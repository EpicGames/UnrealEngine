[CoreRedirects]

; UE 5.5 redirects.
+FunctionRedirects=(OldName="ActivateCameraRigFunctions.ActivateBaseCameraRig", NewName="ActivatePersistentBaseCameraRig")
+FunctionRedirects=(OldName="ActivateCameraRigFunctions.ActivateGlobalCameraRig", NewName="ActivatePersistentGlobalCameraRig")
+FunctionRedirects=(OldName="ActivateCameraRigFunctions.ActivateVisualCameraRig", NewName="ActivatePersistentVisualCameraRig")

; UE 5.6 redirects.
+ClassRedirects=(OldName="/Script/GameplayCameras.CameraRigInterfaceParameter", NewName="CameraObjectInterfaceBlendableParameter")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraTargetInfo",NewName="CameraActorTargetInfo")
+FunctionRedirects=(OldName="BlueprintCameraDirectorEvaluator.ActivateCameraRigPrefab", NewName="ActivateCameraRig")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CameraRigInterface.InterfaceParameters", NewName="BlendableParameters")

+ClassRedirects=(OldName="/Script/GameplayCameras.CameraRigBlendableParameter",NewName="CameraObjectInterfaceBlendableParameter")
+ClassRedirects=(OldName="/Script/GameplayCameras.CameraRigDataParameter",NewName="CameraObjectInterfaceDataParameter")
+ClassRedirects=(OldName="/Script/GameplayCameras.CameraRigInterfaceParameterBase",NewName="CameraObjectInterfaceParameterBase")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraRigAssetReferenceParameterMetaData",NewName="CameraObjectInterfaceParameterMetaData")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraRigInterface",NewName="CameraObjectInterface")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraRigParameterDefinition",NewName="CameraObjectInterfaceParameterDefinition")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraRigAllocationInfo",NewName="CameraObjectAllocationInfo")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CameraObjectInterface.InterfaceParameters", NewName="BlendableParameters")
+EnumRedirects=(OldName="/Script/GameplayCameras.ECameraRigInterfaceParameterType",NewName="ECameraObjectInterfaceCameraParameterType")

+StructRedirects=(OldName="/Script/GameplayCameras.BlueprintCameraNodeEvaluationResult",NewName="BlueprintCameraEvaluationDataRef")
+PropertyRedirects=(OldName="GetCameraPose.Data",NewName="CameraData")
+PropertyRedirects=(OldName="SetCameraPose.Data",NewName="CameraData")

+FunctionRedirects=(OldName="GameplayCameraSystemActor.AutoManageActiveViewTarget", NewName="SetAutoSpawnedCameraSystemActorViewTarget")

+StructRedirects=(OldName="/Script/GameplayCameras.CameraRigProxyTableEntry",NewName="CameraRigProxyRedirectTableEntry")

; UE 5.7 redirects.
+PropertyRedirects=(OldName="/Script/GameplayCameras.CameraVariableAsset.bIsInput",NewName="CameraVariableAsset.bIsPreBlended")

; UE 5.8 redirects.
+PropertyRedirects=(OldName="/Script/GameplayCameras.CustomCameraNodeBlendableParameter.ParameterType",NewName="VariableType")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CustomCameraNodeDataParameter.ParameterType",NewName="DataType")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CustomCameraNodeBlendableParameter.ParameterContainerType",NewName="DataContainerType")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CustomCameraNodeBlendableParameter.ParameterTypeObject",NewName="DataTypeObject")
+PropertyRedirects=(OldName="/Script/GameplayCameras.CameraObjectInterfaceBlendableParameter.ParameterType",NewName="VariableType")

+PropertyRedirects=(OldName="/Script/GameplayCameras.GameplayCameraComponentBase.AutoActivateForPlayer",NewName="DefaultPlayer")

