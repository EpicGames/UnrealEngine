{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Gameplay Cameras",
	"Description" : "A modular and data-driven camera system for Unreal",
	"Category" : "Cameras",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsExperimentalVersion" : true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "StateTree",
			"Enabled": true
		},
		{
			"Name": "TemplateSequence",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "GameplayCameras",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "GameplayCamerasUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name": "GameplayCamerasEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}
