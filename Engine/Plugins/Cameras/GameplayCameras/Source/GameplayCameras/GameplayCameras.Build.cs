// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class GameplayCameras : ModuleRules
{
	public GameplayCameras(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EnhancedInput",
				"GameplayTags",
				"HeadMountedDisplay",
				"MovieScene",
				"MovieSceneTracks",
				"RewindDebuggerRuntimeInterface",
				"StateTreeModule",
				"TemplateSequence",
				"TraceLog"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"UnrealEd"
				});
		}
	}
}
