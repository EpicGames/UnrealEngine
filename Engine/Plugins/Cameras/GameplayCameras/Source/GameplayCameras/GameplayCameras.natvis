<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="FCameraVariableID">
    <DisplayString Condition="Value == (uint32)-1">Invalid</DisplayString>
    <DisplayString Condition="Value != (uint32)-1">Value={Value}</DisplayString>
  </Type>

  <Type Name="FCameraContextDataID">
    <DisplayString Condition="Value == (uint32)-1">Invalid</DisplayString>
    <DisplayString Condition="Value != (uint32)-1">Value={Value}</DisplayString>
  </Type>

  <Type Name="UE::Cameras::FCameraVariableTable">

    <DisplayString>Num={Entries.ArrayNum}</DisplayString>

    <Expand>

      <Synthetic Name="Entries">

        <DisplayString>Num={Entries.ArrayNum}</DisplayString>
        
        <Expand>
            <CustomListItems>
              
              <Variable Name="EntryIndex" InitialValue="0" />
              <Variable Name="EntryPtr" InitialValue="(UE::Cameras::FCameraVariableTable::FEntry*)0" />
              <Variable Name="NumEntries" InitialValue="Entries.ArrayNum" />

              <Variable Name="ValuePtr" InitialValue="Memory" />

              <Loop>
                <Break Condition="EntryIndex == NumEntries" />
                <Exec>EntryPtr = (UE::Cameras::FCameraVariableTable::FEntry*)&amp;Entries[EntryIndex]</Exec>

                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Boolean">*(bool*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Integer32">*(int32*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Float">*(float*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Double">*(double*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector2f">*(FVector2f*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector2d">*(FVector2d*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector3f">*(FVector3f*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector3d">*(FVector3d*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector4f">*(FVector4f*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Vector4d">*(FVector4d*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Rotator3f">*(FRotator3f*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Rotator3d">*(FRotator3d*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Transform3f">*(FTransform3f*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::Transform3d">*(FTransform3d*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraVariableType::BlendableStruct">(void*)(Memory + EntryPtr->Offset)</Item>

                <Exec>EntryIndex = EntryIndex + 1</Exec>
              </Loop>
              
            </CustomListItems>
          </Expand>
      </Synthetic>

      <Synthetic Name="Flags">

        <DisplayString>Num={Entries.ArrayNum}</DisplayString>

        <Expand>
          <CustomListItems>

            <Variable Name="EntryIndex" InitialValue="0" />
            <Variable Name="EntryPtr" InitialValue="(UE::Cameras::FCameraVariableTable::FEntry*)0" />
            
            <Variable Name="NumEntries" InitialValue="Entries.ArrayNum" />

            <Loop>
              <Break Condition="EntryIndex == NumEntries"/>
              <Exec>EntryPtr = (UE::Cameras::FCameraVariableTable::FEntry*)&amp;Entries[EntryIndex]</Exec>

              <Item Name="{EntryPtr->DebugName}">EntryPtr->Flags</Item>

              <Exec>EntryIndex = EntryIndex + 1</Exec>
            </Loop>
            
          </CustomListItems>
        </Expand>
        
      </Synthetic>

      <Item Name="Details">Entries</Item>
      
      <Item Name="Capacity">Capacity</Item>
      <Item Name="Used">Used</Item>

    </Expand>

  </Type>

  <Type Name="UE::Cameras::FCameraContextDataTable">

    <DisplayString>Num={Entries.ArrayNum}</DisplayString>

    <Expand>
      
      <Synthetic Name="Entries">

        <DisplayString>Num={Entries.ArrayNum}</DisplayString>

        <Expand>
          <CustomListItems>

            <Variable Name="EntryIndex" InitialValue="0" />
            <Variable Name="EntryPtr" InitialValue="(UE::Cameras::FCameraContextDataTable::FEntry*)0" />
            <Variable Name="NumEntries" InitialValue="Entries.ArrayNum" />

            <Variable Name="ValuePtr" InitialValue="Memory" />

            <Size>Entries.ArrayNum</Size>

            <Loop>
              <Break Condition="EntryIndex == NumEntries"/>
              <Exec>EntryPtr = (UE::Cameras::FCameraContextDataTable::FEntry*)&amp;Entries[EntryIndex]</Exec>

              <If Condition="EntryPtr->ContainerType == ECameraContextDataContainerType::Array">
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]">*(FEntryScriptArray*)(Memory + EntryPtr->Offset)</Item>
              </If>
              <If Condition="EntryPtr->ContainerType == ECameraContextDataContainerType::None">
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::Name">*(FName*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::String">*(FString*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::Enum">*(uint8*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::Struct">(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::Object">*(UObject*)(Memory + EntryPtr->Offset)</Item>
                <Item Name="{EntryPtr->DebugName} [{EntryPtr->ID.Value}]" Condition="EntryPtr->Type == ECameraContextDataType::Class">*(UClass*)(Memory + EntryPtr->Offset)</Item>
              </If>

              <Exec>EntryIndex = EntryIndex + 1</Exec>
            </Loop>
            
          </CustomListItems>
        </Expand>
        
      </Synthetic>

      <Synthetic Name="Flags">

        <DisplayString>Num={Entries.ArrayNum}</DisplayString>

        <Expand>
          <CustomListItems>

            <Variable Name="EntryIndex" InitialValue="0" />
            <Variable Name="EntryPtr" InitialValue="(UE::Cameras::FCameraContextDataTable::FEntry*)0" />

            <Variable Name="NumEntries" InitialValue="Entries.ArrayNum" />

            <Loop>
              <Break Condition="EntryIndex == NumEntries"/>
              <Exec>EntryPtr = (UE::Cameras::FCameraContextDataTable::FEntry*)&amp;Entries[EntryIndex]</Exec>

              <Item Name="{EntryPtr->DebugName}">EntryPtr->Flags</Item>

              <Exec>EntryIndex = EntryIndex + 1</Exec>
            </Loop>

          </CustomListItems>
        </Expand>

      </Synthetic>

      <Item Name="Details">Entries</Item>

      <Item Name="Capacity">Capacity</Item>
      <Item Name="Used">Used</Item>
      
    </Expand>
    
  </Type>

</AutoVisualizer>

