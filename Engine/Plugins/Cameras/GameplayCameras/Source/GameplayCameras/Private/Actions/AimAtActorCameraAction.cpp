// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/AimAtActorCameraAction.h"

#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AimAtActorCameraAction)

namespace UE::Cameras
{

class FAimAtActorCameraActionEvaluator : public FBaseAimAtCameraActionEvaluator
{
	UE_DECLARE_CAMERA_ACTION_EVALUATOR_EX(, FAimAtActorCameraActionEvaluator, FBaseAimAtCameraActionEvaluator)

protected:

	// FBaseAimAtCameraActionEvaluator
	virtual FVector3d UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result) override;

private:

	AActor* CachedActor = nullptr;
	USkeletalMeshComponent* CachedSkeletalMeshComponent = nullptr;
	FName CachedBoneName;
};

UE_DEFINE_CAMERA_ACTION_EVALUATOR(FAimAtActorCameraActionEvaluator)

FVector3d FAimAtActorCameraActionEvaluator::UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result)
{
	const UAimAtActorCameraAction* ActionData = GetCameraActionAs<UAimAtActorCameraAction>();
	
	const bool bIsSameActor = (ActionData->TargetActor == CachedActor);
	if (!bIsSameActor)
	{
		CachedActor = ActionData->TargetActor.Get();

		CachedSkeletalMeshComponent = nullptr;
		if (ActionData->TargetActor && (!ActionData->TargetSocketName.IsNone() || !ActionData->TargetBoneName.IsNone()))
		{
			CachedSkeletalMeshComponent = ActionData->TargetActor->FindComponentByClass<USkeletalMeshComponent>();
		}

		CachedBoneName = NAME_None;
		if (CachedSkeletalMeshComponent)
		{
			CachedBoneName = ActionData->TargetBoneName;
			if (!ActionData->TargetSocketName.IsNone())
			{
				CachedBoneName = CachedSkeletalMeshComponent->GetSocketBoneName(ActionData->TargetSocketName);
			}
		}
	}

	if (CachedSkeletalMeshComponent && !CachedBoneName.IsNone())
	{
		return CachedSkeletalMeshComponent->GetBoneTransform(CachedBoneName).GetLocation();
	}
	else if (CachedActor)
	{
		return CachedActor->GetTransform().GetLocation();
	}

	return FVector3d::ZeroVector;
}

}  // namespace UE::Cameras

FCameraActionEvaluatorPtr UAimAtActorCameraAction::OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAimAtActorCameraActionEvaluator>();
}

