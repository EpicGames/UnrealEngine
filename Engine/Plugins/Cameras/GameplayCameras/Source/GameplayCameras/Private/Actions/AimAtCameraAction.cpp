// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/AimAtCameraAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AimAtCameraAction)

namespace UE::Cameras
{

class FAimAtCameraActionEvaluator : public FBaseAimAtCameraActionEvaluator
{
	UE_DECLARE_CAMERA_ACTION_EVALUATOR_EX(, FAimAtCameraActionEvaluator, FBaseAimAtCameraActionEvaluator)

protected:

	// FBaseAimAtCameraActionEvaluator
	virtual FVector3d UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result) override;
};

UE_DEFINE_CAMERA_ACTION_EVALUATOR(FAimAtCameraActionEvaluator)

FVector3d FAimAtCameraActionEvaluator::UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result)
{
	const UAimAtCameraAction* ActionData = GetCameraActionAs<UAimAtCameraAction>();
	return ActionData->TargetLocation;
}

}  // namespace UE::Cameras

FCameraActionEvaluatorPtr UAimAtCameraAction::OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAimAtCameraActionEvaluator>();
}

