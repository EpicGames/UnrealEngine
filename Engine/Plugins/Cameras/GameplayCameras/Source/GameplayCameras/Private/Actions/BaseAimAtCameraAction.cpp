// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/BaseAimAtCameraAction.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"  // IWYU pragma: keep
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Math/CameraAimingMath.h"
#include "Math/CameraPoseMath.h"
#include "Math/ColorList.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Services/CameraActionEvaluator.h"
#include "Services/CameraActionScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseAimAtCameraAction)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_ACTION_EVALUATOR(FBaseAimAtCameraActionEvaluator)

void FBaseAimAtCameraActionEvaluator::OnInitialize(const FCameraActionEvaluatorInitializeParams& Params, FCameraActionEvaluationResult& OutResult)
{
	const UBaseAimAtCameraAction* ActionData = GetCameraActionAs<UBaseAimAtCameraAction>();

	if (ActionData->Interpolator)
	{
		Interpolator = ActionData->Interpolator->BuildVector2dInterpolator();
	}
	else
	{
		Interpolator = MakeUnique<TPopValueInterpolator<FVector2d>>();
	}

	const FCameraRigEvaluationInfo CameraRigEvaluationInfo = Params.Scope->GetCameraRigEvaluationInfo();
	ChildHierarchy.Build(CameraRigEvaluationInfo.RootEvaluator);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugElapsedTime = 0.f;
#endif
}

FVector3d FBaseAimAtCameraActionEvaluator::UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result)
{
	return FVector3d::ZeroVector;
}

void FBaseAimAtCameraActionEvaluator::OnPreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	if (Params.EvaluationParams.IsStatelessEvaluation())
	{
		return;
	}

	// Update target location and framing every frame in case it changes.
	TargetLocation = UpdateTargetLocation(Params, OutResult);

	const UBaseAimAtCameraAction* ActionData = GetCameraActionAs<UBaseAimAtCameraAction>();
	TargetFraming = ActionData->TargetFraming;

	bool bIsActionStillRunning = true;
	if (bIsLockedOn)
	{
		// We are locked on the target, and are supposed to keep it that way.
		bIsActionStillRunning = KeepLockOn(Params, OutResult);
	}
	else
	{
		// Aim the camera towards the target before the camera rig is run.
		bIsActionStillRunning = ContinueAimAction(Params, OutResult);
	}
	OutResult.bIsActionFinished = !bIsActionStillRunning;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugElapsedTime += Params.EvaluationParams.DeltaTime;
#endif
}

void FBaseAimAtCameraActionEvaluator::LockUserInputThisFrame(const FCameraActionEvaluationParams& Params)
{
	// Execute the operation to turn the camera.
	FCameraOperationParams OperationParams;
	OperationParams.Evaluator = Params.EvaluationParams.Evaluator;
	OperationParams.EvaluationContext = Params.EvaluationParams.EvaluationContext;

	FLockUserInputCameraOperation Operation;
	Operation.bLockThisFrame = true;

	ChildHierarchy.CallExecuteOperation(OperationParams, Operation);
}

void FBaseAimAtCameraActionEvaluator::RunPreviewEvaluation(const FCameraActionEvaluationParams& Params, const FCameraNodeEvaluationResult& Result)
{
	ScratchResult.OverrideAll(Result, true);

	// Save the initial state of the camera rig.
	{
		EvaluatorSnapshot.Reset();

		FCameraNodeEvaluatorSerializeParams SerializeParams;
		FMemoryWriter Writer(EvaluatorSnapshot);
		ChildHierarchy.CallSerialize(SerializeParams, Writer);
	}

	// Run the camera rig.
	{
		FCameraNodeEvaluationParams PreviewParams(Params.EvaluationParams);
		PreviewParams.EvaluationType = ECameraNodeEvaluationType::IK;

		const FCameraRigEvaluationInfo CameraRigEvaluationInfo = Params.Scope->GetCameraRigEvaluationInfo();
		FCameraNodeEvaluator* ChildEvaluator = CameraRigEvaluationInfo.RootEvaluator;
		ChildEvaluator->Run(PreviewParams, ScratchResult);
	}

	// Restore the state of the camera rig.
	{
		FCameraNodeEvaluatorSerializeParams SerializeParams;
		FMemoryReader Reader(EvaluatorSnapshot);
		ChildHierarchy.CallSerialize(SerializeParams, Reader);
	}
}

bool FBaseAimAtCameraActionEvaluator::ComputeDesiredCorrection(const FCameraActionEvaluationParams& Params, FRotator3d& OutCorrection)
{
	// Find the pivot for this frame.
	const FCameraRigJoint* PreviewPivot = FCameraAimingMath::FindPivotJoint(ScratchResult.CameraRigJoints);
	if (!PreviewPivot)
	{
		// If there's no pivot in the camera, we can't aim it.
		UE_LOGF(LogCameraSystem, Warning, 
				"Can't aim camera rig '%ls': it has no pivot joint.",
				*GetNameSafe(Params.Scope->GetCameraRig()));
		return false;
	}

	static const FVector2d CenterFraming(0.5, 0.5);
	const bool bNeedsTargetFraming = (TargetFraming != CenterFraming);
	FVector3d TargetFramingAim = ScratchResult.CameraPose.GetAimDir();
	if (bNeedsTargetFraming)
	{
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext = Params.Scope->GetEvaluationContext();
		const FRay3d TargetRay = FCameraPoseMath::UnprojectScreenToWorld(
				ScratchResult.CameraPose, EvaluationContext, TargetFraming);
		TargetFramingAim = TargetRay.Direction;
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugTargetFramingAim = TargetFramingAim;
	DebugCameraLocation = ScratchResult.CameraPose.GetLocation();
	DebugCameraAim = ScratchResult.CameraPose.GetAimDir();
#endif

	// Compute the total correction need to aim at the target.
	FRotator3d TotalCorrection;
	const bool bGotCorrection = FCameraAimingMath::ComputeTwoBonesCorrection(
			ScratchResult.CameraPose, 
			TargetFramingAim, 
			PreviewPivot->Transform.GetLocation(), 
			TargetLocation,
			TotalCorrection);
	if (!bGotCorrection)
	{
		UE_LOGF(LogCameraSystem, Warning, 
				"Can't aim camera rig '%ls': we can't solve an IK correction for the desired target.",
				*GetNameSafe(Params.Scope->GetCameraRig()));
		return false;
	}

	OutCorrection = TotalCorrection.GetNormalized();
	return true;
}

bool FBaseAimAtCameraActionEvaluator::ExecuteYawPitchCorrection(const FCameraActionEvaluationParams& Params, const FRotator3d& Correction)
{
	// Execute the operation to turn the camera.
	FCameraOperationParams OperationParams;
	OperationParams.Evaluator = Params.EvaluationParams.Evaluator;
	OperationParams.EvaluationContext = Params.EvaluationParams.EvaluationContext;

	FYawPitchCameraOperation Operation;
	Operation.Yaw = FConsumableDouble::Delta(Correction.Yaw);
	Operation.Pitch = FConsumableDouble::Delta(Correction.Pitch);

	ChildHierarchy.CallExecuteOperation(OperationParams, Operation);

	if (Operation.Yaw.HasValue() || Operation.Pitch.HasValue())
	{
		UE_LOGF(LogCameraSystem, Warning, 
				"Aborting aiming of camera rig '%ls': not all corrections were consumed by the camera nodes.",
				*GetNameSafe(Params.Scope->GetCameraRig()));
		return false;
	}

	return true;
}

bool FBaseAimAtCameraActionEvaluator::ContinueAimAction(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	const UBaseAimAtCameraAction* ActionData = GetCameraActionAs<UBaseAimAtCameraAction>();
	if (!ensure(ActionData))
	{
		return false;
	}

	TSharedPtr<const FCameraEvaluationContext> EvaluationContext = Params.EvaluationParams.EvaluationContext;
	if (!ensure(EvaluationContext))
	{
		return false;
	}

	// Prevent the user from turning the camera this frame, since we want to continue aiming.
	LockUserInputThisFrame(Params);

	// Run a preview evaluation of the camera rig.
	RunPreviewEvaluation(Params, OutResult.Result);

	// Compute exactly how much we still need to turn to get to the target.
	FRotator3d TotalCorrection;
	if (!ComputeDesiredCorrection(Params, TotalCorrection))
	{
		return false;
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugCorrectionLeft = TotalCorrection;
#endif

	// See if we're close enough to the target given our tolerance margin.
	if (FMath::Abs(TotalCorrection.Yaw) < ActionData->LockOnAngleTolerance && 
			FMath::Abs(TotalCorrection.Pitch) < ActionData->LockOnAngleTolerance)
	{
		bIsLockedOn = true;
		return (ActionData->LockOnPolicy == EAimAtCameraActionLockOnPolicy::KeepLock);
	}

	// Update the interpolator to know how much we need to turn _this frame_ to get to the target over time.
	{
		FCameraValueInterpolationParams InterpolatorParams;
		InterpolatorParams.DeltaTime = Params.EvaluationParams.DeltaTime;
		InterpolatorParams.bIsCameraCut = OutResult.Result.bIsCameraCut;

		FCameraValueInterpolationResult InterpolatorResult(OutResult.Result.VariableTable);

		Interpolator->Reset(FVector2d(TotalCorrection.Yaw, TotalCorrection.Pitch), FVector2d::ZeroVector);
		Interpolator->Run(InterpolatorParams, InterpolatorResult);
	}

	const FVector2d InterpValue = Interpolator->GetCurrentValue();
	const FRotator3d RemainingCorrection(InterpValue.Y, InterpValue.X, 0.0);
	const FRotator3d CurCorrection = (TotalCorrection - RemainingCorrection).GetNormalized();
#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugCurrentCorrection = CurCorrection;
#endif

	// Execute the operation to turn the camera.
	if (!ExecuteYawPitchCorrection(Params, CurCorrection))
	{
		return false;
	}

	// If the interpolator has finished, we're done too.
	if (Interpolator->IsFinished())
	{
		bIsLockedOn = true;
		return (ActionData->LockOnPolicy == EAimAtCameraActionLockOnPolicy::KeepLock);
	}

	return true;
}

bool FBaseAimAtCameraActionEvaluator::KeepLockOn(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	// Prevent the user from turning the camera this frame, since we want to keep a lock on the target.
	LockUserInputThisFrame(Params);

	// Run a preview of what the camera will look at this frame.
	RunPreviewEvaluation(Params, OutResult.Result);

	// Figure out the correction to keep it locked on the target.
	FRotator3d TotalCorrection;
	if (!ComputeDesiredCorrection(Params, TotalCorrection))
	{
		return false;
	}

	// Run the correction on the camera rig.
	if (!ExecuteYawPitchCorrection(Params, TotalCorrection))
	{
		return false;
	}

	return true;
}

void FBaseAimAtCameraActionEvaluator::OnSerialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar)
{
	if (Interpolator)
	{
		FCameraValueInterpolatorSerializeParams InterpolatorParams;
		Interpolator->Serialize(InterpolatorParams, Ar);
	}

	Ar << bIsLockedOn;

	// Other fields don't need to be serialized since they are either recomputed on action clone and then never change
	// (e.g. ChildHierarchy), or recomputed every frame (e.g. TargetLocation, TargetFraming, and the stuff for running
	// the temporary IK evaluation).
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(, FAimAtCameraActionDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, ElapsedTime)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, CorrectionLeft)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, CurrentCorrection)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, TargetFraming)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, TargetFramingAim)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, CameraLocation)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, CameraAim)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsLockedOn)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

void FBaseAimAtCameraActionEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FAimAtCameraActionDebugBlock& DebugBlock = Builder.AttachDebugBlock<FAimAtCameraActionDebugBlock>();
	DebugBlock.ElapsedTime = DebugElapsedTime;
	DebugBlock.CorrectionLeft = DebugCorrectionLeft;
	DebugBlock.CurrentCorrection = DebugCurrentCorrection;
	DebugBlock.TargetFraming = TargetFraming;
	DebugBlock.TargetFramingAim = DebugTargetFramingAim;
	DebugBlock.CameraLocation = DebugCameraLocation;
	DebugBlock.CameraAim = DebugCameraAim;
	DebugBlock.bIsLockedOn = bIsLockedOn;
}

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FAimAtCameraActionDebugBlock)

void FAimAtCameraActionDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bIsLockedOn)
	{
		Renderer.AddText(TEXT("locked on target"));
	}
	else
	{
		Renderer.AddText(
				TEXT("elapsed time %.1f ; correction yaw=%.3f/%.3f  pitch=%.3f/%.3f"),
				ElapsedTime,
				CurrentCorrection.Yaw, CorrectionLeft.Yaw,
				CurrentCorrection.Pitch, CorrectionLeft.Pitch);
	}

	if (!Renderer.IsExternalRendering())
	{
		const FVector2D CanvasSize = Renderer.GetCanvasSize();

		const FVector2D TargetFramingPosition = TargetFraming * CanvasSize;
		Renderer.Draw2DPointCross(TargetFramingPosition, 20, FColorList::LightGrey);
	}
	else
	{
		Renderer.DrawLine(CameraLocation, CameraLocation + 2000.f * CameraAim, FColorList::LightBlue);
		Renderer.DrawLine(CameraLocation, CameraLocation + 2000.f * TargetFramingAim, FColorList::LightBlue);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

