// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraAssetAssembleUtils.h"

namespace UE::Cameras
{

FCameraRigAssetAssembler::FCameraRigAssetAssembler(FName Name, UObject* Outer)
	: TCameraRigAssetAssemblerBase<FCameraRigAssetAssembler>(nullptr, Name, Outer)
{
}

FCameraRigAssetAssembler::FCameraRigAssetAssembler(TSharedPtr<FNamedObjectRegistry> InNamedObjectRegistry, FName Name, UObject* Outer)
	: TCameraRigAssetAssemblerBase<FCameraRigAssetAssembler>(InNamedObjectRegistry, Name, Outer)
{
}

FCameraAssetAssembler::FCameraAssetAssembler(UObject* Owner)
{
	if (Owner == nullptr)
	{
		Owner = GetTransientPackage();
	}

	CameraAsset = NewObject<UCameraAsset>(Owner);

	TCameraObjectInitializer<UCameraAsset>::SetObject(CameraAsset);

	NamedObjectRegistry = MakeShared<FNamedObjectRegistry>();
}

FCameraEvaluationContextAssembler::FCameraEvaluationContextAssembler(UObject* Owner)
{
	if (Owner == nullptr)
	{
		Owner = GetTransientPackage();
	}

	CameraAsset = NewObject<UCameraAsset>(Owner);

	FCameraEvaluationContextInitializeParams InitParams;
	InitParams.Owner = Owner;
	InitParams.CameraAsset = CameraAsset;
	EvaluationContext = MakeShared<FCameraEvaluationContext>(InitParams);

	TCameraObjectInitializer<FCameraEvaluationContext>::SetObject(EvaluationContext.Get());

	NamedObjectRegistry = MakeShared<FNamedObjectRegistry>();
}

}  // namespace UE::Cameras

