// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraAssetBuilder.h"

#include "Build/CameraBuildContext.h"
#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Build/CameraRigAssetBuilder.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraRigAsset.h"
#include "GameplayCamerasDelegates.h"
#include "Logging/TokenizedMessage.h"

#define LOCTEXT_NAMESPACE "CameraAssetBuilder"

namespace UE::Cameras
{

namespace Private
{

void GatherCameraAssetRigUsage(const UCameraAsset* InCameraAsset, FCameraDirectorRigUsageInfo& OutUsageInfo)
{
	UCameraDirector* CameraDirector = InCameraAsset->GetCameraDirector();
	if (CameraDirector)
	{
		CameraDirector->GatherRigUsageInfo(OutUsageInfo);
	}
}

void BuildCameraAssetDefaultParameterDefinitions(UCameraAsset* CameraAsset, TArrayView<UCameraRigAsset*> AllCameraRigs)
{
	// Get the list of all the camera rigs' interface parameters.
	TMap<FName, TArray<UCameraRigAsset*>> UsedParameterNames;
	TMap<UCameraRigAsset*, TArray<FCameraObjectInterfaceParameterDefinition>> DefinitionsByCameraRig;
	for (UCameraRigAsset* CameraRig : AllCameraRigs)
	{
		if (ensure(!DefinitionsByCameraRig.Contains(CameraRig)))
		{
			TArray<FCameraObjectInterfaceParameterDefinition>& DefinitionsForCameraRig = DefinitionsByCameraRig.Add(CameraRig);
			for (const FCameraObjectInterfaceParameterDefinition& Definition : CameraRig->GetParameterDefinitions())
			{
				DefinitionsForCameraRig.Add(Definition);
				UsedParameterNames.FindOrAdd(Definition.ParameterName).Add(CameraRig);
			}
		}
	}
	// Resolve name conflicts.
	TMap<TTuple<UCameraRigAsset*, FName>, FString> SourceParameterToInterfaceName;
	for (const TPair<FName, TArray<UCameraRigAsset*>>& Pair : UsedParameterNames)
	{
		const TArray<UCameraRigAsset*>& ConflictCameraRigs(Pair.Value);
		if (ConflictCameraRigs.Num() > 1)
		{
			// For each parameter with the same name, append the source camera rig's name.
			const FName ConflictName(Pair.Key);
			for (UCameraRigAsset* CameraRig : ConflictCameraRigs)
			{
				const FString NewName = FString::Format(TEXT("{0}_{1}"), { *GetNameSafe(CameraRig), *ConflictName.ToString() });
				SourceParameterToInterfaceName.Add({ CameraRig, Pair.Key }, NewName);
			}
		}
		// else: the interface name will be the same as the source parameter's.
	}

	// Build the final list of parameter definitions.
	for (const TPair<UCameraRigAsset*, TArray<FCameraObjectInterfaceParameterDefinition>>& Pair : DefinitionsByCameraRig)
	{
		UCameraRigAsset* CameraRig = Pair.Key;
		const TArray<FCameraObjectInterfaceParameterDefinition>& DefinitionsForCameraRig(Pair.Value);
		for (const FCameraObjectInterfaceParameterDefinition& Definition : DefinitionsForCameraRig)
		{
			const FString DefaultInterfaceName = Definition.ParameterName.ToString();
			const FString ResolvedInterfaceName = SourceParameterToInterfaceName.FindRef(
					{ CameraRig, Definition.ParameterName }, DefaultInterfaceName);

			UCameraAssetInterfaceParameter* NewInterfaceParameter = NewObject<UCameraAssetInterfaceParameter>(CameraAsset);
			NewInterfaceParameter->InterfaceParameterName = ResolvedInterfaceName;
			NewInterfaceParameter->SourceCameraRig = CameraRig;
			NewInterfaceParameter->SourceParameterName = Definition.ParameterName;
			CameraAsset->Interface.Parameters.Add(NewInterfaceParameter);
		}
	}
}

}  // namespace Private

FCameraAssetBuilder::FCameraAssetBuilder(FCameraBuildContext& InBuildContext)
	: BuildContext(InBuildContext)
{
}

void FCameraAssetBuilder::BuildCamera(UCameraAsset* InCameraAsset, bool bBuildReferencedAssets)
{
	if (!ensure(InCameraAsset))
	{
		return;
	}

	CameraAsset = InCameraAsset;
	BuildContext.BuildLog.SetLoggingPrefix(InCameraAsset->GetPathName() + TEXT(": "));
	{
		BuildCameraImpl(bBuildReferencedAssets);
	}
	BuildContext.BuildLog.SetLoggingPrefix(FString());
	UpdateBuildStatus();

	FGameplayCamerasDelegates::OnCameraAssetBuilt().Broadcast(CameraAsset);
}

void FCameraAssetBuilder::BuildCameraImpl(bool bBuildReferencedAssets)
{
	// Build the camera director.
	UCameraDirector* CameraDirector = CameraAsset->GetCameraDirector();
	if (CameraDirector)
	{
		CameraDirector->BuildCameraDirector(BuildContext);
	}
	else
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, LOCTEXT("MissingDirector", "Camera has no director set."));
	}

	// Get the list of referenced camera rigs and proxies.
	FCameraDirectorRigUsageInfo UsageInfo;
	Private::GatherCameraAssetRigUsage(CameraAsset, UsageInfo);
	if (CameraDirector && UsageInfo.CameraRigs.IsEmpty() && UsageInfo.CameraRigProxies.IsEmpty())
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Warning, LOCTEXT("MissingRigs", "Camera director isn't using any camera rigs or proxies."));
	}

	BuiltReferencedAssets.Reset();
	if (bBuildReferencedAssets)
	{
		// Build each of the camera rigs.
		for (UCameraRigAsset* CameraRig : UsageInfo.CameraRigs)
		{
			const bool bNeedsBuild = CameraRig->BuildStatus == ECameraBuildStatus::Dirty;
			if (bNeedsBuild)
			{
				FCameraRigAssetBuilder CameraRigBuilder(BuildContext);
				CameraRigBuilder.BuildCameraRig(CameraRig);
				BuiltReferencedAssets.Add(CameraRig);
			}
		}
	}

	// If we have some old data, we may need to build a default interface for it, but don't dirty the asset unless we're cooking and
	// want to save the proper data. In other contexts, we don't want to cause the user to be prompted with an asset to save without
	// them knowing what changed and why.
	if (CameraAsset->bNeedsDefaultInterface)
	{
		if (BuildContext.IsCooking())
		{
			CameraAsset->Modify();
		}
		Private::BuildCameraAssetDefaultParameterDefinitions(CameraAsset, UsageInfo.CameraRigs);
		CameraAsset->bNeedsDefaultInterface = false;
	}

	// Build the new parameter definitions, and then the new default parameters struct.
	BuildParameters();

	// Accumulate all the camera rigs' allocation infos and store that on the asset.
	// Only do this when cooking. In editor builds we don't want to mark the asset as modified just because someone
	// changed some parameters on a camera rig, or whatever.
	if (BuildContext.IsCooking())
	{
		FCameraAssetAllocationInfo AllocationInfo;
		for (const UCameraRigAsset* CameraRig : UsageInfo.CameraRigs)
		{
			AllocationInfo.VariableTableInfo.Combine(CameraRig->AllocationInfo.VariableTableInfo);
			AllocationInfo.ContextDataTableInfo.Combine(CameraRig->AllocationInfo.ContextDataTableInfo);
		}

		if (AllocationInfo != CameraAsset->AllocationInfo)
		{
			CameraAsset->Modify();
			CameraAsset->AllocationInfo = AllocationInfo;
		}
	}
}

void FCameraAssetBuilder::BuildParameters()
{
	// First build the new parameter definitions. Keep track of which ones are exposed from which camera rig.
	TArray<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions;
	TArray<const UCameraRigAsset*> SourceCameraRigs;

	for (auto It = CameraAsset->Interface.Parameters.CreateIterator(); It; ++It)
	{
		const UCameraAssetInterfaceParameter* InterfaceParameter(*It);
		if (!InterfaceParameter)
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Warning,
					CameraAsset,
					LOCTEXT("InvalidInterfaceParameter", "Invalid interface parameter was found and removed."));

			CameraAsset->Modify();
			It.RemoveCurrent();
			continue;
		}

		if (InterfaceParameter->InterfaceParameterName.IsEmpty())
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					InterfaceParameter,
					LOCTEXT(
						"InvalidInterfaceParameterName",
						"Invalid interface parameter name."));
			continue;
		}

		const UCameraRigAsset* CameraRig = InterfaceParameter->SourceCameraRig.Get();
		if (!CameraRig)
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					InterfaceParameter,
					FText::Format(LOCTEXT(
						"InterfaceParameterMissingSourceCameraRig",
						"Interface parameter references a moved or deleted camera rig asset: {0}"),
						FText::FromString(InterfaceParameter->SourceCameraRig.GetAssetName())));
			continue;
		}

		if (InterfaceParameter->SourceParameterName.IsNone())
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					InterfaceParameter,
					FText::Format(LOCTEXT(
						"InterfaceParameterNoSourceParameter",
						"Interface parameter doesn't specify the parameter name to reference on camera rig '{0}'."),
						FText::FromName(CameraRig->GetFName())));
			continue;
		}

		FCameraObjectInterfaceParameterDefinition SourceParameterDefinition;
		const bool bFoundSourceParameter = CameraRig->FindParameterDefinitionByName(
				InterfaceParameter->SourceParameterName, SourceParameterDefinition);
		if (!bFoundSourceParameter)
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					InterfaceParameter,
					FText::Format(LOCTEXT(
						"InterfaceParameterMissingSourceParameter",
						"Interface parameter references renamed or deleted parameter '{0}' on camera rig '{1}'."),
						FText::FromName(InterfaceParameter->SourceParameterName),
						FText::FromName(CameraRig->GetFName())));
			continue;
		}

		FCameraObjectInterfaceParameterDefinition ParameterDefinition(SourceParameterDefinition);
		ParameterDefinition.ParameterName = FName(InterfaceParameter->InterfaceParameterName);
		ParameterDefinitions.Add(ParameterDefinition);

		SourceCameraRigs.Add(CameraRig);
	}

	if (CameraAsset->ParameterDefinitions != ParameterDefinitions)
	{
		CameraAsset->Modify();
		CameraAsset->ParameterDefinitions = ParameterDefinitions;
	}

	// Now build the new default parameters structure.
	TArray<FPropertyBagPropertyDesc> DefaultParameterProperties;
	FCameraObjectInterfaceParameterBuilder::AppendDefaultParameterProperties(
			CameraAsset->ParameterDefinitions, DefaultParameterProperties);
	const UPropertyBag* DefaultParametersStruct = UPropertyBag::GetOrCreateFromDescs(DefaultParameterProperties);

	// Asssign the default values by looking up the same value on the source camera rig.
	FInstancedPropertyBag DefaultParameters;
	DefaultParameters.InitializeFromBagStruct(DefaultParametersStruct);
	ensure(ParameterDefinitions.Num() == SourceCameraRigs.Num());
	for (int32 Index = 0; Index < ParameterDefinitions.Num(); ++Index)
	{
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition = ParameterDefinitions[Index];
		const UCameraRigAsset* SourceCameraRig = SourceCameraRigs[Index];
		ensure(SourceCameraRig);
		CopyDefaultParameterValue(SourceCameraRig, ParameterDefinition.ParameterGuid, DefaultParameters);
	}

	// If the list or types of parameters changed, or if just the default values changed, let's update the asset.
	if (!DefaultParameters.Identical(&CameraAsset->DefaultParameters, 0))
	{
		CameraAsset->Modify();
		CameraAsset->DefaultParameters = DefaultParameters;
	}
}

void FCameraAssetBuilder::CopyDefaultParameterValue(const UCameraRigAsset* InCameraRig, const FGuid& PropertyID, FInstancedPropertyBag& DefaultParameters)
{
	// Our default parameters property bag is an aggregation of various camera rigs' parameters.
	// Copy their default values over.
	const FInstancedPropertyBag& SourceValues = InCameraRig->GetDefaultParameters();
	const void* SourceValuesPtr = SourceValues.GetValue().GetMemory();
	void* TargetValuesPtr = DefaultParameters.GetMutableValue().GetMemory();
	const UPropertyBag* SourceValueStruct = SourceValues.GetPropertyBagStruct();
	const UPropertyBag* TargetValueStruct = DefaultParameters.GetPropertyBagStruct();
	if (!ensure(SourceValuesPtr && TargetValuesPtr && SourceValueStruct && TargetValueStruct))
	{
		// We shouldn't have empty property bags if we got this far, since we're supposed to copy at least
		// one property from one to the other.
		return;
	}

	const FPropertyBagPropertyDesc* SourcePropertyDesc = SourceValueStruct->FindPropertyDescByID(PropertyID);
	const FPropertyBagPropertyDesc* TargetPropertyDesc = TargetValueStruct->FindPropertyDescByID(PropertyID);
	if (!ensure(SourcePropertyDesc && TargetPropertyDesc))
	{
		// We should have a valid source property here since, in theory, we built our camera asset's parameters
		// using the referenced camera rigs' built parameter definitions. So their default parameters structs
		// should be up to date with that and we shouldn't have missing properties.
		// As for the target property, we should have it since we just made that property bag.
		return;
	}
	if (!ensure(SourcePropertyDesc->CachedProperty && TargetPropertyDesc->CachedProperty))
	{
		return;
	}
	if (!ensure(SourcePropertyDesc->CompatibleType(*TargetPropertyDesc)))
	{
		return;
	}

	const void* SourcePtr = SourcePropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(SourceValuesPtr);
	void* TargetPtr = TargetPropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(TargetValuesPtr);
	SourcePropertyDesc->CachedProperty->CopyCompleteValue(TargetPtr, SourcePtr);
}

void FCameraAssetBuilder::UpdateBuildStatus()
{
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Clean;
	if (BuildContext.BuildLog.HasErrors())
	{
		BuildStatus = ECameraBuildStatus::WithErrors;
	}
	else if (BuildContext.BuildLog.HasWarnings())
	{
		BuildStatus = ECameraBuildStatus::CleanWithWarnings;
	}

	// Don't modify the camera rig: BuildStatus is transient.
	CameraAsset->SetBuildStatus(BuildStatus);
}

bool FCameraAssetBuilder::BuildDefaultInterfaceIfNeeded(UCameraAsset* InCameraAsset)
{
	if (InCameraAsset->bNeedsDefaultInterface)
	{
		FCameraDirectorRigUsageInfo UsageInfo;
		Private::GatherCameraAssetRigUsage(InCameraAsset, UsageInfo);

		Private::BuildCameraAssetDefaultParameterDefinitions(InCameraAsset, UsageInfo.CameraRigs);
		InCameraAsset->bNeedsDefaultInterface = false;

		return true;
	}

	return false;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

