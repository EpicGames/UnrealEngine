// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraBuildLog.h"

#include "GameplayCameras.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/CString.h"
#include "Misc/UObjectToken.h"
#include "UObject/Object.h"

namespace UE::Cameras
{

FString FCameraBuildLogMessage::ToString() const
{
	TStringBuilder<256> StringBuilder;
	if (Object)
	{
		StringBuilder.Append(Object->GetName());
		StringBuilder.Append(TEXT(": "));
	}
	StringBuilder.Append(Text.ToString());
	return StringBuilder.ToString();
}

void FCameraBuildLogMessage::SendToLogging(const FString& InLoggingPrefix) const
{
#define UE_LOG_FORWARD_CAMERA_RIG_BULID_LOG_MESSAGE(Verbosity)\
	UE_LOGF(LogCameraSystem, Verbosity, "%ls%ls", *InLoggingPrefix, *ToString());

	switch (Severity)
	{
	case EMessageSeverity::Error:
		UE_LOG_FORWARD_CAMERA_RIG_BULID_LOG_MESSAGE(Error);
		break;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		UE_LOG_FORWARD_CAMERA_RIG_BULID_LOG_MESSAGE(Warning);
		break;
	case EMessageSeverity::Info:
	default:
		UE_LOG_FORWARD_CAMERA_RIG_BULID_LOG_MESSAGE(Log);
		break;
	};

#undef UE_LOG_FORWARD_CAMERA_RIG_BULID_LOG_MESSAGE
}

void FCameraBuildLog::SetLoggingPrefix(const FString& InPrefix)
{
	if (InPrefix.IsEmpty())
	{
		LoggingPrefix.Empty();
	}
	else
	{
		LoggingPrefix = InPrefix + FString(": ");
	}
}

void FCameraBuildLog::SetForwardMessagesToLogging(bool bInForwardToLogging)
{
	bForwardToLogging = bInForwardToLogging;
}

void FCameraBuildLog::AddMessage(EMessageSeverity::Type InSeverity, FText&& InText)
{
	AddMessage(InSeverity, nullptr, MoveTemp(InText));
}

void FCameraBuildLog::AddMessage(EMessageSeverity::Type InSeverity, const UObject* InObject, FText&& InText)
{
	Messages.Add(FCameraBuildLogMessage{ InSeverity, InObject, MoveTemp(InText) });

	switch (InSeverity)
	{
		case EMessageSeverity::Warning:
		case EMessageSeverity::PerformanceWarning:
			bHasWarnings = true;
			break;
		case EMessageSeverity::Error:
			bHasErrors = true;
			break;
		default:
			break;
	}

	if (bForwardToLogging)
	{
		const FCameraBuildLogMessage& LastMessage = Messages.Last();
		LastMessage.SendToLogging(LoggingPrefix);
	}
}

void FCameraBuildLog::ResetMessages()
{
	Messages.Reset();
	bHasWarnings = false;
	bHasErrors = false;
}

}  // namespace UE::Cameras

