// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraNodeHierarchyBuilder.h"

#include "Build/CameraObjectBuildContext.h"
#include "Core/BaseCameraObject.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluatorStorage.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/ICustomCameraNodeParameterProvider.h"

#define LOCTEXT_NAMESPACE "CameraNodeHierarchyBuilder"

namespace UE::Cameras
{

namespace Internal
{

void AddVariableToAllocationInfo(UCameraVariableAsset* Variable, FCameraVariableTableAllocationInfo& AllocationInfo)
{
	if (Variable)
	{
		FCameraVariableDefinition VariableDefinition = Variable->GetVariableDefinition();
		AllocationInfo.VariableDefinitions.Add(VariableDefinition);
	}
}

void AddVariableToAllocationInfo(FCameraVariableID VariableID, ECameraVariableType VariableType, const UScriptStruct* BlendableStructType, FCameraVariableTableAllocationInfo& AllocationInfo)
{
	if (VariableID)
	{
		FCameraVariableDefinition VariableDefinition;
		VariableDefinition.VariableID = VariableID;
		VariableDefinition.VariableType = VariableType;
		VariableDefinition.BlendableStructType = BlendableStructType;
		VariableDefinition.bIsPrivate = true;
		VariableDefinition.bIsInput = true;
		AllocationInfo.VariableDefinitions.Add(VariableDefinition);
	}
}

}  // namespace Internal

FCameraNodeHierarchyBuilder::FCameraNodeHierarchyBuilder(FCameraBuildContext& InBuildContext, UBaseCameraObject* InCameraObject)
	: BuildContext(InBuildContext)
	, CameraObject(InCameraObject)
{
	CameraNodeHierarchy.Build(CameraObject);
}

void FCameraNodeHierarchyBuilder::PreBuild()
{
	for (UCameraNode* CameraNode : CameraNodeHierarchy.GetFlattenedHierarchy())
	{
		CameraNode->PreBuild(BuildContext);
	}
}

void FCameraNodeHierarchyBuilder::Build()
{
	FCameraObjectBuildContext ObjectBuildContext(BuildContext);

	// Get the size of the evaluators' allocation when cooking. When not cooking, we ignore it because
	// we don't want to have to re-save assets because someone added a new C++ field to an evaluator.
	if (BuildContext.IsCooking())
	{
		// To get the information we need, we build a mock tree of evaluators and see how much room
		// that takes.
		FCameraNodeEvaluatorTreeBuildParams BuildParams;
		BuildParams.RootCameraNode = CameraObject->GetRootNode();
		FCameraNodeEvaluatorStorage Storage;
		Storage.BuildEvaluatorTree(BuildParams);

		Storage.GetAllocationInfo(ObjectBuildContext.AllocationInfo.EvaluatorInfo);
	}

	// Call Build() on all camera nodes in the hierarchy (detached/orphaned camera nodes don't get called).
	for (UCameraNode* CameraNode : CameraNodeHierarchy.GetFlattenedHierarchy())
	{
		CallBuild(ObjectBuildContext, CameraNode);
	}

	// Add parameters to the allocation info.
	BuildParametersAllocationInfo(ObjectBuildContext);

	// Set the final allocation info on the camera rig asset.
	if (CameraObject->AllocationInfo != ObjectBuildContext.AllocationInfo)
	{
		// Previously we would save the evaluator allocation info, and any referenced camera rigs' infos.
		// Now not anymore (see above, and see UCameraRigCameraNode::OnBuild... we only do this when cooking now). 
		// However, we don't want to make all camera rigs modified because this allocation info is now empty or 
		// something.  So we do this extra check.
		bool bShouldModify = true;
		if (!BuildContext.IsCooking())
		{
			bShouldModify = false;
			const FCameraObjectAllocationInfo& CurInfo = CameraObject->AllocationInfo;
			const FCameraObjectAllocationInfo& NewInfo = ObjectBuildContext.AllocationInfo;
			if (!CurInfo.VariableTableInfo.Contains(NewInfo.VariableTableInfo) ||
					!CurInfo.ContextDataTableInfo.Contains(NewInfo.ContextDataTableInfo))
			{
				bShouldModify = true;
			}
		}

		if (bShouldModify)
		{
			CameraObject->Modify();
			CameraObject->AllocationInfo = ObjectBuildContext.AllocationInfo;
		}
	}
}

void FCameraNodeHierarchyBuilder::CallBuild(FCameraObjectBuildContext& ObjectBuildContext, UCameraNode* CameraNode)
{
	using namespace UE::Cameras::Internal;

	// Look for properties that are camera parameters, and gather what camera variables they reference. 
	// This is only for user-defined variable overrides. We will do the same for exposed camera rig
	// parameters later, in BuildParametersAllocationInfo.
	UClass* CameraNodeClass = CameraNode->GetClass();
	FCameraObjectAllocationInfo& AllocationInfo = ObjectBuildContext.AllocationInfo;
	for (TFieldIterator<FProperty> It(CameraNodeClass); It; ++It)
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(*It);
		if (!StructProperty)
		{
			continue;
		}

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		if (StructProperty->Struct == F##ValueName##CameraParameter::StaticStruct())\
		{\
			auto* CameraParameterPtr = StructProperty->ContainerPtrToValuePtr<F##ValueName##CameraParameter>(CameraNode);\
			AddVariableToAllocationInfo(CameraParameterPtr->Variable, AllocationInfo.VariableTableInfo);\
		}\
		else if (StructProperty->Struct == F##ValueName##CameraVariableReference::StaticStruct())\
		{\
			auto* CameraVariableReferencePtr = StructProperty->ContainerPtrToValuePtr<F##ValueName##CameraVariableReference>(CameraNode);\
			AddVariableToAllocationInfo(CameraVariableReferencePtr->Variable, AllocationInfo.VariableTableInfo);\
		}\
		else
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
		{
			// Another struct property...
		}
	}

	// Now do the same with custom parameters handled by the node itself.
	if (ICustomCameraNodeParameterProvider* CustomParameterProvider = Cast<ICustomCameraNodeParameterProvider>(CameraNode))
	{
		FCameraNodeParameterInfos CameraNodeParameters;
		CustomParameterProvider->GetCustomCameraNodeParameters(CameraNodeParameters);

		for (const FCameraNodeBlendableParameterInfo& BlendableParameter : CameraNodeParameters.BlendableParameters)
		{
			AddVariableToAllocationInfo(BlendableParameter.OverrideVariable, AllocationInfo.VariableTableInfo);
		}
	}

	// Let the camera node add any custom variables or extra memory.
	CameraNode->Build(ObjectBuildContext);
}

void FCameraNodeHierarchyBuilder::BuildParametersAllocationInfo(FCameraObjectBuildContext& ObjectBuildContext)
{
	// The variables and context data definitions should have already been added by the camera nodes
	// who have override variable IDs and data IDs set on them. 

	for (const UCameraObjectInterfaceBlendableParameter* BlendableParameter : CameraObject->Interface.BlendableParameters)
	{
		if (!BlendableParameter->PrivateVariableID.IsValid())
		{
			continue;
		}

		FCameraVariableDefinition Definition = BlendableParameter->GetVariableDefinition();
		ObjectBuildContext.AllocationInfo.VariableTableInfo.VariableDefinitions.Add(Definition);
	}

	for (const UCameraObjectInterfaceDataParameter* DataParameter : CameraObject->Interface.DataParameters)
	{
		if (!DataParameter->PrivateDataID.IsValid())
		{
			continue;
		}

		FCameraContextDataDefinition Definition = DataParameter->GetDataDefinition();
		ObjectBuildContext.AllocationInfo.ContextDataTableInfo.DataDefinitions.Add(Definition);
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

