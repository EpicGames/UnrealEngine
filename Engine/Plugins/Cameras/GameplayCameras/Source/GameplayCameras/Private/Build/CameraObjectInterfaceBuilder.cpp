// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraObjectInterfaceBuilder.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeHierarchy.h"
#include "Core/CameraObjectInterface.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "GameplayCameras.h"

#define LOCTEXT_NAMESPACE "CameraObjectInterfaceBuilder"

namespace UE::Cameras
{

namespace Internal
{

struct FInterfaceParameterBindingBuilder
{
	UBaseCameraObject* CameraObject;
	TMap<const UCameraNode*, FCameraNodeParameterInfos> BuiltCameraNodeParameters;

	FInterfaceParameterBindingBuilder(FCameraObjectInterfaceBuilder& InOwner)
		: Owner(InOwner)
	{
		CameraObject = Owner.CameraObject;
	}

	void ReportError(FText&& ErrorMessage)
	{
		ReportError(nullptr, MoveTemp(ErrorMessage));
	}

	void ReportError(UObject* Object, FText&& ErrorMessage)
	{
		Owner.BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, MoveTemp(ErrorMessage));
	}

	void SetParameterOverride(
			UCameraObjectInterfaceBlendableParameter* SourceParameter,
			UCameraNode* TargetNode,
			FName TargetNodeParameterName)
	{
		const FCameraNodeBlendableParameterInfo* TargetParameterInfo = FindBlendableParameter(TargetNode, TargetNodeParameterName);
		if (!TargetParameterInfo)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"InvalidConnectionTargetProperty",
							"Invalid connection to property '{0}' on '{1}', no such property found."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		if (TargetParameterInfo->VariableType != SourceParameter->VariableType ||
				TargetParameterInfo->BlendableStructType != SourceParameter->BlendableStructType)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"ConnectionVariableTypeMismatch",
							"Invalid connection to property '{0}' on '{1}', expected type '{2}' but is '{3}'."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName()),
						UEnum::GetDisplayValueAsText(SourceParameter->VariableType),
						UEnum::GetDisplayValueAsText(TargetParameterInfo->VariableType)));
			return;
		}

		if (!TargetParameterInfo->OverrideVariableID)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"InvalidConnectionTargetVariableID",
							"Can't connect to property '{0}' on '{1}', no camera variable ID storage available."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		if (TargetParameterInfo->OverrideVariableID->IsValid())
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"ConnectionTargetVariableIDAlreadyUsed",
							"Can't connect to property '{0}' on '{1}', it is already connected with something else."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		ensure(SourceParameter->PrivateVariableID.IsValid());
		FCameraVariableID PreviousVariableID = FindOldDrivingVariableID(TargetNode, TargetNodeParameterName);
		if (PreviousVariableID != SourceParameter->PrivateVariableID)
		{
			TargetNode->Modify();
		}
		*TargetParameterInfo->OverrideVariableID = SourceParameter->PrivateVariableID;
	}

	void SetParameterOverride(
			UCameraObjectInterfaceDataParameter* SourceParameter,
			UCameraNode* TargetNode,
			FName TargetNodeParameterName)
	{
		const FCameraNodeDataParameterInfo* TargetParameterInfo = FindDataParameter(TargetNode, TargetNodeParameterName);
		if (!TargetParameterInfo)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"InvalidConnectionTargetProperty",
							"Invalid connection to property '{0}' on '{1}', no such property found."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		if (TargetParameterInfo->DataType != SourceParameter->DataType ||
				TargetParameterInfo->DataContainerType != SourceParameter->DataContainerType ||
				TargetParameterInfo->DataTypeObject != SourceParameter->DataTypeObject)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"ConnectionDataTypeMismatch",
							"Invalid connection to property '{0}' on '{1}', expected type '{2}' but is '{3}'."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName()),
						UEnum::GetDisplayValueAsText(SourceParameter->DataType),
						UEnum::GetDisplayValueAsText(TargetParameterInfo->DataType)));
			return;
		}

		if (!TargetParameterInfo->OverrideDataID)
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"InvalidConnectionTargetDataID",
							"Can't connect to property '{0}' on '{1}', no camera data ID storage available."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		if (TargetParameterInfo->OverrideDataID->IsValid())
		{
			ReportError(
					TargetNode,
					FText::Format(LOCTEXT(
							"ConnectionTargetDataIDAlreadyUsed",
							"Can't connect to property '{0}' on '{1}', it is already connected with something else."),
						FText::FromName(TargetNodeParameterName),
						FText::FromName(TargetNode->GetFName())));
			return;
		}

		ensure(SourceParameter->PrivateDataID.IsValid());
		FCameraContextDataID PreviousDataID = FindOldDrivingDataID(TargetNode, TargetNodeParameterName);
		if (PreviousDataID != SourceParameter->PrivateDataID)
		{
			TargetNode->Modify();
		}
		*TargetParameterInfo->OverrideDataID = SourceParameter->PrivateDataID;
	}

private:

	const FCameraNodeParameterInfos& GetCameraNodeParameterInfos(UCameraNode* CameraNode)
	{
		const FCameraNodeParameterInfos* CameraNodeParameters = BuiltCameraNodeParameters.Find(CameraNode);
		if (CameraNodeParameters)
		{
			return *CameraNodeParameters;
		}
		
		FCameraNodeParameterInfos& NewCameraNodeParameters = BuiltCameraNodeParameters.Add(CameraNode);
		NewCameraNodeParameters.BuildFrom(CameraNode);
		return NewCameraNodeParameters;
	}

	const FCameraNodeBlendableParameterInfo* FindBlendableParameter(UCameraNode* CameraNode, FName ParameterName)
	{
		const FCameraNodeParameterInfos& CameraNodeParameters = GetCameraNodeParameterInfos(CameraNode);
		return CameraNodeParameters.FindBlendableParameter(ParameterName);
	}

	const FCameraNodeDataParameterInfo* FindDataParameter(UCameraNode* CameraNode, FName ParameterName)
	{
		const FCameraNodeParameterInfos& CameraNodeParameters = GetCameraNodeParameterInfos(CameraNode);
		return CameraNodeParameters.FindDataParameter(ParameterName);
	}

private:

	FCameraVariableID FindOldDrivingVariableID(UObject* ForObject, FName ForParameterName)
	{
		using FDrivenParameterKey = FCameraObjectInterfaceBuilder::FDrivenParameterKey;

		FDrivenParameterKey ParameterKey{ ForObject, ForParameterName };
		FCameraVariableID ReusedVariableID;
		Owner.OldDrivenBlendableParameters.RemoveAndCopyValue(ParameterKey, ReusedVariableID);
		return ReusedVariableID;
	}

	FCameraContextDataID FindOldDrivingDataID(UObject* ForObject, FName ForParameterName)
	{
		using FDrivenParameterKey = FCameraObjectInterfaceBuilder::FDrivenParameterKey;

		FDrivenParameterKey ParameterKey{ ForObject, ForParameterName };
		FCameraContextDataID ReusedDataID;
		Owner.OldDrivenDataParameters.RemoveAndCopyValue(ParameterKey, ReusedDataID);
		return ReusedDataID;
	}

private:

	FCameraObjectInterfaceBuilder& Owner;
};

}  // namespace Internal

FCameraObjectInterfaceBuilder::FCameraObjectInterfaceBuilder(FCameraBuildContext& InBuildContext)
	: BuildContext(InBuildContext)
{
}

void FCameraObjectInterfaceBuilder::BuildInterface(UBaseCameraObject* InCameraObject, const FCameraNodeHierarchy& InHierarchy, bool bCollectStrayNodes)
{
	TSet<UCameraNode*> CameraNodesToGather(InHierarchy.GetFlattenedHierarchy());

	if (bCollectStrayNodes)
	{
		// Get the list of nodes, both connected and disconnected from the root hierarchy.
		// We could use AllNodeTreeObjects for that, but it only exists in editor builds, and we 
		// don't want to rely on unit tests or runtime data manipulation to have correctly populated 
		// it, so we'll try to gather any stray nodes by looking at objects outer'ed to the camera rig.
		ForEachObjectWithOuter(InCameraObject, [&CameraNodesToGather](UObject* Obj)
				{
					if (UCameraNode* CameraNode = Cast<UCameraNode>(Obj))
					{
						CameraNodesToGather.Add(CameraNode);
					}
				});
		const int32 NumStrayCameraNodes = (CameraNodesToGather.Num() - InHierarchy.Num());
		if (NumStrayCameraNodes > 0)
		{
			UE_LOGF(LogCameraSystem, Verbose, "Collected %d stray camera nodes while building camera rig '%ls'.",
					NumStrayCameraNodes, *GetPathNameSafe(CameraObject));
		}
	}

	BuildInterface(InCameraObject, CameraNodesToGather.Array());
}

void FCameraObjectInterfaceBuilder::BuildInterface(UBaseCameraObject* InCameraObject, TArrayView<UCameraNode*> InCameraObjectNodes)
{
	if (!ensure(InCameraObject))
	{
		return;
	}

	CameraObject = InCameraObject;
	CameraObjectNodes = InCameraObjectNodes;
	{
		BuildInterfaceImpl();
	}
	CameraObject = nullptr;
	CameraObjectNodes.Reset();
}

void FCameraObjectInterfaceBuilder::BuildInterfaceImpl()
{
	GatherOldDrivenParameters();
	BuildInterfaceParameters();
	BuildInterfaceParameterBindings();
	DiscardUnusedParameters();
}

void FCameraObjectInterfaceBuilder::GatherOldDrivenParameters()
{
	// Keep track of which blendable/data parameters were previously overriden with private IDs.
	// Then clear those private IDs. This is because it's easier to rebuild all this from a blank 
	// slate than trying to figure out what changed.
	//
	// As we rebuild things in BuildInterfaceParameterBindings, we compare to the old state to
	// figure out if we need to flag anything as modified for the current transaction.
	//
	// Note that parameters driven by user-defined variables are left alone.

	OldDrivenBlendableParameters.Reset();
	OldDrivenDataParameters.Reset();

	FCameraNodeParameterInfos CameraNodeParameters;

	for (UCameraNode* CameraNode : CameraObjectNodes)
	{
		CameraNodeParameters.BuildFrom(CameraNode);

		for (const FCameraNodeBlendableParameterInfo& BlendableParameter : CameraNodeParameters.GetBlendableParameters())
		{
			if (BlendableParameter.OverrideVariableID &&
					BlendableParameter.OverrideVariableID->IsValid() && 
					// Don't touch user-defined variable overrides.
					!BlendableParameter.OverrideVariable)
			{
				OldDrivenBlendableParameters.Add(
						FDrivenParameterKey{ CameraNode, BlendableParameter.ParameterName },
						*BlendableParameter.OverrideVariableID);
				*BlendableParameter.OverrideVariableID = FCameraVariableID();
			}
		}

		for (const FCameraNodeDataParameterInfo& DataParameter : CameraNodeParameters.GetDataParameters())
		{
			if (DataParameter.OverrideDataID && 
					DataParameter.OverrideDataID->IsValid())
			{
				OldDrivenDataParameters.Add(
						FDrivenParameterKey{ CameraNode, DataParameter.ParameterName },
						*DataParameter.OverrideDataID);
				*DataParameter.OverrideDataID = FCameraContextDataID();
			}
		}
	}
}

void FCameraObjectInterfaceBuilder::BuildInterfaceParameters()
{
	// Here we simply validate all blendable/data interface parameters and create IDs for their entries in the
	// variable and context data tables.

	using namespace Internal;

	for (auto It = CameraObject->Interface.BlendableParameters.CreateIterator(); It; ++It)
	{
		UCameraObjectInterfaceBlendableParameter* BlendableParameter(*It);

		// Basic validations.
		if (!BlendableParameter)
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Warning,
					CameraObject,
					LOCTEXT("InvalidBlendableParameter", "Invalid interface parameter was found and removed."));

			CameraObject->Modify();
			It.RemoveCurrent();

			continue;
		}

		if (BlendableParameter->InterfaceParameterName.IsEmpty())
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					BlendableParameter,
					LOCTEXT(
						"InvalidBlendableParameterName",
						"Invalid interface parameter name."));
			continue;
		}

		// Create a new private variable ID for this interface parameter. Flag the parameter as changed if
		// the ID is different, generally when it's a new parameter.
		FCameraVariableID VariableID = FCameraVariableID::FromHashValue(GetTypeHash(BlendableParameter->GetGuid()));
		if (BlendableParameter->PrivateVariableID != VariableID)
		{
			BlendableParameter->Modify();
			BlendableParameter->PrivateVariableID = VariableID;
		}
	}

	for (auto It = CameraObject->Interface.DataParameters.CreateIterator(); It; ++It)
	{
		UCameraObjectInterfaceDataParameter* DataParameter(*It);

		// Basic validations.
		if (!DataParameter)
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Warning,
					CameraObject,
					LOCTEXT("InvalidDataParameter", "Invalid interface parameter was found and removed."));

			CameraObject->Modify();
			It.RemoveCurrent();

			continue;
		}

		if (DataParameter->InterfaceParameterName.IsEmpty())
		{
			BuildContext.BuildLog.AddMessage(EMessageSeverity::Error,
					DataParameter,
					LOCTEXT(
						"InvalidDataParameterName",
						"Invalid interface parameter name."));
			continue;
		}

		// Create a new private data ID for this interface parameter. Flag the parameter as changed if
		// the ID is different, generally when it's a new parameter.
		FCameraContextDataID DataID = FCameraContextDataID::FromHashValue(GetTypeHash(DataParameter->GetGuid()));
		if (DataParameter->PrivateDataID != DataID)
		{
			DataParameter->Modify();
			DataParameter->PrivateDataID = DataID;
		}
	}
}

void FCameraObjectInterfaceBuilder::BuildInterfaceParameterBindings()
{
	// Now we connect the interface parameters to whatever node property they are supposed to drive.
	// Each time we need to check for either a custom property (via ICustomCameraNodeParameterProvider),
	// or a UObject property found with reflection.

	const FString CameraObjectName = CameraObject->GetName();
	const FString CameraObjectPathName = CameraObject->GetPathName();

	Internal::FInterfaceParameterBindingBuilder Builder(*this);

	for (const FCameraObjectConnection& Connection : CameraObject->Connections.Connections)
	{
		if (Connection.Source == nullptr || Connection.Target == nullptr)
		{
			continue;
		}

		// For now limit ourselves to connections from interface parameters to camera nodes.
		const UCameraObjectInterfaceParameterGetter* ParameterGetter = Cast<UCameraObjectInterfaceParameterGetter>(Connection.Source);
		if (!ParameterGetter || !ParameterGetter->ParameterGuid.IsValid())
		{
			continue;
		}

		UCameraNode* TargetNode = Cast<UCameraNode>(Connection.Target);
		if (!TargetNode)
		{
			continue;
		}

		if (UCameraObjectInterfaceBlendableParameter* BlendableParameter = CameraObject->Interface.FindBlendableParameterByGuid(ParameterGetter->ParameterGuid))
		{
			Builder.SetParameterOverride(BlendableParameter, TargetNode, Connection.TargetPropertyName);
		}
		else if (UCameraObjectInterfaceDataParameter* DataParameter = CameraObject->Interface.FindDataParameterByGuid(ParameterGetter->ParameterGuid))
		{
			Builder.SetParameterOverride(DataParameter, TargetNode, Connection.TargetPropertyName);
		}
	}
}

void FCameraObjectInterfaceBuilder::DiscardUnusedParameters()
{
	// Now that we've rebuilt all exposed parameters, anything left from the old list 
	// must be discarded. These are nodes and properties that used to be driven by
	// variables and now aren't, so we need to flag them as modified.

	for (TPair<FDrivenParameterKey, FCameraVariableID> Pair : OldDrivenBlendableParameters)
	{
		UObject* Target = Pair.Key.Key;
		Target->Modify();
	}
	OldDrivenBlendableParameters.Reset();

	for (TPair<FDrivenParameterKey, FCameraContextDataID> Pair : OldDrivenDataParameters)
	{
		UObject* Target = Pair.Key.Key;
		Target->Modify();
	}
	OldDrivenDataParameters.Reset();
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

