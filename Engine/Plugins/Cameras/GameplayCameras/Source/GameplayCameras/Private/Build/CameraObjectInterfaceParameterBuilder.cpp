// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraObjectInterfaceParameterBuilder.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraNode.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraParameters.h"  // IWYU pragma: keep
#include "Core/CameraVariableReferences.h"  // IWYU pragma: keep
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "Misc/EngineVersionComparison.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/UnrealType.h"

namespace UE::Cameras
{

FCameraObjectInterfaceParameterBuilder::FCameraObjectInterfaceParameterBuilder()
{
}

void FCameraObjectInterfaceParameterBuilder::BuildParameters(UBaseCameraObject* InCameraObject)
{
	CameraObject = InCameraObject;
	{
		BuildParametersImpl();
	}
	CameraObject = nullptr;
}

void FCameraObjectInterfaceParameterBuilder::BuildParametersImpl()
{
	BuildParameterDefinitions();
	BuildDefaultParameters();
}

void FCameraObjectInterfaceParameterBuilder::BuildParameterDefinitions()
{
	TArray<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions;

	for (const UCameraObjectInterfaceBlendableParameter* BlendableParameter : CameraObject->Interface.BlendableParameters)
	{
		if (BlendableParameter && BlendableParameter->PrivateVariableID)
		{
			FCameraObjectInterfaceParameterDefinition Definition;
			BlendableParameter->GetParameterDefinition(Definition);
			ParameterDefinitions.Add(Definition);
		}
	}

	for (const UCameraObjectInterfaceDataParameter* DataParameter : CameraObject->Interface.DataParameters)
	{
		if (DataParameter && DataParameter->PrivateDataID)
		{
			FCameraObjectInterfaceParameterDefinition Definition;
			DataParameter->GetParameterDefinition(Definition);
			ParameterDefinitions.Add(Definition);
		}
	}

	if (ParameterDefinitions != CameraObject->ParameterDefinitions)
	{
		CameraObject->Modify();
		CameraObject->ParameterDefinitions = ParameterDefinitions;
	}
}

void FCameraObjectInterfaceParameterBuilder::BuildDefaultParameters()
{
	TArray<FPropertyBagPropertyDesc> DefaultParameterProperties;
	AppendDefaultParameterProperties(CameraObject, DefaultParameterProperties);
	const UPropertyBag* DefaultParametersStruct = UPropertyBag::GetOrCreateFromDescs(DefaultParameterProperties);
	if (CameraObject->DefaultParameters.GetPropertyBagStruct() != DefaultParametersStruct)
	{
		CameraObject->Modify();
		// In theory, the default values were set when the editor called SetDefaultParameterValue, which happens when the user
		// creates a new parameter or changes that parameter's default value in the Details View. So the DefaultParameters
		// structure should have the correct values already and we only need to migrate them to the new struct if anything
		// changed.
		CameraObject->DefaultParameters.MigrateToNewBagStruct(DefaultParametersStruct);
	}

}

void FCameraObjectInterfaceParameterBuilder::AppendDefaultParameterProperties(const UBaseCameraObject* CameraObject, TArray<FPropertyBagPropertyDesc>& OutProperties)
{
	AppendDefaultParameterProperties(CameraObject->GetParameterDefinitions(), OutProperties);
}

void FCameraObjectInterfaceParameterBuilder::AppendDefaultParameterProperties(TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions, TArray<FPropertyBagPropertyDesc>& OutProperties)
{
	for (const FCameraObjectInterfaceParameterDefinition& Definition : ParameterDefinitions)
	{
		bool bIsValidProperty = true;
		EPropertyBagPropertyType PropertyType = EPropertyBagPropertyType::Struct;
		EPropertyBagContainerType ContainerType = EPropertyBagContainerType::None;
		const UObject* PropertyTypeObject = nullptr;
		EPropertyFlags PropertyFlags = CPF_None;

		if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
		{
			switch (Definition.VariableType)
			{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
				case ECameraVariableType::ValueName:\
					PropertyTypeObject = F##ValueName##CameraParameter::StaticStruct();\
					break;
				UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
				case ECameraVariableType::BlendableStruct:
					PropertyTypeObject = Definition.BlendableStructType;
					PropertyFlags = CPF_Interp;
					break;
				default:
					ensure(false);
					break;
			}
		}
		else if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Data)
		{
			PropertyTypeObject = Definition.DataTypeObject;

			switch (Definition.DataType)
			{
				case ECameraContextDataType::Name:
					PropertyType = EPropertyBagPropertyType::Name;
					PropertyFlags = CPF_Interp;
					break;
				case ECameraContextDataType::String:
					PropertyType = EPropertyBagPropertyType::String;
					PropertyFlags = CPF_Interp;
					break;
				case ECameraContextDataType::Enum:
					PropertyType = EPropertyBagPropertyType::Enum;
					ensure(PropertyTypeObject && PropertyTypeObject->IsA<UEnum>());
					PropertyFlags = CPF_Interp;
					break;
				case ECameraContextDataType::Struct:
					PropertyType = EPropertyBagPropertyType::Struct;
					ensure(PropertyTypeObject && PropertyTypeObject->IsA<UScriptStruct>());
					break;
				case ECameraContextDataType::Object:
					PropertyType = EPropertyBagPropertyType::Object;
					PropertyFlags = CPF_Interp;
					break;
				case ECameraContextDataType::Class:
					PropertyType = EPropertyBagPropertyType::Class;
					PropertyFlags = CPF_Interp;
					break;
				default:
					bIsValidProperty = false;
					break;
			}

			switch (Definition.DataContainerType)
			{
				case ECameraContextDataContainerType::Array:
					ContainerType = EPropertyBagContainerType::Array;
					break;
			}
		}
		else
		{
			bIsValidProperty = false;
		}

		if (ensure(bIsValidProperty))
		{
			FPropertyBagPropertyDesc NewProperty(Definition.ParameterName, ContainerType, PropertyType, PropertyTypeObject);
			// Make the property bag match the camera interface parameter GUIDs.
			NewProperty.ID = Definition.ParameterGuid;
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
			NewProperty.PropertyFlags |= PropertyFlags;
			if (!Definition.bIsVisible)
			{
				NewProperty.PropertyFlags &= ~CPF_Edit;
			}
#endif

			OutProperties.Add(NewProperty);
		}
	}
}

bool FCameraObjectInterfaceParameterBuilder::SetDefaultParameterValue(UBaseCameraObject* CameraObject, const FCameraObjectInterfaceParameterDefinition& ParameterDefinition, UCameraNode* TargetNode, FName TargetPropertyName, bool bAddParameterIfMissing)
{
	if (!ensure(CameraObject && TargetNode && TargetPropertyName != NAME_None))
	{
		return false;
	}

	// Start by finding the source value, i.e. the value of the camera node property being exposed as a parameter.
	const void* RawSourceValuePtr = nullptr;

	// First check if the value is found on a custom parameter.
	if (ICustomCameraNodeParameterProvider* CustomParameterProvider = Cast<ICustomCameraNodeParameterProvider>(TargetNode))
	{
		FCameraNodeParameterInfos CustomParameters;
		CustomParameterProvider->GetCustomCameraNodeParameters(CustomParameters);

		FCameraNodeBlendableParameterInfo* CustomBlendableParameter = 
			CustomParameters.BlendableParameters.FindByPredicate(
					[TargetPropertyName](FCameraNodeBlendableParameterInfo& CustomParameter)
					{
						return CustomParameter.ParameterName == TargetPropertyName;
					});
		if (CustomBlendableParameter)
		{
			RawSourceValuePtr = CustomBlendableParameter->DefaultValue;
			goto DoneSearchingRawSourceValuePtr;
		}

		FCameraNodeDataParameterInfo* CustomDataParameter =
			CustomParameters.DataParameters.FindByPredicate(
					[TargetPropertyName](FCameraNodeDataParameterInfo& CustomParameter)
					{
						return CustomParameter.ParameterName == TargetPropertyName;
					});
		if (CustomDataParameter)
		{
			RawSourceValuePtr = CustomDataParameter->DefaultValue;
			goto DoneSearchingRawSourceValuePtr;
		}
	}

	// If not found, check on a reflected UObject property.
	// Blendable parameters would be hooked to a camera parameter struct, or a blendable struct.
	// Data parameters could be hooked up to anything.
	if (!RawSourceValuePtr)
	{
		const UClass* TargetClass = TargetNode->GetClass();
		FProperty* TargetProperty = TargetClass->FindPropertyByName(TargetPropertyName);
		FStructProperty* StructProperty = CastField<FStructProperty>(TargetProperty);
		if (StructProperty && ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
		{
			switch (ParameterDefinition.VariableType)
			{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
				case ECameraVariableType::ValueName:\
					{\
						using CameraParameterType = F##ValueName##CameraParameter;\
						using CameraVariableReferenceType = F##ValueName##CameraVariableReference;\
						if (StructProperty->Struct == CameraParameterType::StaticStruct())\
						{\
							CameraParameterType* CameraParameterPtr = StructProperty->ContainerPtrToValuePtr<CameraParameterType>(TargetNode);\
							RawSourceValuePtr = static_cast<void*>(&CameraParameterPtr->Value);\
						}\
						else if (StructProperty->Struct == CameraVariableReferenceType::StaticStruct())\
						{\
							CameraVariableReferenceType* VariableReferencePtr = StructProperty->ContainerPtrToValuePtr<CameraVariableReferenceType>(TargetNode);\
							RawSourceValuePtr = VariableReferencePtr->Variable ? VariableReferencePtr->Variable->GetDefaultValuePtr() : nullptr;\
						}\
					}\
					break;
				UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
				case ECameraVariableType::BlendableStruct:
					{
						RawSourceValuePtr = StructProperty->ContainerPtrToValuePtr<void>(TargetNode);
					}
					break;
			}
		}
		else if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Data)
		{
			RawSourceValuePtr = TargetProperty->ContainerPtrToValuePtr<void>(TargetNode);
		}
	}

DoneSearchingRawSourceValuePtr:

	if (!RawSourceValuePtr)
	{
		return false;
	}

	// Second, find the corresponding property on the default parameters' property bag. We will write the source 
	// value to this property. If we don't find it, either bail out, or add it to the property bag structure and 
	// continue.
	FInstancedPropertyBag& DefaultParameters = CameraObject->DefaultParameters;
	const UPropertyBag* PropertyBagStruct = DefaultParameters.GetPropertyBagStruct();
	if (!PropertyBagStruct)
	{
		if (!bAddParameterIfMissing)
		{
			return false;
		}
		// else, we will (re)create the property bag struct with the new property below.
	}

	const FPropertyBagPropertyDesc* PropertyDesc = PropertyBagStruct ? 
		PropertyBagStruct->FindPropertyDescByID(ParameterDefinition.ParameterGuid) :
		nullptr;
	if (!PropertyDesc)
	{
		if (!bAddParameterIfMissing)
		{
			return false;
		}

		TArray<FPropertyBagPropertyDesc> NewProperties;
		TConstArrayView<FCameraObjectInterfaceParameterDefinition> NewParameterDefinitions = MakeConstArrayView(&ParameterDefinition, 1);
		AppendDefaultParameterProperties(NewParameterDefinitions, NewProperties);
		const EPropertyBagAlterationResult Result = DefaultParameters.AddProperties(NewProperties, false);
		if (!ensure(Result == EPropertyBagAlterationResult::Success))
		{
			return false;
		}

		// Re-query the struct, since it changed when we added the property.
		PropertyBagStruct = DefaultParameters.GetPropertyBagStruct();
		PropertyDesc = PropertyBagStruct->FindPropertyDescByID(ParameterDefinition.ParameterGuid);
	}

	if (!ensure(PropertyDesc && PropertyDesc->CachedProperty))
	{
		return false;
	}

	uint8* PropertyBagValue = DefaultParameters.GetMutableValue().GetMemory();
	if (!ensure(PropertyBagValue && PropertyBagStruct))
	{
		return false;
	}

	// The given property should be a structure for blendable parameters (either a camera parameter or a blendable 
	// structure). For data parameters, it can be anything we support. Now do the value copying!
	bool bSuccess = true;
	if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
	{
		const FStructProperty* DefaultParameterProperty = CastField<FStructProperty>(PropertyDesc->CachedProperty);
		if (!ensure(DefaultParameterProperty))
		{
			return false;
		}

		switch (ParameterDefinition.VariableType)
		{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			case ECameraVariableType::ValueName:\
				{\
					using CameraParameterType = F##ValueName##CameraParameter;\
					if (ensure(DefaultParameterProperty->Struct == CameraParameterType::StaticStruct()))\
					{\
						const ValueType* SourceValuePtr = reinterpret_cast<const ValueType*>(RawSourceValuePtr);\
						CameraParameterType* DestinationParameter = DefaultParameterProperty->ContainerPtrToValuePtr<CameraParameterType>(PropertyBagValue);\
						DestinationParameter->Value = *SourceValuePtr;\
					}\
				}\
				break;
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			case ECameraVariableType::BlendableStruct:
				if (ensure(DefaultParameterProperty->Struct == ParameterDefinition.BlendableStructType))
				{
					void* RawDestinationValuePtr = DefaultParameterProperty->ContainerPtrToValuePtr<void>(PropertyBagValue);
					ParameterDefinition.BlendableStructType->CopyScriptStruct(RawDestinationValuePtr, RawSourceValuePtr);
				}
				break;
			default:
				ensureMsgf(false, TEXT("Unsupported or unknown variable type!"));
				bSuccess = false;
				break;
		}
	}
	else if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Data)
	{
		if (ParameterDefinition.DataContainerType == ECameraContextDataContainerType::None)
		{
			void* RawDestinationValuePtr = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(PropertyBagValue);
			if (!ensure(RawDestinationValuePtr))
			{
				return false;
			}
			
			switch (ParameterDefinition.DataType)
			{
				case ECameraContextDataType::Name:
					*((FName*)RawDestinationValuePtr) = *((FName*)RawSourceValuePtr);
					break;
				case ECameraContextDataType::String:
					*((FString*)RawDestinationValuePtr) = *((FString*)RawSourceValuePtr);
					break;
				case ECameraContextDataType::Enum:
					*((uint8*)RawDestinationValuePtr) = *((uint8*)RawSourceValuePtr);
					break;
				case ECameraContextDataType::Struct:
					{
						const UScriptStruct* StructType = CastChecked<const UScriptStruct>(ParameterDefinition.DataTypeObject);
						StructType->CopyScriptStruct(RawDestinationValuePtr, RawSourceValuePtr);
					}
					break;
				case ECameraContextDataType::Object:
					*((FObjectPtr*)RawDestinationValuePtr) = *((FObjectPtr*)RawSourceValuePtr);
					break;
				case ECameraContextDataType::Class:
					*((FObjectPtr*)RawDestinationValuePtr) = *((FObjectPtr*)RawSourceValuePtr);
					break;
				default:
					ensureMsgf(false, TEXT("Unsupported or unknown data type!"));
					bSuccess = false;
					break;
			}
		}
		else if (ParameterDefinition.DataContainerType == ECameraContextDataContainerType::Array)
		{
			// Array properties are empty by default.
		}
	}

	return bSuccess;
}

void FCameraObjectInterfaceParameterBuilder::FixUpDefaultParameterProperties(TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions, FInstancedPropertyBag& InOutPropertyBag)
{
	const UPropertyBag* PropertyBag = InOutPropertyBag.GetPropertyBagStruct();
	if (!PropertyBag)
	{
		return;
	}

	// Before UE 5.8, there was a bug with FInstancedPropertyBag not saving the PropertyFlags of its UPropertyBag
	// struct, resulting in camera parameters losing their CPF_Interp flag, or still having a CPF_Edit flag even
	// though they're supposed to not be visible/editable.
	//
	// This meant that rebuilding a camera asset would create a new parameter struct that was slightly different
	// from the loaded one (it had those flags). This made camera assets always dirty and in need of re-saving.
	//
	// This function aims to fix that as a band-aid: it gets called on PostLoad by camera assets and re-adds the
	// missing flags.

	bool bFixedAny = false;
	TArray<FPropertyBagPropertyDesc> FixedPropertyDescs(PropertyBag->GetPropertyDescs());

	for (const FCameraObjectInterfaceParameterDefinition& Definition : ParameterDefinitions)
	{
		FPropertyBagPropertyDesc* PropertyDesc = FixedPropertyDescs.FindByPredicate(
				[&Definition](FPropertyBagPropertyDesc& Item)
				{
					return Definition.ParameterGuid == Item.ID;
				});
		if (!PropertyDesc)
		{
			continue;
		}

		// Fixup blendable struct properties, and any data properties that aren't a struct.
		bool bFixPropertyDesc = false;
		if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
		{
			bFixPropertyDesc = (Definition.VariableType == ECameraVariableType::BlendableStruct);
		}
		else if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Data)
		{
			bFixPropertyDesc = (Definition.DataType != ECameraContextDataType::Struct);
		}
		if (bFixPropertyDesc)
		{
			bFixedAny |= ((PropertyDesc->PropertyFlags & CPF_Interp) == 0);
			PropertyDesc->PropertyFlags |= CPF_Interp;
		}

		// Remove CPF_Edit if the property isn't visible/editable.
		if (!Definition.bIsVisible)
		{
			bFixedAny = true;
			PropertyDesc->PropertyFlags &= ~CPF_Edit;
		}
	}

	if (bFixedAny)
	{
		const UPropertyBag* FixedPropertyBag = UPropertyBag::GetOrCreateFromDescs(FixedPropertyDescs);
		InOutPropertyBag.MigrateToNewBagStruct(FixedPropertyBag);
	}
}

}  // namespace UE::Cameras

