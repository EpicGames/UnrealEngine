// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraRigAssetBuilder.h"

#include "Build/CameraNodeHierarchyBuilder.h"
#include "Build/CameraObjectInterfaceBuilder.h"
#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/CameraRigAsset.h"
#include "GameplayCamerasDelegates.h"

#define LOCTEXT_NAMESPACE "CameraRigAssetBuilder"

namespace UE::Cameras
{

FCameraRigAssetBuilder::FCameraRigAssetBuilder(FCameraBuildContext& InBuildContext)
	: BuildContext(InBuildContext)
{
}

void FCameraRigAssetBuilder::BuildCameraRig(UCameraRigAsset* InCameraRig)
{
	if (!ensure(InCameraRig))
	{
		return;
	}

	CameraRig = InCameraRig;

	BuildContext.BuildLog.SetLoggingPrefix(CameraRig->GetPathName() + TEXT(": "));
	{
		BuildCameraRigImpl();

		CameraRig->EventHandlers.Notify(&ICameraRigAssetEventHandler::OnCameraRigBuilt, CameraRig);
	}
	BuildContext.BuildLog.SetLoggingPrefix(FString());
	UpdateBuildStatus();

	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().Broadcast(CameraRig);

	CameraRig = nullptr;
}

void FCameraRigAssetBuilder::BuildCameraRigImpl()
{
	FCameraNodeHierarchyBuilder NodeBuilder(BuildContext, CameraRig);
	NodeBuilder.PreBuild();

	FCameraObjectInterfaceBuilder InterfaceBuilder(BuildContext);
	InterfaceBuilder.BuildInterface(CameraRig, NodeBuilder.GetHierarchy(), true);

	NodeBuilder.Build();

	FCameraObjectInterfaceParameterBuilder ParameterBuilder;
	ParameterBuilder.BuildParameters(CameraRig);
}

void FCameraRigAssetBuilder::UpdateBuildStatus()
{
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Clean;
	if (BuildContext.BuildLog.HasErrors())
	{
		BuildStatus = ECameraBuildStatus::WithErrors;
	}
	else if (BuildContext.BuildLog.HasWarnings())
	{
		BuildStatus = ECameraBuildStatus::CleanWithWarnings;
	}

	// Don't modify the camera rig: BuildStatus is transient.
	CameraRig->BuildStatus = BuildStatus;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

