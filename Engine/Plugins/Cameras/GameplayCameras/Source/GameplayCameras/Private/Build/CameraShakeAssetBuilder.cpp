// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraShakeAssetBuilder.h"

#include "Build/CameraNodeHierarchyBuilder.h"
#include "Build/CameraObjectInterfaceBuilder.h"
#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/CameraShakeAsset.h"
#include "GameplayCamerasDelegates.h"

#define LOCTEXT_NAMESPACE "CameraShakeAssetBuilder"

namespace UE::Cameras
{

FCameraShakeAssetBuilder::FCameraShakeAssetBuilder(FCameraBuildContext& InBuildContext)
	: BuildContext(InBuildContext)
{
}

void FCameraShakeAssetBuilder::BuildCameraShake(UCameraShakeAsset* InCameraShake)
{
	if (!ensure(InCameraShake))
	{
		return;
	}

	CameraShake = InCameraShake;

	BuildContext.BuildLog.SetLoggingPrefix(CameraShake->GetPathName() + TEXT(": "));
	{
		BuildCameraShakeImpl();
	}
	BuildContext.BuildLog.SetLoggingPrefix(FString());
	UpdateBuildStatus();

	FGameplayCamerasDelegates::OnCameraShakeAssetBuilt().Broadcast(CameraShake);

	CameraShake = nullptr;
}

void FCameraShakeAssetBuilder::BuildCameraShakeImpl()
{
	FCameraNodeHierarchyBuilder NodeBuilder(BuildContext, CameraShake);
	NodeBuilder.PreBuild();

	FCameraObjectInterfaceBuilder InterfaceBuilder(BuildContext);
	InterfaceBuilder.BuildInterface(CameraShake, NodeBuilder.GetHierarchy(), true);

	NodeBuilder.Build();

	FCameraObjectInterfaceParameterBuilder ParameterBuilder;
	ParameterBuilder.BuildParameters(CameraShake);
}

void FCameraShakeAssetBuilder::UpdateBuildStatus()
{
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Clean;
	if (BuildContext.BuildLog.HasErrors())
	{
		BuildStatus = ECameraBuildStatus::WithErrors;
	}
	else if (BuildContext.BuildLog.HasWarnings())
	{
		BuildStatus = ECameraBuildStatus::CleanWithWarnings;
	}

	// Don't modify the camera shake: BuildStatus is transient.
	CameraShake->BuildStatus = BuildStatus;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

