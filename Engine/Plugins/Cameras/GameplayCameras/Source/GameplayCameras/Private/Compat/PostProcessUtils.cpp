// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compat/PostProcessUtils.h"

#if !UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

#include "Engine/Scene.h"

bool FPostProcessUtils::OverridePostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo)
{
	bool bAnyOverwritten = false;

#define UE_SET_PP(Name)\
	if (OtherTo.bOverride_##Name)\
	{\
		ThisFrom.bOverride_##Name = true;\
		ThisFrom.Name = OtherTo.Name;\
		bAnyOverwritten = true;\
	}

	{
		UE_SET_PP(TemperatureType);
		UE_SET_PP(WhiteTemp);
		UE_SET_PP(WhiteTint);

		UE_SET_PP(ColorSaturation);
		UE_SET_PP(ColorContrast);
		UE_SET_PP(ColorGamma);
		UE_SET_PP(ColorGain);
		UE_SET_PP(ColorOffset);

		UE_SET_PP(ColorSaturationShadows);
		UE_SET_PP(ColorContrastShadows);
		UE_SET_PP(ColorGammaShadows);
		UE_SET_PP(ColorGainShadows);
		UE_SET_PP(ColorOffsetShadows);

		UE_SET_PP(ColorSaturationMidtones);
		UE_SET_PP(ColorContrastMidtones);
		UE_SET_PP(ColorGammaMidtones);
		UE_SET_PP(ColorGainMidtones);
		UE_SET_PP(ColorOffsetMidtones);

		UE_SET_PP(ColorSaturationHighlights);
		UE_SET_PP(ColorContrastHighlights);
		UE_SET_PP(ColorGammaHighlights);
		UE_SET_PP(ColorGainHighlights);
		UE_SET_PP(ColorOffsetHighlights);

		UE_SET_PP(ColorCorrectionShadowsMax);
		UE_SET_PP(ColorCorrectionHighlightsMin);
		UE_SET_PP(ColorCorrectionHighlightsMax);

		UE_SET_PP(BlueCorrection);
		UE_SET_PP(ExpandGamut);
		UE_SET_PP(ToneCurveAmount);

		UE_SET_PP(FilmSlope);
		UE_SET_PP(FilmToe);
		UE_SET_PP(FilmShoulder);
		UE_SET_PP(FilmBlackClip);
		UE_SET_PP(FilmWhiteClip);

		UE_SET_PP(SceneColorTint);
		UE_SET_PP(SceneFringeIntensity);
		UE_SET_PP(ChromaticAberrationStartOffset);
		UE_SET_PP(BloomIntensity);
		UE_SET_PP(BloomGaussianIntensity);
		UE_SET_PP(BloomThreshold);
		UE_SET_PP(Bloom1Tint);
		UE_SET_PP(BloomSizeScale);
		UE_SET_PP(Bloom1Size);
		UE_SET_PP(Bloom2Tint);
		UE_SET_PP(Bloom2Size);
		UE_SET_PP(Bloom3Tint);
		UE_SET_PP(Bloom3Size);
		UE_SET_PP(Bloom4Tint);
		UE_SET_PP(Bloom4Size);
		UE_SET_PP(Bloom5Tint);
		UE_SET_PP(Bloom5Size);
		UE_SET_PP(Bloom6Tint);
		UE_SET_PP(Bloom6Size);
		UE_SET_PP(BloomDirtMaskIntensity);
		UE_SET_PP(BloomDirtMaskTint);
		UE_SET_PP(BloomConvolutionIntensity);
		UE_SET_PP(BloomConvolutionScatterDispersion);
		UE_SET_PP(BloomConvolutionSize);
		UE_SET_PP(BloomConvolutionCenterUV);
		UE_SET_PP(BloomConvolutionPreFilterMin);
		UE_SET_PP(BloomConvolutionPreFilterMax);
		UE_SET_PP(BloomConvolutionPreFilterMult);
		UE_SET_PP(AmbientCubemapIntensity);
		UE_SET_PP(AmbientCubemapTint);
		UE_SET_PP(CameraShutterSpeed);
		UE_SET_PP(CameraISO);
		UE_SET_PP(AutoExposureLowPercent);
		UE_SET_PP(AutoExposureHighPercent);
		UE_SET_PP(AutoExposureMinBrightness);
		UE_SET_PP(AutoExposureMaxBrightness);
		UE_SET_PP(AutoExposureSpeedUp);
		UE_SET_PP(AutoExposureSpeedDown);
		UE_SET_PP(AutoExposureBias);
		UE_SET_PP(HistogramLogMin);
		UE_SET_PP(HistogramLogMax);
		UE_SET_PP(LocalExposureMethod);
		UE_SET_PP(LocalExposureContrastScale_DEPRECATED);
		UE_SET_PP(LocalExposureHighlightContrastScale);
		UE_SET_PP(LocalExposureShadowContrastScale);
		UE_SET_PP(LocalExposureHighlightThreshold);
		UE_SET_PP(LocalExposureShadowThreshold);
		UE_SET_PP(LocalExposureDetailStrength);
		UE_SET_PP(LocalExposureBlurredLuminanceBlend);
		UE_SET_PP(LocalExposureBlurredLuminanceKernelSizePercent);
		UE_SET_PP(LocalExposureMiddleGreyBias);
		UE_SET_PP(LensFlareIntensity);
		UE_SET_PP(LensFlareTint);
		UE_SET_PP(LensFlareBokehSize);
		UE_SET_PP(LensFlareThreshold);
		UE_SET_PP(VignetteIntensity);
		UE_SET_PP(VignetteType);
		UE_SET_PP(VignetteCenter);
		UE_SET_PP(VignetteColor);
		UE_SET_PP(VignetteSize);
		UE_SET_PP(VignetteSoftness);
		UE_SET_PP(VignetteTexture);
		UE_SET_PP(Sharpen);
		UE_SET_PP(FilmGrainIntensity);
		UE_SET_PP(FilmGrainIntensityShadows);
		UE_SET_PP(FilmGrainIntensityMidtones);
		UE_SET_PP(FilmGrainIntensityHighlights);
		UE_SET_PP(FilmGrainShadowsMax);
		UE_SET_PP(FilmGrainHighlightsMin);
		UE_SET_PP(FilmGrainHighlightsMax);
		UE_SET_PP(FilmGrainScale);
		UE_SET_PP(AmbientOcclusionIntensity);
		UE_SET_PP(AmbientOcclusionStaticFraction);
		UE_SET_PP(AmbientOcclusionRadius);
		UE_SET_PP(AmbientOcclusionFadeDistance);
		UE_SET_PP(AmbientOcclusionFadeRadius);
		UE_SET_PP(AmbientOcclusionDistance_DEPRECATED);
		UE_SET_PP(AmbientOcclusionPower);
		UE_SET_PP(AmbientOcclusionBias);
		UE_SET_PP(AmbientOcclusionQuality);
		UE_SET_PP(AmbientOcclusionMipBlend);
		UE_SET_PP(AmbientOcclusionMipScale);
		UE_SET_PP(AmbientOcclusionMipThreshold);
		UE_SET_PP(AmbientOcclusionTemporalBlendWeight);
		UE_SET_PP(IndirectLightingColor);
		UE_SET_PP(IndirectLightingIntensity);
		UE_SET_PP(DepthOfFieldEnabled);
		UE_SET_PP(DepthOfFieldFocalDistance);
		UE_SET_PP(DepthOfFieldFstop);
		UE_SET_PP(DepthOfFieldMinFstop);
		UE_SET_PP(DepthOfFieldSensorWidth);
		UE_SET_PP(DepthOfFieldSqueezeFactor);
		UE_SET_PP(DepthOfFieldDepthBlurRadius);
		UE_SET_PP(DepthOfFieldUseHairDepth)
		UE_SET_PP(DepthOfFieldDepthBlurAmount);
		UE_SET_PP(DepthOfFieldFocalRegion);
		UE_SET_PP(DepthOfFieldNearTransitionRegion);
		UE_SET_PP(DepthOfFieldFarTransitionRegion);
		UE_SET_PP(DepthOfFieldScale);
		UE_SET_PP(DepthOfFieldNearBlurSize);
		UE_SET_PP(DepthOfFieldFarBlurSize);
		UE_SET_PP(DepthOfFieldOcclusion);
		UE_SET_PP(DepthOfFieldSkyFocusDistance);
		UE_SET_PP(DepthOfFieldVignetteSize);

		UE_SET_PP(MotionBlurAmount);
		UE_SET_PP(MotionBlurMax);
		UE_SET_PP(MotionBlurPerObjectSize);
		UE_SET_PP(ScreenSpaceReflectionQuality);
		UE_SET_PP(ScreenSpaceReflectionIntensity);
		UE_SET_PP(ScreenSpaceReflectionMaxRoughness);

		UE_SET_PP(TranslucencyType);
		UE_SET_PP(RayTracingTranslucencyMaxRoughness);
		UE_SET_PP(RayTracingTranslucencyRefractionRays);
		UE_SET_PP(RayTracingTranslucencySamplesPerPixel);
		UE_SET_PP(RayTracingTranslucencyShadows);
		UE_SET_PP(RayTracingTranslucencyRefraction);

		UE_SET_PP(DynamicGlobalIlluminationMethod);
		UE_SET_PP(LumenSurfaceCacheResolution);
		UE_SET_PP(LumenSceneLightingQuality);
		UE_SET_PP(LumenSceneDetail);
		UE_SET_PP(LumenSceneViewDistance);
		UE_SET_PP(LumenSceneLightingUpdateSpeed);
		UE_SET_PP(LumenFinalGatherQuality);
		UE_SET_PP(LumenFinalGatherLightingUpdateSpeed);
		UE_SET_PP(LumenFinalGatherScreenTraces);
		UE_SET_PP(LumenMaxTraceDistance);

		UE_SET_PP(LumenDiffuseColorBoost);
		UE_SET_PP(LumenAmbientOcclusionIntensity);
		UE_SET_PP(LumenSkylightLeaking);
		UE_SET_PP(LumenFullSkylightLeakingDistance);

		UE_SET_PP(LumenRayLightingMode);
		UE_SET_PP(LumenReflectionsScreenTraces);
		UE_SET_PP(LumenFrontLayerTranslucencyReflections);
		UE_SET_PP(LumenMaxRoughnessToTraceReflections);
		UE_SET_PP(LumenMaxReflectionBounces);
		UE_SET_PP(LumenMaxRefractionBounces);
		UE_SET_PP(ReflectionMethod);
		UE_SET_PP(LumenReflectionQuality);
		UE_SET_PP(RayTracingAO);
		UE_SET_PP(RayTracingAOSamplesPerPixel);
		UE_SET_PP(RayTracingAOIntensity);
		UE_SET_PP(RayTracingAORadius);

		UE_SET_PP(PathTracingMaxBounces);
		UE_SET_PP(PathTracingSamplesPerPixel);
		UE_SET_PP(PathTracingMaxPathIntensity);
		UE_SET_PP(PathTracingEnableEmissiveMaterials);
		UE_SET_PP(PathTracingEnableReferenceDOF);
		UE_SET_PP(PathTracingEnableReferenceAtmosphere);
		UE_SET_PP(PathTracingEnableDenoiser);
		UE_SET_PP(PathTracingIncludeEmissive);
		UE_SET_PP(PathTracingIncludeDiffuse);
		UE_SET_PP(PathTracingIncludeIndirectDiffuse);
		UE_SET_PP(PathTracingIncludeSpecular);
		UE_SET_PP(PathTracingIncludeIndirectSpecular);
		UE_SET_PP(PathTracingIncludeVolume);
		UE_SET_PP(PathTracingIncludeIndirectVolume);

		UE_SET_PP(DepthOfFieldBladeCount);

		// This is no bOverride_AmbientCubemap so just see if it is set.
		if (OtherTo.AmbientCubemap)
		{
			ThisFrom.AmbientCubemap = OtherTo.AmbientCubemap;
			bAnyOverwritten = true;
		}

		UE_SET_PP(ColorGradingIntensity);
		UE_SET_PP(ColorGradingLUT);

		UE_SET_PP(BloomDirtMask);
		UE_SET_PP(BloomMethod);
		UE_SET_PP(BloomConvolutionTexture)
		UE_SET_PP(FilmGrainTexture)

		UE_SET_PP(BloomConvolutionBufferScale);

		UE_SET_PP(AutoExposureBiasCurve);
		UE_SET_PP(AutoExposureMeterMask);
		UE_SET_PP(LocalExposureHighlightContrastCurve);
		UE_SET_PP(LocalExposureShadowContrastCurve);
		UE_SET_PP(LensFlareBokehShape);

		if (OtherTo.bOverride_LensFlareTints)
		{
			for (uint32 i = 0; i < 8; ++i)
			{
				ThisFrom.LensFlareTints[i] = OtherTo.LensFlareTints[i];
			}
			bAnyOverwritten = true;
		}

		if (OtherTo.bOverride_MobileHQGaussian)
		{
			ThisFrom.bMobileHQGaussian = OtherTo.bMobileHQGaussian;
			bAnyOverwritten = true;
		}

		UE_SET_PP(AutoExposureMethod);
		UE_SET_PP(AmbientOcclusionRadiusInWS);
		UE_SET_PP(MotionBlurTargetFPS);
		UE_SET_PP(AutoExposureApplyPhysicalCameraExposure);
		UE_SET_PP(UserFlags);
	}

#undef UE_SET_PP

	// Override blendable objects.
	bAnyOverwritten |= (ThisFrom.WeightedBlendables.Array.Num() > 0 || OtherTo.WeightedBlendables.Array.Num() > 0);
	ThisFrom.WeightedBlendables = OtherTo.WeightedBlendables;

	return bAnyOverwritten;
}

bool FPostProcessUtils::BlendPostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo, float BlendFactor)
{
	if (UNLIKELY(BlendFactor <= 0.f))
	{
		return false;
	}

	BlendFactor = FMath::Clamp(BlendFactor, 0.f, 1.f);
	const bool bShouldFlip = (BlendFactor >= 0.5f);

	bool bAnyOverwritten = false;

#define UE_SET_PP(Name)\
	if ((OtherTo.bOverride_##Name || ThisFrom.bOverride_##Name) && bShouldFlip)\
	{\
		ThisFrom.bOverride_##Name = true;\
		ThisFrom.Name = OtherTo.Name;\
		bAnyOverwritten = true;\
	}

#define UE_LERP_PP(Name)\
	if (OtherTo.bOverride_##Name || ThisFrom.bOverride_##Name)\
	{\
		ThisFrom.bOverride_##Name = true;\
		ThisFrom.Name = FMath::Lerp(ThisFrom.Name, OtherTo.Name, BlendFactor);\
		bAnyOverwritten = true;\
	}

	{
		UE_SET_PP(TemperatureType);
		UE_LERP_PP(WhiteTemp);
		UE_LERP_PP(WhiteTint);

		UE_LERP_PP(ColorSaturation);
		UE_LERP_PP(ColorContrast);
		UE_LERP_PP(ColorGamma);
		UE_LERP_PP(ColorGain);
		UE_LERP_PP(ColorOffset);

		UE_LERP_PP(ColorSaturationShadows);
		UE_LERP_PP(ColorContrastShadows);
		UE_LERP_PP(ColorGammaShadows);
		UE_LERP_PP(ColorGainShadows);
		UE_LERP_PP(ColorOffsetShadows);

		UE_LERP_PP(ColorSaturationMidtones);
		UE_LERP_PP(ColorContrastMidtones);
		UE_LERP_PP(ColorGammaMidtones);
		UE_LERP_PP(ColorGainMidtones);
		UE_LERP_PP(ColorOffsetMidtones);

		UE_LERP_PP(ColorSaturationHighlights);
		UE_LERP_PP(ColorContrastHighlights);
		UE_LERP_PP(ColorGammaHighlights);
		UE_LERP_PP(ColorGainHighlights);
		UE_LERP_PP(ColorOffsetHighlights);

		UE_LERP_PP(ColorCorrectionShadowsMax);
		UE_LERP_PP(ColorCorrectionHighlightsMin);
		UE_LERP_PP(ColorCorrectionHighlightsMax);

		UE_LERP_PP(BlueCorrection);
		UE_LERP_PP(ExpandGamut);
		UE_LERP_PP(ToneCurveAmount);

		UE_LERP_PP(FilmSlope);
		UE_LERP_PP(FilmToe);
		UE_LERP_PP(FilmShoulder);
		UE_LERP_PP(FilmBlackClip);
		UE_LERP_PP(FilmWhiteClip);

		UE_LERP_PP(SceneColorTint);
		UE_LERP_PP(SceneFringeIntensity);
		UE_LERP_PP(ChromaticAberrationStartOffset);
		UE_LERP_PP(BloomIntensity);
		UE_LERP_PP(BloomGaussianIntensity);
		UE_LERP_PP(BloomThreshold);
		UE_LERP_PP(Bloom1Tint);
		UE_LERP_PP(BloomSizeScale);
		UE_LERP_PP(Bloom1Size);
		UE_LERP_PP(Bloom2Tint);
		UE_LERP_PP(Bloom2Size);
		UE_LERP_PP(Bloom3Tint);
		UE_LERP_PP(Bloom3Size);
		UE_LERP_PP(Bloom4Tint);
		UE_LERP_PP(Bloom4Size);
		UE_LERP_PP(Bloom5Tint);
		UE_LERP_PP(Bloom5Size);
		UE_LERP_PP(Bloom6Tint);
		UE_LERP_PP(Bloom6Size);
		UE_LERP_PP(BloomDirtMaskIntensity);
		UE_LERP_PP(BloomDirtMaskTint);
		UE_LERP_PP(BloomConvolutionIntensity);
		UE_LERP_PP(BloomConvolutionScatterDispersion);
		UE_LERP_PP(BloomConvolutionSize);
		UE_LERP_PP(BloomConvolutionCenterUV);
		UE_LERP_PP(BloomConvolutionPreFilterMin);
		UE_LERP_PP(BloomConvolutionPreFilterMax);
		UE_LERP_PP(BloomConvolutionPreFilterMult);
		UE_LERP_PP(AmbientCubemapIntensity);
		UE_LERP_PP(AmbientCubemapTint);
		UE_LERP_PP(CameraShutterSpeed);
		UE_LERP_PP(CameraISO);
		UE_LERP_PP(AutoExposureLowPercent);
		UE_LERP_PP(AutoExposureHighPercent);
		UE_LERP_PP(AutoExposureMinBrightness);
		UE_LERP_PP(AutoExposureMaxBrightness);
		UE_LERP_PP(AutoExposureSpeedUp);
		UE_LERP_PP(AutoExposureSpeedDown);
		UE_LERP_PP(AutoExposureBias);
		UE_LERP_PP(HistogramLogMin);
		UE_LERP_PP(HistogramLogMax);
		UE_SET_PP(LocalExposureMethod);
		UE_LERP_PP(LocalExposureContrastScale_DEPRECATED);
		UE_LERP_PP(LocalExposureHighlightContrastScale);
		UE_LERP_PP(LocalExposureShadowContrastScale);
		UE_LERP_PP(LocalExposureHighlightThreshold);
		UE_LERP_PP(LocalExposureShadowThreshold);
		UE_LERP_PP(LocalExposureDetailStrength);
		UE_LERP_PP(LocalExposureBlurredLuminanceBlend);
		UE_LERP_PP(LocalExposureBlurredLuminanceKernelSizePercent);
		UE_LERP_PP(LocalExposureMiddleGreyBias);
		UE_LERP_PP(LensFlareIntensity);
		UE_LERP_PP(LensFlareTint);
		UE_LERP_PP(LensFlareBokehSize);
		UE_LERP_PP(LensFlareThreshold);
		UE_LERP_PP(VignetteIntensity);
		UE_SET_PP(VignetteType);
		UE_LERP_PP(VignetteCenter);
		UE_LERP_PP(VignetteColor);
		UE_LERP_PP(VignetteSize);
		UE_LERP_PP(VignetteSoftness);
		UE_SET_PP(VignetteTexture);
		UE_LERP_PP(Sharpen);
		UE_LERP_PP(FilmGrainIntensity);
		UE_LERP_PP(FilmGrainIntensityShadows);
		UE_LERP_PP(FilmGrainIntensityMidtones);
		UE_LERP_PP(FilmGrainIntensityHighlights);
		UE_LERP_PP(FilmGrainShadowsMax);
		UE_LERP_PP(FilmGrainHighlightsMin);
		UE_LERP_PP(FilmGrainHighlightsMax);
		UE_LERP_PP(FilmGrainScale);
		UE_LERP_PP(AmbientOcclusionIntensity);
		UE_LERP_PP(AmbientOcclusionStaticFraction);
		UE_LERP_PP(AmbientOcclusionRadius);
		UE_LERP_PP(AmbientOcclusionFadeDistance);
		UE_LERP_PP(AmbientOcclusionFadeRadius);
		UE_LERP_PP(AmbientOcclusionDistance_DEPRECATED);
		UE_LERP_PP(AmbientOcclusionPower);
		UE_LERP_PP(AmbientOcclusionBias);
		UE_LERP_PP(AmbientOcclusionQuality);
		UE_LERP_PP(AmbientOcclusionMipBlend);
		UE_LERP_PP(AmbientOcclusionMipScale);
		UE_LERP_PP(AmbientOcclusionMipThreshold);
		UE_LERP_PP(AmbientOcclusionTemporalBlendWeight);
		UE_LERP_PP(IndirectLightingColor);
		UE_LERP_PP(IndirectLightingIntensity);

		if (OtherTo.bOverride_DepthOfFieldFocalDistance)
		{
			if (ThisFrom.DepthOfFieldFocalDistance == 0.0f || OtherTo.DepthOfFieldFocalDistance == 0.0f)
			{
				ThisFrom.DepthOfFieldFocalDistance = OtherTo.DepthOfFieldFocalDistance;
			}
			else
			{
				ThisFrom.DepthOfFieldFocalDistance = FMath::Lerp(ThisFrom.DepthOfFieldFocalDistance, OtherTo.DepthOfFieldFocalDistance, BlendFactor);
			}
			bAnyOverwritten = true;
		}
		UE_SET_PP(DepthOfFieldEnabled)
		UE_LERP_PP(DepthOfFieldFstop);
		UE_LERP_PP(DepthOfFieldMinFstop);
		UE_LERP_PP(DepthOfFieldSensorWidth);
		UE_LERP_PP(DepthOfFieldSqueezeFactor);
		UE_LERP_PP(DepthOfFieldDepthBlurRadius);
		UE_SET_PP(DepthOfFieldUseHairDepth)
		UE_LERP_PP(DepthOfFieldDepthBlurAmount);
		UE_LERP_PP(DepthOfFieldFocalRegion);
		UE_LERP_PP(DepthOfFieldNearTransitionRegion);
		UE_LERP_PP(DepthOfFieldFarTransitionRegion);
		UE_LERP_PP(DepthOfFieldScale);
		UE_LERP_PP(DepthOfFieldNearBlurSize);
		UE_LERP_PP(DepthOfFieldFarBlurSize);
		UE_LERP_PP(DepthOfFieldOcclusion);
		UE_LERP_PP(DepthOfFieldSkyFocusDistance);
		UE_LERP_PP(DepthOfFieldVignetteSize);
		UE_LERP_PP(MotionBlurAmount);
		UE_LERP_PP(MotionBlurMax);
		UE_LERP_PP(MotionBlurPerObjectSize);
		UE_LERP_PP(ScreenSpaceReflectionQuality);
		UE_LERP_PP(ScreenSpaceReflectionIntensity);
		UE_LERP_PP(ScreenSpaceReflectionMaxRoughness);

		UE_SET_PP(TranslucencyType);
		UE_SET_PP(RayTracingTranslucencyMaxRoughness);
		UE_SET_PP(RayTracingTranslucencyRefractionRays);
		UE_SET_PP(RayTracingTranslucencySamplesPerPixel);
		UE_SET_PP(RayTracingTranslucencyShadows);
		UE_SET_PP(RayTracingTranslucencyRefraction);

		UE_SET_PP(DynamicGlobalIlluminationMethod);
		UE_SET_PP(LumenSurfaceCacheResolution);
		UE_SET_PP(LumenSceneLightingQuality);
		UE_SET_PP(LumenSceneDetail);
		UE_SET_PP(LumenSceneViewDistance);
		UE_SET_PP(LumenSceneLightingUpdateSpeed);
		UE_SET_PP(LumenFinalGatherQuality);
		UE_SET_PP(LumenFinalGatherLightingUpdateSpeed);
		UE_SET_PP(LumenFinalGatherScreenTraces);
		UE_SET_PP(LumenMaxTraceDistance);

		UE_LERP_PP(LumenDiffuseColorBoost);
		UE_LERP_PP(LumenAmbientOcclusionIntensity);
		UE_LERP_PP(LumenSkylightLeaking);
		UE_LERP_PP(LumenFullSkylightLeakingDistance);

		UE_SET_PP(LumenRayLightingMode);
		UE_SET_PP(LumenReflectionsScreenTraces);
		UE_SET_PP(LumenFrontLayerTranslucencyReflections);
		UE_SET_PP(LumenMaxRoughnessToTraceReflections);
		UE_SET_PP(LumenMaxReflectionBounces);
		UE_SET_PP(LumenMaxRefractionBounces);
		UE_SET_PP(ReflectionMethod);
		UE_SET_PP(LumenReflectionQuality);
		UE_SET_PP(RayTracingAO);
		UE_SET_PP(RayTracingAOSamplesPerPixel);
		UE_SET_PP(RayTracingAOIntensity);
		UE_SET_PP(RayTracingAORadius);

		UE_SET_PP(PathTracingMaxBounces);
		UE_SET_PP(PathTracingSamplesPerPixel);
		UE_LERP_PP(PathTracingMaxPathIntensity);
		UE_SET_PP(PathTracingEnableEmissiveMaterials);
		UE_SET_PP(PathTracingEnableReferenceDOF);
		UE_SET_PP(PathTracingEnableReferenceAtmosphere);
		UE_SET_PP(PathTracingEnableDenoiser);
		UE_SET_PP(PathTracingIncludeEmissive);
		UE_SET_PP(PathTracingIncludeDiffuse);
		UE_SET_PP(PathTracingIncludeIndirectDiffuse);
		UE_SET_PP(PathTracingIncludeSpecular);
		UE_SET_PP(PathTracingIncludeIndirectSpecular);
		UE_SET_PP(PathTracingIncludeVolume);
		UE_SET_PP(PathTracingIncludeIndirectVolume);

		UE_SET_PP(DepthOfFieldBladeCount);

		// No cubemap blending (only supported for FFinalPostProcessSettings)
		if (bShouldFlip)
		{
			ThisFrom.AmbientCubemap = OtherTo.AmbientCubemap;
			bAnyOverwritten = true;
		}

		// No color grading texture blending (only supported for FFinalPostProcessSettings)
		UE_LERP_PP(ColorGradingIntensity);
		UE_SET_PP(ColorGradingLUT);

		UE_SET_PP(BloomDirtMask);
		UE_SET_PP(BloomMethod);
		UE_SET_PP(BloomConvolutionTexture)
		UE_SET_PP(FilmGrainTexture)

		// Flipping this instead of blending, as per the comment in SceneView.cpp
		UE_SET_PP(BloomConvolutionBufferScale);

		UE_SET_PP(AutoExposureBiasCurve);
		UE_SET_PP(AutoExposureMeterMask);
		UE_SET_PP(LocalExposureHighlightContrastCurve);
		UE_SET_PP(LocalExposureShadowContrastCurve);
		UE_SET_PP(LensFlareBokehShape);

		if (OtherTo.bOverride_LensFlareTints)
		{
			for (uint32 i = 0; i < 8; ++i)
			{
				ThisFrom.LensFlareTints[i] = FMath::Lerp(ThisFrom.LensFlareTints[i], OtherTo.LensFlareTints[i], BlendFactor);
			}
			bAnyOverwritten = true;
		}

		if (OtherTo.bOverride_MobileHQGaussian && bShouldFlip)
		{
			ThisFrom.bMobileHQGaussian = OtherTo.bMobileHQGaussian;
			bAnyOverwritten = true;
		}

		UE_SET_PP(AutoExposureMethod);
		UE_SET_PP(AmbientOcclusionRadiusInWS);
		UE_SET_PP(MotionBlurTargetFPS);
		UE_SET_PP(AutoExposureApplyPhysicalCameraExposure);
		UE_SET_PP(UserFlags);
	}

#undef UE_SET_PP
#undef UE_LERP_PP

	// Blend out old objects, blend in new objects.
	// This uses an O(n2) lookup for now since 99% of the time we have zero blendables, and the rest
	// of the time we have only one or two.
	bAnyOverwritten |= (ThisFrom.WeightedBlendables.Array.Num() > 0 || OtherTo.WeightedBlendables.Array.Num() > 0);
	for (FWeightedBlendable& ThisBlendable : ThisFrom.WeightedBlendables.Array)
	{
		const bool bFoundInOther = OtherTo.WeightedBlendables.Array.ContainsByPredicate(
				[&ThisBlendable](const FWeightedBlendable& OtherBlendable)
				{
					return ThisBlendable.Object == OtherBlendable.Object;
				});
		if (!bFoundInOther)
		{
			ThisBlendable.Weight *= (1.0f - BlendFactor);
		}
	}
	for (const FWeightedBlendable& OtherBlendable : OtherTo.WeightedBlendables.Array)
	{
		const bool bFoundInThis = ThisFrom.WeightedBlendables.Array.ContainsByPredicate(
				[&OtherBlendable](FWeightedBlendable& ThisBlendable)
				{
					return ThisBlendable.Object == OtherBlendable.Object;
				});
		if (!bFoundInThis)
		{
			ThisFrom.WeightedBlendables.Array.Emplace(OtherBlendable.Weight * BlendFactor, OtherBlendable.Object);
		}
	}

	return bAnyOverwritten;
}

#endif  // !5.6.0

