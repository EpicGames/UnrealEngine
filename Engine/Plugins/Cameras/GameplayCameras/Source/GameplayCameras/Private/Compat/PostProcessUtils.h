// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EngineVersionComparison.h"

struct FPostProcessSettings;

#if !UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

/**
 * Copy of UE 5.6's FPostProcessUtils class for backwards compatible code.
 * Implementation only supports post process settings that existed up to UE 5.5.
 */
struct FPostProcessUtils
{
	static bool OverridePostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo);
	static bool BlendPostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo, float BlendFactor);
};

#endif  // <5.6.0

