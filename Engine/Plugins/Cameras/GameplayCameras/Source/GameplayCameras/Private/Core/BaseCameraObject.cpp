// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BaseCameraObject.h"

#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/CameraNode.h"
#include "Core/ObjectTreeGraphRootObject.h"
#include "Misc/EngineVersionComparison.h"
#include "StructUtils/OverridablePropertyBag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCameraObject)

void FCameraObjectAllocationInfo::Append(const FCameraObjectAllocationInfo& OtherAllocationInfo)
{
	const FCameraNodeEvaluatorAllocationInfo& OtherEvaluatorInfo(OtherAllocationInfo.EvaluatorInfo);
	EvaluatorInfo.MaxAlignof = FMath::Max(EvaluatorInfo.MaxAlignof, OtherEvaluatorInfo.MaxAlignof);
	EvaluatorInfo.TotalSizeof = Align(EvaluatorInfo.TotalSizeof, OtherEvaluatorInfo.MaxAlignof) + OtherEvaluatorInfo.TotalSizeof;

	const FCameraVariableTableAllocationInfo& OtherVariableTableInfo(OtherAllocationInfo.VariableTableInfo);
	VariableTableInfo.VariableDefinitions.Append(OtherVariableTableInfo.VariableDefinitions);

	const FCameraContextDataTableAllocationInfo& OtherContextDataTableInfo(OtherAllocationInfo.ContextDataTableInfo);
	ContextDataTableInfo.DataDefinitions.Append(OtherContextDataTableInfo.DataDefinitions);
}

void FCameraObjectConnections::Add(UObject* InSource, FName InSourcePropertyName, UObject* InTarget, FName InTargetPropertyName)
{
	FCameraObjectConnection NewConnection;
	NewConnection.Source = InSource;
	NewConnection.SourcePropertyName = InSourcePropertyName;
	NewConnection.Target = InTarget;
	NewConnection.TargetPropertyName = InTargetPropertyName;
	Connections.Add(NewConnection);
}

FCameraObjectConnection* FCameraObjectConnections::FindBySource(UObject* InSource)
{
	return Connections.FindByPredicate([InSource](FCameraObjectConnection& Item)
			{
				return Item.Source == InSource;
			});
}

FCameraObjectConnection* FCameraObjectConnections::FindBySource(UObject* InSource, FName InSourcePropertyName)
{
	return Connections.FindByPredicate([InSource, InSourcePropertyName](FCameraObjectConnection& Item)
			{
				return Item.Source == InSource && Item.SourcePropertyName == InSourcePropertyName;
			});
}

FCameraObjectConnection* FCameraObjectConnections::FindByTarget(UObject* InTarget)
{
	return Connections.FindByPredicate([InTarget](FCameraObjectConnection& Item)
			{
				return Item.Target == InTarget;
			});
}

FCameraObjectConnection* FCameraObjectConnections::FindByTarget(UObject* InTarget, FName InTargetPropertyName)
{
	return Connections.FindByPredicate([InTarget, InTargetPropertyName](FCameraObjectConnection& Item)
			{
				return Item.Target == InTarget && Item.TargetPropertyName == InTargetPropertyName;
			});
}

void UBaseCameraObject::Serialize(FArchive& Ar)
{
	// Before 5.8, property bags are never saved with property flags, so always fix them up in PostLoad.
	// With 5.8, that bug is fixed so we only fix up the property bags if the asset was saved before.
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
	Ar.UsingCustomVersion(FOverridablePropertyBagCustomVersion::GUID);
#endif

	Super::Serialize(Ar);

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
	bDefaultParametersMayHaveMissingPropertyFlags = Ar.CustomVer(FOverridablePropertyBagCustomVersion::GUID) < FOverridablePropertyBagCustomVersion::MissingPropertyFlags;
#else
	bDefaultParametersMayHaveMissingPropertyFlags = true;
#endif
}

void UBaseCameraObject::PostLoad()
{
	Super::PostLoad();

	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}

	if (bDefaultParametersMayHaveMissingPropertyFlags)
	{
		using namespace UE::Cameras;
		FCameraObjectInterfaceParameterBuilder::FixUpDefaultParameterProperties(ParameterDefinitions, DefaultParameters);
		bDefaultParametersMayHaveMissingPropertyFlags = false;
	}
}

void UBaseCameraObject::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && 
			!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UBaseCameraObject::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		Guid = FGuid::NewGuid();
	}
}

bool UBaseCameraObject::FindParameterDefinitionByName(const FName ParameterName, FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const
{
	const FCameraObjectInterfaceParameterDefinition* FoundItem = ParameterDefinitions.FindByPredicate(
			[ParameterName](const FCameraObjectInterfaceParameterDefinition& Item)
			{
				return Item.ParameterName == ParameterName;
			});
	if (FoundItem)
	{
		OutParameterDefinition = *FoundItem;
		return true;
	}
	return false;
}

bool UBaseCameraObject::FindParameterDefinitionByGuid(const FGuid& ParameterGuid, FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const
{
	const FCameraObjectInterfaceParameterDefinition* FoundItem = ParameterDefinitions.FindByPredicate(
			[&ParameterGuid](const FCameraObjectInterfaceParameterDefinition& Item)
			{
				return Item.ParameterGuid == ParameterGuid;
			});
	if (FoundItem)
	{
		OutParameterDefinition = *FoundItem;
		return true;
	}
	return false;
}

#if WITH_EDITORONLY_DATA

void UBaseCameraObject::UpgradeInterfaceConnections(IObjectTreeGraphRootObject* RootObject, FName DefaultGraphName)
{
	if (!ensure(RootObject))
	{
		return;
	}

	TArray<UCameraObjectInterfaceParameterBase*> AllParameters;
	AllParameters.Append(Interface.BlendableParameters);
	AllParameters.Append(Interface.DataParameters);

	// Move the connection information to the new connection list.
	// Create getter nodes for parameters that were added to the camera node graph.
	for (UCameraObjectInterfaceParameterBase* Parameter : AllParameters)
	{
		UCameraObjectInterfaceParameterGetter* GetterNode = nullptr;
		const bool bIsConnected = (Parameter->Target_DEPRECATED && Parameter->TargetPropertyName_DEPRECATED != NAME_None);

		if (Parameter->bHasGraphNode_DEPRECATED || bIsConnected)
		{
			GetterNode = NewObject<UCameraObjectInterfaceParameterGetter>(this);
			GetterNode->ParameterGuid = Parameter->GetGuid();
			GetterNode->GraphNodePos = Parameter->GraphNodePos_DEPRECATED;
			RootObject->AddConnectableObject(DefaultGraphName, GetterNode);

			Parameter->bHasGraphNode_DEPRECATED = false;
			Parameter->GraphNodePos_DEPRECATED = FIntVector2::ZeroValue;
		}

		if (bIsConnected)
		{
			ensure(GetterNode);

			FCameraObjectConnection Connection;
			Connection.Source = GetterNode;
			Connection.Target = Parameter->Target_DEPRECATED;
			Connection.TargetPropertyName = Parameter->TargetPropertyName_DEPRECATED;

			Connections.Connections.Add(Connection);

			Parameter->Target_DEPRECATED = nullptr;
			Parameter->TargetPropertyName_DEPRECATED = NAME_None;
		}
	}
}

#endif  // WITH_EDITORONLY_DATA

