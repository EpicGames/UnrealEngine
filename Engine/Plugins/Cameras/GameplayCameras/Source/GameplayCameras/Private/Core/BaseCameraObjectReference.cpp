// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BaseCameraObjectReference.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraParameters.h"  // IWYU pragma: keep
#include "Core/ICustomCameraNodeParameterProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCameraObjectReference)

const FCameraObjectInterfaceParameterMetaData* FBaseCameraObjectReference::FindMetaData(const FGuid& PropertyID) const
{
	return ParameterMetaData.FindByPredicate(
			[PropertyID](const FCameraObjectInterfaceParameterMetaData& Item)
			{
				return Item.ParameterGuid == PropertyID;
			});
}

FCameraObjectInterfaceParameterMetaData& FBaseCameraObjectReference::GetMetaData(const FGuid& PropertyID)
{
	FCameraObjectInterfaceParameterMetaData* MetaData = ParameterMetaData.FindByPredicate(
			[PropertyID](const FCameraObjectInterfaceParameterMetaData& Item)
			{
				return Item.ParameterGuid == PropertyID;
			});
	check(MetaData);
	return *MetaData;
}

bool FBaseCameraObjectReference::IsParameterOverridden(const FGuid& PropertyID) const
{
	return Parameters.IsPropertyOverriden(PropertyID);
}

void FBaseCameraObjectReference::SetParameterOverridden(const FGuid& PropertyID, bool bIsOverridden)
{
	Parameters.SetPropertyOverriden(PropertyID, bIsOverridden);
}

bool FBaseCameraObjectReference::NeedsRebuildParameters() const
{
	const UBaseCameraObject* CameraObject = GetCameraObject();

	if ((!CameraObject && Parameters.IsValid()) || (CameraObject && !Parameters.IsValid()))
	{
		return true;
	}

	if (CameraObject)
	{
		const UPropertyBag* AssetParametersType = CameraObject->GetDefaultParameters().GetPropertyBagStruct();
		const UPropertyBag* ReferenceParametersType = Parameters.GetPropertyBagStruct();
		if (AssetParametersType != ReferenceParametersType)
		{
			return true;
		}
	}

	return false;
}

bool FBaseCameraObjectReference::RebuildParametersIfNeeded()
{
	if (NeedsRebuildParameters())
	{
		RebuildParameters();
		return true;
	}
	return false;
}

void FBaseCameraObjectReference::RebuildParameters()
{
	const UBaseCameraObject* CameraObject = GetCameraObject();

	if (CameraObject)
	{
		Parameters.MigrateToNewBagInstanceWithOverrides(CameraObject->GetDefaultParameters());

		if (const UPropertyBag* ParametersType = Parameters.GetPropertyBagStruct())
		{
			// Remove metadata for parameters that don't exist anymore, and add default metadata for
			// new parameters.
			TSet<FGuid> ExistingMetaDataIDs;
			for (TArray<FCameraObjectInterfaceParameterMetaData>::TIterator It = ParameterMetaData.CreateIterator(); It; ++It)
			{
				ExistingMetaDataIDs.Add(It->ParameterGuid);
			}

			TSet<FGuid> WantedMetaDataIDs;
			TMap<FGuid, ECameraObjectInterfaceParameterType> WantedParameterTypes;
			for (const FCameraObjectInterfaceParameterDefinition& Definition : CameraObject->GetParameterDefinitions())
			{
				WantedMetaDataIDs.Add(Definition.ParameterGuid);
				WantedParameterTypes.Add(Definition.ParameterGuid, Definition.ParameterType);
			}

			TSet<FGuid> RemovedMetaDataIDs = ExistingMetaDataIDs.Difference(WantedMetaDataIDs);
			for (TArray<FCameraObjectInterfaceParameterMetaData>::TIterator It = ParameterMetaData.CreateIterator(); It; ++It)
			{
				if (RemovedMetaDataIDs.Contains(It->ParameterGuid))
				{
					It.RemoveCurrentSwap();
				}
			}

			TSet<FGuid> AddedMetaDataIDs = WantedMetaDataIDs.Difference(ExistingMetaDataIDs);
			for (const FGuid& ParameterGuid : AddedMetaDataIDs)
			{
				FCameraObjectInterfaceParameterMetaData NewMetaData;
				NewMetaData.ParameterGuid = ParameterGuid;
				ParameterMetaData.Add(NewMetaData);
			}
		}
		else
		{
			ParameterMetaData.Reset();
		}
	}
	else
	{
		Parameters.Reset();
		ParameterMetaData.Reset();
	}
}

void FBaseCameraObjectReference::GetCustomCameraNodeParameters(UE::Cameras::FCameraNodeParameterInfos& OutParameterInfos)
{
	RebuildParametersIfNeeded();

	const UBaseCameraObject* CameraObject = GetCameraObject();
	if (!CameraObject)
	{
		return;
	}

	uint8* ParametersMemory = Parameters.GetMutableValue().GetMemory();
	const UPropertyBag* ParametersStruct = Parameters.GetPropertyBagStruct();
	const FInstancedPropertyBag& DefaultParameters = CameraObject->GetDefaultParameters();
	if (!ensure(ParametersMemory && ParametersStruct && ParametersStruct == DefaultParameters.GetPropertyBagStruct()))
	{
		return;
	}

	for (const FCameraObjectInterfaceParameterDefinition& Definition : CameraObject->GetParameterDefinitions())
	{
		if (!Definition.bIsVisible)
		{
			continue;
		}

		const FPropertyBagPropertyDesc* PropertyDesc = ParametersStruct->FindPropertyDescByID(Definition.ParameterGuid);
		if (!ensure(PropertyDesc && PropertyDesc->CachedProperty))
		{
			continue;
		}

		if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
		{
			if (!ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Struct))
			{
				continue;
			}

			switch (Definition.VariableType)
			{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
				case ECameraVariableType::ValueName:\
					if (ensure(PropertyDesc->ValueTypeObject == F##ValueName##CameraParameter::StaticStruct()))\
					{\
						using CameraParameterType = F##ValueName##CameraParameter;\
						void* PropertyValue = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(ParametersMemory);\
						if (ensure(PropertyValue))\
						{\
							CameraParameterType* CameraParameter = static_cast<CameraParameterType*>(PropertyValue);\
							OutParameterInfos.AddBlendableParameter(\
									Definition.ParameterName,\
									Definition.VariableType,\
									nullptr,\
									reinterpret_cast<uint8*>(&CameraParameter->Value),\
									&CameraParameter->VariableID);\
						}\
					}\
					break;
				UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
				case ECameraVariableType::BlendableStruct:
					{
						void* PropertyValue = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(ParametersMemory);
						if (ensure(PropertyValue))
						{
							FCameraObjectInterfaceParameterMetaData& MetaData = GetMetaData(Definition.ParameterGuid);
							OutParameterInfos.AddBlendableParameter(
									Definition.ParameterName,
									Definition.VariableType,
									Definition.BlendableStructType,
									reinterpret_cast<uint8*>(PropertyValue),
									&MetaData.OverrideVariableID);
						}
					}
					break;
			}
		}
		else if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Data)
		{
			FCameraObjectInterfaceParameterMetaData& MetaData = GetMetaData(Definition.ParameterGuid);

			ECameraContextDataContainerType ContainerType = ECameraContextDataContainerType::None;
			if (PropertyDesc->ContainerTypes.GetFirstContainerType() == EPropertyBagContainerType::Array)
			{
				ContainerType = ECameraContextDataContainerType::Array;
			}

			const uint8* DefaultValue = reinterpret_cast<uint8*>(PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(ParametersMemory));

			switch (Definition.DataType)
			{
				case ECameraContextDataType::Name:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Name))
					{
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::Name, ContainerType, nullptr, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
				case ECameraContextDataType::String:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::String))
					{
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::String, ContainerType, nullptr, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
				case ECameraContextDataType::Enum:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Enum &&
								PropertyDesc->ValueTypeObject == Definition.DataTypeObject))
					{
						const UEnum* EnumType = CastChecked<const UEnum>(Definition.DataTypeObject);
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::Enum, ContainerType, EnumType, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
				case ECameraContextDataType::Struct:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Struct &&
								PropertyDesc->ValueTypeObject == Definition.DataTypeObject))
					{
						const UScriptStruct* DataType = CastChecked<const UScriptStruct>(Definition.DataTypeObject);
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::Struct, ContainerType, DataType, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
				case ECameraContextDataType::Object:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Object))
					{
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::Object, ContainerType, Definition.DataTypeObject, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
				case ECameraContextDataType::Class:
					if (ensure(PropertyDesc->ValueType == EPropertyBagPropertyType::Class))
					{
						OutParameterInfos.AddDataParameter(Definition.ParameterName, ECameraContextDataType::Class, ContainerType, Definition.DataTypeObject, DefaultValue, &MetaData.OverrideDataID);
					}
					break;
			}
		}
	}
}

bool FBaseCameraObjectReference::PostBuild()
{
	const UPropertyBag* ParametersStruct = Parameters.GetPropertyBagStruct();
	const uint8* ParametersMemory = Parameters.GetValue().GetMemory();
	if (!ParametersStruct || !ParametersMemory)
	{
		return false;
	}

	// Parameters that are driven by a connection (such as a camera rig parameter) should be set as overriden.
	// For most blendable parameters, we can tell about the connection from the VariableID being set on them.
	// For blendable structs and data parameters, the VariableID or DataID is in our metadata.
	bool bDidModify = false;
	for (const FCameraObjectInterfaceParameterMetaData& MetaData : ParameterMetaData)
	{
		const FPropertyBagPropertyDesc* PropertyDesc = ParametersStruct->FindPropertyDescByID(MetaData.ParameterGuid);
		if (!ensure(PropertyDesc))
		{
			continue;
		}

		if (PropertyDesc->ValueType == EPropertyBagPropertyType::Struct)
		{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			if (PropertyDesc->ValueTypeObject == F##ValueName##CameraParameter::StaticStruct())\
			{\
				using CameraParameterType = F##ValueName##CameraParameter;\
				const CameraParameterType* PropertyValue = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<CameraParameterType>(ParametersMemory);\
				if (ensure(PropertyValue) && PropertyValue->VariableID.IsValid())\
				{\
					if (!Parameters.IsPropertyOverriden(MetaData.ParameterGuid))\
					{\
						Parameters.SetPropertyOverriden(MetaData.ParameterGuid, true);\
						bDidModify = true;\
					}\
				}\
			}\
			else
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			{
				// Other struct, probably a blendable struct or data parameter, handled below.
			}
		}

		if (MetaData.OverrideVariableID.IsValid() || MetaData.OverrideDataID.IsValid())
		{
			if (!Parameters.IsPropertyOverriden(MetaData.ParameterGuid))
			{
				Parameters.SetPropertyOverriden(MetaData.ParameterGuid, true);
				bDidModify = true;
			}
		}
	}
	return bDidModify;
}

void FBaseCameraObjectReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FCameraObjectInterfaceParameterMetaData& MetaData : ParameterMetaData)
		{
			if (MetaData.bIsOverridden_DEPRECATED)
			{
				Parameters.SetPropertyOverriden(MetaData.ParameterGuid, true);
				MetaData.bIsOverridden_DEPRECATED = false;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

