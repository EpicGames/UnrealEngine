// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendCameraNode)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBlendCameraNodeEvaluator)

void FBlendCameraNodeEvaluator::BlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	OnBlendParameters(Params, OutResult);
}

void FBlendCameraNodeEvaluator::BlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	OnBlendResults(Params, OutResult);
}

bool FBlendCameraNodeEvaluator::InitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params)
{
	return OnInitializeFromInterruption(Params);
}

bool FBlendCameraNodeEvaluator::SetReversed(bool bInReverse)
{
	return OnSetReversed(bInReverse);
}

void FBlendCameraNodeEvaluator::Freeze()
{
	OnFreeze();

	// When frozen, we can't access the camera node anymore, as we may have been frozen
	// because the data we came from has been unloaded.
	SetPrivateCameraNode(nullptr);
}

}  // namespace UE::Cameras

