// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BlendStackCameraNode.h"

#include "Build/CameraRigAssetBuilder.h"
#include "Build/CameraBuildLog.h"
#include "Core/BlendStackCameraRigEvent.h"
#include "Core/BlendStackRootCameraNode.h"
#include "Core/CameraAsset.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/PersistentBlendStackCameraNode.h"
#include "Core/TransientBlendStackCameraNode.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraNodeEvaluationResultDebugBlock.h"
#include "Debug/CameraPoseDebugBlock.h"
#include "Debug/CameraPoseLocationTrailDebugBlock.h"
#include "Debug/ContextDataTableDebugBlock.h"
#include "Debug/VariableTableDebugBlock.h"
#include "Engine/World.h"
#include "GameplayCamerasSettings.h"
#include "HAL/IConsoleManager.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasModule.h"
#include "Math/ColorList.h"
#include "Modules/ModuleManager.h"
#include "Nodes/Blends/PopBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendStackCameraNode)

FCameraNodeEvaluatorPtr UBlendStackCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;

	switch (BlendStackType)
	{
		case ECameraBlendStackType::AdditivePersistent:
			return Builder.BuildEvaluator<FPersistentBlendStackCameraNodeEvaluator>();
		case ECameraBlendStackType::IsolatedTransient:
			return Builder.BuildEvaluator<FTransientBlendStackCameraNodeEvaluator>();
		default:
			ensure(false);
			return nullptr;
	}
}

namespace UE::Cameras
{

bool GGameplayCamerasDebugBlendStackShowUnchanged = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBlendStackShowUnchanged(
	TEXT("GameplayCameras.Debug.BlendStack.ShowUnchanged"),
	GGameplayCamerasDebugBlendStackShowUnchanged,
	TEXT(""));

bool GGameplayCamerasDebugBlendStackShowVariableIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBlendStackShowVariableIDs(
	TEXT("GameplayCameras.Debug.BlendStack.ShowVariableIDs"),
	GGameplayCamerasDebugBlendStackShowVariableIDs,
	TEXT(""));

bool GGameplayCamerasDebugBlendStackShowDataIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBlendStackShowDataIDs(
	TEXT("GameplayCameras.Debug.BlendStack.ShowDataIDs"),
	GGameplayCamerasDebugBlendStackShowDataIDs,
	TEXT(""));

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBlendStackCameraNodeEvaluator)

FBlendStackCameraNodeEvaluator::FBlendStackCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
	bAutoCameraPoseMovementTrail = false;
#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
}

FBlendStackCameraNodeEvaluator::~FBlendStackCameraNodeEvaluator()
{
	// Pop all our entries to unregister the live-edit callbacks.
	PopEntries(Entries.Num());
}

void FBlendStackCameraNodeEvaluator::InitializeEntry(
		FCameraRigEntry& NewEntry, 
		const UCameraRigAsset* CameraRig,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		UBlendStackRootCameraNode* EntryRootNode,
		bool bIsActiveEntry)
{
	// Clear the evaluator hierarchy in case we are hot-reloading an entry.
	NewEntry.EvaluatorHierarchy.Reset();

	// Generate the hierarchy of node evaluators inside our storage buffer.
	FCameraNodeEvaluatorTreeBuildParams BuildParams;
	BuildParams.RootCameraNode = EntryRootNode;
	BuildParams.AllocationInfo = &CameraRig->AllocationInfo.EvaluatorInfo;
	FCameraNodeEvaluator* RootEvaluator = NewEntry.EvaluatorStorage.BuildEvaluatorTree(BuildParams);

	// Allocate variable table and context data table.
	NewEntry.ContextResult.VariableTable.Initialize(CameraRig->AllocationInfo.VariableTableInfo);
	NewEntry.ContextResult.ContextDataTable.Initialize(CameraRig->AllocationInfo.ContextDataTableInfo);
	NewEntry.Result.VariableTable.Initialize(CameraRig->AllocationInfo.VariableTableInfo);
	NewEntry.Result.ContextDataTable.Initialize(CameraRig->AllocationInfo.ContextDataTableInfo);

	// Set all the data from the context.
	const FCameraNodeEvaluationResult& ContextResult = EvaluationContext->GetInitialResult();
	NewEntry.ContextResult.VariableTable.OverrideAll(ContextResult.VariableTable, true);
	NewEntry.ContextResult.ContextDataTable.OverrideAll(ContextResult.ContextDataTable);

	// Add some conditional result if necessary.
	if (bIsActiveEntry && EvaluationContext)
	{
		const FCameraNodeEvaluationResult* ActiveOnlyResult = EvaluationContext->GetConditionalResult(ECameraEvaluationDataCondition::ActiveCameraRig);
		if (ActiveOnlyResult)
		{
			NewEntry.ContextResult.VariableTable.OverrideAll(ActiveOnlyResult->VariableTable, true);
			NewEntry.ContextResult.ContextDataTable.OverrideAll(ActiveOnlyResult->ContextDataTable);
		}
	}

	// Set default values for unset entries in the variable and context data tables.
	FCameraObjectInterfaceParameterOverrideHelper Helper(NewEntry.ContextResult);
	Helper.ApplyParameterDefaults(CameraRig);

	// Initialize the node evaluators.
	if (RootEvaluator)
	{
		FCameraNodeEvaluatorInitializeParams InitParams(&NewEntry.EvaluatorHierarchy);
		InitParams.Evaluator = OwningEvaluator;
		InitParams.EvaluationContext = EvaluationContext;
		if (!bIsActiveEntry)
		{
			InitParams.LastActiveCameraRigInfo = GetActiveCameraRigEvaluationInfo();
		}
		InitParams.Layer = Layer;
		RootEvaluator->Initialize(InitParams, NewEntry.ContextResult);  // Initializing with the context result here.
	}

	// Apply the initialized result.
	NewEntry.Result.OverrideAll(NewEntry.ContextResult, true);

	// Wrap up!
	NewEntry.EntryID = FBlendStackEntryID(NextEntryID++);
	NewEntry.EvaluationContext = EvaluationContext;
	NewEntry.CameraRig = CameraRig;
	NewEntry.RootNode = EntryRootNode;
	NewEntry.Flags.bWasContextInitialResultValid = EvaluationContext->GetInitialResult().bIsValid;
	NewEntry.Flags.bIsFirstFrame = true;
	if (RootEvaluator)
	{
		NewEntry.RootEvaluator = RootEvaluator->CastThisChecked<FBlendStackRootCameraNodeEvaluator>();
	}
}

int32 FBlendStackCameraNodeEvaluator::IndexOfEntry(const FBlendStackEntryID EntryID) const
{
	return Entries.IndexOfByPredicate([EntryID](const FCameraRigEntry& Item)
			{
				return Item.EntryID == EntryID;
			});
}

void FBlendStackCameraNodeEvaluator::FreezeEntry(FCameraRigEntry& Entry)
{
	if (Entry.RootEvaluator)
	{
		// Freeze the blend.
		FBlendCameraNodeEvaluator* BlendEvaluator = Entry.RootEvaluator->GetBlendEvaluator();
		if (BlendEvaluator)
		{
			BlendEvaluator->Freeze();
		}

		// Teardown and destroy the evaluation tree.
		FCameraNodeEvaluator* RootEvaluator = Entry.RootEvaluator->GetRootEvaluator();
		if (RootEvaluator)
		{
			FCameraNodeEvaluatorTeardownParams TeardownParams;
			TeardownParams.Evaluator = this->OwningEvaluator;
			TeardownParams.EvaluationContext = Entry.EvaluationContext.Pin();
			TeardownParams.Layer = Layer;

			RootEvaluator->Teardown(TeardownParams);

			Entry.EvaluatorStorage.DestroyEvaluatorTree(RootEvaluator, true);

			Entry.RootEvaluator->SetRootEvaluator(nullptr);
		}
	}

	Entry.EvaluatorHierarchy.Reset();

	Entry.RootNode = nullptr;

	Entry.EvaluationContext.Reset();
	
	Entry.Flags.bIsFrozen = true;
}

FCameraRigEvaluationInfo FBlendStackCameraNodeEvaluator::GetActiveCameraRigEvaluationInfo() const
{
	if (Entries.Num() > 0)
	{
		const FCameraRigEntry& ActiveEntry = Entries.Last();
		FCameraRigEvaluationInfo Info(
				FCameraRigInstanceID::FromBlendStackEntryID(ActiveEntry.EntryID, Layer),
				ActiveEntry.EvaluationContext.Pin(),
				ActiveEntry.CameraRig, 
				&ActiveEntry.Result,
				ActiveEntry.RootEvaluator ? ActiveEntry.RootEvaluator->GetRootEvaluator() : nullptr);
		return Info;
	}
	return FCameraRigEvaluationInfo();
}

FCameraRigEvaluationInfo FBlendStackCameraNodeEvaluator::GetCameraRigEvaluationInfo(FBlendStackEntryID EntryID) const
{
	const int32 EntryIndex = IndexOfEntry(EntryID);
	if (EntryIndex != INDEX_NONE)
	{
		const FCameraRigEntry& Entry = Entries[EntryIndex];
		FCameraRigEvaluationInfo Info(
				FCameraRigInstanceID::FromBlendStackEntryID(Entry.EntryID, Layer),
				Entry.EvaluationContext.Pin(),
				Entry.CameraRig, 
				&Entry.Result,
				Entry.RootEvaluator ? Entry.RootEvaluator->GetRootEvaluator() : nullptr);
		return Info;
	}
	return FCameraRigEvaluationInfo();
}

bool FBlendStackCameraNodeEvaluator::HasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const
{
	for (const FCameraRigEntry& Entry : Entries)
	{
		if (Entry.EvaluationContext == InContext)
		{
			return true;
		}
	}
	return false;
}

FCameraNodeEvaluatorChildrenView FBlendStackCameraNodeEvaluator::OnGetChildren()
{
	FCameraNodeEvaluatorChildrenView View;
	for (FCameraRigEntry& Entry : Entries)
	{
		if (Entry.RootEvaluator)
		{
			View.Add(Entry.RootEvaluator);
		}
	}
	return View;
}

void FBlendStackCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	OwningEvaluator = Params.Evaluator;

	const UBlendStackCameraNode* BlendStack = GetCameraNodeAs<UBlendStackCameraNode>();
	Layer = BlendStack->Layer;
}

void FBlendStackCameraNodeEvaluator::OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	ensure(Params.Evaluator == OwningEvaluator);

	// Execute operations from top to bottom.
	for (int32 Index = Entries.Num() - 1; Index >= 0; --Index)
	{
		FCameraRigEntry& Entry(Entries[Index]);
		TSharedPtr<const FCameraEvaluationContext> CurContext = Entry.EvaluationContext.Pin();

		if (!Entry.Flags.bIsFrozen)
		{
			FCameraOperationParams CurParams(Params);
			CurParams.EvaluationContext = CurContext;
			Entry.EvaluatorHierarchy.CallExecuteOperation(CurParams, Operation);
		}
	}
}

void FBlendStackCameraNodeEvaluator::ResolveEntries(const FCameraNodeEvaluationParams& Params, TArray<FResolvedEntry>& OutResolvedEntries)
{
	constexpr ECameraVariableTableFilter VariableTableFilter = ECameraVariableTableFilter::ChangedOnly;
	constexpr ECameraContextDataTableFilter ContextDataTableFilter = ECameraContextDataTableFilter::ChangedOnly;

	// Build up these structures so we don't re-resolve evaluation context weak-pointers
	// multiple times in this function..
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		FCameraRigEntry& Entry(Entries[Index]);
		TSharedPtr<const FCameraEvaluationContext> CurContext = Entry.EvaluationContext.Pin();

		FResolvedEntry& ResolvedEntry = OutResolvedEntries.Emplace_GetRef(Entry, CurContext);
		ResolvedEntry.EntryIndex = Index;
		if (Index == Entries.Num() - 1)
		{
			ResolvedEntry.bIsActiveEntry = true;
		}

		// While we make these resolved entries, emit warnings and errors as needed.
		if (!Entry.Flags.bIsFrozen)
		{
			// Check that we still have a valid context. If not, let's freeze the entry, since
			// we won't be able to evaluate it anymore... unless we are in a stateless evaluation,
			// in which case we're not supposed to change the structure of the camera node 
			// evaluator tree.
			if (UNLIKELY(!CurContext.IsValid()))
			{
				if (!Params.IsStatelessEvaluation())
				{
					FreezeEntry(Entry);

#if UE_GAMEPLAY_CAMERAS_TRACE
					if (Entry.Flags.bLogWarnings)
					{
						UE_LOGF(LogCameraSystem, Warning,
							"Freezing camera rig '%ls' because its evaluation context isn't valid anymore.",
							*GetNameSafe(Entry.CameraRig));
						Entry.Flags.bLogWarnings = false;
					}
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
				}

				continue;
			}

			// Check that we have a valid result for this context.
			const FCameraNodeEvaluationResult& ContextResult(CurContext->GetInitialResult());
			if (UNLIKELY(!ContextResult.bIsValid))
			{
#if UE_GAMEPLAY_CAMERAS_TRACE
				if (Entry.Flags.bLogWarnings)
				{
					UE_LOGF(LogCameraSystem, Warning,
							"Camera rig '%ls' may experience a hitch because its initial result isn't valid.",
							*GetNameSafe(Entry.CameraRig));
					Entry.Flags.bLogWarnings = false;
				}
#endif  // UE_GAMEPLAY_CAMERAS_TRACE

				continue;
			}

			// If the context was previously invalid, and this isn't the first frame, flag
			// this update as a camera cut.
			if (UNLIKELY(!Entry.Flags.bWasContextInitialResultValid && !Entry.Flags.bIsFirstFrame))
			{
				Entry.Flags.bForceCameraCut = true;
			}
			Entry.Flags.bWasContextInitialResultValid = true;

			// Reset this entry's flags for this frame.
			FCameraNodeEvaluationResult& CurResult = Entry.Result;
			CurResult.ResetFrameFlags();

			// Bring the entry's context result up to date with any changes.
			Entry.ContextResult.CameraPose.OverrideChanged(ContextResult.CameraPose);
			Entry.ContextResult.VariableTable.Override(ContextResult.VariableTable, VariableTableFilter);
			Entry.ContextResult.ContextDataTable.Override(ContextResult.ContextDataTable, ContextDataTableFilter);
			if (ResolvedEntry.bIsActiveEntry)
			{
				if (const FCameraNodeEvaluationResult* ActiveOnlyResult = ResolvedEntry.Context->GetConditionalResult(ECameraEvaluationDataCondition::ActiveCameraRig))
				{
					Entry.ContextResult.VariableTable.Override(ActiveOnlyResult->VariableTable, VariableTableFilter);
					Entry.ContextResult.ContextDataTable.Override(ActiveOnlyResult->ContextDataTable, ContextDataTableFilter);
				}
			}
			Entry.ContextResult.bIsCameraCut = ContextResult.bIsCameraCut;
			Entry.ContextResult.bIsValid = ContextResult.bIsValid;
		}
		// else: frozen entries may have null contexts or invalid initial results
		//       because we're not going to update them anyway. We will however blend
		//       them so we add them to the list of entries too.

#if UE_GAMEPLAY_CAMERAS_TRACE
		// This entry might have has warnings before. It's valid now, so let's
		// re-enable warnings if it becomes invalid again in the future.
		Entry.Flags.bLogWarnings = true;
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
	}
}

void FBlendStackCameraNodeEvaluator::OnRunFinished(FCameraNodeEvaluationResult& OutResult)
{
	// Reset transient flags.
	for (FCameraRigEntry& Entry : Entries)
	{
		Entry.Flags.bIsFirstFrame = false;
		Entry.Flags.bForceCameraCut = false;
	}

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

	// Append the motion trail of the active entry so that we see all the steps it took
	// to get to the end result. Also add an extra point for the actual final result,
	// to represent the difference between the active result and the blended result.
	// In theory, this extra segment should blend into nothingness over time.
	if (Entries.Num() > 0)
	{
		const FCameraRigEntry& ActiveEntry = Entries.Last();
		OutResult.AppendCameraPoseLocationTrail(ActiveEntry.Result);

		OutResult.AddCameraPoseTrailPointIfNeeded();
	}

#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
}

void FBlendStackCameraNodeEvaluator::PopEntry(int32 EntryIndex)
{
	if (!ensure(Entries.IsValidIndex(EntryIndex)))
	{
		return;
	}

	FCameraRigEntry& Entry = Entries[EntryIndex];
#if WITH_EDITOR
	RemoveListenedPackages(Entry);
#endif  // WITH_EDITOR

	if (OnCameraRigEventDelegate.IsBound())
	{
		BroadcastCameraRigEvent(EBlendStackCameraRigEventType::Popped, Entry);
	}

	Entries.RemoveAt(EntryIndex);
}

void FBlendStackCameraNodeEvaluator::PopEntries(int32 FirstIndexToKeep)
{
	if (UNLIKELY(Entries.IsEmpty()))
	{
		return;
	}

#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = FModuleManager::GetModuleChecked<IGameplayCamerasModule>("GameplayCameras");
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager();
#endif  // WITH_EDITOR

	for (int32 Index = 0; Index < FirstIndexToKeep; ++Index)
	{
		FCameraRigEntry& FirstEntry = Entries[0];

		if (FirstEntry.RootEvaluator)
		{
			FCameraNodeEvaluatorTeardownParams TeardownParams;
			TeardownParams.Evaluator = this->OwningEvaluator;
			TeardownParams.EvaluationContext = FirstEntry.EvaluationContext.Pin();
			TeardownParams.Layer = Layer;

			FirstEntry.RootEvaluator->Teardown(TeardownParams);
		}

#if WITH_EDITOR
		RemoveListenedPackages(LiveEditManager, FirstEntry);
#endif  // WITH_EDITOR

		if (OnCameraRigEventDelegate.IsBound())
		{
			BroadcastCameraRigEvent(EBlendStackCameraRigEventType::Popped, FirstEntry);
		}

		Entries.RemoveAt(0);
	}
}

#if WITH_EDITOR

void FBlendStackCameraNodeEvaluator::AddPackageListeners(FCameraRigEntry& Entry)
{
	if (!ensure(Entry.CameraRig))
	{
		return;
	}

	IGameplayCamerasModule& GameplayCamerasModule = FModuleManager::GetModuleChecked<IGameplayCamerasModule>("GameplayCameras");
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager();
	if (!LiveEditManager)
	{
		return;
	}

	FCameraRigPackages EntryPackages;
	Entry.CameraRig->GatherPackages(EntryPackages);

	Entry.ListenedPackages.Reset();
	Entry.ListenedPackages.Append(EntryPackages);

	for (const UPackage* ListenPackage : EntryPackages)
	{
		int32& NumListens = AllListenedPackages.FindOrAdd(ListenPackage, 0);
		if (NumListens == 0)
		{
			LiveEditManager->AddListener(ListenPackage, this);
		}
		++NumListens;
	}
}

void FBlendStackCameraNodeEvaluator::RemoveListenedPackages(FCameraRigEntry& Entry)
{
	IGameplayCamerasModule& GameplayCamerasModule = FModuleManager::GetModuleChecked<IGameplayCamerasModule>("GameplayCameras");
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager();
	RemoveListenedPackages(LiveEditManager, Entry);
}

void FBlendStackCameraNodeEvaluator::RemoveListenedPackages(TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager, FCameraRigEntry& Entry)
{
	if (!LiveEditManager)
	{
		return;
	}

	for (TWeakObjectPtr<const UPackage> WeakListenPackage : Entry.ListenedPackages)
	{
		int32* NumListens = AllListenedPackages.Find(WeakListenPackage);
		if (ensure(NumListens))
		{
			--(*NumListens);
			if (*NumListens == 0)
			{
				if (const UPackage* ListenPackage = WeakListenPackage.Get())
				{
					LiveEditManager->RemoveListener(ListenPackage, this);
				}
				AllListenedPackages.Remove(WeakListenPackage);
			}
		}
	}

	Entry.ListenedPackages.Reset();
}

#endif  // WITH_EDITOR

void FBlendStackCameraNodeEvaluator::BroadcastCameraRigEvent(EBlendStackCameraRigEventType EventType, const FCameraRigEntry& Entry, const UCameraRigTransition* Transition) const
{
	FBlendStackCameraRigEvent Event;
	Event.EventType = EventType;
	Event.BlendStackEvaluator = this;
	Event.CameraRigInfo = FCameraRigEvaluationInfo(
			FCameraRigInstanceID::FromBlendStackEntryID(Entry.EntryID, Layer),
			Entry.EvaluationContext.Pin(),
			Entry.CameraRig,
			&Entry.Result,
			Entry.RootEvaluator);
	Event.Transition = Transition;

	OnCameraRigEventDelegate.Broadcast(Event);
}

void FBlendStackCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	for (FCameraRigEntry& Entry : Entries)
	{
		Collector.AddReferencedObject(Entry.CameraRig);
		Collector.AddReferencedObject(Entry.RootNode);
		Entry.ContextResult.AddReferencedObjects(Collector);
		Entry.Result.AddReferencedObjects(Collector);
	}
}

void FBlendStackCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	int32 NumEntriesToSerialize = Entries.Num();

	if (Ar.IsSaving())
	{
		int32 NumEntries = Entries.Num();
		Ar << NumEntries;
	}
	else if (Ar.IsLoading())
	{
		int32 LoadedNumEntries = 0;
		Ar << LoadedNumEntries;

		ensureMsgf(
				LoadedNumEntries == Entries.Num(),
				TEXT("The number of entries changed since this blend stack was serialized!"));
		NumEntriesToSerialize = LoadedNumEntries;
	}

	for (int32 Index = 0; Index < NumEntriesToSerialize; ++Index)
	{
		FCameraRigEntry& Entry(Entries[Index]);
		Entry.ContextResult.Serialize(Ar);
		Entry.Result.Serialize(Ar);
		Ar.SerializeBits(static_cast<void*>(&Entry.Flags), sizeof(FCameraRigEntry::Flags));
	}
}

#if WITH_EDITOR

void FBlendStackCameraNodeEvaluator::OnPostBuildAsset(const FGameplayCameraAssetBuildEvent& BuildEvent)
{
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		FCameraRigEntry& Entry(Entries[Index]);
		const bool bRebuildEntry = Entry.ListenedPackages.Contains(BuildEvent.AssetPackage);
		if (!bRebuildEntry)
		{
			continue;
		}

		if (!Entry.Flags.bIsFrozen)
		{
			// Destroy the existing entry evaluators.
			Entry.EvaluatorStorage.DestroyEvaluatorTree();
			Entry.EvaluatorHierarchy.Reset();

			// Re-assign the root node in case the camera rig's root was changed.
			Entry.RootNode->RootNode = Entry.CameraRig->RootNode;

			// Remove the blend on the root node, since we don't want the reloaded camera rig to re-blend-in
			// for no good reason. This might "pop" if we reloaded this entry while it was blending, but
			// that's acceptable.
			Entry.RootNode->Blend = NewObject<UPopBlendCameraNode>(Entry.RootNode, NAME_None);

			// Rebuild the evaluator tree.
			InitializeEntry(
					Entry,
					Entry.CameraRig,
					Entry.EvaluationContext.Pin(),
					Entry.RootNode,
					Index == Entries.Num() - 1);
		}
		else
		{
			// Leave things frozen, but also remove all the frozen data, in case the new data
			// has different types or names for the same variables, because the user changed them.
			Entry.Result.VariableTable.UnsetAllValues();
			Entry.Result.ContextDataTable.UnsetAllValues();
		}

		OnEntryReinitialized(Index);
	}
}

#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FBlendStackEntryCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(int32, EntryIndex)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FLinearColor, EntryColor)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FBlendStackEntryID, EntryID)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, CameraRigName)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsFrozen)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FBlendStackEntryCameraDebugBlock)

void FBlendStackCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FBlendStackSummaryCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FBlendStackSummaryCameraDebugBlock>(*this);
	for (const FCameraRigEntry& Entry : Entries)
	{
		DebugBlock.AddChild(&Builder.BuildDebugBlock<FCameraPoseLocationTrailDebugBlock>(Entry.Result));
	}
}

FBlendStackCameraDebugBlock* FBlendStackCameraNodeEvaluator::BuildDetailedDebugBlock(
		const FCameraDebugBlockBuildParams& Params, 
		const FLinearColor& StartColor, const FLinearColor& EndColor,
		FCameraDebugBlockBuilder& Builder)
{
	FBlendStackCameraDebugBlock& StackDebugBlock = Builder.BuildDebugBlock<FBlendStackCameraDebugBlock>();
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		const FCameraRigEntry& Entry(Entries[Index]);

		const FLinearColor EntryColor = LerpLinearColorUsingHSV(
				StartColor, EndColor, Index, Entries.Num());

		// Each entry has a wrapper debug block with 2 children blocks:
		// - block for the blend
		// - block for the result
		FBlendStackEntryCameraDebugBlock& EntryDebugBlock = Builder.BuildDebugBlock<FBlendStackEntryCameraDebugBlock>();
		EntryDebugBlock.EntryIndex = Index;
		EntryDebugBlock.EntryColor = EntryColor;
		EntryDebugBlock.EntryID = Entry.EntryID;
		EntryDebugBlock.CameraRigName = GetNameSafe(Entry.CameraRig);
		EntryDebugBlock.bIsFrozen = Entry.Flags.bIsFrozen;
		StackDebugBlock.AddChild(&EntryDebugBlock);
		{
			FCameraNodeEvaluator* BlendEvaluator = Entry.RootEvaluator ? Entry.RootEvaluator->GetBlendEvaluator() : nullptr;
			if (BlendEvaluator)
			{
				Builder.StartParentDebugBlockOverride(EntryDebugBlock);
				{
					BlendEvaluator->BuildDebugBlocks(Params, Builder);
				}
				Builder.EndParentDebugBlockOverride();
			}
			else
			{
				// Dummy debug block.
				EntryDebugBlock.AddChild(&Builder.BuildDebugBlock<FCameraDebugBlock>());
			}

			FCameraNodeEvaluationResultDebugBlock& ResultDebugBlock = Builder.BuildDebugBlock<FCameraNodeEvaluationResultDebugBlock>();
			EntryDebugBlock.AddChild(&ResultDebugBlock);
			{
				ResultDebugBlock.Initialize(Entry.Result, Builder);
				ResultDebugBlock.GetCameraPoseDebugBlock()->WithShowUnchangedCVar(TEXT("GameplayCameras.Debug.BlendStack.ShowUnchanged"));
				ResultDebugBlock.GetVariableTableDebugBlock()->WithShowVariableIDsCVar(TEXT("GameplayCameras.Debug.BlendStack.ShowVariableIDs"));
				ResultDebugBlock.GetContextDataTableDebugBlock()->WithShowDataIDsCVar(TEXT("GameplayCameras.Debug.BlendStack.ShowDataIDs"));
			}
		}
	}
	return &StackDebugBlock;
}

UE_DEFINE_CAMERA_DEBUG_BLOCK(FBlendStackSummaryCameraDebugBlock);

FBlendStackSummaryCameraDebugBlock::FBlendStackSummaryCameraDebugBlock()
{
}

FBlendStackSummaryCameraDebugBlock::FBlendStackSummaryCameraDebugBlock(const FBlendStackCameraNodeEvaluator& InEvaluator)
{
	const UBlendStackCameraNode* BlendStack = InEvaluator.GetCameraNodeAs<UBlendStackCameraNode>();

	NumEntries = InEvaluator.Entries.Num();
	BlendStackType = BlendStack->BlendStackType;
	BlendStackLayer = BlendStack->Layer;
}

void FBlendStackSummaryCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (BlendStackLayer != ECameraRigLayer::None)
	{
		UEnum* LayerEnum = StaticEnum<ECameraRigLayer>();
		Renderer.AddText(TEXT("(%s) "), *LayerEnum->GetDisplayNameTextByValue((int64)BlendStackLayer).ToString());
	}
	Renderer.AddText(TEXT("%d entries"), NumEntries);
}

void FBlendStackSummaryCameraDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << NumEntries;
	Ar << BlendStackType;
	Ar << BlendStackLayer;
}

void FBlendStackEntryCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(
			TEXT("{cam_passive}[%d ID=%s] {cam_notice}%s{cam_default}"),
			EntryIndex + 1, *LexToString(EntryID), *CameraRigName);
	if (bIsFrozen)
	{
		Renderer.AddText(" {cam_notice2}FROZEN{cam_default}");
	}
	Renderer.NewLine();

	// Render children with some color/label override for the entry's result pose.
	{
		FScopedGlobalCameraPoseRenderingParams ScopedParams(*LexToString(EntryIndex + 1), EntryColor);

		TArrayView<FCameraDebugBlock*> ChildrenBlocks(GetChildren());
		for (FCameraDebugBlock* ChildBlock : ChildrenBlocks)
		{
			ChildBlock->DebugDraw(Params, Renderer);
		}

		Renderer.SkipChildrenBlocks();
	}
}

UE_DEFINE_CAMERA_DEBUG_BLOCK(FBlendStackCameraDebugBlock);

void FBlendStackCameraDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << StartEndColors;
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

