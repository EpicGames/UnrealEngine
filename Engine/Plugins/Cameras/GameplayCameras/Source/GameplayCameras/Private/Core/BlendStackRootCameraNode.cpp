// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BlendStackRootCameraNode.h"

#include "Algo/Transform.h"
#include "Core/BlendCameraNode.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"
#include "Core/CameraVariableTableAllocationInfo.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "Nodes/Common/CameraRigCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendStackRootCameraNode)

UBlendStackRootCameraNode::UBlendStackRootCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView UBlendStackRootCameraNode::OnGetChildren()
{
	FCameraNodeChildrenView Children;
	if (Blend)
	{
		Children.Add(Blend);
	}
	if (RootNode)
	{
		Children.Add(RootNode);
	}
	return Children;
}

FCameraNodeEvaluatorPtr UBlendStackRootCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FBlendStackRootCameraNodeEvaluator>();
}

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBlendStackRootCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FBlendStackRootCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, CameraRigAssetName)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TArray<FString>, BlendedParameterOverridesEntries)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FBlendStackRootCameraDebugBlock)

FBlendStackRootCameraNodeEvaluator::FBlendStackRootCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsParameterUpdate);
}

FCameraNodeEvaluatorChildrenView FBlendStackRootCameraNodeEvaluator::OnGetChildren()
{
	FCameraNodeEvaluatorChildrenView Children;
	if (BlendEvaluator)
	{
		Children.Add(BlendEvaluator);
	}
	for (FBlendedParameterOverrides& BlendedParameterOverrides : BlendedParameterOverridesStack)
	{
		Children.Add(BlendedParameterOverrides.BlendEvaluator);
	}
	if (RootEvaluator)
	{
		Children.Add(RootEvaluator);
	}
	return Children;
}

void FBlendStackRootCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UBlendStackRootCameraNode* RootNode = GetCameraNodeAs<UBlendStackRootCameraNode>();

	BlendEvaluator = Params.BuildEvaluatorAs<FBlendCameraNodeEvaluator>(RootNode->Blend);
	RootEvaluator = Params.BuildEvaluator(RootNode->RootNode);
}

void FBlendStackRootCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UBlendStackRootCameraNode* RootNode = GetCameraNodeAs<UBlendStackRootCameraNode>();
	if (RootNode->RootNode)
	{
		const UCameraRigAsset* CameraRig = RootNode->RootNode->GetTypedOuter<const UCameraRigAsset>();
		BlendablePrefabCameraRig = FindInnermostCameraRigPrefab(CameraRig);

#if UE_GAMEPLAY_CAMERAS_DEBUG
		CameraRigAssetName = GetNameSafe(CameraRig);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	}
}

void FBlendStackRootCameraNodeEvaluator::SetBlendEvaluator(FBlendCameraNodeEvaluator* InBlendEvaluator)
{
	BlendEvaluator = InBlendEvaluator;
}

void FBlendStackRootCameraNodeEvaluator::SetRootEvaluator(FCameraNodeEvaluator* InRootEvaluator)
{
	RootEvaluator = InRootEvaluator;
}

ECameraRigMergingEligibility FBlendStackRootCameraNodeEvaluator::CompareCameraRigForMerging(const UCameraRigAsset* CameraRig) const
{
	const UCameraRigAsset* NewCameraRigPrefab = FindInnermostCameraRigPrefab(CameraRig);

	if (NewCameraRigPrefab != BlendablePrefabCameraRig)
	{
		return ECameraRigMergingEligibility::Different;
	}

	if (BlendedParameterOverridesStack.IsEmpty())
	{
		return ECameraRigMergingEligibility::EligibleForMerge;
	}
	
	const FBlendedParameterOverrides& TopEntry = BlendedParameterOverridesStack.Top();
	if (TopEntry.CameraRig == CameraRig)
	{
		return ECameraRigMergingEligibility::Active;
	}

	return ECameraRigMergingEligibility::EligibleForMerge;
}

void FBlendStackRootCameraNodeEvaluator::MergeCameraRig(
		const FCameraNodeEvaluatorBuildParams& BuildParams,
		const FCameraNodeEvaluatorInitializeParams& InitParams,
		FCameraNodeEvaluationResult& InitResult,
		const UCameraRigAsset* CameraRig,
		const UBlendCameraNode* Blend)
{
	if (!ensureMsgf(CameraRig, TEXT("No camera rig given.")))
	{
		return;
	}

	if (!ensureMsgf(BlendablePrefabCameraRig,
				TEXT("Adding blended parameter overrides for a camera rig that doesn't support it.")))
	{
		return;
	}

	InitializeBlendedParameterOverridesStack();

	FBlendedParameterOverrides BlendedParameterOverrides;
	BlendedParameterOverrides.CameraRig = CameraRig;
	BlendedParameterOverrides.Blend = Blend;

	BuildNestedPrefabTrail(CameraRig, BlendedParameterOverrides.PrefabTrail);

	const FCameraObjectAllocationInfo& AllocationInfo = BlendablePrefabCameraRig->AllocationInfo;
	BlendedParameterOverrides.Result.VariableTable.Initialize(AllocationInfo.VariableTableInfo);

	if (Blend)
	{
		BlendedParameterOverrides.BlendEvaluator = BuildParams.BuildEvaluatorAs<FBlendCameraNodeEvaluator>(Blend);
	}
	if (BlendedParameterOverrides.BlendEvaluator)
	{
		BlendedParameterOverrides.BlendEvaluator->Initialize(InitParams, InitResult);
	}

	BlendedParameterOverridesStack.Add(MoveTemp(BlendedParameterOverrides));

	// Show the new active camera rig in the debug info.
#if UE_GAMEPLAY_CAMERAS_DEBUG
	CameraRigAssetName = GetNameSafe(CameraRig);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

const UCameraRigAsset* FBlendStackRootCameraNodeEvaluator::FindInnermostCameraRigPrefab(const UCameraRigAsset* CameraRig)
{
	if (CameraRig)
	{
		TArray<TObjectPtr<const UCameraRigCameraNode>> PrefabTrail;
		return BuildNestedPrefabTrail(CameraRig, PrefabTrail);
	}
	return nullptr;
}

FCameraNodeEvaluator* FBlendStackRootCameraNodeEvaluator::FindInnermostCameraRigEvaluator(FCameraNodeEvaluator* CameraNodeEvaluator)
{
	if (CameraNodeEvaluator)
	{
		TArray<FCameraRigCameraNodeEvaluator*> EvaluatorTrail;
		return BuildNestedEvaluatorTrail(CameraNodeEvaluator, EvaluatorTrail);
	}
	return CameraNodeEvaluator;
}

void FBlendStackRootCameraNodeEvaluator::InitializeBlendedParameterOverridesStack()
{
	if (!BlendedParameterOverridesStack.IsEmpty())
	{
		return;
	}

	// Swap out the current root evaluator for the innermost rig one, because we want to apply parameter
	// overrides ourselves from now on.
	FCameraNodeEvaluator* InnermostRootEvaluator = FindInnermostCameraRigEvaluator(RootEvaluator);
	RootEvaluator = InnermostRootEvaluator;

	const UBlendStackRootCameraNode* ThisNode = GetCameraNodeAs<UBlendStackRootCameraNode>();
	const UCameraRigAsset* OriginalCameraRig = ThisNode->RootNode->GetTypedOuter<const UCameraRigAsset>();

	FBlendedParameterOverrides InitialParameterOverrides;
	InitialParameterOverrides.CameraRig = OriginalCameraRig;
	// No blend, the initial entry is always at 100%.

	BuildNestedPrefabTrail(OriginalCameraRig, InitialParameterOverrides.PrefabTrail);

	const FCameraObjectAllocationInfo& AllocationInfo = BlendablePrefabCameraRig->AllocationInfo;
	InitialParameterOverrides.Result.VariableTable.Initialize(AllocationInfo.VariableTableInfo);

	BlendedParameterOverridesStack.Add(MoveTemp(InitialParameterOverrides));
}

const UCameraRigAsset* FBlendStackRootCameraNodeEvaluator::BuildNestedPrefabTrail(const UCameraRigAsset* CameraRig, TArray<TObjectPtr<const UCameraRigCameraNode>>& OutPrefabNodes)
{
	if (const UCameraRigCameraNode* PrefabNode = Cast<const UCameraRigCameraNode>(CameraRig->RootNode))
	{
		if (ensureMsgf(!OutPrefabNodes.Contains(PrefabNode), TEXT("Circular camera rig prefab reference detected!")))
		{
			OutPrefabNodes.Add(PrefabNode);

			if (const UCameraRigAsset* InnerCameraRig = PrefabNode->CameraRigReference.GetCameraRig())
			{
				return BuildNestedPrefabTrail(InnerCameraRig, OutPrefabNodes);
			}
		}
	}
	return CameraRig;
}

FCameraNodeEvaluator* FBlendStackRootCameraNodeEvaluator::BuildNestedEvaluatorTrail(FCameraNodeEvaluator* CameraNodeEvaluator, TArray<FCameraRigCameraNodeEvaluator*>& OutPrefabEvaluators)
{
	if (FCameraRigCameraNodeEvaluator* PrefabNodeEvaluator = CameraNodeEvaluator->CastThis<FCameraRigCameraNodeEvaluator>())
	{
		if (ensureMsgf(!OutPrefabEvaluators.Contains(PrefabNodeEvaluator), TEXT("Circular camera rig prefab reference detected!")))
		{
			OutPrefabEvaluators.Add(PrefabNodeEvaluator);

			if (FCameraNodeEvaluator* InnerNodeEvaluator = PrefabNodeEvaluator->GetCameraRigRootEvaluator())
			{
				return BuildNestedEvaluatorTrail(InnerNodeEvaluator, OutPrefabEvaluators);
			}
		}
	}
	return CameraNodeEvaluator;
}

void FBlendStackRootCameraNodeEvaluator::OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	RunBlendedParameterOverridesStack(Params, OutResult);
}

void FBlendStackRootCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (BlendEvaluator)
	{
		BlendEvaluator->Run(Params, OutResult);
	}
	if (RootEvaluator)
	{
		RootEvaluator->Run(Params, OutResult);
	}
}

void FBlendStackRootCameraNodeEvaluator::RunBlendedParameterOverridesStack(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	if (BlendedParameterOverridesStack.IsEmpty())
	{
		return;
	}

	int32 PopEntriesBelow = INDEX_NONE;
	for (int32 EntryIndex = 0; EntryIndex < BlendedParameterOverridesStack.Num(); ++EntryIndex)
	{
		FBlendedParameterOverrides& BlendedParameterOverrides(BlendedParameterOverridesStack[EntryIndex]);
		FCameraNodeEvaluationResult& CurResult(BlendedParameterOverrides.Result);

		// Start by setting the default values of all parameters. If we don't do this, parameter overrides
		// wouldn't have a base value to blend from.
		FCameraObjectInterfaceParameterOverrideHelper Helper(&CurResult.VariableTable, nullptr);
		Helper.ApplyParameterDefaults(BlendablePrefabCameraRig);

		// Next, override the defaults with the specific values of this entry, applied bottoms up.
		for (int32 Index = BlendedParameterOverrides.PrefabTrail.Num() - 1; Index >= 0; --Index)
		{
			const UCameraRigCameraNode* CurPrefabNode = BlendedParameterOverrides.PrefabTrail[Index];
			CurPrefabNode->CameraRigReference.ApplyParameterOverrides(CurResult.VariableTable, false);
		}

		// Finally, update the parameter overrides' blend, and apply it.
		if (BlendedParameterOverrides.BlendEvaluator)
		{
			BlendedParameterOverrides.BlendEvaluator->Run(Params.EvaluationParams, CurResult);

			FCameraNodePreBlendParams BlendParams(Params.EvaluationParams, Params.LastCameraPose, CurResult.VariableTable);
			BlendParams.VariableTableFilter = ECameraVariableTableFilter::InputOnly;
			FCameraNodePreBlendResult BlendResult(OutResult.VariableTable);
			BlendedParameterOverrides.BlendEvaluator->BlendParameters(BlendParams, BlendResult);

			if (BlendResult.bIsBlendFinished && BlendResult.bIsBlendFull)
			{
				PopEntriesBelow = EntryIndex;
			}
		}
		else
		{
			OutResult.VariableTable.Override(CurResult.VariableTable, ECameraVariableTableFilter::InputOnly);

			PopEntriesBelow = EntryIndex;
		}
	}
	if (PopEntriesBelow > 0)
	{
		BlendedParameterOverridesStack.RemoveAt(0, PopEntriesBelow);
	}
}

void FBlendStackRootCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(BlendablePrefabCameraRig);

	for (FBlendedParameterOverrides& BlendedParameterOverrides: BlendedParameterOverridesStack)
	{
		Collector.AddReferencedObject(BlendedParameterOverrides.CameraRig);
		Collector.AddReferencedObject(BlendedParameterOverrides.Blend);
		Collector.AddReferencedObjects(BlendedParameterOverrides.PrefabTrail);
		BlendedParameterOverrides.Result.AddReferencedObjects(Collector);
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FBlendStackRootCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FBlendStackRootCameraDebugBlock& DebugBlock = Builder.StartChildDebugBlock<FBlendStackRootCameraDebugBlock>();
	DebugBlock.CameraRigAssetName = CameraRigAssetName;
	Algo::Transform(BlendedParameterOverridesStack, DebugBlock.BlendedParameterOverridesEntries,
			[](const FBlendedParameterOverrides& Item)
			{
				return GetNameSafe(Item.CameraRig);
			});

	if (BlendEvaluator)
	{
		BlendEvaluator->BuildDebugBlocks(Params, Builder);
	}
	else
	{
		// Dummy block.
		Builder.StartChildDebugBlock<FCameraDebugBlock>();
		Builder.EndChildDebugBlock();
	}

	Builder.StartChildDebugBlock<FCameraDebugBlock>();
	for (const FBlendedParameterOverrides& BlendedParameterOverrides : BlendedParameterOverridesStack)
	{
		if (BlendedParameterOverrides.BlendEvaluator)
		{
			BlendedParameterOverrides.BlendEvaluator->BuildDebugBlocks(Params, Builder);
		}
		else
		{
			// Dummy block.
			Builder.StartChildDebugBlock<FCameraDebugBlock>();
			Builder.EndChildDebugBlock();
		}
	}
	Builder.EndChildDebugBlock();

	if (RootEvaluator)
	{
		RootEvaluator->BuildDebugBlocks(Params, Builder);
	}
	else
	{
		// Dummy block.
		Builder.StartChildDebugBlock<FCameraDebugBlock>();
		Builder.EndChildDebugBlock();
	}

	Builder.EndChildDebugBlock();
	Builder.SkipChildren();
}

void FBlendStackRootCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());

	Renderer.AddText(TEXT("{cam_passive}<Blend>{cam_default}\n"));
	Renderer.AddIndent();
	ChildrenView[0]->DebugDraw(Params, Renderer);
	Renderer.RemoveIndent();

	if (!BlendedParameterOverridesEntries.IsEmpty())
	{
		Renderer.AddText(TEXT("{cam_passive}<%d Merged Camera Rigs>{cam_default}\n"), BlendedParameterOverridesEntries.Num());
		Renderer.AddIndent();
		{
			int32 ChildIndex = 0;
			for (FCameraDebugBlock* ParameterOverridesDebugBlock : ChildrenView[1]->GetChildren())
			{
				const FString& ParameterOverridesCameraRigName = BlendedParameterOverridesEntries[ChildIndex++];
				Renderer.AddText(ParameterOverridesCameraRigName);
				ParameterOverridesDebugBlock->DebugDraw(Params, Renderer);
			}
		}
		Renderer.RemoveIndent();
	}

	Renderer.AddText(TEXT("{cam_passive}<CameraRig> {cam_default}Running {cam_notice}%s{cam_default}\n"), *CameraRigAssetName);
	Renderer.AddIndent();
	ChildrenView[2]->DebugDraw(Params, Renderer);
	Renderer.RemoveIndent();

	Renderer.SkipAllBlocks();
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

