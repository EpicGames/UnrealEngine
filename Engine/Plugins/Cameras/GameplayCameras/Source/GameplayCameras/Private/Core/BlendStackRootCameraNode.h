// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"

#include "BlendStackRootCameraNode.generated.h"

class UBlendCameraNode;
class UCameraRigAsset;
class UCameraRigCameraNode;

/**
 * Root camera node for running a camera rig in a blend stack.
 * This camera node wraps both the camera rig's root node, and the
 * blend node used to blend it.
 */
UCLASS(MinimalAPI, Hidden)
class UBlendStackRootCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	UBlendStackRootCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The blend to use on the camera rig. */
	UPROPERTY()
	TObjectPtr<UBlendCameraNode> Blend;

	/** The root of the instantied camera node tree. */
	UPROPERTY()
	TObjectPtr<UCameraNode> RootNode;
};

namespace UE::Cameras
{

class FBlendCameraNodeEvaluator;
class FCameraRigCameraNodeEvaluator;

enum class ECameraRigMergingEligibility
{
	Different,
	EligibleForMerge,
	Active
};

/**
 * Evaluator for the blend stack entry root node.
 */
class FBlendStackRootCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBlendStackRootCameraNodeEvaluator)

public:

	FBlendStackRootCameraNodeEvaluator();

	FBlendCameraNodeEvaluator* GetBlendEvaluator() const { return BlendEvaluator; }
	FCameraNodeEvaluator* GetRootEvaluator() const { return RootEvaluator; }

	void SetBlendEvaluator(FBlendCameraNodeEvaluator* InBlendEvaluator);
	void SetRootEvaluator(FCameraNodeEvaluator* InRootEvaluator);

	ECameraRigMergingEligibility CompareCameraRigForMerging(const UCameraRigAsset* CameraRig) const;

	void MergeCameraRig(
			const FCameraNodeEvaluatorBuildParams& BuildParams, 
			const FCameraNodeEvaluatorInitializeParams& InitParams, 
			FCameraNodeEvaluationResult& InitResult,
			const UCameraRigAsset* CameraRig, 
			const UBlendCameraNode* Blend);

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	struct FBlendedParameterOverrides;

	static const UCameraRigAsset* FindInnermostCameraRigPrefab(const UCameraRigAsset* CameraRig);
	static FCameraNodeEvaluator* FindInnermostCameraRigEvaluator(FCameraNodeEvaluator* CameraNodeEvaluator);

	static const UCameraRigAsset* BuildNestedPrefabTrail(const UCameraRigAsset* CameraRig, TArray<TObjectPtr<const UCameraRigCameraNode>>& OutPrefabNodes);
	static FCameraNodeEvaluator* BuildNestedEvaluatorTrail(FCameraNodeEvaluator* CameraNodeEvaluator, TArray<FCameraRigCameraNodeEvaluator*>& OutPrefabEvaluators);

	void InitializeBlendedParameterOverridesStack();
	void RunBlendedParameterOverridesStack(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult);

private:

	FBlendCameraNodeEvaluator* BlendEvaluator = nullptr;
	FCameraNodeEvaluator* RootEvaluator = nullptr;

	TObjectPtr<const UCameraRigAsset> BlendablePrefabCameraRig;

	struct FBlendedParameterOverrides
	{
		TObjectPtr<const UCameraRigAsset> CameraRig;
		TObjectPtr<const UBlendCameraNode> Blend;
		TArray<TObjectPtr<const UCameraRigCameraNode>> PrefabTrail;

		FBlendCameraNodeEvaluator* BlendEvaluator = nullptr;
		FCameraNodeEvaluationResult Result;
	};
	TArray<FBlendedParameterOverrides> BlendedParameterOverridesStack;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FString CameraRigAssetName;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

}  // namespace UE::Cameras

