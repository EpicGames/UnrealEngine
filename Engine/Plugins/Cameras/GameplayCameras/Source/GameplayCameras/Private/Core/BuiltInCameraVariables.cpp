// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/BuiltInCameraVariables.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuiltInCameraVariables)

namespace UE::Cameras
{

namespace Private
{

struct FBuiltInCameraVariablesGuids
{
	FGuid YawGuid;       // {6E23348F-290E-460F-9432-ED80E7CA03F2}
	FGuid PitchGuid;     // {B7142BDA-6775-45A4-B06C-D07B712D1F89}
	FGuid RollGuid;      // {DABC149B-DE7D-4917-820C-7EEBE2A9846C}
	FGuid ZoomGuid;      // {56BF6A94-87B3-4648-84A4-391BDAD27061}

	FGuid YawPitchGuid;  // {67B7828D-C645-4907-92BE-B40DEB8C838C}

	FGuid FreezeControlRotationGuid; // {39EDC23E-70F2-4041-971B-7BD680F640D1}
	FGuid ControlRotationGuid;  // {2AB5FCA4-D62C-494C-B700-C8F98CD2D8E8}

	FBuiltInCameraVariablesGuids()
	{
		YawGuid = FGuid(0x6E23348F, 0x290E460F, 0x9432ED80, 0xE7CA03F2);
		PitchGuid = FGuid(0xB7142BDA, 0x677545A4, 0xB06CD07B, 0x712D1F89);
		RollGuid = FGuid(0xDABC149B, 0xDE7D4917, 0x820C7EEB, 0xE2A9846C);
		ZoomGuid = FGuid(0x56BF6A94, 0x87B34648, 0x84A4391B, 0xDAD27061);

		YawPitchGuid = FGuid(0x67B7828D, 0xC6454907, 0x92BEB40D, 0xEB8C838C);

		FreezeControlRotationGuid = FGuid(0x39EDC23E, 0x70F24041, 0x971B7BD6, 0x80F640D1);
		ControlRotationGuid = FGuid(0x2AB5FCA4, 0xD62C494C, 0xB700C8F9, 0x8CD2D8E8);
	}
};

void MakeCameraVariableDefinition(
		FCameraVariableDefinition& OutDefinition,
		const FGuid& VariableGuid, ECameraVariableType VariableType, bool bIsInput,
		const FString& VariableName)
{
	OutDefinition.VariableID =  FCameraVariableID::FromHashValue(GetTypeHash(VariableGuid));
	OutDefinition.VariableType = VariableType;
	OutDefinition.bIsInput = bIsInput;
#if WITH_EDITORONLY_DATA
	OutDefinition.VariableName = VariableName;
#endif
}

}  // namespace Private

const FBuiltInCameraVariables& FBuiltInCameraVariables::Get()
{
	static FBuiltInCameraVariables Instance;
	return Instance;
}

FBuiltInCameraVariables::FBuiltInCameraVariables()
{
	using namespace Private;

	FBuiltInCameraVariablesGuids KnownGuids;

	MakeCameraVariableDefinition(YawDefinition, KnownGuids.YawGuid, ECameraVariableType::Double, true, TEXT("Yaw"));
	MakeCameraVariableDefinition(PitchDefinition, KnownGuids.PitchGuid, ECameraVariableType::Double, true, TEXT("Pitch"));
	MakeCameraVariableDefinition(RollDefinition, KnownGuids.RollGuid, ECameraVariableType::Double, true, TEXT("Roll"));
	MakeCameraVariableDefinition(ZoomDefinition, KnownGuids.ZoomGuid, ECameraVariableType::Double, true, TEXT("Zoom"));

	MakeCameraVariableDefinition(YawPitchDefinition, KnownGuids.YawPitchGuid, ECameraVariableType::Vector2d, true, TEXT("YawPitch"));

	MakeCameraVariableDefinition(FreezeControlRotationDefinition, KnownGuids.FreezeControlRotationGuid, ECameraVariableType::Boolean, true, TEXT("FreezeControlRotation"));
	MakeCameraVariableDefinition(ControlRotationDefinition, KnownGuids.ControlRotationGuid, ECameraVariableType::Rotator3d, true, TEXT("ControlRotation"));
}

const FCameraVariableDefinition& FBuiltInCameraVariables::GetDefinition(EBuiltInDoubleCameraVariable BuiltInVariable) const
{
	static const FCameraVariableDefinition Invalid;
	switch (BuiltInVariable)
	{
		case EBuiltInDoubleCameraVariable::Yaw:
			return YawDefinition;
		case EBuiltInDoubleCameraVariable::Pitch:
			return PitchDefinition;
		case EBuiltInDoubleCameraVariable::Roll:
			return RollDefinition;
		case EBuiltInDoubleCameraVariable::Zoom:
			return ZoomDefinition;
		default:
			return Invalid;
	}
}

const FCameraVariableDefinition& FBuiltInCameraVariables::GetDefinition(EBuiltInVector2dCameraVariable BuiltInVariable) const
{
	static const FCameraVariableDefinition Invalid;
	switch (BuiltInVariable)
	{
		case EBuiltInVector2dCameraVariable::YawPitch:
			return YawPitchDefinition;
		default:
			return Invalid;
	}
}

const FCameraVariableDefinition& FBuiltInCameraVariables::GetDefinition(EBuiltInRotator3dCameraVariable BuiltInVariable) const
{
	static const FCameraVariableDefinition Invalid;
	switch (BuiltInVariable)
	{
		case EBuiltInRotator3dCameraVariable::ControlRotation:
			return ControlRotationDefinition;
		default:
			return Invalid;
	}
}

}  // namespace UE::Cameras

