// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraAsset.h"

#include "Build/CameraAssetBuilder.h"
#include "Build/CameraBuildContext.h"
#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Misc/EngineVersionComparison.h"
#include "Misc/EngineVersionComparison.h"
#include "StructUtils/OverridablePropertyBag.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAsset)

void UCameraAssetInterfaceParameter::PostLoad()
{
	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}

	Super::PostLoad();
}

void UCameraAssetInterfaceParameter::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && 
			!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UCameraAssetInterfaceParameter::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		Guid = FGuid::NewGuid();
	}
}

bool UCameraAssetInterfaceParameter::GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const
{
	UCameraRigAsset* CameraRig = SourceCameraRig.Get();
	if (!CameraRig)
	{
		return false;
	}

	const FCameraObjectInterfaceParameterDefinition* SourceParameter = CameraRig->GetParameterDefinitions()
		.FindByPredicate(
			[this](const FCameraObjectInterfaceParameterDefinition& Item)
			{
				return Item.ParameterName == SourceParameterName;
			});
	if (!SourceParameter)
	{
		return false;
	}

	// The parameter definition is the same except that we might expose it under a different name.
	OutParameterDefinition = *SourceParameter;
	OutParameterDefinition.ParameterName = FName(InterfaceParameterName);

	return true;
}

bool operator==(const FCameraAssetAllocationInfo& A, const FCameraAssetAllocationInfo& B)
{
	return A.VariableTableInfo == B.VariableTableInfo &&
		A.ContextDataTableInfo == B.ContextDataTableInfo;
}

const FName UCameraAsset::SharedTransitionsGraphName("SharedTransitions");

void UCameraAsset::SetCameraDirector(UCameraDirector* InCameraDirector)
{
	using namespace UE::Cameras;

	if (CameraDirector != InCameraDirector)
	{
		CameraDirector = InCameraDirector;

		TCameraPropertyChangedEvent<UCameraDirector*> ChangedEvent;
		ChangedEvent.NewValue = CameraDirector;
		EventHandlers.Notify(&ICameraAssetEventHandler::OnCameraDirectorChanged, this, ChangedEvent);
	}
}

void UCameraAsset::AddEnterTransition(UCameraRigTransition* InTransition)
{
	using namespace UE::Cameras;

	ensure(InTransition);

	EnterTransitions.Add(InTransition);

	TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent;
	ChangedEvent.EventType = ECameraArrayChangedEventType::Add;
	EventHandlers.Notify(&ICameraAssetEventHandler::OnEnterTransitionsChanged, this, ChangedEvent);
}

int32 UCameraAsset::RemoveEnterTransition(UCameraRigTransition* InTransition)
{
	using namespace UE::Cameras;
	
	const int32 NumRemoved = EnterTransitions.Remove(InTransition);
	if (NumRemoved > 0)
	{
		TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent;
		ChangedEvent.EventType = ECameraArrayChangedEventType::Remove;
		EventHandlers.Notify(&ICameraAssetEventHandler::OnEnterTransitionsChanged, this, ChangedEvent);
	}
	return NumRemoved;
}

void UCameraAsset::AddExitTransition(UCameraRigTransition* InTransition)
{
	using namespace UE::Cameras;

	ensure(InTransition);

	ExitTransitions.Add(InTransition);

	TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent;
	ChangedEvent.EventType = ECameraArrayChangedEventType::Add;
	EventHandlers.Notify(&ICameraAssetEventHandler::OnExitTransitionsChanged, this, ChangedEvent);
}

int32 UCameraAsset::RemoveExitTransition(UCameraRigTransition* InTransition)
{
	using namespace UE::Cameras;

	const int32 NumRemoved = ExitTransitions.Remove(InTransition);
	if (NumRemoved > 0)
	{
		TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent;
		ChangedEvent.EventType = ECameraArrayChangedEventType::Remove;
		EventHandlers.Notify(&ICameraAssetEventHandler::OnExitTransitionsChanged, this, ChangedEvent);
	}
	return NumRemoved;
}

void UCameraAsset::Serialize(FArchive& Ar)
{
	// Before 5.8, property bags are never saved with property flags, so always fix them up in PostLoad.
	// With 5.8, that bug is fixed so we only fix up the property bags if the asset was saved before.
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
	Ar.UsingCustomVersion(FOverridablePropertyBagCustomVersion::GUID);
#endif

	Super::Serialize(Ar);

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
	bDefaultParametersMayHaveMissingPropertyFlags = Ar.CustomVer(FOverridablePropertyBagCustomVersion::GUID) < FOverridablePropertyBagCustomVersion::MissingPropertyFlags;
#else
	bDefaultParametersMayHaveMissingPropertyFlags = true;
#endif
}

void UCameraAsset::PostLoad()
{
	Super::PostLoad();

	if (bDefaultParametersMayHaveMissingPropertyFlags)
	{
		using namespace UE::Cameras;
		FCameraObjectInterfaceParameterBuilder::FixUpDefaultParameterProperties(ParameterDefinitions, DefaultParameters);
		bDefaultParametersMayHaveMissingPropertyFlags = false;
	}

#if WITH_EDITOR
	if (CameraDirector)
	{
		EObjectFlags Flags = CameraDirector->GetFlags();
		if (EnumHasAnyFlags(Flags, (RF_Public | RF_Standalone)))
		{
			UE_LOGF(LogCameraSystem, Warning, 
					"Removing incorrect object flags from camera director inside '%ls', please re-save the asset.",
					*GetPathNameSafe(this));
			CameraDirector->Modify();
			CameraDirector->ClearFlags(RF_Public | RF_Standalone);
		}
	}
#endif  // WITH_EDITOR

	if (ParameterDefinitions.Num() > 0 && Interface.Parameters.IsEmpty())
	{
		// We have saved parameter definitions but they didn't come from building our interface,
		// so this must be an old asset that pre-dates the interface field. It needs a default
		// interface to be built for it.
		bNeedsDefaultInterface = true;
	}
}

void UCameraAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (CameraDirector)
	{
		CameraDirector->ExtendAssetRegistryTags(Context);
	}

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR

void UCameraAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::Cameras;

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCameraAsset, CameraDirector))
	{
		TCameraPropertyChangedEvent<UCameraDirector*> ChangedEvent;
		ChangedEvent.NewValue = CameraDirector;
		EventHandlers.Notify(&ICameraAssetEventHandler::OnCameraDirectorChanged, this, ChangedEvent);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCameraAsset, EnterTransitions))
	{
		TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent(PropertyChangedEvent.ChangeType);
		EventHandlers.Notify(&ICameraAssetEventHandler::OnEnterTransitionsChanged, this, ChangedEvent);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCameraAsset, ExitTransitions))
	{
		TCameraArrayChangedEvent<UCameraRigTransition*> ChangedEvent(PropertyChangedEvent.ChangeType);
		EventHandlers.Notify(&ICameraAssetEventHandler::OnExitTransitionsChanged, this, ChangedEvent);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void UCameraAsset::BuildCamera()
{
	using namespace UE::Cameras;

	FCameraBuildLog BuildLog;
	BuildLog.SetForwardMessagesToLogging(true);
	FCameraBuildContext BuildContext(BuildLog);
	BuildCamera(BuildContext);
}

void UCameraAsset::BuildCamera(UE::Cameras::FCameraBuildContext& InBuildContext)
{
	using namespace UE::Cameras;

	FCameraAssetBuilder Builder(InBuildContext);
	Builder.BuildCamera(this);
}

void UCameraAsset::DirtyBuildStatus()
{
	BuildStatus = ECameraBuildStatus::Dirty;
}

void UCameraAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR

	const bool bIsUserObject = !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
	const bool bIsEditorAutoSave = ObjectSaveContext.IsFromAutoSave();
#else
	const bool bIsEditorAutoSave = ((ObjectSaveContext.GetSaveFlags() & SAVE_FromAutosave) != 0);
#endif  // UE >= 5.7.0
	if (bIsUserObject && !bIsEditorAutoSave)
	{
		// Build when saving/cooking.
		BuildCamera();
	}

#endif

	Super::PreSave(ObjectSaveContext);
}

#if WITH_EDITOR

void UCameraAsset::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = TransitionGraphNodePos.X;
	NodePosY = TransitionGraphNodePos.Y;
}

void UCameraAsset::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	TransitionGraphNodePos.X = NodePosX;
	TransitionGraphNodePos.Y = NodePosY;
}

const FString& UCameraAsset::GetGraphNodeCommentText(FName InGraphName) const
{
	return TransitionGraphNodeComment;
}

void UCameraAsset::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	TransitionGraphNodeComment = NewComment;
}

void UCameraAsset::GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const
{
	OutObjects.Append(AllSharedTransitionsObjects);
}

void UCameraAsset::AddConnectableObject(FName InGraphName, UObject* InObject)
{
	Modify();

	const int32 Index = AllSharedTransitionsObjects.AddUnique(InObject);
	ensure(Index == AllSharedTransitionsObjects.Num() - 1);
}

void UCameraAsset::RemoveConnectableObject(FName InGraphName, UObject* InObject)
{
	Modify();

	const int32 NumRemoved = AllSharedTransitionsObjects.Remove(InObject);
	ensure(NumRemoved == 1);
}

#endif  // WITH_EDITOR

