// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraAssetReference.h"

#include "Core/CameraAsset.h"
#include "Core/CameraNodeEvaluator.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAssetReference)

FCameraAssetReference::FCameraAssetReference()
{
}

FCameraAssetReference::FCameraAssetReference(UCameraAsset* InCameraAsset)
	: CameraAsset(InCameraAsset)
{
}

void FCameraAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const
{
	using namespace UE::Cameras;
	if (CameraAsset)
	{
		FCameraObjectInterfaceParameterOverrideHelper Helper(OutResult);
		Helper.bDrivenOnly = bDrivenOnly;
		Helper.ApplyFilteredParameters(
				CameraAsset, 
				CameraAsset->GetParameterDefinitions(),
				Parameters,
				[this](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
				{
					return Parameters.IsPropertyOverriden(Definition.ParameterGuid);
				});
	}
}

bool FCameraAssetReference::IsParameterOverridden(const FGuid PropertyID) const
{
	return Parameters.IsPropertyOverriden(PropertyID);
}

void FCameraAssetReference::SetParameterOverridden(const FGuid PropertyID, bool bIsOverridden)
{
	Parameters.SetPropertyOverriden(PropertyID, bIsOverridden);
}

bool FCameraAssetReference::NeedsRebuildParameters() const
{
	if ((!CameraAsset && Parameters.IsValid()) || (CameraAsset && !Parameters.IsValid()))
	{
		return true;
	}

	if (CameraAsset)
	{
		const UPropertyBag* AssetParametersType = CameraAsset->GetDefaultParameters().GetPropertyBagStruct();
		const UPropertyBag* ReferenceParametersType = Parameters.GetPropertyBagStruct();
		if (AssetParametersType != ReferenceParametersType)
		{
			return true;
		}
	}

	return false;
}

bool FCameraAssetReference::RebuildParametersIfNeeded()
{
	if (NeedsRebuildParameters())
	{
		RebuildParameters();
		return true;
	}
	return false;
}

void FCameraAssetReference::RebuildParameters()
{
	if (CameraAsset)
	{
		Parameters.MigrateToNewBagInstanceWithOverrides(CameraAsset->GetDefaultParameters());
	}
	else
	{
		Parameters.Reset();
	}
}

bool FCameraAssetReference::SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_SoftObjectProperty)
	{
		FSoftObjectPtr CameraAssetPath;
		Slot << CameraAssetPath;
		CameraAsset = Cast<UCameraAsset>(CameraAssetPath.Get());
		return true;
	}
	return false;
}

void FCameraAssetReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const FGuid& Guid : ParameterOverrideGuids_DEPRECATED)
		{
			Parameters.SetPropertyOverriden(Guid, true);
		}
		ParameterOverrideGuids_DEPRECATED.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

