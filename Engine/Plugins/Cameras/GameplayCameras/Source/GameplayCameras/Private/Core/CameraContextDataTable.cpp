// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraContextDataTable.h"

#include "Core/CameraContextDataTableAllocationInfo.h"

namespace UE::Cameras
{

const uint32 FCameraContextDataTable::GDefaultCapacity = 64;
const uint32 FCameraContextDataTable::GDefaultAlignment = alignof(void*);

FCameraContextDataTable::FCameraContextDataTable()
{
}

FCameraContextDataTable::~FCameraContextDataTable()
{
}

void FCameraContextDataTable::AddReferencedObjects(FReferenceCollector& ReferenceCollector)
{
	for (FEntry& Entry : Entries)
	{
		ReferenceCollector.AddReferencedObject(Entry.TypeObject);

		if (Entry.ContainerType == ECameraContextDataContainerType::None)
		{
			uint8* RawData = Memory + Entry.Offset;

			switch (Entry.Type)
			{
				case ECameraContextDataType::Struct:
					{
						const UScriptStruct* StructType = Cast<const UScriptStruct>(Entry.TypeObject);
						if (ensure(StructType))
						{
							ReferenceCollector.AddPropertyReferencesWithStructARO(StructType, RawData);
						}
					}
					break;
				case ECameraContextDataType::Object:
					{
						TObjectPtr<UObject>* TypedData = reinterpret_cast<TObjectPtr<UObject>*>(RawData);
						ReferenceCollector.AddReferencedObject(*TypedData);
					}
					break;
				case ECameraContextDataType::Class:
					{
						TObjectPtr<UClass>* TypedData = reinterpret_cast<TObjectPtr<UClass>*>(RawData);
						ReferenceCollector.AddReferencedObject(*TypedData);
					}
					break;
			}
		}
		else if (Entry.ContainerType == ECameraContextDataContainerType::Array)
		{
			FArrayEntryHelper Helper(Entry, Memory);

			switch (Entry.Type)
			{
				case ECameraContextDataType::Struct:
					{
						const UScriptStruct* StructType = Cast<const UScriptStruct>(Entry.TypeObject);
						if (ensure(StructType))
						{
							for (int32 Index = 0; Index < Helper.Num(); ++Index)
							{
								ReferenceCollector.AddPropertyReferencesWithStructARO(StructType, Helper.GetRawPtr(Index));
							}
						}
					}
					break;
				case ECameraContextDataType::Object:
					{
						for (int32 Index = 0; Index < Helper.Num(); ++Index)
						{
							TObjectPtr<UObject>* TypedData = reinterpret_cast<TObjectPtr<UObject>*>(Helper.GetRawPtr(Index));
							ReferenceCollector.AddReferencedObject(*TypedData);
						}
					}
					break;
				case ECameraContextDataType::Class:
					{
						for (int32 Index = 0; Index < Helper.Num(); ++Index)
						{
							TObjectPtr<UClass>* TypedData = reinterpret_cast<TObjectPtr<UClass>*>(Helper.GetRawPtr(Index));
							ReferenceCollector.AddReferencedObject(*TypedData);
						}
					}
					break;
			}
		}
	}
}

void FCameraContextDataTable::Initialize(const FCameraContextDataTableAllocationInfo& AllocationInfo)
{
	// Reset any previous state.
	DestroyBuffer();
	Entries.Reset();
	EntryLookup.Reset();

	// Compute the total buffer size we need, and create our entries as we go.
	uint32 TotalSizeOf = 0;
	uint32 MaxAlignOf = 0;
	uint32 CurSizeOf, CurAlignOf;

	for (const FCameraContextDataDefinition& DataDefinition : AllocationInfo.DataDefinitions)
	{
		GetDataTypeAllocationInfo(DataDefinition.DataType, DataDefinition.DataContainerType, DataDefinition.DataTypeObject, CurSizeOf, CurAlignOf);
		const uint32 NewEntryOffset = Align(TotalSizeOf, CurAlignOf);
		TotalSizeOf = NewEntryOffset + CurSizeOf;
		MaxAlignOf = FMath::Max(MaxAlignOf, CurAlignOf);

		FEntry NewEntry;
		CreateEntry(DataDefinition, NewEntry);
		NewEntry.Offset = NewEntryOffset;
		Entries.Add(NewEntry);
		EntryLookup.Add(NewEntry.ID, Entries.Num() - 1);
	}

	// Allocate the memory buffer.
	MaxAlignOf = FMath::Max(GDefaultAlignment, MaxAlignOf);
	Memory = reinterpret_cast<uint8*>(FMemory::Malloc(TotalSizeOf, MaxAlignOf));
	Capacity = TotalSizeOf;
	Alignment = MaxAlignOf;
	Used = TotalSizeOf;

	// Go back to our entries and initialize each entry to the default value for that data type.
	for (const FEntry& Entry : Entries)
	{
		uint8* DataPtr = Memory + Entry.Offset;
		ConstructDataValue(Entry.Type, Entry.ContainerType, Entry.TypeObject, DataPtr);
	}
}

bool FCameraContextDataTable::EnsureData(const FCameraContextDataTableAllocationInfo& AllocationInfo)
{
	uint32 TotalSizeOf = Used;
	uint32 MaxAlignOf = 0;
	uint32 CurSizeOf, CurAlignOf;

	bool bAllExist = true;
	int32 FirstNewEntry = Entries.Num();

	for (const FCameraContextDataDefinition& DataDefinition : AllocationInfo.DataDefinitions)
	{
		if (EntryLookup.Contains(DataDefinition.DataID))
		{
			continue;
		}

		GetDataTypeAllocationInfo(DataDefinition.DataType, DataDefinition.DataContainerType, DataDefinition.DataTypeObject, CurSizeOf, CurAlignOf);
		const uint32 NewEntryOffset = Align(TotalSizeOf, CurAlignOf);
		TotalSizeOf = NewEntryOffset + CurSizeOf;
		MaxAlignOf = FMath::Max(MaxAlignOf, CurAlignOf);

		FEntry NewEntry;
		CreateEntry(DataDefinition, NewEntry);
		NewEntry.Offset = NewEntryOffset;
		Entries.Add(NewEntry);
		EntryLookup.Add(NewEntry.ID, Entries.Num() - 1);

		bAllExist = false;
	}

	if (!bAllExist)
	{
		// Alignment may not have been initialized yet.
		Alignment = FMath::Max(Alignment, MaxAlignOf);

		if (TotalSizeOf > Capacity)
		{
			ReallocateBuffer(TotalSizeOf);
		}

		Used = TotalSizeOf;

		for (int32 Index = FirstNewEntry; Index < Entries.Num(); ++Index)
		{
			const FEntry& Entry = Entries[Index];
			uint8* DataPtr = Memory + Entry.Offset;
			ConstructDataValue(Entry.Type, Entry.ContainerType, Entry.TypeObject, DataPtr);
		}
	}

	return bAllExist;
}

void FCameraContextDataTable::AddData(const FCameraContextDataDefinition& DataDefinition)
{
	if (!ensure(!EntryLookup.Contains(DataDefinition.DataID)))
	{
		return;
	}

	uint32 SizeOf, AlignOf;
	GetDataTypeAllocationInfo(DataDefinition.DataType, DataDefinition.DataContainerType, DataDefinition.DataTypeObject, SizeOf, AlignOf);

	uint8* DataPtr = Align(Memory + Used, AlignOf);
	uint32 NewUsed = (DataPtr + SizeOf) - Memory;

	if (NewUsed > Capacity)
	{
		// Alignment may not have been initialized yet.
		Alignment = FMath::Max(AlignOf, GDefaultAlignment);

		ReallocateBuffer(NewUsed);

		DataPtr = Align(Memory + Used, AlignOf);
	}

	Used = NewUsed;

	FEntry NewEntry;
	CreateEntry(DataDefinition, NewEntry);
	NewEntry.Offset = DataPtr - Memory;
	Entries.Add(NewEntry);
	EntryLookup.Add(DataDefinition.DataID, Entries.Num() - 1);

	ConstructDataValue(NewEntry.Type, NewEntry.ContainerType, NewEntry.TypeObject, Memory + NewEntry.Offset);
}

void FCameraContextDataTable::CreateEntry(const FCameraContextDataDefinition& DataDefinition, FEntry& OutEntry)
{
	OutEntry.ID = DataDefinition.DataID;
	OutEntry.Type = DataDefinition.DataType;
	OutEntry.ContainerType = DataDefinition.DataContainerType;
	OutEntry.TypeObject = DataDefinition.DataTypeObject;
	OutEntry.Offset = 0;
	OutEntry.Flags = EEntryFlags::None;
	if (DataDefinition.bAutoReset)
	{
		OutEntry.Flags |= EEntryFlags::AutoReset;
	}
#if WITH_EDITORONLY_DATA
	OutEntry.DebugName = DataDefinition.DataName;
#endif
}

bool FCameraContextDataTable::GetDataTypeAllocationInfo(ECameraContextDataType DataType, const UObject* DataTypeObject, uint32& OutSizeOf, uint32& OutAlignOf)
{
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			OutSizeOf = sizeof(FName);
			OutAlignOf = alignof(FName);
			break;
		case ECameraContextDataType::String:
			OutSizeOf = sizeof(FString);
			OutAlignOf = alignof(FString);
			break;
		case ECameraContextDataType::Enum:
			OutSizeOf = sizeof(uint32);
			OutAlignOf = alignof(uint32);
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(DataTypeObject);
				if (ensure(StructType))
				{
					OutSizeOf = StructType->GetPropertiesSize();
					OutAlignOf = StructType->GetMinAlignment();
				}
			}
			break;
		case ECameraContextDataType::Object:
			OutSizeOf = sizeof(TObjectPtr<UObject>);
			OutAlignOf = alignof(TObjectPtr<UObject>);
			break;
		case ECameraContextDataType::Class:
			OutSizeOf = sizeof(TObjectPtr<UClass>);
			OutAlignOf = alignof(TObjectPtr<UClass>);
			break;
		default:
			ensure(false);
			return false;
	}
	return true;
}

bool FCameraContextDataTable::GetDataTypeAllocationInfo(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint32& OutSizeOf, uint32& OutAlignOf)
{
	if (DataContainerType == ECameraContextDataContainerType::None)
	{
		return GetDataTypeAllocationInfo(DataType, DataTypeObject, OutSizeOf, OutAlignOf);
	}
	else if (DataContainerType == ECameraContextDataContainerType::Array)
	{
		OutSizeOf = sizeof(FEntryScriptArray);
		OutAlignOf = alignof(FEntryScriptArray);
		return true;
	}
	return false;
}

bool FCameraContextDataTable::ConstructDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DataPtr)
{
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			new (DataPtr) FName();
			break;
		case ECameraContextDataType::String:
			new (DataPtr) FString();
			break;
		case ECameraContextDataType::Enum:
			{
				const UEnum* EnumType = Cast<const UEnum>(DataTypeObject);
				if (ensure(EnumType))
				{
					*reinterpret_cast<uint32*>(DataPtr) = (uint32)EnumType->GetValueByIndex(0);
				}
			}
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(DataTypeObject);
				if (ensure(StructType))
				{
					StructType->InitializeDefaultValue(DataPtr);
				}
			}
			break;
		case ECameraContextDataType::Object:
			new (DataPtr) TObjectPtr<UObject>();
			break;
		case ECameraContextDataType::Class:
			new (DataPtr) TObjectPtr<UClass>();
			break;
		default:
			ensure(false);
			return false;
	}
	return true;
}

bool FCameraContextDataTable::ConstructDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DataPtr)
{
	if (DataContainerType == ECameraContextDataContainerType::None)
	{
		return ConstructDataValue(DataType, DataTypeObject, DataPtr);
	}
	else if (DataContainerType == ECameraContextDataContainerType::Array)
	{
		new (DataPtr) FEntryScriptArray();
	}
	return false;
}

bool FCameraContextDataTable::DestroyDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DataPtr)
{
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			((FName*)DataPtr)->~FName();
			break;
		case ECameraContextDataType::String:
			((FString*)DataPtr)->~FString();
			break;
		case ECameraContextDataType::Enum:
			// Nothing to do.
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(DataTypeObject);
				if (ensure(StructType))
				{
					StructType->DestroyStruct(DataPtr);
				}
			}
			break;
		case ECameraContextDataType::Object:
			((TObjectPtr<UObject>*)DataPtr)->~TObjectPtr<UObject>();
			break;
		case ECameraContextDataType::Class:
			((TObjectPtr<UClass>*)DataPtr)->~TObjectPtr<UClass>();
			break;
		default:
			ensure(false);
			return false;
	}
	return true;
}

bool FCameraContextDataTable::DestroyDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DataPtr)
{
	if (DataContainerType == ECameraContextDataContainerType::None)
	{
		return DestroyDataValue(DataType, DataTypeObject, DataPtr);
	}
	else if (DataContainerType == ECameraContextDataContainerType::Array)
	{
		FArrayEntryHelper Helper(DataType, DataTypeObject, DataPtr);
		for (int32 Index = 0; Index < Helper.Num(); ++Index)
		{
			uint8* RawElementPtr = Helper.GetRawPtr(Index);
			DestroyDataValue(DataType, DataTypeObject, RawElementPtr);
		}
		((FEntryScriptArray*)DataPtr)->~FEntryScriptArray();
	}
	return false;
}

void FCameraContextDataTable::ReallocateBuffer(uint32 MinRequired)
{
	uint32 NewCapacity = Capacity <= 0 ? GDefaultCapacity : Capacity * 2;
	if (MinRequired > 0)
	{
		NewCapacity = FMath::Max(NewCapacity, MinRequired);
	}

	uint8* OldMemory = Memory;
	uint8* NewMemory = reinterpret_cast<uint8*>(FMemory::Malloc(NewCapacity, Alignment));

	if (OldMemory)
	{
		// UE implements names, strings, and UStructs as bitwise relocatable so we can just
		// move the memory around.
		FMemory::Memmove(NewMemory, OldMemory, Capacity);
		FMemory::Free(OldMemory);
	}

	Memory = NewMemory;
	Capacity = NewCapacity;
}

void FCameraContextDataTable::DestroyBuffer()
{
	if (!Memory)
	{
		return;
	}

	for (const FEntry& Entry : Entries)
	{
		uint8* DataPtr = Memory + Entry.Offset;
		DestroyDataValue(Entry.Type, Entry.ContainerType, Entry.TypeObject, DataPtr);
	}

	FMemory::Free(Memory);

	Memory = nullptr;
	Capacity = 0;
	Used = 0;
}

const FName& FCameraContextDataTable::GetNameData(FCameraContextDataID InID) const
{
	if (const FName* Value = GetDataImpl<FName>(InID, ECameraContextDataType::Name, nullptr))
	{
		return *Value;
	}

	static FName DefaultValue(NAME_None);
	return DefaultValue;
}

const FString& FCameraContextDataTable::GetStringData(FCameraContextDataID InID) const
{
	if (const FString* Value = GetDataImpl<FString>(InID, ECameraContextDataType::String, nullptr))
	{
		return *Value;
	}

	static FString DefaultValue;
	return DefaultValue;
}

uint32 FCameraContextDataTable::GetEnumData(FCameraContextDataID InID, const UEnum* EnumType) const
{
	if (const uint32* Value = GetDataImpl<uint32>(InID, ECameraContextDataType::Enum, EnumType))
	{
		return *Value;
	}
	return 0;
}

FConstStructView FCameraContextDataTable::GetStructViewData(FCameraContextDataID InID, const UScriptStruct* StructType) const
{
	const uint8* RawData = TryGetData(InID, ECameraContextDataType::Struct, StructType);
	if (RawData)
	{
		FConstStructView ReturnValue(StructType, RawData);
		return ReturnValue;
	}
	return FStructView();
}

FInstancedStruct FCameraContextDataTable::GetInstancedStructData(FCameraContextDataID InID, const UScriptStruct* StructType) const
{
	const uint8* RawData = TryGetData(InID, ECameraContextDataType::Struct, StructType);
	if (RawData)
	{
		FInstancedStruct ReturnValue;
		ReturnValue.InitializeAs(StructType, RawData);
		return ReturnValue;
	}
	return FInstancedStruct();
}

UObject* FCameraContextDataTable::GetObjectData(FCameraContextDataID InID) const
{
	if (const TObjectPtr<UObject>* Value = GetDataImpl<TObjectPtr<UObject>>(InID, ECameraContextDataType::Object, nullptr))
	{
		return Value->Get();
	}
	return nullptr;
}

UClass* FCameraContextDataTable::GetClassData(FCameraContextDataID InID) const
{
	if (const TObjectPtr<UClass>* Value = GetDataImpl<TObjectPtr<UClass>>(InID, ECameraContextDataType::Class, nullptr))
	{
		return Value->Get();
	}
	return nullptr;
}

void FCameraContextDataTable::SetNameData(FCameraContextDataID InID, const FName& InData)
{
	SetDataImpl(InID, ECameraContextDataType::Name, nullptr, InData);
}

void FCameraContextDataTable::SetStringData(FCameraContextDataID InID, const FString& InData)
{
	SetDataImpl(InID, ECameraContextDataType::String, nullptr, InData);
}

void FCameraContextDataTable::SetEnumData(FCameraContextDataID InID, const UEnum* EnumType, uint32 InData)
{
	SetDataImpl(InID, ECameraContextDataType::Enum, EnumType, InData);
}

void FCameraContextDataTable::SetObjectData(FCameraContextDataID InID, UObject* InData)
{
	TObjectPtr<UObject> ActualData(InData);
	SetDataImpl(InID, ECameraContextDataType::Object, nullptr, ActualData);
}

void FCameraContextDataTable::SetClassData(FCameraContextDataID InID, UClass* InData)
{
	TObjectPtr<UClass> ActualData(InData);
	SetDataImpl(InID, ECameraContextDataType::Class, nullptr, ActualData);
}

void FCameraContextDataTable::SetStructViewData(FCameraContextDataID InID, const FStructView& InData)
{
	FEntry* Entry = FindEntry(InID);
	if (ensure(Entry && 
				Entry->Type == ECameraContextDataType::Struct && 
				Entry->ContainerType == ECameraContextDataContainerType::None &&
				InData.GetScriptStruct() == Entry->TypeObject))
	{
		uint8* DataPtr = Memory + Entry->Offset;
		const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Entry->TypeObject);
		StructType->CopyScriptStruct(DataPtr, InData.GetMemory());
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

void FCameraContextDataTable::SetInstancedStructData(FCameraContextDataID InID, const FInstancedStruct& InData)
{
	FEntry* Entry = FindEntry(InID);
	if (ensure(Entry && 
				Entry->Type == ECameraContextDataType::Struct && 
				Entry->ContainerType == ECameraContextDataContainerType::None &&
				InData.GetScriptStruct() == Entry->TypeObject))
	{
		uint8* DataPtr = Memory + Entry->Offset;
		const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Entry->TypeObject);
		StructType->CopyScriptStruct(DataPtr, InData.GetMemory());
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

void FCameraContextDataTable::SetNameArrayData(FCameraContextDataID InID, TConstArrayView<FName> InData)
{
	SetArrayDataImpl(InID, ECameraContextDataType::Name, nullptr, InData);
}

void FCameraContextDataTable::SetStringArrayData(FCameraContextDataID InID, TConstArrayView<FString> InData)
{
	SetArrayDataImpl(InID, ECameraContextDataType::String, nullptr, InData);
}

void FCameraContextDataTable::SetEnumArrayData(FCameraContextDataID InID, const UEnum* EnumType, TConstArrayView<uint32> InData)
{
	SetArrayDataImpl(InID, ECameraContextDataType::Enum, EnumType, InData);
}

void FCameraContextDataTable::SetObjectArrayData(FCameraContextDataID InID, TConstArrayView<UObject*> InData)
{
	TArray<TObjectPtr<UObject>> InDataPtrs(InData);
	SetArrayDataImpl(InID, ECameraContextDataType::Object, nullptr, TConstArrayView<TObjectPtr<UObject>>(InDataPtrs));
}

void FCameraContextDataTable::SetClassArrayData(FCameraContextDataID InID, TConstArrayView<UClass*> InData)
{
	TArray<TObjectPtr<UClass>> InDataPtrs(InData);
	SetArrayDataImpl(InID, ECameraContextDataType::Class, nullptr, TConstArrayView<TObjectPtr<UClass>>(InDataPtrs));
}

void FCameraContextDataTable::SetStructViewArrayData(FCameraContextDataID InID, TConstArrayView<FStructView> InData)
{
	FEntry* Entry = FindEntry(InID);
	if (ensure(Entry && 
				Entry->Type == ECameraContextDataType::Struct && 
				Entry->ContainerType == ECameraContextDataContainerType::Array))
	{
		const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Entry->TypeObject);

		FArrayEntryHelper Helper(*Entry, Memory);
		Helper.Resize(InData.Num());
		for (int32 Index = 0; Index < InData.Num(); ++Index)
		{
			uint8* RawData = Helper.GetRawPtr(Index);
			if (ensure(StructType == InData[Index].GetScriptStruct()))
			{
				StructType->CopyScriptStruct(RawData, InData[Index].GetMemory());
			}
		}
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

void FCameraContextDataTable::SetInstancedStructArrayData(FCameraContextDataID InID, TConstArrayView<FInstancedStruct> InData)
{
	FEntry* Entry = FindEntry(InID);
	if (ensure(Entry && 
				Entry->Type == ECameraContextDataType::Struct && 
				Entry->ContainerType == ECameraContextDataContainerType::Array))
	{
		const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Entry->TypeObject);

		FArrayEntryHelper Helper(*Entry, Memory);
		Helper.Resize(InData.Num());
		for (int32 Index = 0; Index < InData.Num(); ++Index)
		{
			uint8* RawData = Helper.GetRawPtr(Index);
			if (ensure(StructType == InData[Index].GetScriptStruct()))
			{
				StructType->CopyScriptStruct(RawData, InData[Index].GetMemory());
			}
		}
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

const FCameraContextDataTable::FEntry* FCameraContextDataTable::FindEntry(FCameraContextDataID InID) const
{
	const int32* IndexPtr = EntryLookup.Find(InID);
	return IndexPtr ? &Entries[*IndexPtr] : nullptr;
}

FCameraContextDataTable::FEntry* FCameraContextDataTable::FindEntry(FCameraContextDataID InID)
{
	const int32* IndexPtr = EntryLookup.Find(InID);
	return IndexPtr ? &Entries[*IndexPtr] : nullptr;
}

const uint8* FCameraContextDataTable::GetData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		bool bOnlyIfWritten) const
{
	const uint8* Data = TryGetData(DataID, ExpectedDataType, ExpectedDataTypeObject, bOnlyIfWritten);
	ensureMsgf(
			Data, 
			TEXT("Can't get camera context data (ID '%d') because it doesn't exist in the table, or isn't of the expected data type."), 
			DataID.GetValue());
	return Data;
}

const uint8* FCameraContextDataTable::TryGetData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		bool bOnlyIfWritten) const
{
	const FEntry* Entry = FindEntry(DataID);
	if (Entry)
	{
		if (Entry->Type == ExpectedDataType && 
				Entry->ContainerType == ECameraContextDataContainerType::None && 
				Entry->TypeObject == ExpectedDataTypeObject &&
				(!bOnlyIfWritten || EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written)))
		{
			return Memory + Entry->Offset;
		}
	}

	return nullptr;
}

const FCameraContextDataTable::FEntryScriptArray* FCameraContextDataTable::TryGetArrayData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		bool bOnlyIfWritten) const
{
	const FEntry* Entry = FindEntry(DataID);
	if (Entry)
	{
		if (Entry->Type == ExpectedDataType && 
				Entry->ContainerType == ECameraContextDataContainerType::Array && 
				Entry->TypeObject == ExpectedDataTypeObject &&
				(!bOnlyIfWritten || EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written)))
		{
			return (FEntryScriptArray*)(Memory + Entry->Offset);
		}
	}

	return nullptr;
}

const uint8* FCameraContextDataTable::TryGetRawDataPtr(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		bool bOnlyIfWritten) const
{
	const FEntry* Entry = FindEntry(DataID);
	if (Entry)
	{
		if (Entry->Type == ExpectedDataType && 
				Entry->TypeObject == ExpectedDataTypeObject &&
				(!bOnlyIfWritten || EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written)))
		{
			return Memory + Entry->Offset;
		}
	}

	return nullptr;
}

void FCameraContextDataTable::SetData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		const uint8* InRawDataPtr,
		bool bMarkAsWrittenThisFrame)
{
	const bool bDidSet = TrySetData(DataID, ExpectedDataType, ExpectedDataTypeObject, InRawDataPtr, bMarkAsWrittenThisFrame);
	ensureMsgf(bDidSet, TEXT("Can't set camera context data (ID '%d') beacuse it doesn't exist in the table."), DataID.GetValue());
}

bool FCameraContextDataTable::TrySetData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		const uint8* InRawDataPtr,
		bool bMarkAsWrittenThisFrame)
{
	FEntry* Entry = FindEntry(DataID);
	if (!Entry)
	{
		return false;
	}

	if (!ensure(Entry->Type == ExpectedDataType && 
				Entry->ContainerType == ECameraContextDataContainerType::None &&
				Entry->TypeObject == ExpectedDataTypeObject))
	{
		return false;
	}

	uint8* DataPtr = Memory + Entry->Offset;
	SetDataValue(Entry->Type, Entry->TypeObject, DataPtr, InRawDataPtr);

	Entry->Flags |= EEntryFlags::Written;
	if (bMarkAsWrittenThisFrame)
	{
		Entry->Flags |= EEntryFlags::WrittenThisFrame;
	}
	
	return true;
}

bool FCameraContextDataTable::TrySetArrayDataNum(
		FCameraContextDataID DataID, 
		int32 Count,
		bool bMarkAsWrittenThisFrame)
{
	FEntry* Entry = FindEntry(DataID);
	if (!Entry)
	{
		return false;
	}

	if (!ensure(Entry->ContainerType == ECameraContextDataContainerType::Array))
	{
		return false;
	}

	FArrayEntryHelper Helper(*Entry, Memory);
	Helper.Resize(Count);

	Entry->Flags |= EEntryFlags::Written;
	if (bMarkAsWrittenThisFrame)
	{
		Entry->Flags |= EEntryFlags::WrittenThisFrame;
	}

	return true;
}

bool FCameraContextDataTable::TrySetArrayData(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		int32 Index,
		const uint8* InRawDataPtr,
		bool bMarkAsWrittenThisFrame)
{
	FEntry* Entry = FindEntry(DataID);
	if (!Entry)
	{
		return false;
	}

	if (!ensure(Entry->Type == ExpectedDataType && 
				Entry->ContainerType == ECameraContextDataContainerType::Array &&
				Entry->TypeObject == ExpectedDataTypeObject))
	{
		return false;
	}

	FArrayEntryHelper Helper(*Entry, Memory);
	if (!ensure(Helper.IsValidIndex(Index)))
	{
		return false;
	}

	uint8* DataPtr = Helper.GetRawPtr(Index);
	SetDataValue(Entry->Type, Entry->TypeObject, DataPtr, InRawDataPtr);

	Entry->Flags |= EEntryFlags::Written;
	if (bMarkAsWrittenThisFrame)
	{
		Entry->Flags |= EEntryFlags::WrittenThisFrame;
	}
	
	return true;
}

uint8* FCameraContextDataTable::TryGetMutableRawDataPtr(
		FCameraContextDataID DataID,
		ECameraContextDataType ExpectedDataType,
		const UObject* ExpectedDataTypeObject,
		bool bMarkAsWrittenThisFrame)
{
	FEntry* Entry = FindEntry(DataID);
	if (Entry)
	{
		if (Entry->Type == ExpectedDataType && Entry->TypeObject == ExpectedDataTypeObject)
		{
			Entry->Flags |= EEntryFlags::Written;
			if (bMarkAsWrittenThisFrame)
			{
				Entry->Flags |= EEntryFlags::WrittenThisFrame;
			}
			return Memory + Entry->Offset;
		}
	}

	return nullptr;
}

bool FCameraContextDataTable::SetDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DestDataPtr, const uint8* SrcDataPtr)
{
	if (DataContainerType == ECameraContextDataContainerType::None)
	{
		return SetDataValue(DataType, DataTypeObject, DestDataPtr, SrcDataPtr);
	}
	else if (DataContainerType == ECameraContextDataContainerType::Array)
	{
		FArrayEntryHelper DestHelper(DataType, DataTypeObject, DestDataPtr);
		FArrayEntryHelper SrcHelper(DataType, DataTypeObject, const_cast<uint8*>(SrcDataPtr));
		DestHelper.Resize(SrcHelper.Num());
		for (int32 Index = 0; Index < SrcHelper.Num(); ++Index)
		{
			uint8* DestElementPtr = DestHelper.GetRawPtr(Index);
			const uint8* SrcElementPtr = SrcHelper.GetRawPtr(Index);
			SetDataValue(DataType, DataTypeObject, DestElementPtr, SrcElementPtr);
		}
	}
	return false;
}

bool FCameraContextDataTable::SetDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DestDataPtr, const uint8* SrcDataPtr)
{
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			*reinterpret_cast<FName*>(DestDataPtr) = *reinterpret_cast<const FName*>(SrcDataPtr);
			break;
		case ECameraContextDataType::String:
			*reinterpret_cast<FString*>(DestDataPtr) = *reinterpret_cast<const FString*>(SrcDataPtr);
			break;
		case ECameraContextDataType::Enum:
			*reinterpret_cast<uint32*>(DestDataPtr) = *reinterpret_cast<const uint32*>(SrcDataPtr);
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(DataTypeObject);
				if (ensure(StructType))
				{
					StructType->CopyScriptStruct(DestDataPtr, SrcDataPtr);
				}
			}
			break;
		case ECameraContextDataType::Object:
			*reinterpret_cast<TObjectPtr<UObject>*>(DestDataPtr) = *reinterpret_cast<const TObjectPtr<UObject>*>(SrcDataPtr);
			break;
		case ECameraContextDataType::Class:
			*reinterpret_cast<TObjectPtr<UClass>*>(DestDataPtr) = *reinterpret_cast<const TObjectPtr<UClass>*>(SrcDataPtr);
			break;
		default:
			ensure(false);
			return false;
	}
	return true;
}

bool FCameraContextDataTable::IsValueWritten(FCameraContextDataID InID) const
{
	if (const FEntry* Entry = FindEntry(InID))
	{
		return EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written);
	}
	return false;
}

void FCameraContextDataTable::UnsetValue(FCameraContextDataID InID)
{
	if (FEntry* Entry = FindEntry(InID))
	{
		return EnumRemoveFlags(Entry->Flags, EEntryFlags::Written);
	}
}

void FCameraContextDataTable::UnsetAllValues()
{
	for (FEntry& Entry : Entries)
	{
		EnumRemoveFlags(Entry.Flags, EEntryFlags::Written);
	}
}

bool FCameraContextDataTable::IsValueWrittenThisFrame(FCameraContextDataID InID) const
{
	if (const FEntry* Entry = FindEntry(InID))
	{
		return EnumHasAnyFlags(Entry->Flags, EEntryFlags::WrittenThisFrame);
	}
	return false;
}

void FCameraContextDataTable::ClearAllWrittenThisFrameFlags()
{
	for (FEntry& Entry : Entries)
	{
		EnumRemoveFlags(Entry.Flags, EEntryFlags::WrittenThisFrame);
	}
}

void FCameraContextDataTable::AutoResetValues()
{
	for (FEntry& Entry : Entries)
	{
		if (EnumHasAnyFlags(Entry.Flags, EEntryFlags::AutoReset))
		{
			EnumRemoveFlags(Entry.Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
		}
	}
}

void FCameraContextDataTable::OverrideAll(const FCameraContextDataTable& OtherTable)
{
	InternalOverride(OtherTable, ECameraContextDataTableFilter::None);
}

void FCameraContextDataTable::OverrideKnown(const FCameraContextDataTable& OtherTable)
{
	InternalOverride(OtherTable, ECameraContextDataTableFilter::KnownOnly);
}

void FCameraContextDataTable::Override(const FCameraContextDataTable& OtherTable, ECameraContextDataTableFilter Filter)
{
	InternalOverride(OtherTable, Filter);
}

void FCameraContextDataTable::InternalOverride(const FCameraContextDataTable& OtherTable, ECameraContextDataTableFilter Filter)
{
	const bool bKnownOnly = EnumHasAllFlags(Filter, ECameraContextDataTableFilter::KnownOnly);
	const bool bChangedOnly = EnumHasAllFlags(Filter, ECameraContextDataTableFilter::ChangedOnly);

	for (const FEntry& OtherEntry : OtherTable.Entries)
	{
		const EEntryFlags OtherFlags = OtherEntry.Flags;
		if (EnumHasAnyFlags(OtherFlags, EEntryFlags::Written)
				&& (!bChangedOnly || EnumHasAnyFlags(OtherFlags, EEntryFlags::WrittenThisFrame)))
		{
			int32 ThisIndex = EntryLookup.FindRef(OtherEntry.ID, INDEX_NONE);
			if (ThisIndex == INDEX_NONE)
			{
				if (bKnownOnly)
				{
					continue;
				}

				FCameraContextDataDefinition OtherEntryDefinition;
				OtherEntryDefinition.DataID = OtherEntry.ID;
				OtherEntryDefinition.DataType = OtherEntry.Type;
				OtherEntryDefinition.DataContainerType = OtherEntry.ContainerType;
				OtherEntryDefinition.DataTypeObject = OtherEntry.TypeObject;
				AddData(OtherEntryDefinition);
				ThisIndex = Entries.Num() - 1;
			}

			ensure(ThisIndex != INDEX_NONE);
			
			FEntry& ThisEntry = Entries[ThisIndex];
			if (!ensure(ThisEntry.Type == OtherEntry.Type && ThisEntry.TypeObject == OtherEntry.TypeObject))
			{
				continue;
			}

			uint8* ThisDataPtr = Memory + ThisEntry.Offset;
			uint8* OtherDataPtr = OtherTable.Memory + OtherEntry.Offset;
			SetDataValue(ThisEntry.Type, ThisEntry.ContainerType, ThisEntry.TypeObject, ThisDataPtr, OtherDataPtr);

			EnumAddFlags(ThisEntry.Flags, EEntryFlags::Written | (OtherFlags & EEntryFlags::WrittenThisFrame));
		}
	}
}

FCameraContextDataTable::FArrayEntryHelper::FArrayEntryHelper(const FEntry& Entry, uint8* TableMemory)
	: FArrayEntryHelper(Entry.Type, Entry.TypeObject, TableMemory + Entry.Offset)
{
	check(Entry.ContainerType == ECameraContextDataContainerType::Array);
}

FCameraContextDataTable::FArrayEntryHelper::FArrayEntryHelper(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* RawPtr)
{
	ElementType = DataType;
	ElementTypeObject = DataTypeObject;

	uint32 SizeOf, AlignOf;
	FCameraContextDataTable::GetDataTypeAllocationInfo(DataType, DataTypeObject, SizeOf, AlignOf);

	// ElementSizeOf is the total size of an array entry, including padding.
	ElementSizeOf = Align(SizeOf, AlignOf);
	ElementAlignOf = AlignOf;

	ScriptArray = (FEntryScriptArray*)RawPtr;
}

bool FCameraContextDataTable::FArrayEntryHelper::IsValidIndex(int32 Index) const
{
	return Index >= 0 && Index < Num();
}

int32 FCameraContextDataTable::FArrayEntryHelper::Num() const
{
	return ScriptArray->Num();
}

uint8* FCameraContextDataTable::FArrayEntryHelper::GetRawPtr(int32 Index)
{
	checkSlow(IsValidIndex(Index));
	return (uint8*)ScriptArray->GetData() + Index * ElementSizeOf;
}

void FCameraContextDataTable::FArrayEntryHelper::Resize(int32 Count)
{
	int32 OldNum = Num();
	if (Count > OldNum)
	{
		ScriptArray->Add(Count - OldNum, (int32)ElementSizeOf, ElementAlignOf);
		for (int32 Index = OldNum; Index < Count; ++Index)
		{
			uint8* ElementPtr = GetRawPtr(Index);
			FCameraContextDataTable::ConstructDataValue(ElementType, ElementTypeObject, ElementPtr);
		}
	}
	else if (Count < OldNum)
	{
		for (int32 Index = Count; Index < OldNum; ++Index)
		{
			uint8* ElementPtr = GetRawPtr(Index);
			FCameraContextDataTable::DestroyDataValue(ElementType, ElementTypeObject, ElementPtr);
		}
		ScriptArray->Remove(Count, OldNum - Count, (int32)ElementSizeOf, ElementAlignOf);
	}
}

}  // namespace UE::Cameras

