// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraContextDataTableAllocationInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraContextDataTableAllocationInfo)

namespace UE::Cameras::Private
{

void GetKnownContextDataDefinitions(
		const FCameraContextDataTableAllocationInfo& AllocationInfo,
		TMap<FCameraContextDataID, int32>& OutKnownIDs)
{
	for (auto It = AllocationInfo.DataDefinitions.CreateConstIterator(); It; ++It)
	{
		const FCameraContextDataDefinition& ContextDataDefinition(*It);
		OutKnownIDs.Add(ContextDataDefinition.DataID, It.GetIndex());
	}
}

}  // namespace UE::Cameras::Private

void FCameraContextDataTableAllocationInfo::Combine(const FCameraContextDataTableAllocationInfo& OtherInfo)
{
	TMap<FCameraContextDataID, int32> KnownIDs;
	UE::Cameras::Private::GetKnownContextDataDefinitions(*this, KnownIDs);

	for (const FCameraContextDataDefinition& OtherDataDefinition : OtherInfo.DataDefinitions)
	{
		const int32 KnownIndex = KnownIDs.FindRef(OtherDataDefinition.DataID, INDEX_NONE);
		if (KnownIndex == INDEX_NONE)
		{
			DataDefinitions.Add(OtherDataDefinition);
		}
		else
		{
			const FCameraContextDataDefinition& KnownDataDefinition(DataDefinitions[KnownIndex]);
			ensure(KnownDataDefinition == OtherDataDefinition);
		}
	}
}

bool FCameraContextDataTableAllocationInfo::Contains(const FCameraContextDataTableAllocationInfo& OtherInfo) const
{
	TMap<FCameraContextDataID, int32> KnownIDs;
	UE::Cameras::Private::GetKnownContextDataDefinitions(*this, KnownIDs);

	for (const FCameraContextDataDefinition& OtherDataDefinition : OtherInfo.DataDefinitions)
	{
		const int32 KnownIndex = KnownIDs.FindRef(OtherDataDefinition.DataID, INDEX_NONE);
		if (KnownIndex == INDEX_NONE)
		{
			return false;
		}
	}

	return true;
}

