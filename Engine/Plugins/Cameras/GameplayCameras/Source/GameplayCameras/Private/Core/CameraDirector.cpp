// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraDirector.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Core/CameraRigProxyRedirectTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraDirector)

void UCameraDirector::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (CameraRigProxyTable_DEPRECATED)
	{
		CameraRigProxyRedirectTable.Entries = CameraRigProxyTable_DEPRECATED->Entries;
		CameraRigProxyTable_DEPRECATED = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FCameraDirectorEvaluatorPtr UCameraDirector::BuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	FCameraDirectorEvaluator* NewEvaluator = OnBuildEvaluator(Builder);
	NewEvaluator->SetPrivateCameraDirector(this);
	return NewEvaluator;
}

void UCameraDirector::BuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext)
{
	OnBuildCameraDirector(BuildContext);
}

void UCameraDirector::GatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const
{
	OnGatherRigUsageInfo(UsageInfo);

	// Add the rigs referenced by the proxy table.
	for (const FCameraRigProxyRedirectTableEntry& Entry : CameraRigProxyRedirectTable.Entries)
	{
		if (Entry.CameraRigProxy)
		{
			UsageInfo.CameraRigProxies.AddUnique(Entry.CameraRigProxy);
		}
		if (Entry.CameraRig)
		{
			UsageInfo.CameraRigs.AddUnique(Entry.CameraRig);
		}
	}

	// Some directors might add CDOs for some reason (for instance if they use a reference collector).
	// We need to remove them and leave them untouched.
	UsageInfo.CameraRigs.Remove(GetMutableDefault<UCameraRigAsset>());
	UsageInfo.CameraRigProxies.Remove(GetMutableDefault<UCameraRigProxyAsset>());
}

void UCameraDirector::ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	FCameraDirectorRigUsageInfo UsageInfo;
	GatherRigUsageInfo(UsageInfo);

	{
		FAssetRegistryTag NumCameraRigsTag;
		NumCameraRigsTag.Name = TEXT("NumUsedCameraRigs");
		NumCameraRigsTag.Value = LexToString(UsageInfo.CameraRigs.Num());
		Context.AddTag(NumCameraRigsTag);

		TStringBuilder<256> CameraRigListBuilder;
		for (const UCameraRigAsset* CameraRig : UsageInfo.CameraRigs)
		{
			const UPackage* CameraRigPackage = CameraRig->GetPackage();
			CameraRigListBuilder << CameraRigPackage->GetName();
			CameraRigListBuilder << TEXT("\n");
		}

		FAssetRegistryTag CameraRigsTag;
		CameraRigsTag.Name = TEXT("UsedCameraRigs");
		CameraRigsTag.Value = CameraRigListBuilder.ToString();
		Context.AddTag(CameraRigsTag);
	}

	{
		FAssetRegistryTag NumCameraRigProxiesTag;
		NumCameraRigProxiesTag.Name = TEXT("NumUsedCameraRigProxies");
		NumCameraRigProxiesTag.Value = LexToString(UsageInfo.CameraRigProxies.Num());
		Context.AddTag(NumCameraRigProxiesTag);

		TStringBuilder<256> CameraRigProxyListBuilder;
		for (const UCameraRigProxyAsset* CameraRigProxy : UsageInfo.CameraRigProxies)
		{
			const UPackage* CameraRigPackage = CameraRigProxy->GetPackage();
			CameraRigProxyListBuilder << CameraRigPackage->GetName();
			CameraRigProxyListBuilder << TEXT("\n");
		}

		FAssetRegistryTag CameraRigProxiesTag;
		CameraRigProxiesTag.Name = TEXT("UsedCameraRigProxies");
		CameraRigProxiesTag.Value = CameraRigProxyListBuilder.ToString();
		Context.AddTag(CameraRigProxiesTag);
	}

	OnExtendAssetRegistryTags(Context);
}

#if WITH_EDITOR

void UCameraDirector::FactoryCreateAsset(const FCameraDirectorFactoryCreateParams& InParams)
{
	OnFactoryCreateAsset(InParams);
}

#endif  // WITH_EDITOR

