// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraDirectorEvaluator.h"

#include "Core/CameraDirector.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigProxyRedirectTable.h"
#include "Core/CameraRigTransition.h"

namespace UE::Cameras
{

bool FCameraRigActivationDeactivationRequest::ResolveCameraRigProxyIfNeeded(const FCameraDirectorEvaluator* InDirectorEvaluator)
{
	return ResolveCameraRigProxyIfNeeded(InDirectorEvaluator->GetCameraDirector());
}

bool FCameraRigActivationDeactivationRequest::ResolveCameraRigProxyIfNeeded(const UCameraDirector* InDirector)
{
	if (CameraRig == nullptr && CameraRigProxy != nullptr && ensure(InDirector))
	{
		const FCameraRigProxyRedirectTable& ProxyTable = InDirector->CameraRigProxyRedirectTable;

		FCameraRigProxyResolveParams ResolveParams;
		ResolveParams.CameraRigProxy = CameraRigProxy;
		CameraRig = ProxyTable.ResolveProxy(ResolveParams);
	}
	return CameraRig != nullptr;
}

void FCameraDirectorEvaluatorStorage::DestroyEvaluator()
{
	Evaluator.Reset();
}

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraDirectorEvaluator)

FCameraDirectorEvaluator::FCameraDirectorEvaluator()
{
}

void FCameraDirectorEvaluator::SetPrivateCameraDirector(const UCameraDirector* InCameraDirector)
{
	PrivateCameraDirector = InCameraDirector;
}

void FCameraDirectorEvaluator::Initialize(const FCameraDirectorInitializeParams& Params)
{
	WeakOwnerContext = Params.OwnerContext;

	OnInitialize(Params);
}

void FCameraDirectorEvaluator::Activate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	Evaluator = Params.Evaluator;

	OnActivate(Params, OutResult);
}

void FCameraDirectorEvaluator::Deactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	OnDeactivate(Params, OutResult);

	Evaluator = nullptr;
}

void FCameraDirectorEvaluator::Run(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	OnRun(Params, OutResult);

	// Set some overrides on the first main-layer activation we find.
	if (NextActivationTransitionOverride || bNextActivationForce)
	{
		for (FCameraRigActivationDeactivationRequest& Request : OutResult.Requests)
		{
			if (Request.RequestType == ECameraRigActivationDeactivationRequestType::Activate && Request.Layer == ECameraRigLayer::Main)
			{
				Request.TransitionOverride = NextActivationTransitionOverride;
				Request.bForceActivateDeactivate |= bNextActivationForce;
			}
		}
	}

	NextActivationTransitionOverride = nullptr;
	bNextActivationForce = false;
}

const UCameraRigAsset* FCameraDirectorEvaluator::FindCameraRigByProxy(const UCameraRigProxyAsset* InProxy) const
{
	if (!ensure(PrivateCameraDirector))
	{
		return nullptr;
	}

	const FCameraRigProxyRedirectTable& ProxyTable = PrivateCameraDirector->CameraRigProxyRedirectTable;

	FCameraRigProxyResolveParams ResolveParams;
	ResolveParams.CameraRigProxy = InProxy;
	return ProxyTable.ResolveProxy(ResolveParams);
}

void FCameraDirectorEvaluator::OnEndCameraSystemUpdate()
{
	// Also clear these here, in case we didn't run this frame.
	NextActivationTransitionOverride = nullptr;
	bNextActivationForce = false;
}

void FCameraDirectorEvaluator::OverrideNextActivationTransition(const UCameraRigTransition* TransitionOverride)
{
	NextActivationTransitionOverride = TransitionOverride;
}

void FCameraDirectorEvaluator::ForceNextActivation()
{
	bNextActivationForce = true;
}

bool FCameraDirectorEvaluator::AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext)
{
	TSharedPtr<FCameraEvaluationContext> OwnerContext = WeakOwnerContext.Pin();
	if (!ensureMsgf(OwnerContext, TEXT("Can't add child evaluation context when the parent/owner context is invalid!")))
	{
		return false;
	}

	FChildContextManulationParams Params;
	Params.ParentContext = OwnerContext;
	Params.ChildContext = InContext.ToSharedPtr();
	FChildContextManulationResult Result;
	OnAddChildEvaluationContext(Params, Result);

	bool bReturn = false;
	bool bRegisterAndActivateChildContext = false;
	switch (Result.Result)
	{
		case EChildContextManipulationResult::Failure:
		default:
			// Nothing to do.
			break;
		case EChildContextManipulationResult::Success:
			// Our director evaluator accepted the child context.
			bRegisterAndActivateChildContext = true;
			bReturn = true;
			break;
		case EChildContextManipulationResult::ChildContextSuccess:
			// A sub-director of our director accepted the child context, so it already
			// activated it and we don't need to do it ourselves.
			bReturn = true;
			break;
	}
	if (bRegisterAndActivateChildContext)
	{
		OwnerContext->RegisterChildContext(InContext);

		FCameraEvaluationContextActivateParams ActivateParams;
		ActivateParams.Evaluator = Evaluator;
		ActivateParams.ParentContext = OwnerContext;
		InContext->Activate(ActivateParams);
	}
	return bReturn;
}

bool FCameraDirectorEvaluator::RemoveChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext)
{
	TSharedPtr<FCameraEvaluationContext> OwnerContext = WeakOwnerContext.Pin();
	if (!ensureMsgf(OwnerContext, TEXT("Can't remove child evaluation context when the parent/owner context is invalid!")))
	{
		return false;
	}

	FChildContextManulationParams Params;
	Params.ParentContext = OwnerContext;
	Params.ChildContext = InContext;
	FChildContextManulationResult Result;
	OnRemoveChildEvaluationContext(Params, Result);

	bool bReturn = false;
	bool bUnregisterAndDeactivateChildContext = false;
	switch (Result.Result)
	{
		case EChildContextManipulationResult::Failure:
		default:
			break;
		case EChildContextManipulationResult::Success:
			bUnregisterAndDeactivateChildContext = true;
			bReturn = true;
			break;
		case EChildContextManipulationResult::ChildContextSuccess:
			bReturn = true;
			break;
	}
	if (bUnregisterAndDeactivateChildContext)
	{
		OwnerContext->UnregisterChildContext(InContext);

		FCameraEvaluationContextDeactivateParams DeactivateParams;
		InContext->Deactivate(DeactivateParams);
	}
	return bReturn;
}

void FCameraDirectorEvaluator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PrivateCameraDirector);

	OnAddReferencedObjects(Collector);
}

}  // namespace UE::Cameras

