// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraEvaluationContext.h"

#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraEvaluationContext)

namespace UE::Cameras
{

#if WITH_EDITOR

class FEditorPreviewCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FEditorPreviewCameraDirectorEvaluator)

public:

	FEditorPreviewCameraDirectorEvaluator()
	{}

	FEditorPreviewCameraDirectorEvaluator(TConstArrayView<const UCameraRigAsset*> InCameraRigs)
		: CameraRigs(InCameraRigs)
	{}

	int32 GetCameraRigIndex() const { return PreviewIndex; }
	void SetCameraRigIndex(int32 Index) { PreviewIndex = Index; }

protected:

	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override
	{
		if (!CameraRigs.IsValidIndex(PreviewIndex))
		{
			PreviewIndex = (CameraRigs.IsEmpty() ? INDEX_NONE : 0);
		}

		if (PreviewIndex != INDEX_NONE && CameraRigs[PreviewIndex] != nullptr)
		{
			OutResult.Add(GetEvaluationContext(), CameraRigs[PreviewIndex]);
		}
	}

	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects(CameraRigs);
	}

private:

	TArray<TObjectPtr<const UCameraRigAsset>> CameraRigs;
	int32 PreviewIndex = 0;
};

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FEditorPreviewCameraDirectorEvaluator)

#endif  // WITH_EDITOR

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraEvaluationContext)

FCameraEvaluationContext::FCameraEvaluationContext()
{
}

FCameraEvaluationContext::FCameraEvaluationContext(const FCameraEvaluationContextInitializeParams& Params)
{
	Initialize(Params);
}

void FCameraEvaluationContext::Initialize(const FCameraEvaluationContextInitializeParams& Params)
{
	if (!ensureMsgf(!bInitialized, TEXT("This evaluation context has already been initialized!")))
	{
		return;
	}

	WeakOwner = Params.Owner;
	CameraAsset = Params.CameraAsset;
	WeakPlayerController = Params.PlayerController;

	if (CameraAsset)
	{
		const FCameraAssetAllocationInfo& AllocationInfo = CameraAsset->GetAllocationInfo();

#if WITH_EDITOR
		// In editor/PIE builds the camera asset's AllocationInfo is only populated when cooking,
		// so it will be empty at runtime. Gather the allocation info transiently from the camera
		// rigs instead, mirroring what CameraAssetBuilder does at cook time.
		if (AllocationInfo.VariableTableInfo.VariableDefinitions.IsEmpty() &&
				AllocationInfo.ContextDataTableInfo.DataDefinitions.IsEmpty())
		{
			FCameraDirectorRigUsageInfo UsageInfo;
			if (UCameraDirector* Director = CameraAsset->GetCameraDirector())
			{
				Director->GatherRigUsageInfo(UsageInfo);
			}

			FCameraAssetAllocationInfo RuntimeAllocationInfo;
			for (const UCameraRigAsset* Rig : UsageInfo.CameraRigs)
			{
				if (Rig)
				{
					RuntimeAllocationInfo.VariableTableInfo.Combine(Rig->AllocationInfo.VariableTableInfo);
					RuntimeAllocationInfo.ContextDataTableInfo.Combine(Rig->AllocationInfo.ContextDataTableInfo);
				}
			}

			InitialResult.VariableTable.Initialize(RuntimeAllocationInfo.VariableTableInfo);
			InitialResult.ContextDataTable.Initialize(RuntimeAllocationInfo.ContextDataTableInfo);
		}
		else
#endif  // WITH_EDITOR
		{
			InitialResult.VariableTable.Initialize(AllocationInfo.VariableTableInfo);
			InitialResult.ContextDataTable.Initialize(AllocationInfo.ContextDataTableInfo);
		}
	}

	bInitialized = true;
}

FCameraEvaluationContext::~FCameraEvaluationContext()
{
	// Camera director evaluator usually gets destroyed here since the storage object generally
	// holds the only shared pointer to it.
}

UWorld* FCameraEvaluationContext::GetWorld() const
{
	if (UObject* Owner = GetOwner())
	{
		return Owner->GetWorld();
	}
	return nullptr;
}

FIntPoint FCameraEvaluationContext::GetViewportSize() const
{
	if (OverrideViewportSize.IsSet())
	{
		return OverrideViewportSize.GetValue();
	}

	if (APlayerController* PlayerController = GetPlayerController())
	{
		int32 ViewportSizeX = 0, ViewportSizeY = 0;
		PlayerController->GetViewportSize(ViewportSizeX, ViewportSizeY);
		return FIntPoint(ViewportSizeX, ViewportSizeY);
	}

	return FIntPoint(EForceInit::ForceInit);
}

void FCameraEvaluationContext::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraAsset);

	OnAddReferencedObjects(Collector);

	if (DirectorEvaluator)
	{
		DirectorEvaluator->AddReferencedObjects(Collector);
	}

	for (TSharedPtr<FCameraEvaluationContext> ChildContext : ChildrenContexts)
	{
		ChildContext->AddReferencedObjects(Collector);
	}
}

void FCameraEvaluationContext::OnEndCameraSystemUpdate()
{
	InitialResult.CameraPose.ClearAllChangedFlags();

	InitialResult.VariableTable.AutoResetValues();
	InitialResult.VariableTable.ClearAllWrittenThisFrameFlags();

	InitialResult.ContextDataTable.AutoResetValues();
	InitialResult.ContextDataTable.ClearAllWrittenThisFrameFlags();

	for (FConditionalResults::ElementType& Pair : ConditionalResults)
	{
		FCameraNodeEvaluationResult& Result = Pair.Value;
		Result.CameraPose.ClearAllChangedFlags();
		Result.VariableTable.AutoResetValues();
		Result.VariableTable.ClearAllWrittenThisFrameFlags();
		Result.ContextDataTable.AutoResetValues();
		Result.ContextDataTable.ClearAllWrittenThisFrameFlags();
	}

	if (DirectorEvaluator)
	{
		DirectorEvaluator->OnEndCameraSystemUpdate();
	}

	for (TSharedPtr<FCameraEvaluationContext> ChildContext : ChildrenContexts)
	{
		ChildContext->OnEndCameraSystemUpdate();
	}
}

void FCameraEvaluationContext::AutoCreateDirectorEvaluator()
{
	if (DirectorEvaluator == nullptr)
	{
		if (!CameraAsset)
		{
			UE_LOGF(LogCameraSystem, Error, "Activating an evaluation context without a camera!");
			return;
		}
		if (!CameraAsset->GetCameraDirector())
		{
			UE_LOGF(LogCameraSystem, Error, "Activating an evaluation context without a camera director!");
			return;
		}

		const UCameraDirector* CameraDirector = CameraAsset->GetCameraDirector();
		FCameraDirectorEvaluatorBuilder DirectorBuilder(DirectorEvaluatorStorage);
		DirectorEvaluator = CameraDirector->BuildEvaluator(DirectorBuilder);

		FCameraDirectorInitializeParams InitParams;
		InitParams.OwnerContext = SharedThis(this);
		DirectorEvaluator->Initialize(InitParams);
	}
}

#if WITH_EDITOR

void FCameraEvaluationContext::AutoCreateEditorPreviewDirectorEvaluator(const FCameraEvaluationContextActivateParams& Params)
{
	// If a sub-class already created an evaluator, just use that.
	if (DirectorEvaluator)
	{
		return;
	}

	// If we're not meant for an editor preview, bail out.
	if (!ensure(Params.Evaluator))
	{
		return;
	}
	if (Params.Evaluator->GetRole() != ECameraSystemEvaluatorRole::EditorPreview)
	{
		return;
	}

	// Find the camera director to use. If it is OK to run in the editor, bail out and let AutoCreateDirectorEvaluator
	// create it.
	UCameraDirector* CameraDirector = nullptr;
	if (CameraAsset && CameraAsset->GetCameraDirector())
	{
		CameraDirector = CameraAsset->GetCameraDirector();
	}
	
	if (CameraDirector && CameraDirector->bRunInEditor)
	{
		return;
	}

	// Create a simple preview director evaluator that just lets the user switch between the known rigs.
	FCameraDirectorRigUsageInfo UsageInfo;
	if (CameraDirector)
	{
		CameraDirector->GatherRigUsageInfo(UsageInfo);
	}

	FCameraDirectorEvaluatorBuilder DirectorBuilder(DirectorEvaluatorStorage);
	DirectorEvaluator = DirectorBuilder.BuildEvaluator<FEditorPreviewCameraDirectorEvaluator>(UsageInfo.CameraRigs);

	FCameraDirectorInitializeParams InitParams;
	InitParams.OwnerContext = SharedThis(this);
	DirectorEvaluator->Initialize(InitParams);
}

void FCameraEvaluationContext::SetEditorPreviewCameraRigIndex(int32 Index)
{
	if (!DirectorEvaluator)
	{
		return;
	}

	FEditorPreviewCameraDirectorEvaluator* EditorPreviewEvaluator = DirectorEvaluator->CastThis<FEditorPreviewCameraDirectorEvaluator>();
	if (EditorPreviewEvaluator)
	{
		EditorPreviewEvaluator->SetCameraRigIndex(Index);
	}
}

#endif  // WITH_EDITOR

FCameraNodeEvaluationResult& FCameraEvaluationContext::GetOrAddConditionalResult(ECameraEvaluationDataCondition Condition)
{
	if (FCameraNodeEvaluationResult* ExistingResult = ConditionalResults.Find(Condition))
	{
		return *ExistingResult;
	}

	FCameraNodeEvaluationResult& NewResult = ConditionalResults.Add(Condition);

	if (CameraAsset)
	{
		const FCameraAssetAllocationInfo& AllocationInfo = CameraAsset->GetAllocationInfo();

		NewResult.VariableTable.Initialize(AllocationInfo.VariableTableInfo);
		NewResult.ContextDataTable.Initialize(AllocationInfo.ContextDataTableInfo);
	}

	return NewResult;
}

void FCameraEvaluationContext::Activate(const FCameraEvaluationContextActivateParams& Params)
{
	if (!ensureMsgf(bInitialized, TEXT("This evaluation context needs to be initialized!")))
	{
		return;
	}
	if (!ensureMsgf(!bActivated, TEXT("This evaluation context has already been activated!")))
	{
		return;
	}

	OnActivate(Params);

#if WITH_EDITOR
	AutoCreateEditorPreviewDirectorEvaluator(Params);
#endif  // WITH_EDITOR

	AutoCreateDirectorEvaluator();

	CameraSystemEvaluator = Params.Evaluator;
	bActivated = true;

	if (DirectorEvaluator)
	{
		FCameraDirectorActivateParams DirectorParams;
		DirectorParams.Evaluator = Params.Evaluator;
		FCameraDirectorEvaluationResult DirectorResult;
		DirectorEvaluator->Activate(DirectorParams, DirectorResult);

		// Execute initial requests from the director.
		ExecuteSetupAndTeardownRequests(DirectorResult);
	}
}

void FCameraEvaluationContext::Deactivate(const FCameraEvaluationContextDeactivateParams& Params)
{
	if (!ensureMsgf(bActivated, TEXT("This evaluation context has not been activated!")))
	{
		return;
	}

	if (DirectorEvaluator)
	{
		FCameraDirectorDeactivateParams DirectorParams;
		FCameraDirectorEvaluationResult DirectorResult;
		DirectorEvaluator->Deactivate(DirectorParams, DirectorResult);

		// Execute last requests from the director.
		ExecuteSetupAndTeardownRequests(DirectorResult);
	}

	// Don't destroy the camera director evaluator, it could still be useful. We only destroy it
	// along with this context.

	OnDeactivate(Params);

	CameraSystemEvaluator = nullptr;
	bActivated = false;
}

void FCameraEvaluationContext::ExecuteSetupAndTeardownRequests(FCameraDirectorEvaluationResult& DirectorResult)
{
	if (!ensure(CameraSystemEvaluator))
	{
		return;
	}

	FRootCameraNodeEvaluator* RootEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();

	for (FCameraRigActivationDeactivationRequest& Request : DirectorResult.Requests)
	{
		if (Request.Layer == ECameraRigLayer::Main)
		{
			// Executing main-layer requests would require being able to handle dynamically combined camera rigs,
			// which is possible but requries refactoring that away from FCameraSystemEvaluator, so leave that
			// for later, especially since it's arguably wrong to try to activate main rigs in Activate/Deactivate.
			UObject* RequestedCameraObject = Request.CameraRig ? (UObject*)Request.CameraRig : (UObject*)Request.CameraRigProxy;
			UE_LOGF(LogCameraSystem, Error, 
					"Main layer camera rigs can only be activated/deactivated during the director's normal update. "
	 					 "Ignoring request to activate/deactivate '%ls'.",
					*GetNameSafe(RequestedCameraObject));
			continue;
		}

		Request.ResolveCameraRigProxyIfNeeded(DirectorEvaluator);
		if (Request.IsValid())
		{
			RootEvaluator->ExecuteCameraDirectorRequest(Request);
		}
	}
}

TSharedPtr<FCameraSystemEvaluator> FCameraEvaluationContext::GetCameraSystemEvaluator() const
{
	if (CameraSystemEvaluator)
	{
		return CameraSystemEvaluator->AsShared();
	}
	return TSharedPtr<FCameraSystemEvaluator>();
}

bool FCameraEvaluationContext::AddChildContext(TSharedRef<FCameraEvaluationContext> ChildContext)
{
	if (DirectorEvaluator)
	{
		ensure(DirectorEvaluator->GetEvaluationContext() == SharedThis(this));
		return DirectorEvaluator->AddChildEvaluationContext(ChildContext);
	}

	return false;
}

bool FCameraEvaluationContext::RemoveChildContext(TSharedRef<FCameraEvaluationContext> ChildContext)
{
	if (DirectorEvaluator)
	{
		ensure(DirectorEvaluator->GetEvaluationContext() == SharedThis(this));
		return DirectorEvaluator->RemoveChildEvaluationContext(ChildContext);
	}

	return false;
}

bool FCameraEvaluationContext::RegisterChildContext(TSharedRef<FCameraEvaluationContext> ChildContext)
{
	if (!ensureMsgf(ChildContext->WeakParent == nullptr, TEXT("The given evaluation context already has a parent!")))
	{
		return false;
	}

	ChildContext->WeakParent = SharedThis(this);
	ChildrenContexts.Add(ChildContext);
	return true;
}

bool FCameraEvaluationContext::UnregisterChildContext(TSharedRef<FCameraEvaluationContext> ChildContext)
{
	if (!ensureMsgf(ChildContext->WeakParent == SharedThis(this), TEXT("The given evaluation context isn't our child!")))
	{
		return false;
	}

	ChildContext->WeakParent = nullptr;
	const int32 NumRemoved = ChildrenContexts.Remove(ChildContext);
	ensureMsgf(NumRemoved == 1, TEXT("The given evaluation context wasn't in our list of children!"));
	return true;
}

}  // namespace UE::Cameras

