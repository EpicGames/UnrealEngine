// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraEvaluationContextStack.h"

#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraVariableTable.h"

namespace UE::Cameras
{

FCameraEvaluationContextStack::~FCameraEvaluationContextStack()
{
	Reset();
}

TSharedPtr<FCameraEvaluationContext> FCameraEvaluationContextStack::GetActiveContext() const
{
	for (const FContextEntry& Entry : ReverseIterate(Entries))
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			return Context;
		}
	}
	return nullptr;
}

bool FCameraEvaluationContextStack::HasContext(TSharedRef<FCameraEvaluationContext> Context) const
{
	for (const FContextEntry& Entry : ReverseIterate(Entries))
	{
		if (Context == Entry.WeakContext)
		{
			return true;
		}
	}
	return false;
}

void FCameraEvaluationContextStack::PushContext(TSharedRef<FCameraEvaluationContext> Context)
{
	checkf(Evaluator, TEXT("Can't push context when no evaluator is set! Did you call Initialize?"));

	// If we're pushing an existing context, move it to the top.
	const int32 ExistingIndex = Entries.IndexOfByPredicate(
			[Context](const FContextEntry& Entry) { return Entry.WeakContext == Context; });
	if (ExistingIndex != INDEX_NONE)
	{
		if (ExistingIndex < Entries.Num() - 1)
		{
			FContextEntry EntryCopy(MoveTemp(Entries[ExistingIndex]));
			Entries.RemoveAt(ExistingIndex);
			Entries.Add(MoveTemp(EntryCopy));

			OnStackChangedEvent.Broadcast();
		}
		return;
	}

	// Make a new entry and activate the context. This will build the director evaluator.
	FContextEntry NewEntry;

	FCameraEvaluationContextActivateParams ActivateParams;
	ActivateParams.Evaluator = Evaluator;
	Context->Activate(ActivateParams);
	
	NewEntry.WeakContext = Context;
	Entries.Push(MoveTemp(NewEntry));

	OnStackChangedEvent.Broadcast();
}

bool FCameraEvaluationContextStack::AddChildContext(TSharedRef<FCameraEvaluationContext> Context, TSharedPtr<FCameraEvaluationContext> ParentContext)
{
	if (ParentContext == nullptr && !Entries.IsEmpty())
	{
		ParentContext = GetActiveContext();
	}

	// No parent context provided, and no active context found in the stack.
	if (!ParentContext.IsValid())
	{
		return false;
	}

	// The context is already in the stack. The caller should remove it first.
	const int32 ExistingIndex = Entries.IndexOfByPredicate(
			[Context](const FContextEntry& Entry) { return Entry.WeakContext == Context; });
	if (ExistingIndex != INDEX_NONE)
	{
		return false;
	}

	// Check invalid situation.
	if (!ensureMsgf(Context != ParentContext, TEXT("Can't add a context as a child of itself")))
	{
		return false;
	}

	FCameraDirectorEvaluator* DirectorEvaluator = ParentContext->GetDirectorEvaluator();
	if (ensureMsgf(DirectorEvaluator, TEXT("Can't add child context, active context has no camera director evaluator!")))
	{
		return DirectorEvaluator->AddChildEvaluationContext(Context);
	}

	return false;
}

bool FCameraEvaluationContextStack::RemoveContext(TSharedRef<FCameraEvaluationContext> Context)
{
	for (auto It = Entries.CreateIterator(); It; ++It)
	{
		FContextEntry& Entry = (*It);
		if (Entry.WeakContext == Context)
		{
			FCameraEvaluationContextDeactivateParams DeactivateParams;
			Context->Deactivate(DeactivateParams);

			It.RemoveCurrent();

			OnStackChangedEvent.Broadcast();
			return true;
		}
	}
	return false;
}

bool FCameraEvaluationContextStack::RemoveContextsOwnedBy(UObject* ContextOwner, bool bAlsoRemoveInnerOwners)
{
	bool bRemovedAny = false;

	for (auto It = Entries.CreateIterator(); It; ++It)
	{
		FContextEntry& Entry = (*It);
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			if (!Context->GetOwner())
			{
				continue;
			}

			if (Context->GetOwner() == ContextOwner || (bAlsoRemoveInnerOwners && Context->GetOwner()->IsIn(ContextOwner)))
			{
				FCameraEvaluationContextDeactivateParams DeactivateParams;
				Context->Deactivate(DeactivateParams);

				It.RemoveCurrent();
				bRemovedAny = true;
			}
		}
	}

	if (bRemovedAny)
	{
		OnStackChangedEvent.Broadcast();
	}

	return bRemovedAny;
}

void FCameraEvaluationContextStack::PopContext()
{
	Entries.Pop();
	OnStackChangedEvent.Broadcast();
}

void FCameraEvaluationContextStack::GetAllContexts(TArray<TSharedPtr<FCameraEvaluationContext>>& OutContexts) const
{
	for (const FContextEntry& Entry : Entries)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			OutContexts.Add(Context);
		}
	}
}

TSharedPtr<FCameraEvaluationContext> FCameraEvaluationContextStack::FindContextByOwner(const UObject* InOwner) const
{
	for (const FContextEntry& Entry : Entries)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			if (Context->GetOwner() == InOwner)
			{
				return Context;
			}
		}
	}
	return TSharedPtr<FCameraEvaluationContext>();
}

void FCameraEvaluationContextStack::Reset()
{
	for (FContextEntry& Entry : Entries)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			FCameraEvaluationContextDeactivateParams DeactivateParams;
			Context->Deactivate(DeactivateParams);
		}
	}
	Entries.Reset();
	OnStackChangedEvent.Broadcast();
}

void FCameraEvaluationContextStack::Initialize(FCameraSystemEvaluator& InEvaluator)
{
	Evaluator = &InEvaluator;
}

void FCameraEvaluationContextStack::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FContextEntry& Entry : Entries)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			Context->AddReferencedObjects(Collector);
		}
	}
}

void FCameraEvaluationContextStack::OnEndCameraSystemUpdate()
{
	// Reset all written-this-frame flags on evaluation contexts, so we properly get those flags set
	// regardless of when, during next frame, they set their variables. This is because various 
	// gameplay systems, Blueprint scripting, whatever, might set variables at any time.
	bool bRemovedAny = false;
	TArray<TSharedPtr<FCameraEvaluationContext>> ContextsToVisit;
	for (auto It = Entries.CreateIterator(); It; ++It)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = It->WeakContext.Pin())
		{
			ContextsToVisit.Add(Context);
		}
		else
		{
			It.RemoveCurrent();
			bRemovedAny = true;
		}
	}
	while (!ContextsToVisit.IsEmpty())
	{
		TSharedPtr<FCameraEvaluationContext> Context = ContextsToVisit.Pop();
		Context->OnEndCameraSystemUpdate();

		TArrayView<const TSharedPtr<FCameraEvaluationContext>> ChildrenContexts(Context->GetChildrenContexts());
		for (TSharedPtr<FCameraEvaluationContext> ChildContext : ReverseIterate(ChildrenContexts))
		{
			if (ChildContext)
			{
				ContextsToVisit.Add(ChildContext);
			}
		}
	}

	if (bRemovedAny)
	{
		OnStackChangedEvent.Broadcast();
	}
}

}  // namespace UE::Cameras

