// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraEvaluationService.h"

#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"

namespace UE::Cameras
{

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraEvaluationService)

FCameraEvaluationService::FCameraEvaluationService()
{
}

FCameraEvaluationService::~FCameraEvaluationService()
{
	if (!ensureMsgf(
				Evaluator == nullptr, 
				TEXT("Evaluation service being destroyed without having been torn down properly")))
	{
		if (EnumHasAllFlags(PrivateFlags, ECameraEvaluationServiceFlags::NeedsRootCameraNodeEvents))
		{
			if (FRootCameraNodeEvaluator* RootNodeEvaluator = Evaluator->GetRootNodeEvaluator())
			{
				RootNodeEvaluator->OnCameraRigEvent().RemoveAll(this);
			}
		}
	}
}

void FCameraEvaluationService::Initialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	ensureMsgf(!Evaluator, TEXT("Evaluation service has already been initialized"));
	ensureMsgf(Params.Evaluator, TEXT("Evaluation service initialized with an invalid system evaluator"));
	Evaluator = Params.Evaluator;

	OnInitialize(Params);

	if (Evaluator && EnumHasAllFlags(PrivateFlags, ECameraEvaluationServiceFlags::NeedsRootCameraNodeEvents))
	{
		FRootCameraNodeEvaluator* RootNodeEvaluator = Params.Evaluator->GetRootNodeEvaluator();
		if (ensure(RootNodeEvaluator))
		{
			RootNodeEvaluator->OnCameraRigEvent().AddRaw(this, &FCameraEvaluationService::RootCameraNodeEventHandler);
		}
	}
}

void FCameraEvaluationService::PreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	OnPreUpdate(Params, OutResult);
}

void FCameraEvaluationService::PostCameraDirectorUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	OnPostCameraDirectorUpdate(Params, OutResult);
}

void FCameraEvaluationService::PostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	OnPostUpdate(Params, OutResult);
}

void FCameraEvaluationService::Teardown(const FCameraEvaluationServiceTeardownParams& Params)
{
	ensureMsgf(
			Params.Evaluator && Evaluator == Params.Evaluator,
			TEXT("Evaluation service torn down with an invalid system evaluator"));

	OnTeardown(Params);

	if (Evaluator && EnumHasAllFlags(PrivateFlags, ECameraEvaluationServiceFlags::NeedsRootCameraNodeEvents))
	{
		FRootCameraNodeEvaluator* RootNodeEvaluator = Evaluator->GetRootNodeEvaluator();
		if (ensure(RootNodeEvaluator))
		{
			RootNodeEvaluator->OnCameraRigEvent().RemoveAll(this);
		}
	}

	Evaluator = nullptr;
}

void FCameraEvaluationService::RootCameraNodeEventHandler(const FRootCameraNodeCameraRigEvent& InEvent)
{
	OnRootCameraNodeEvent(InEvent);
}

void FCameraEvaluationService::AddReferencedObjects(FReferenceCollector& Collector)
{
	OnAddReferencedObjects(Collector);
}

bool FCameraEvaluationService::HasAllEvaluationServiceFlags(ECameraEvaluationServiceFlags InFlags) const
{
	return EnumHasAllFlags(PrivateFlags, InFlags);
}

void FCameraEvaluationService::SetEvaluationServiceFlags(ECameraEvaluationServiceFlags InFlags)
{
	PrivateFlags = InFlags;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCameraEvaluationService::BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	OnBuildDebugBlocks(Params, Builder);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras
