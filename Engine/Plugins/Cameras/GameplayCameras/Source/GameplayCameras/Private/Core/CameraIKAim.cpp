// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraIKAim.h"

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigEvaluationInfo.h"
#include "Core/CameraRigJoints.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraVariableTable.h"
#include "Core/RootCameraNode.h"
#include "Debug/CameraDebugRenderer.h"
#include "Engine/Engine.h"
#include "GameplayCameras.h"
#include "GameplayCamerasSettings.h"
#include "Math/CameraAimingMath.h"
#include "Math/Ray.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace UE::Cameras
{

bool FCameraIKAim::Run(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo)
{
	FCameraIKAimParams ValidatedParams(Params);

	// Replace invalid parameter values with defaults from the settings.
	const UGameplayCamerasSettings* Settings = UGameplayCamerasSettings::StaticClass()->GetDefaultObject<UGameplayCamerasSettings>();
	if (ValidatedParams.AngleTolerance <= 0)
	{
		ValidatedParams.AngleTolerance = FMath::Max(0.1, Settings->DefaultIKAimingAngleTolerance);
	}
	if (ValidatedParams.DistanceTolerance <= 0)
	{
		ValidatedParams.DistanceTolerance = FMath::Max(0.1, Settings->DefaultIKAimingDistanceTolerance);
	}
	if (ValidatedParams.MinDistance <= 0)
	{
		ValidatedParams.MinDistance = FMath::Max(0.1, Settings->DefaultIKAimingMinDistance);
	}
	if (ValidatedParams.MaxIterations == 0)
	{
		ValidatedParams.MaxIterations = FMath::Max((uint8)1, Settings->DefaultIKAimingMaxIterations);
	}

	return DoRun(ValidatedParams, CameraRigInfo);
}

bool FCameraIKAim::DoRun(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo)
{
	if (!ensureMsgf(CameraRigInfo.CameraRig, TEXT("Can't aim invalid camera rig!")))
	{
		return false;
	}
	if (!ensureMsgf(CameraRigInfo.EvaluationContext, TEXT("Can't aim camera rig '%s', it has no evalution context!"), *CameraRigInfo.CameraRig->GetPathName()))
	{
		return false;
	}

	FCameraSystemEvaluator* CameraSystemEvaluator = Params.Evaluator;

	// Initialize our scratch result.
	ScratchResult.VariableTable.Initialize(CameraRigInfo.CameraRig->AllocationInfo.VariableTableInfo);

	// Initialize our hierarchy caches.
	FRootCameraNodeEvaluator* CameraSystemRootEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
	FSingleCameraRigHierarchyBuildParams HierarchyParams;
	HierarchyParams.CameraRigInfo = CameraRigInfo;
	CameraSystemRootEvaluator->BuildSingleCameraRigHierarchy(HierarchyParams, CameraSystemHierarchy);

	// Iterate on the solution.
	FAimIterationInfo IterationInfo;
	double LastErrorAngle = TNumericLimits<double>::Max();
	double LastErrorDistance = TNumericLimits<double>::Max();

	while (IterationInfo.IterationIndex < Params.MaxIterations)
	{
		DoRunIteration(Params, CameraRigInfo, IterationInfo);

		if (IterationInfo.Result == EAimResult::Failed || IterationInfo.Result == EAimResult::Aborted || IterationInfo.Result == EAimResult::Completed)
		{
			break;
		}
		
		ensure(IterationInfo.Result == EAimResult::Corrected);

		// Check that we are getting closer to a solution.
		if (IterationInfo.ErrorAngle >= LastErrorAngle || IterationInfo.ErrorDistance >= LastErrorDistance)
		{
			UE_LOGF(LogCameraSystem, Error, "Can't converge towards a solution while aiming camera rig '%ls'. Aborting.", *CameraRigInfo.CameraRig->GetPathName());
			IterationInfo.Result = EAimResult::Aborted;
			break;
		}

		LastErrorAngle = IterationInfo.ErrorAngle;
		LastErrorDistance = IterationInfo.ErrorDistance;
		++IterationInfo.IterationIndex;
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	LastRunDebugInfo.DesiredTarget = Params.TargetLocation;
	LastRunDebugInfo.bSucceeded = (IterationInfo.Result == EAimResult::Completed);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	return IterationInfo.Result == EAimResult::Completed || IterationInfo.Result == EAimResult::Corrected;
}

void FCameraIKAim::DoRunIteration(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, FAimIterationInfo& IterationInfo)
{
#if UE_GAMEPLAY_CAMERAS_DEBUG
	FCameraIKAimIterationDebugInfo& IterationDebugInfo = LastRunDebugInfo.Iterations.Emplace_GetRef();
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// Run the system. We restore its state after each run.
	RunRootCameraNode(Params, CameraRigInfo);

	// Check how far we are from the desired target.
	const bool bContinueIteration = CheckTolerance(Params, CameraRigInfo, IterationInfo);
#if UE_GAMEPLAY_CAMERAS_DEBUG
	IterationDebugInfo.CameraPoseLocation = ScratchResult.CameraPose.GetLocation();
	IterationDebugInfo.CameraPoseRotation = ScratchResult.CameraPose.GetRotation();
	IterationDebugInfo.ErrorAngle = IterationInfo.ErrorAngle;
	IterationDebugInfo.ErrorDistance = IterationInfo.ErrorDistance;
	IterationDebugInfo.bNeededSolver = bContinueIteration;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	if (!bContinueIteration)
	{
		return;
	}

	// Look at what we have... in 99% of cases, we're dealing with a camera rig that can be boiled down
	// to 2 bones and one yaw/pitch articulation.
	const FCameraRigJoint* FoundItem = FCameraAimingMath::FindPivotJoint(ScratchResult);
#if UE_GAMEPLAY_CAMERAS_DEBUG
	IterationDebugInfo.bFoundSolver = (FoundItem != nullptr);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	if (!FoundItem)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't figure out how to aim camera rig '%ls'.", *CameraRigInfo.CameraRig->GetPathName());
		IterationInfo.Result = EAimResult::Failed;
		return;
	}

	const FCameraRigJoint& PivotJoint(*FoundItem);
	AimTwoBonesCameraRig(Params, CameraRigInfo, PivotJoint.Transform, IterationInfo);
}

void FCameraIKAim::RunRootCameraNode(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo)
{
	// Reset the scratch result the same way the camera system does it at the beginning of each frame.
	ScratchResult.Reset();

	// Make sure the camera rig will get its private variables, such as rig interface parameter overrides.
	ScratchResult.VariableTable.OverrideAll(CameraRigInfo.LastResult->VariableTable, true);

	// Save the initial state of the camera rig.
	{
		EvaluatorSnapshot.Reset();

		FCameraNodeEvaluatorSerializeParams SerializeParams;
		FMemoryWriter Writer(EvaluatorSnapshot);
		CameraSystemHierarchy.CallSerialize(SerializeParams, Writer);
	}

	// Run the system.
	{
		FSingleCameraRigEvaluationParams SingleParams;
		SingleParams.EvaluationParams.DeltaTime = Params.DeltaTime;
		SingleParams.EvaluationParams.bIsFirstFrame = Params.bIsFirstFrame;
		SingleParams.EvaluationParams.bIsActiveCameraRig = Params.bIsActiveCameraRig;
		SingleParams.EvaluationParams.EvaluationType = ECameraNodeEvaluationType::IK;
		SingleParams.EvaluationParams.EvaluationContext = CameraRigInfo.EvaluationContext;
		SingleParams.EvaluationParams.Evaluator = Params.Evaluator;
		SingleParams.CameraRigInfo = CameraRigInfo;

		FRootCameraNodeEvaluator* RootEvaluator = Params.Evaluator->GetRootNodeEvaluator();
		RootEvaluator->RunSingleCameraRig(SingleParams, ScratchResult);
	}

	// Restore the state of the camera rig.
	{
		FCameraNodeEvaluatorSerializeParams SerializeParams;
		FMemoryReader Reader(EvaluatorSnapshot);
		CameraSystemHierarchy.CallSerialize(SerializeParams, Reader);
	}
}

bool FCameraIKAim::CheckTolerance(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, FAimIterationInfo& IterationInfo)
{
	const FCameraPose& ResultPose = ScratchResult.CameraPose;

	// Figure out whether the distance to the current and/or desired target is too short.
	// This is sometimes a good indication that the player is up against an obstacle and it's
	// undesirable to turn their camera.
	const double TargetDistance = ResultPose.GetTargetDistance();
	const FVector3d CurrentAim(ResultPose.GetAimDir() * TargetDistance);
	const FVector3d CurrentLocationToDesiredTarget(Params.TargetLocation - ResultPose.GetLocation());
	const double DistanceToDesiredTarget(CurrentLocationToDesiredTarget.Length());

	if (TargetDistance < Params.MinDistance || DistanceToDesiredTarget < Params.MinDistance)
	{
		UE_LOGF(LogCameraSystem, Warning, "Aborting aiming of camera rig '%ls': current target is %f away, minimium distance is %f",
				*CameraRigInfo.CameraRig->GetPathName(), TargetDistance, Params.MinDistance);
		IterationInfo.Result = EAimResult::Aborted;
		return false;
	}

	// See if we are within the angle or distance tolerance.
	//
	//    ||R ^ D|| = ||R||*||D||*sin(A)
	//
	// ..with R being the line of sight to the current target, D being the vector pointing to the 
	// desired target, and A being the angle between the two. So:
	//
	//    A = asin(||R ^ D|| / (||R||*||D||)) 
	//
	// We also want to find H, the distance between the current line of sight and the desired target:
	//
	//    sin(A) = H / D
	//    H = sin(A) * D
	//    H = ||R ^ D|| / ||R||
	//
	// In the following code:
	//   OrthLength = ||R ^ D||
	//   SinAngle = sin(A)
	//   ErrorAngle = A
	//   ErrorDistance = H
	//
	const double OrthLength = FVector3d::CrossProduct(CurrentAim, CurrentLocationToDesiredTarget).Length();
	const double SinAngle = OrthLength / (TargetDistance * DistanceToDesiredTarget);

	const double ErrorAngle = FMath::RadiansToDegrees(FMath::Asin(SinAngle));
	IterationInfo.ErrorAngle = ErrorAngle;

	const double ErrorDistance = OrthLength / TargetDistance;
	IterationInfo.ErrorDistance = ErrorDistance;

	if (ErrorAngle <= Params.AngleTolerance)
	{
		IterationInfo.Result = EAimResult::Completed;
		return false;
	}

	if (ErrorDistance <= Params.DistanceTolerance)
	{
		IterationInfo.Result = EAimResult::Completed;
		return false;
	}

	return true;
}

void FCameraIKAim::AimTwoBonesCameraRig(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, const FTransform3d& PivotTransform, FAimIterationInfo& IterationInfo)
{
	FRotator3d Correction;
	const bool bGotCorrection = FCameraAimingMath::ComputeTwoBonesCorrection(
			ScratchResult.CameraPose, PivotTransform.GetLocation(), Params.TargetLocation, Correction);
#if UE_GAMEPLAY_CAMERAS_DEBUG
	FCameraIKAimIterationDebugInfo& IterationDebugInfo = LastRunDebugInfo.Iterations.Last();
	IterationDebugInfo.PivotJointLocation = PivotTransform.GetLocation();
	IterationDebugInfo.bSolvingSuccess = bGotCorrection;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	if (!bGotCorrection)
	{
		IterationInfo.Result = EAimResult::Failed;
		return;
	}

	FCameraOperationParams OperationParams;
	OperationParams.Evaluator = Params.Evaluator;
	OperationParams.EvaluationContext = CameraRigInfo.EvaluationContext;

	FYawPitchCameraOperation Operation;
	Operation.Yaw = FConsumableDouble::Delta(Correction.Yaw);
	Operation.Pitch = FConsumableDouble::Delta(Correction.Pitch);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	IterationDebugInfo.YawPitchCorrection = FVector2d(Correction.Yaw, Correction.Pitch);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	CameraSystemHierarchy.ForEachEvaluator(TEXT("ActiveCameraRig"), ECameraNodeEvaluatorFlags::SupportsOperations,
			[&OperationParams, &Operation](FCameraNodeEvaluator* Evaluator)
			{
				Evaluator->ExecuteOperation(OperationParams, Operation);
			});

	if (Operation.Yaw.HasValue() || Operation.Pitch.HasValue())
	{
		UE_LOGF(LogCameraSystem, Warning, 
				"Aborting aiming of camera rig '%ls': not all corrections were consumed by the camera nodes.",
				*CameraRigInfo.CameraRig->GetPathName());
		IterationInfo.Result = EAimResult::Aborted;
	}
	else
	{
		IterationInfo.Result = EAimResult::Corrected;
	}
#if UE_GAMEPLAY_CAMERAS_DEBUG
	IterationDebugInfo.bSolvingSuccess = (IterationInfo.Result == EAimResult::Corrected);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCameraIKAim::GetLastRunDebugInfo(FCameraIKAimDebugInfo& OutDebugInfo) const
{
	OutDebugInfo = LastRunDebugInfo;
}

void FCameraIKAimDebugInfo::DebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) const
{
	Renderer.AddText(TEXT("IK aiming "));
	if (bSucceeded)
	{
		Renderer.AddText(TEXT("{cam_good}succeeded"));
	}
	else
	{
		Renderer.AddText(TEXT("{cam_error}failed"));
	}
	Renderer.AddText(TEXT("{cam_default} in %d iterations\n"), Iterations.Num());

	UFont* SmallFont = GEngine->GetSmallFont();
	Renderer.DrawPoint(DesiredTarget, 2.f, FLinearColor::Yellow, 2.f);

	Renderer.AddIndent();
	{
		int32 IterationIndex = 0;
		const FVector2d TextOffset(-20, -20);
		for (const FCameraIKAimIterationDebugInfo& IterationDebugInfo : Iterations)
		{
			const bool bDirectionIsNormalized = true;
			const FVector3d TargetDir = FVector3d::ForwardVector;
			FRay3d DirectionRay(
					IterationDebugInfo.CameraPoseLocation, 
					IterationDebugInfo.CameraPoseRotation.RotateVector(TargetDir), bDirectionIsNormalized);

			const FLinearColor IterationColor = LerpLinearColorUsingHSV(
					FLinearColor::Yellow, FLinearColor::Red, IterationIndex, Iterations.Num());

			Renderer.DrawLine(
					IterationDebugInfo.CameraPoseLocation,
					DirectionRay.PointAt(1000.0),
					IterationColor);
			Renderer.DrawTextView(
					IterationDebugInfo.CameraPoseLocation,
					TextOffset,
					FString::Format(TEXT("Iteration {0}"), { IterationIndex + 1 }),
					IterationColor,
					SmallFont);

			Renderer.AddText(
					TEXT("%d : error angle %.2fdeg, error distance %.1fcm, "),
					IterationIndex + 1,
					IterationDebugInfo.ErrorAngle, IterationDebugInfo.ErrorDistance);

			++IterationIndex;

			if (!IterationDebugInfo.bNeededSolver)
			{
				Renderer.AddText(TEXT(" {cam_good}reached tolerance{cam_default}\n"));
				continue;
			}
			if (!IterationDebugInfo.bFoundSolver)
			{
				Renderer.AddText(TEXT(" {cam_error}couldn't find solver{cam_default}\n"));
				continue;
			}
			
			Renderer.AddText(TEXT(" pivot %s"), *IterationDebugInfo.PivotJointLocation.ToString());
			Renderer.AddText(
					TEXT(" correction Yaw=%.1f Pitch=%.1f"), 
					IterationDebugInfo.YawPitchCorrection.X,
					IterationDebugInfo.YawPitchCorrection.Y);

			if (!IterationDebugInfo.bSolvingSuccess)
			{
				Renderer.AddText(TEXT(", {cam_error}couldn't compute correction{cam_default}\n"));
				continue;
			}

			Renderer.NewLine();
		}
	}
	Renderer.RemoveIndent();
}

FArchive& operator<< (FArchive& Ar, FCameraIKAimIterationDebugInfo& IterationDebugInfo)
{
	Ar << IterationDebugInfo.CameraPoseLocation;
	Ar << IterationDebugInfo.CameraPoseRotation;
	Ar << IterationDebugInfo.ErrorAngle;
	Ar << IterationDebugInfo.ErrorDistance;

	Ar << IterationDebugInfo.PivotJointLocation;
	Ar << IterationDebugInfo.YawPitchCorrection;

	Ar << IterationDebugInfo.bNeededSolver;
	Ar << IterationDebugInfo.bFoundSolver;
	Ar << IterationDebugInfo.bSolvingSuccess;

	return Ar;
}

FArchive& operator<< (FArchive& Ar, FCameraIKAimDebugInfo& DebugInfo)
{
	Ar << DebugInfo.Iterations;
	Ar << DebugInfo.DesiredTarget;
	Ar << DebugInfo.bSucceeded;

	return Ar;
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

