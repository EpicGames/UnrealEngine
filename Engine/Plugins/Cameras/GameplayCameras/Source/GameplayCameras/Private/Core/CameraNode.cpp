// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraNode.h"

#include "Core/CameraNodeEvaluator.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraNode)

void UCameraNode::PostLoad()
{
#if WITH_EDITORONLY_DATA

	if (GraphNodePosX_DEPRECATED != 0 || GraphNodePosY_DEPRECATED != 0)
	{
		GraphNodePos = FIntVector2(GraphNodePosX_DEPRECATED, GraphNodePosY_DEPRECATED);

		GraphNodePosX_DEPRECATED = 0;
		GraphNodePosY_DEPRECATED = 0;
	}

#endif

	Super::PostLoad();
}

FCameraNodeChildrenView UCameraNode::GetChildren()
{
	if (EnumHasAnyFlags(PrivateFlags, ECameraNodeFlags::CustomGetChildren))
	{
		return OnGetChildren();
	}

	const UClass* ThisClass = GetClass();
	FCameraNodeChildrenView ChildrenView;
	for (TFieldIterator<FProperty> PropertyIt(ThisClass); PropertyIt; ++PropertyIt)
	{
		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(*PropertyIt))
		{
			if (ObjectProperty->PropertyClass->IsChildOf<UCameraNode>())
			{
				UObject* Child = ObjectProperty->GetObjectPropertyValue_InContainer(this);
				ChildrenView.Add(CastChecked<UCameraNode>(Child, ECastCheckedType::NullAllowed));
			}
		}
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*PropertyIt))
		{
			FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);
			if (InnerObjectProperty && InnerObjectProperty->PropertyClass->IsChildOf<UCameraNode>())
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(this));
				for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
				{
					UObject* Child = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
					ChildrenView.Add(CastChecked<UCameraNode>(Child, ECastCheckedType::NullAllowed));
				}
			}
		}
	}
	return ChildrenView;
}

void UCameraNode::PreBuild(FCameraBuildContext& BuildContext)
{
	OnPreBuild(BuildContext);
}

void UCameraNode::Build(FCameraObjectBuildContext& BuildContext)
{
	OnBuild(BuildContext);
}

FCameraNodeEvaluatorPtr UCameraNode::BuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	FCameraNodeEvaluator* NewEvaluator = OnBuildEvaluator(Builder);
	NewEvaluator->SetPrivateCameraNode(this);
	return NewEvaluator;
}

#if WITH_EDITOR

void UCameraNode::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UCameraNode::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

const FString& UCameraNode::GetGraphNodeCommentText(FName InGraphName) const
{
	return GraphNodeComment;
}

void UCameraNode::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	GraphNodeComment = NewComment;
}

#endif  // WITH_EDITOR

