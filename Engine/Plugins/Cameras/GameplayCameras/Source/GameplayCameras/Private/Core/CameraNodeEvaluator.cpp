// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraNodeEvaluator.h"

#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraNodeEvaluatorDebugBlock.h"
#include "UObject/UObjectGlobals.h"

namespace UE::Cameras
{

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraNodeEvaluator)

FCameraNodeEvaluatorInitializeParams::FCameraNodeEvaluatorInitializeParams(FCameraNodeEvaluatorHierarchy* InHierarchy)
	: Hierarchy(InHierarchy)
{
}

void FCameraNodeEvaluationResult::Reset()
{
	CameraPose.Reset();
	CameraRigJoints.Reset();
	PostProcessSettings.Reset();

	bIsCameraCut = false;
	bIsValid = false;

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
	CameraPoseLocationTrail.Reset();
#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
}

void FCameraNodeEvaluationResult::ResetFrameFlags()
{
	CameraPose.ClearAllChangedFlags();
	VariableTable.ClearAllWrittenThisFrameFlags();
	ContextDataTable.ClearAllWrittenThisFrameFlags();
}

void FCameraNodeEvaluationResult::OverrideAll(const FCameraNodeEvaluationResult& OtherResult, bool bIncludePrivateValues)
{
	CameraPose.OverrideAll(OtherResult.CameraPose);
	VariableTable.OverrideAll(OtherResult.VariableTable, bIncludePrivateValues);
	ContextDataTable.OverrideAll(OtherResult.ContextDataTable);
	CameraRigJoints.OverrideAll(OtherResult.CameraRigJoints);
	PostProcessSettings.OverrideAll(OtherResult.PostProcessSettings);
	bIsCameraCut = OtherResult.bIsCameraCut;
	bIsValid = OtherResult.bIsValid;
}

void FCameraNodeEvaluationResult::LerpAll(const FCameraNodeEvaluationResult& ToResult, float BlendFactor, bool bIncludePrivateValues)
{
	// Blend all properties.
	CameraPose.LerpAll(ToResult.CameraPose, BlendFactor);
	VariableTable.LerpAll(ToResult.VariableTable, BlendFactor, bIncludePrivateValues);

	// Merge/blend the joints.
	CameraRigJoints.LerpAll(ToResult.CameraRigJoints, BlendFactor);

	// Merge/blend the post-process settings.
	PostProcessSettings.LerpAll(ToResult.PostProcessSettings, BlendFactor);

	// If we have even a fraction of a camera cut, we need to make the
	// whole result into a camera cut.
	if (BlendFactor > 0.f && ToResult.bIsCameraCut)
	{
		bIsCameraCut = true;
	}

	// The blended result is valid if both input results are valid.
	bIsValid = (bIsValid && ToResult.bIsValid);
}

void FCameraNodeEvaluationResult::GetViewInfo(FMinimalViewInfo& OutViewInfo) const
{
	// Get the view info from the camera pose.
	CameraPose.GetViewInfo(OutViewInfo);

	OutViewInfo.PostProcessSettings = PostProcessSettings.Get();
	OutViewInfo.PostProcessBlendWeight = 1.f;
	// Create the physical camera settings if needed. Don't overwrite settings that were set by hand.
	CameraPose.ApplyPhysicalCameraSettings(OutViewInfo.PostProcessSettings, false);
}

void FCameraNodeEvaluationResult::Serialize(FArchive& Ar)
{
	CameraPose.SerializeWithFlags(Ar);
	VariableTable.Serialize(Ar);
	CameraRigJoints.Serialize(Ar);
	PostProcessSettings.Serialize(Ar);
	Ar << bIsCameraCut;
	Ar << bIsValid;
}

void FCameraNodeEvaluationResult::AddReferencedObjects(FReferenceCollector& Collector)
{
	ContextDataTable.AddReferencedObjects(Collector);
}

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

void FCameraNodeEvaluationResult::AddCameraPoseTrailPointIfNeeded()
{
	AddCameraPoseTrailPointIfNeeded(CameraPose.GetLocation());
}

void FCameraNodeEvaluationResult::AddCameraPoseTrailPointIfNeeded(const FVector3d& Point)
{
	if (CameraPoseLocationTrail.IsEmpty() || !CameraPoseLocationTrail.Last().Equals(Point))
	{
		CameraPoseLocationTrail.Add(Point);
	}
}

void FCameraNodeEvaluationResult::AppendCameraPoseLocationTrail(const FCameraNodeEvaluationResult& InResult)
{
	ensure(this != &InResult);
	CameraPoseLocationTrail.Append(InResult.CameraPoseLocationTrail);
}

TConstArrayView<FVector3d> FCameraNodeEvaluationResult::GetCameraPoseLocationTrail() const
{
	return CameraPoseLocationTrail;
}

#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

FCameraNodeEvaluator* FCameraNodeEvaluatorBuildParams::BuildEvaluator(const UCameraNode* InNode) const
{
	if (InNode)
	{
		FCameraNodeEvaluator* NewEvaluator = InNode->BuildEvaluator(Builder);
		NewEvaluator->Build(*this);
		return NewEvaluator;
	}
	return nullptr;
}

void FCameraNodeEvaluator::SetPrivateCameraNode(TObjectPtr<const UCameraNode> InCameraNode)
{
	PrivateCameraNode = InCameraNode;
}

void FCameraNodeEvaluator::AddNodeEvaluatorFlags(ECameraNodeEvaluatorFlags InFlags)
{
	PrivateFlags |= InFlags;
}

void FCameraNodeEvaluator::SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags InFlags)
{
	PrivateFlags = InFlags;
}

void FCameraNodeEvaluator::Build(const FCameraNodeEvaluatorBuildParams& Params)
{
	OnBuild(Params);
}

void FCameraNodeEvaluator::Initialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (Params.Hierarchy)
	{
		Params.Hierarchy->AddEvaluator(this);
	}

	OnInitialize(Params, OutResult);

	for (FCameraNodeEvaluator* Child : GetChildren())
	{
		if (Child)
		{
			Child->Initialize(Params, OutResult);
		}
	}
}

void FCameraNodeEvaluator::Teardown(const FCameraNodeEvaluatorTeardownParams& Params)
{
	OnTeardown(Params);

	for (FCameraNodeEvaluator* Child : GetChildren())
	{
		if (Child)
		{
			Child->Teardown(Params);
		}
	}
}

void FCameraNodeEvaluator::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PrivateCameraNode)
	{
		Collector.AddReferencedObject(PrivateCameraNode);
	}

	OnAddReferencedObjects(Collector);

	for (FCameraNodeEvaluator* Child : GetChildren())
	{
		if (Child)
		{
			Child->AddReferencedObjects(Collector);
		}
	}
}

FCameraNodeEvaluatorChildrenView FCameraNodeEvaluator::GetChildren()
{
	return OnGetChildren();
}

void FCameraNodeEvaluator::UpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	if (!PrivateCameraNode || PrivateCameraNode->bIsEnabled)
	{
		OnUpdateParameters(Params, OutResult);
	}
}

void FCameraNodeEvaluator::Run(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (!PrivateCameraNode || PrivateCameraNode->bIsEnabled)
	{
		OnRun(Params, OutResult);

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
		if (bAutoCameraPoseMovementTrail)
		{
			OutResult.AddCameraPoseTrailPointIfNeeded();
		}
#endif  // WITH_EDITOR
	}
}

void FCameraNodeEvaluator::ExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	if (!PrivateCameraNode || PrivateCameraNode->bIsEnabled)
	{
		OnExecuteOperation(Params, Operation);
	}
}

void FCameraNodeEvaluator::Serialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	OnSerialize(Params, Ar);
}

#if WITH_EDITOR

void FCameraNodeEvaluator::DrawEditorPreview(const FCameraEditorPreviewDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	OnDrawEditorPreview(Params, Renderer);

	FCameraNodeEvaluatorChildrenView ChildrenView(GetChildren());
	for (FCameraNodeEvaluator* Child : ChildrenView)
	{
		if (Child)
		{
			Child->DrawEditorPreview(Params, Renderer);
		}
	}
}

#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCameraNodeEvaluator::BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	// Let's start by adding the default debug block for a node evaluator.
	Builder.StartChildDebugBlock<FCameraNodeEvaluatorDebugBlock>(PrivateCameraNode);
	{
		// Then let the node evaluator attach or add other custom debug blocks.
		const int32 PreviousLevel = Builder.GetHierarchyLevel();
		OnBuildDebugBlocks(Params, Builder);
		if (!ensureMsgf(
				PreviousLevel == Builder.GetHierarchyLevel(), 
				TEXT("Node evaluator added new children debug blocks but forgot to end them!")))
		{
			const int32 LevelsToEnd = Builder.GetHierarchyLevel() - PreviousLevel;
			for (int32 Index = 0; Index < LevelsToEnd; ++Index)
			{
				Builder.EndChildDebugBlock();
			}
		}

		// Build debug blocks for children node evaluators.
		ECameraDebugBlockBuildVisitFlags VisitFlags = Builder.GetVisitFlags();
		Builder.ResetVisitFlags();
		if (!EnumHasAnyFlags(VisitFlags, ECameraDebugBlockBuildVisitFlags::SkipChildren))
		{
			FCameraNodeEvaluatorChildrenView ChildrenView(GetChildren());
			for (FCameraNodeEvaluator* Child : ChildrenView)
			{
				if (Child)
				{
					Child->BuildDebugBlocks(Params, Builder);
				}
			}
		}
	}
	Builder.EndChildDebugBlock();
}

void FCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

