// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraNodeEvaluatorHierarchy.h"

namespace UE::Cameras
{

FCameraNodeEvaluatorHierarchy::FCameraNodeEvaluatorHierarchy()
{
}

FCameraNodeEvaluatorHierarchy::FCameraNodeEvaluatorHierarchy(FCameraNodeEvaluator* InRootEvaluator)
{
	Build(InRootEvaluator);
}

TArrayView<FCameraNodeEvaluator* const> FCameraNodeEvaluatorHierarchy::GetFlattenedHierarchy() const
{
	return FlattenedHierarchy;
}

void FCameraNodeEvaluatorHierarchy::GetFlattenedHierarchy(ECameraNodeEvaluatorFlags FilterFlags, TArray<FCameraNodeEvaluator*> OutEvaluators) const
{
	for (FCameraNodeEvaluator* Evaluator : FlattenedHierarchy)
	{
		if (EnumHasAllFlags(Evaluator->GetNodeEvaluatorFlags(), FilterFlags))
		{
			OutEvaluators.Add(Evaluator);
		}
	}
}

void FCameraNodeEvaluatorHierarchy::Build(FCameraNodeEvaluator* InRootEvaluator)
{
	Reset();

	Append(InRootEvaluator);
}

void FCameraNodeEvaluatorHierarchy::Append(FCameraNodeEvaluator* InRootEvaluator)
{
	if (!InRootEvaluator)
	{
		return;
	}

	TArray<FCameraNodeEvaluator*> EvaluatorStack;
	EvaluatorStack.Add(InRootEvaluator);
	while (!EvaluatorStack.IsEmpty())
	{
		FCameraNodeEvaluator* TopEvaluator = EvaluatorStack.Pop();
		FlattenedHierarchy.Add(TopEvaluator);

		FCameraNodeEvaluatorChildrenView TopEvaluatorChildren(TopEvaluator->GetChildren());
		for (FCameraNodeEvaluator* ChildEvaluator : ReverseIterate(TopEvaluatorChildren))
		{
			if (ChildEvaluator)
			{
				EvaluatorStack.Add(ChildEvaluator);
			}
		}
	}
}

void FCameraNodeEvaluatorHierarchy::AppendTagged(const FName TaggedRangeName, FCameraNodeEvaluator* InRootEvaluator)
{
	FTaggedRange& NewRange = TaggedRanges.Add(TaggedRangeName);
	NewRange.StartIndex = FlattenedHierarchy.Num();
	Append(InRootEvaluator);
	NewRange.EndIndex = FlattenedHierarchy.Num();
}

void FCameraNodeEvaluatorHierarchy::AddEvaluator(FCameraNodeEvaluator* Evaluator)
{
	if (Evaluator)
	{
		FlattenedHierarchy.Add(Evaluator);
	}
}

void FCameraNodeEvaluatorHierarchy::Reset()
{
	FlattenedHierarchy.Reset();
}

void FCameraNodeEvaluatorHierarchy::CallUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	ForEachEvaluator(ECameraNodeEvaluatorFlags::NeedsParameterUpdate, 
			[&Params, &OutResult](FCameraNodeEvaluator* Evaluator)
			{
				Evaluator->UpdateParameters(Params, OutResult);
			});
}

void FCameraNodeEvaluatorHierarchy::CallExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	ForEachEvaluator(ECameraNodeEvaluatorFlags::SupportsOperations, 
			[&Params, &Operation](FCameraNodeEvaluator* Evaluator)
			{
				Evaluator->ExecuteOperation(Params, Operation);
			});
}

void FCameraNodeEvaluatorHierarchy::CallSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	ForEachEvaluator(ECameraNodeEvaluatorFlags::NeedsSerialize,
			[&Params, &Ar](FCameraNodeEvaluator* Evaluator)
			{
				Evaluator->Serialize(Params, Ar);
			});
}

}  // namespace UE::Cameras

