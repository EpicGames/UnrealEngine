// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraNodeEvaluatorStorage.h"

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorBuilder.h"

namespace UE::Cameras
{

FCameraNodeEvaluatorPtr FCameraNodeEvaluatorStorage::BuildEvaluatorTree(const FCameraNodeEvaluatorTreeBuildParams& Params)
{
	if (Params.AllocationInfo)
	{
		const uint16 NewCapacity = Params.AllocationInfo->TotalSizeof;
		const uint16 NewAlignment = Params.AllocationInfo->MaxAlignof;
		if (NewCapacity > 0 && NewAlignment > 0)
		{
			Super::AllocatePage(NewCapacity, NewAlignment);
		}
	}

	FCameraNodeEvaluatorBuilder Builder(*this);
	FCameraNodeEvaluatorBuildParams BuildParams(Builder);

	FCameraNodeEvaluatorPtr RootEvaluator = BuildParams.BuildEvaluator(Params.RootCameraNode);

	return RootEvaluator;
}

void FCameraNodeEvaluatorStorage::DestroyEvaluatorTree(bool bFreeAllocations)
{
	Super::DestroyObjects(bFreeAllocations);
}

void FCameraNodeEvaluatorStorage::DestroyEvaluatorTree(FCameraNodeEvaluator* InRootEvaluator, bool bResetMemory)
{
	if (!InRootEvaluator)
	{
		return;
	}

	const FCameraObjectTypeRegistry& TypeRegistry = FCameraObjectTypeRegistry::Get();

	TArray<FCameraNodeEvaluator*> Stack;
	Stack.Add(InRootEvaluator);
	while (Stack.Num() > 0)
	{
		FCameraNodeEvaluator* Evaluator = Stack.Pop(EAllowShrinking::No);
		FCameraNodeEvaluatorChildrenView Children(Evaluator->GetChildren());
		for (FCameraNodeEvaluator* Child : ReverseIterate(Children))
		{
			if (Child)
			{
				Stack.Add(Child);
			}
		}

		const FCameraObjectTypeID TypeID = Evaluator->GetTypeID();
		const FCameraObjectTypeInfo* TypeInfo = TypeRegistry.GetTypeInfo(TypeID);

		if (ensure(TypeInfo))
		{
			Super::DestroyObject(Evaluator, bResetMemory ? TypeInfo->Sizeof : 0);
		}
	}
}

void FCameraNodeEvaluatorStorage::GetAllocationInfo(FCameraNodeEvaluatorAllocationInfo& OutAllocationInfo)
{
	uint32 TotalUsed;
	uint32 FirstAlignment;
	Super::GetAllocationInfo(TotalUsed, FirstAlignment);

	OutAllocationInfo.TotalSizeof = TotalUsed;
	OutAllocationInfo.MaxAlignof = FirstAlignment;
}

}  // namespace UE::Cameras

