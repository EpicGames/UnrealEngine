// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraNodeHierarchy.h"

#include "Core/BaseCameraObject.h"

namespace UE::Cameras
{

FCameraNodeHierarchy::FCameraNodeHierarchy()
{
}

FCameraNodeHierarchy::FCameraNodeHierarchy(UBaseCameraObject* InCameraObject)
{
	Build(InCameraObject);
}

TArrayView<UCameraNode* const> FCameraNodeHierarchy::GetFlattenedHierarchy() const
{
	return FlattenedHierarchy;
}

int32 FCameraNodeHierarchy::Num() const
{
	return FlattenedHierarchy.Num();
}

void FCameraNodeHierarchy::Build(UBaseCameraObject* InCameraObject)
{
	Build(InCameraObject ? InCameraObject->GetRootNode() : nullptr);
}

void FCameraNodeHierarchy::Build(UCameraNode* InRootCameraNode)
{
	Reset();

	if (!InRootCameraNode)
	{
		return;
	}

	TArray<UCameraNode*> NodeStack;
	NodeStack.Add(InRootCameraNode);
	while (!NodeStack.IsEmpty())
	{
		UCameraNode* CurrentNode = NodeStack.Pop();
		FlattenedHierarchy.Add(CurrentNode);

		FCameraNodeChildrenView CurrentChildren = CurrentNode->GetChildren();
		for (UCameraNode* ChildNode : ReverseIterate(CurrentChildren))
		{
			if (ChildNode)
			{
				NodeStack.Add(ChildNode);
			}
		}
	}
}

void FCameraNodeHierarchy::Reset()
{
	FlattenedHierarchy.Reset();
}

#if WITH_EDITORONLY_DATA

bool FCameraNodeHierarchy::FindMissingConnectableObjects(TArrayView<UObject* const> ConnectableObjects, TSet<UObject*>& OutMissingObjects)
{
	TSet<UObject*> ConnectableObjectsSet(ConnectableObjects);
	return FindMissingConnectableObjects(ConnectableObjectsSet, OutMissingObjects);
}

bool FCameraNodeHierarchy::FindMissingConnectableObjects(const TSet<UObject*> ConnectableObjectsSet, TSet<UObject*>& OutMissingObjects)
{
	TSet<UObject*> FlattenedHierarchySet(MakeArrayView((UObject**)FlattenedHierarchy.GetData(), FlattenedHierarchy.Num()));
	OutMissingObjects = FlattenedHierarchySet.Difference(ConnectableObjectsSet);
	return !OutMissingObjects.IsEmpty();
}

#endif  // WITH_EDITORONLY_DATA

}  // namespace UE::Cameras

