// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraObjectInterface.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraContextDataTableAllocationInfo.h"
#include "Core/CameraVariableTableAllocationInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraObjectInterface)

void UCameraObjectInterfaceParameterBase::PostLoad()
{
	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}

	Super::PostLoad();
}

void UCameraObjectInterfaceParameterBase::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && 
			!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UCameraObjectInterfaceParameterBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		Guid = FGuid::NewGuid();
	}
}

FCameraVariableDefinition UCameraObjectInterfaceBlendableParameter::GetVariableDefinition() const
{
	FCameraVariableDefinition Definition;
	Definition.VariableID = PrivateVariableID;
	Definition.VariableType = VariableType;
	Definition.BlendableStructType = BlendableStructType;
	Definition.bIsPrivate = true;
	Definition.bIsInput = bIsPreBlended;
#if WITH_EDITORONLY_DATA
	Definition.VariableName = GetVariableName();
#endif
	return Definition;
}

void UCameraObjectInterfaceBlendableParameter::GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const
{
	OutParameterDefinition.ParameterName = FName(InterfaceParameterName);
	OutParameterDefinition.ParameterGuid = Guid;
	OutParameterDefinition.ParameterType = ECameraObjectInterfaceParameterType::Blendable;
	OutParameterDefinition.bIsVisible = bIsVisible;
	OutParameterDefinition.VariableID = PrivateVariableID;
	OutParameterDefinition.VariableType = VariableType;
	OutParameterDefinition.BlendableStructType = BlendableStructType;
}

FCameraContextDataDefinition UCameraObjectInterfaceDataParameter::GetDataDefinition() const
{
	FCameraContextDataDefinition Definition;
	Definition.DataID = PrivateDataID;
	Definition.DataType = DataType;
	Definition.DataContainerType = DataContainerType;
	Definition.DataTypeObject = DataTypeObject;
#if WITH_EDITORONLY_DATA
	Definition.DataName = GetDataName();
#endif
	return Definition;
}

void UCameraObjectInterfaceDataParameter::GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const
{
	OutParameterDefinition.ParameterName = FName(InterfaceParameterName);
	OutParameterDefinition.ParameterGuid = Guid;
	OutParameterDefinition.ParameterType = ECameraObjectInterfaceParameterType::Data;
	OutParameterDefinition.bIsVisible = bIsVisible;
	OutParameterDefinition.DataID = PrivateDataID;
	OutParameterDefinition.DataType = DataType;
	OutParameterDefinition.DataContainerType = DataContainerType;
	OutParameterDefinition.DataTypeObject = DataTypeObject;
}

#if WITH_EDITORONLY_DATA

FString UCameraObjectInterfaceBlendableParameter::GetVariableName() const
{
	const UObject* Owner = GetOuter();
	return FString::Printf(TEXT("Override_%s_%s"), *GetNameSafe(Owner), *InterfaceParameterName);
}

FString UCameraObjectInterfaceDataParameter::GetDataName() const
{
	const UObject* Owner = GetOuter();
	return FString::Printf(TEXT("Override_%s_%s"), *GetNameSafe(Owner), *InterfaceParameterName);
}

#endif  // WITH_EDITORONLY_DATA

UCameraObjectInterfaceBlendableParameter* FCameraObjectInterface::FindBlendableParameterByName(const FString& ParameterName) const
{
	const TObjectPtr<UCameraObjectInterfaceBlendableParameter>* FoundItem = BlendableParameters.FindByPredicate(
			[&ParameterName](UCameraObjectInterfaceBlendableParameter* Item)
			{
				return Item->InterfaceParameterName == ParameterName;
			});
	return FoundItem ? *FoundItem : nullptr;
}

UCameraObjectInterfaceDataParameter* FCameraObjectInterface::FindDataParameterByName(const FString& ParameterName) const
{
	const TObjectPtr<UCameraObjectInterfaceDataParameter>* FoundItem = DataParameters.FindByPredicate(
			[&ParameterName](UCameraObjectInterfaceDataParameter* Item)
			{
				return Item->InterfaceParameterName == ParameterName;
			});
	return FoundItem ? *FoundItem : nullptr;
}

UCameraObjectInterfaceBlendableParameter* FCameraObjectInterface::FindBlendableParameterByGuid(const FGuid& ParameterGuid) const
{
	const TObjectPtr<UCameraObjectInterfaceBlendableParameter>* FoundItem = BlendableParameters.FindByPredicate(
			[&ParameterGuid](UCameraObjectInterfaceBlendableParameter* Item)
			{
				return Item->GetGuid() == ParameterGuid;
			});
	return FoundItem ? *FoundItem : nullptr;
}

UCameraObjectInterfaceDataParameter* FCameraObjectInterface::FindDataParameterByGuid(const FGuid& ParameterGuid) const
{
	const TObjectPtr<UCameraObjectInterfaceDataParameter>* FoundItem = DataParameters.FindByPredicate(
			[&ParameterGuid](UCameraObjectInterfaceDataParameter* Item)
			{
				return Item->GetGuid() == ParameterGuid;
			});
	return FoundItem ? *FoundItem : nullptr;
}

bool FCameraObjectInterface::HasBlendableParameter(const FString& ParameterName) const
{
	return FindBlendableParameterByName(ParameterName) != nullptr;
}

UCameraObjectInterfaceParameterBase* UCameraObjectInterfaceParameterGetter::GetInterfaceParameter() const
{
	if (ParameterGuid.IsValid())
	{
		if (UBaseCameraObject* CameraObject = GetTypedOuter<UBaseCameraObject>())
		{
			if (UCameraObjectInterfaceBlendableParameter* BlendableParameter = CameraObject->Interface.FindBlendableParameterByGuid(ParameterGuid))
			{
				return BlendableParameter;
			}
			if (UCameraObjectInterfaceDataParameter* DataParameter = CameraObject->Interface.FindDataParameterByGuid(ParameterGuid))
			{
				return DataParameter;
			}
		}
	}
	return nullptr;
}

FString UCameraObjectInterfaceParameterGetter::GetInterfaceParameterName() const
{
	if (UCameraObjectInterfaceParameterBase* Parameter = GetInterfaceParameter())
	{
		return Parameter->InterfaceParameterName;
	}
	return FString();
}

#if WITH_EDITOR

void UCameraObjectInterfaceParameterGetter::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UCameraObjectInterfaceParameterGetter::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

#endif

