// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraObjectRtti.h"

namespace UE::Cameras
{

uint32 FCameraObjectTypeID::RegisterNewID()
{
	static uint32 ID = 0;
	return ID++;
}

FCameraObjectTypeRegistry& FCameraObjectTypeRegistry::Get()
{
	static FCameraObjectTypeRegistry StaticRegistry;
	return StaticRegistry;
}

void FCameraObjectTypeRegistry::RegisterType(FCameraObjectTypeID TypeID, FCameraObjectTypeInfo&& TypeInfo)
{
	ensureMsgf(
			!TypeIDsByName.Contains(TypeInfo.TypeName), 
			TEXT("Type '%s' has already been registered!"), *TypeInfo.TypeName.ToString());
	TypeIDsByName.Add(TypeInfo.TypeName, TypeID.GetTypeID());
	TypeInfos.Insert(TypeID.GetTypeID(), MoveTemp(TypeInfo));
}

FCameraObjectTypeID FCameraObjectTypeRegistry::FindTypeByName(const FName& TypeName) const
{
	if (const uint32* RegisteredTypeID = TypeIDsByName.Find(TypeName))
	{
		return FCameraObjectTypeID{ *RegisteredTypeID };
	}
	return FCameraObjectTypeID::Invalid();
}

const FCameraObjectTypeInfo* FCameraObjectTypeRegistry::GetTypeInfo(FCameraObjectTypeID TypeID) const
{
	if (ensureMsgf(
				TypeID.IsValid() && TypeInfos.IsValidIndex(TypeID.GetTypeID()),
				TEXT("Given type ID is not valid, or not registered.")))
	{
		const FCameraObjectTypeInfo& TypeInfo = TypeInfos[TypeID.GetTypeID()];
		return &TypeInfo;
	}
	return nullptr;
}

FName FCameraObjectTypeRegistry::GetTypeNameSafe(FCameraObjectTypeID TypeID) const
{
	if (TypeID.IsValid())
	{
		if (const FCameraObjectTypeInfo* TypeInfo = GetTypeInfo(TypeID))
		{
			return TypeInfo->TypeName;
		}
	}

	return NAME_None;
}

void FCameraObjectTypeRegistry::ConstructObject(FCameraObjectTypeID TypeID, void* Ptr)
{
	if (ensureMsgf(TypeInfos.IsValidIndex(TypeID.GetTypeID()), TEXT("Invalid camera object type ID!")))
	{
		const FCameraObjectTypeInfo& TypeInfo = TypeInfos[TypeID.GetTypeID()];
		TypeInfo.Constructor(Ptr);
	}
}

}  // namespace UE::Cameras

