// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraOperation.h"

namespace UE::Cameras
{

FCameraOperationTypeID FCameraOperationTypeID::RegisterNew()
{
	static uint32 NextID = (uint32)EBuiltInCameraOperationTypes::MAX + 1;
	return FCameraOperationTypeID(NextID++);
}

}  // namespace UE::Cameras

