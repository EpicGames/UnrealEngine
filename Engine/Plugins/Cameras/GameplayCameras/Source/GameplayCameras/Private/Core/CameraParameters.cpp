// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraParameters.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraVariableTable.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraParameters)

bool FBooleanCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_BoolProperty)
	{
		Value = (Tag.BoolVal != 0);
		return true;
	}

	return false;
}

bool FInteger32CameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_IntProperty || Tag.Type == NAME_Int32Property)
	{
		Slot << Value;
		return true;
	}

	return false;
}

bool FFloatCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_FloatProperty)
	{
		Slot << Value;
		return true;
	}

	return false;
}


bool FDoubleCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_DoubleProperty))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector2fCameraParameter::FVector2fCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector2fCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector2f) || Tag.GetType().IsStruct(NAME_Vector2D))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector2dCameraParameter::FVector2dCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector2dCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector2d))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector3fCameraParameter::FVector3fCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector3fCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector3f))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector3dCameraParameter::FVector3dCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector3dCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector3d))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector4fCameraParameter::FVector4fCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector4fCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector4f))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FVector4dCameraParameter::FVector4dCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FVector4dCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector4d))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FRotator3fCameraParameter::FRotator3fCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FRotator3fCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Rotator3f))
	{
		Slot << Value;
		return true;
	}

	return false;
}

FRotator3dCameraParameter::FRotator3dCameraParameter()
	: Value(EForceInit::ForceInit)
{
}

bool FRotator3dCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Rotator3d))
	{
		Slot << Value;
		return true;
	}

	return false;
}

bool FTransform3fCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Transform3f))
	{
		Slot << Value;
		return true;
	}

	return false;
}

bool FTransform3dCameraParameter::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Transform3d))
	{
		Slot << Value;
		return true;
	}

	return false;
}

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
void F##ValueName##CameraParameter::PostSerialize(const FArchive& Ar)\
{\
	if (Ar.IsLoading())\
	{\
		if (Variable && Variable->GetOuter()->IsA<UCameraRigAsset>())\
		{\
			Variable = nullptr;\
		}\
	}\
}\
ValueType F##ValueName##CameraParameter::GetValue(const UE::Cameras::FCameraVariableTable& VariableTable) const\
{\
	if (!VariableID.IsValid())\
	{\
		return Value;\
	}\
	else\
	{\
		if (const ValueType* ActualValue = VariableTable.FindValue<ValueType>(VariableID))\
		{\
			return *ActualValue;\
		}\
		return Value;\
	}\
}
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

