// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraPose.h"

#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "GameplayCameras.h"
#include "HAL/IConsoleManager.h"
#include "Math/Ray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraPose)

float GGameplayCamerasDefaultMinFstop = 0.f;
static FAutoConsoleVariableRef CVarGameplayCamerasDefaultMinFstop(
	TEXT("GameplayCameras.DefaultMinFstop"),
	GGameplayCamerasDefaultMinFstop,
	TEXT("(Default: 0. Minimum camera lens aperture (f-stop) that defines the curvature of the diaphragm blades."));

const FCameraPoseFlags& FCameraPoseFlags::All()
{
	static FCameraPoseFlags Instance(true);
	return Instance;
}

FCameraPoseFlags::FCameraPoseFlags()
{
}

FCameraPoseFlags::FCameraPoseFlags(bool bInValue)
{
	SetAllFlags(bInValue);
}

FCameraPoseFlags& FCameraPoseFlags::SetAllFlags(bool bInValue)
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	PropName = bInValue;

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

	return *this;
}

FCameraPoseFlags& FCameraPoseFlags::ExclusiveCombine(const FCameraPoseFlags& OtherFlags)
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	if (OtherFlags.PropName)\
	{\
		ensureMsgf(!PropName, TEXT("Exclusive combination failed: " #PropName " set on both flags!"));\
		PropName = true;\
	}

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

	return *this;
}

FCameraPoseFlags& FCameraPoseFlags::AND(const FCameraPoseFlags& OtherFlags)
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	PropName = PropName && OtherFlags.PropName;

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

	return *this;
}

FCameraPoseFlags& FCameraPoseFlags::OR(const FCameraPoseFlags& OtherFlags)
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	PropName = PropName || OtherFlags.PropName;

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

	return *this;
}

FCameraPose::FCameraPose()
{
}

void FCameraPose::Reset()
{
	*this = FCameraPose();

	ClearAllChangedFlags();
}

void FCameraPose::SetAllChangedFlags()
{
	ChangedFlags.SetAllFlags(true);
}

void FCameraPose::ClearAllChangedFlags()
{
	ChangedFlags.SetAllFlags(false);
}

FTransform3d FCameraPose::GetTransform() const
{
	FTransform3d Transform;
	Transform.SetLocation(Location);
	Transform.SetRotation(Rotation.Quaternion());
	return Transform;
}

void FCameraPose::SetTransform(FTransform3d Transform, bool bForceSet)
{
	SetLocation(Transform.GetLocation(), bForceSet);
	SetRotation(Transform.GetRotation().Rotator(), bForceSet);
}

double FCameraPose::GetEffectiveFieldOfView(bool bIncludeOverscan) const
{
	return GetEffectiveFieldOfView(FocalLength, FieldOfView, SensorWidth, SensorHeight, SqueezeFactor, bIncludeOverscan ? Overscan : 1.0f);
}

double FCameraPose::GetEffectiveFieldOfView(float FocalLength, float FieldOfView, float SensorWidth, float SensorHeight, float SqueezeFactor, float Overscan)
{
	const bool bValidFocalLength = (FocalLength > 0.f);
	const bool bValidFieldOfView = (FieldOfView > 0.f);

#if !NO_LOGGING
	static bool GEmitZeroFocalLenthAndFieldOfViewWarning = true;
	static bool GEmitFocalLengthPrioritizationWarning = true;

	if (!bValidFocalLength && !bValidFieldOfView && GEmitZeroFocalLenthAndFieldOfViewWarning)
	{
		UE_LOGF(LogCameraSystem, Warning,
				"Both FocalLength and FieldOfView have a zero or negative value! Using default FocalLength.");
		GEmitZeroFocalLenthAndFieldOfViewWarning = false;
	}

	if (bValidFocalLength && bValidFieldOfView && GEmitFocalLengthPrioritizationWarning)
	{
		UE_LOGF(LogCameraSystem, Warning,
				"Both FocalLength and FieldOfView are specified on a camera pose! Using FocalLength first.");
		GEmitFocalLengthPrioritizationWarning = false;
	}
#endif  // NO_LOGGING	

	if (!bValidFocalLength && !bValidFieldOfView)
	{
		FocalLength = 35.f;
	}

	if (FocalLength > 0.f)
	{
		// Compute FOV with similar code to UCineCameraComponent...
		double CroppedSensorWidth = SensorWidth * SqueezeFactor;
		const double AspectRatio = GetSensorAspectRatio(SensorWidth, SensorHeight);
		if (AspectRatio > 0.0)
		{
			double DesqueezeAspectRatio = AspectRatio * SqueezeFactor;
			if (AspectRatio < DesqueezeAspectRatio)
			{
				CroppedSensorWidth *= AspectRatio / DesqueezeAspectRatio;
			}
		}

		const double EffectiveOverscan = (1.0 + Overscan);

		return FMath::RadiansToDegrees(2.0 * FMath::Atan(CroppedSensorWidth * EffectiveOverscan / (2.0 * FocalLength)));
	}
	else
	{
		// Let's use the FOV directly, like in the good old times.
		return FieldOfView;
	}
}

double FCameraPose::GetSensorAspectRatio() const
{
	return GetSensorAspectRatio(SensorWidth, SensorHeight);
}

double FCameraPose::GetSensorAspectRatio(float SensorWidth, float SensorHeight)
{
	return (SensorHeight > 0.f) ? (SensorWidth / SensorHeight) : 0.0;
}

void FCameraPose::GetDefaultSensorSize(float& OutSensorWidth, float& OutSensorHeight)
{
	OutSensorWidth = 24.89f;
	OutSensorHeight = 18.67f;
}

EAspectRatioAxisConstraint FCameraPose::GetEffectiveAspectRatioAxisConstraint() const
{
	// Use the same default as UCameraComponent.
	return GetEffectiveAspectRatioAxisConstraint(EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV);
}

EAspectRatioAxisConstraint FCameraPose::GetEffectiveAspectRatioAxisConstraint(EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint) const
{
	if (OverrideAspectRatioAxisConstraint)
	{
		return AspectRatioAxisConstraint;
	}
	return DefaultAspectRatioAxisConstraint;
}

double FCameraPose::GetHorizontalProjectionOffset() const
{
	// Compute projection offset with similar code to UCineCameraComponent...
	double CroppedSensorWidth = SensorWidth * SqueezeFactor;
	const double AspectRatio = GetSensorAspectRatio(SensorWidth, SensorHeight);
	if (AspectRatio > 0.0)
	{
		double DesqueezeAspectRatio = AspectRatio * SqueezeFactor;
		if (AspectRatio < DesqueezeAspectRatio)
		{
			CroppedSensorWidth *= AspectRatio / DesqueezeAspectRatio;
		}
	}

	const double EffectiveOverscan = (1.0f + Overscan);

	return 2.0 * SensorHorizontalOffset / (CroppedSensorWidth * EffectiveOverscan);
}

double FCameraPose::GetVerticalProjectionOffset() const
{
	// Compute projection offset with similar code to UCineCameraComponent...
	double CroppedSensorHeight = SensorHeight;
	const double AspectRatio = GetSensorAspectRatio(SensorWidth, SensorHeight);
	if (AspectRatio > 0.0)
	{
		double DesqueezeAspectRatio = AspectRatio * SqueezeFactor;
		if (DesqueezeAspectRatio < AspectRatio)
		{
			CroppedSensorHeight *= DesqueezeAspectRatio / AspectRatio;
		}
	}

	const double EffectiveOverscan = (1.0f + Overscan);

	return 2.0 * SensorVerticalOffset / (CroppedSensorHeight * EffectiveOverscan);
}

void FCameraPose::GetViewInfo(FMinimalViewInfo& OutViewInfo) const
{
	OutViewInfo.Location = Location;
	OutViewInfo.Rotation = Rotation;
	OutViewInfo.FOV = GetEffectiveFieldOfView();
	OutViewInfo.DesiredFOV = OutViewInfo.FOV;

	OutViewInfo.AspectRatio = GetSensorAspectRatio();
	OutViewInfo.bConstrainAspectRatio = ConstrainAspectRatio;
	OutViewInfo.AspectRatioAxisConstraint = OverrideAspectRatioAxisConstraint ? 
		TOptional<EAspectRatioAxisConstraint>(AspectRatioAxisConstraint.GetValue()) : 
		TOptional<EAspectRatioAxisConstraint>();

	OutViewInfo.ProjectionMode = ProjectionMode;
	if (ProjectionMode == ECameraProjectionMode::Orthographic)
	{
		OutViewInfo.OrthoWidth = OrthographicWidth;
	}

	OutViewInfo.PerspectiveNearClipPlane = NearClippingPlane > 0.0f ? NearClippingPlane : GNearClippingPlane;

	OutViewInfo.bUseFirstPersonParameters = EnableFirstPerson;
	if (EnableFirstPerson)
	{
		OutViewInfo.FirstPersonFOV = FirstPersonFieldOfView > 0 ? FirstPersonFieldOfView : OutViewInfo.FOV;
		OutViewInfo.FirstPersonScale = FirstPersonScale > 0 ? FirstPersonScale : 1.f;
	}
	else
	{
		OutViewInfo.FirstPersonFOV = OutViewInfo.FOV;
		OutViewInfo.FirstPersonScale = 1.f;
	}

	OutViewInfo.OffCenterProjectionOffset.X = GetHorizontalProjectionOffset();
	OutViewInfo.OffCenterProjectionOffset.Y = GetVerticalProjectionOffset();

	OutViewInfo.ApplyOverscan(GetOverscan());
}

bool FCameraPose::ApplyPhysicalCameraSettings(FPostProcessSettings& PostProcessSettings, bool bOverwriteSettings) const
{
	if (!EnablePhysicalCamera || PhysicalCameraBlendWeight <= 0.f)
	{
		return false;
	}

#define UE_LERP_PP(SettingName, Value)\
	if (!PostProcessSettings.bOverride_##SettingName || bOverwriteSettings)\
	{\
		PostProcessSettings.bOverride_##SettingName = true;\
		PostProcessSettings.SettingName = FMath::Lerp(PostProcessSettings.SettingName, Value, PhysicalCameraBlendWeight);\
	}

	UE_LERP_PP(CameraISO, ISO);
	UE_LERP_PP(CameraShutterSpeed, ShutterSpeed);

	UE_LERP_PP(DepthOfFieldFstop, Aperture);
	UE_LERP_PP(DepthOfFieldBladeCount, DiaphragmBladeCount);

	// TODO: add this to the camera pose?
	UE_LERP_PP(DepthOfFieldMinFstop, GGameplayCamerasDefaultMinFstop);

	// TODO: support minimum-focus-distance?
	UE_LERP_PP(DepthOfFieldFocalDistance, FocusDistance);

	const float EffectiveOverscan = (1.0f + Overscan);
	UE_LERP_PP(DepthOfFieldSensorWidth, SensorWidth * EffectiveOverscan);
	UE_LERP_PP(DepthOfFieldSqueezeFactor, SqueezeFactor);

#undef UE_LERP_PP

	return true;
}

FRay3d FCameraPose::GetAimRay() const
{
	const bool bDirectionIsNormalized = false;
	const FVector3d TargetDir{ TargetDistance, 0, 0 };
	return FRay3d(Location, Rotation.RotateVector(TargetDir), bDirectionIsNormalized);
}

FVector3d FCameraPose::GetAimDir() const
{
	return Rotation.RotateVector(FVector3d{ 1, 0, 0 });
}

FVector3d FCameraPose::GetTarget() const
{
	return Location + TargetDistance * GetAimDir();
}

FVector3d FCameraPose::GetTarget(double InTargetDistance) const
{
	return Location + InTargetDistance * GetAimDir();
}

void FCameraPose::OverrideAll(const FCameraPose& OtherPose)
{
	InternalOverrideChanged(OtherPose, false);
}

void FCameraPose::OverrideChanged(const FCameraPose& OtherPose)
{
	InternalOverrideChanged(OtherPose, true);
}

void FCameraPose::InternalOverrideChanged(const FCameraPose& OtherPose, bool bChangedOnly)
{
	const FCameraPoseFlags& OtherPoseChangedFlags = OtherPose.GetChangedFlags();

#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	if (!bChangedOnly || OtherPoseChangedFlags.PropName)\
	{\
		Set##PropName(OtherPose.Get##PropName());\
	}

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY
}

void FCameraPose::LerpAll(const FCameraPose& ToPose, float Factor)
{
	FCameraPoseFlags DummyFlags(true);
	InternalLerpChanged(ToPose, Factor, DummyFlags, false, DummyFlags, false);
}

void FCameraPose::LerpChanged(const FCameraPose& ToPose, float Factor)
{
	FCameraPoseFlags DummyFlags(true);
	InternalLerpChanged(ToPose, Factor, DummyFlags, false, DummyFlags, true);
}

void FCameraPose::LerpChanged(const FCameraPose& ToPose, float Factor, const FCameraPoseFlags& InMask, bool bInvertMask, FCameraPoseFlags& OutMask)
{
	InternalLerpChanged(ToPose, Factor, InMask, bInvertMask, OutMask, true);
}

void FCameraPose::InternalLerpChanged(const FCameraPose& ToPose, float Factor, const FCameraPoseFlags& InMask, bool bInvertMask, FCameraPoseFlags& OutMask, bool bChangedOnly)
{
	if (UNLIKELY(Factor == 0.f))
	{
		return;
	}

	const bool bIsOverride = (Factor == 1.f);
	const FCameraPoseFlags& ToPoseChangedFlags = ToPose.GetChangedFlags();

	if (bIsOverride)
	{
		// The interpolation factor is 1 so we just override the properties.
		// We do all of them except the FOV/Focal Length, which is done in a special way.
	
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
		if ((!bInvertMask && InMask.PropName) || (bInvertMask && !InMask.PropName))\
		{\
			if (!bChangedOnly || ToPoseChangedFlags.PropName)\
			{\
				Set##PropName(ToPose.Get##PropName());\
			}\
			OutMask.PropName = true;\
		}

		UE_CAMERA_POSE_FOR_TRANSFORM_PROPERTIES()
		UE_CAMERA_POSE_FOR_INTERPOLABLE_PROPERTIES()
		UE_CAMERA_POSE_FOR_FLIPPING_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

		if (
				(!bInvertMask && (InMask.FieldOfView || InMask.FocalLength)) ||
				(bInvertMask && (!InMask.FieldOfView || !InMask.FocalLength)))
		{
			if (!bChangedOnly || (ToPoseChangedFlags.FieldOfView || ToPoseChangedFlags.FocalLength))
			{
				SetFocalLength(ToPose.GetFocalLength());
				SetFieldOfView(ToPose.GetFieldOfView());
			}
		}
	}
	else
	{
		// Interpolate all the properties.
		//
		// Start with those we can simply feed to a LERP formula. Some properties don't
		// necessarily make sense to interpolate (like, what does it mean to interpolate the
		// sensor size of a camera?) but, well, we use whatever we have been given at this
		// point.

#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
		if ((!bInvertMask && InMask.PropName) || (bInvertMask && !InMask.PropName))\
		{\
			if (!bChangedOnly || ToPoseChangedFlags.PropName)\
			{\
				Set##PropName(FMath::Lerp(Get##PropName(), ToPose.Get##PropName(), Factor));\
			}\
			OutMask.PropName = true;\
		}

		UE_CAMERA_POSE_FOR_TRANSFORM_PROPERTIES()
		UE_CAMERA_POSE_FOR_INTERPOLABLE_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

		// Next, handle the special case of FOV, where we might have to blend between a pose
		// specifying FOV directly and a pose using focal length.
		if (
				(!bInvertMask && (InMask.FieldOfView || InMask.FocalLength)) ||
				(bInvertMask && (!InMask.FieldOfView || !InMask.FocalLength)))
		{
			ensureMsgf(
					(FocalLength <= 0 || FieldOfView <= 0) &&
					(ToPose.FocalLength <= 0 || ToPose.FieldOfView <= 0),
					TEXT("Can't specify both FocalLength and FieldOfView on a camera pose!"));

			if (!bChangedOnly || (ToPoseChangedFlags.FocalLength || ToPoseChangedFlags.FieldOfView))
			{
				// Interpolate FocalLength, or FieldOfView, if both poses use the same.
				// If there's a mix, interpolate the effective FieldOfView.
				//
				// We realize that linearly interpolating FocalLength won't linearly interpolate
				// the effective FOV, so this will actually behave differently between the two
				// code branches, but this also means that an "all proper" camera setup will
				// enjoy more realistic camera behavior.
				//
				const float FromFocalLength = GetFocalLength();
				const float ToFocalLength = ToPose.GetFocalLength();
				if (FromFocalLength > 0 && ToFocalLength > 0)
				{
					SetFocalLength(FMath::Lerp(FromFocalLength, ToFocalLength, Factor));
				}
				else // only FieldOfView is specified on both, or we have a mix.
				{
					const float FromFieldOfView = GetEffectiveFieldOfView();
					const float ToFieldOfView = ToPose.GetEffectiveFieldOfView();
					SetFieldOfView(FMath::Lerp(FromFieldOfView, ToFieldOfView, Factor));
					SetFocalLength(-1);
				}
				OutMask.FieldOfView = true;
				OutMask.FocalLength = true;
			}
		}

		// Last, do booleans and other properties that just flip their value once we reach 50% interpolation.

#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
		if ((!bInvertMask && InMask.PropName) || (bInvertMask && !InMask.PropName))\
		{\
			if ((!bChangedOnly || ToPoseChangedFlags.PropName) && Factor >= 0.5f)\
			{\
				Set##PropName(ToPose.Get##PropName());\
			}\
			OutMask.PropName = true;\
		}

		UE_CAMERA_POSE_FOR_FLIPPING_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

	}
}

void FCameraPose::SerializeWithFlags(FArchive& Ar, FCameraPose& CameraPose)
{
	Ar.Serialize(static_cast<void*>(&CameraPose), sizeof(FCameraPose));
}

void FCameraPose::SerializeWithFlags(FArchive& Ar)
{
	FCameraPose::SerializeWithFlags(Ar, *this);
}

