// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigAsset.h"

#include "Build/CameraBuildContext.h"
#include "Build/CameraBuildLog.h"
#include "Build/CameraRigAssetBuilder.h"
#include "Core/CameraNode.h"
#include "Core/CameraVariableAssets.h"
#include "Core/IAssetReferenceCameraNode.h"
#include "Misc/EngineVersionComparison.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigAsset)

const FName UCameraRigAsset::NodeTreeGraphName(TEXT("NodeTree"));
const FName UCameraRigAsset::TransitionsGraphName(TEXT("Transitions"));

void UCameraRigAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA

	if (GraphNodePosX_DEPRECATED != 0 || GraphNodePosY_DEPRECATED != 0)
	{
		NodeGraphNodePos = FIntVector2(GraphNodePosX_DEPRECATED, GraphNodePosY_DEPRECATED);

		GraphNodePosX_DEPRECATED = 0;
		GraphNodePosY_DEPRECATED = 0;
	}

	// Any interface parameters found in the list of graph objects should be removed and instead
	// flagged as having a node. This is because the way of handling parameter graph nodes has
	// changed.
	for (auto It = AllNodeTreeObjects.CreateIterator(); It; ++It)
	{
		UObject* Item(*It);
		if (!Item)
		{
			It.RemoveCurrent();
			continue;
		}
		if (UCameraObjectInterfaceParameterBase* InterfaceParameter = Cast<UCameraObjectInterfaceParameterBase>(Item))
		{
			// This will get further upgraded below in UpgradeInterfaceConnections.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InterfaceParameter->bHasGraphNode_DEPRECATED = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			It.RemoveCurrent();
			continue;
		}
	}

#endif

	// Initialize the VariableType of blendable parameters that pre-existed the new interface data.
	// The type defaults to Boolean so check only those.
	for (UCameraObjectInterfaceBlendableParameter* BlendableParameter : Interface.BlendableParameters)
	{
		if (BlendableParameter->VariableType == ECameraVariableType::Boolean)
		{
			if (BlendableParameter->PrivateVariable_DEPRECATED)
			{
				BlendableParameter->VariableType = BlendableParameter->PrivateVariable_DEPRECATED->GetVariableType();
				BlendableParameter->PrivateVariable_DEPRECATED = nullptr;
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// Upgrade the old parameter connections.
	UpgradeInterfaceConnections(this, NodeTreeGraphName);
#endif
}

void UCameraRigAsset::GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const
{
	TagContainer.AppendTags(GameplayTags);
}

void UCameraRigAsset::BuildCameraRig()
{
	using namespace UE::Cameras;

	FCameraBuildLog BuildLog;
	BuildLog.SetForwardMessagesToLogging(true);
	FCameraBuildContext BuildContext(BuildLog);
	BuildCameraRig(BuildContext);
}

void UCameraRigAsset::BuildCameraRig(UE::Cameras::FCameraBuildContext& InBuildContext)
{
	using namespace UE::Cameras;

	FCameraRigAssetBuilder Builder(InBuildContext);
	Builder.BuildCameraRig(this);
}

void UCameraRigAsset::DirtyBuildStatus()
{
	BuildStatus = ECameraBuildStatus::Dirty;
}

void UCameraRigAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR

	const bool bIsUserObject = !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
	const bool bIsEditorAutoSave = ObjectSaveContext.IsFromAutoSave();
#else
	const bool bIsEditorAutoSave = ((ObjectSaveContext.GetSaveFlags() & SAVE_FromAutosave) != 0);
#endif  // UE >= 5.7.0
	if (bIsUserObject && !bIsEditorAutoSave)
	{
		// Build when saving/cooking.
		BuildCameraRig();
	}

#endif

	Super::PreSave(ObjectSaveContext);
}

#if WITH_EDITOR

void UCameraRigAsset::GatherPackages(FCameraRigPackages& OutPackages) const
{
	TArray<UCameraNode*> NodeStack;
	if (RootNode)
	{
		NodeStack.Add(RootNode);
	}
	while (!NodeStack.IsEmpty())
	{
		UCameraNode* CurrentNode = NodeStack.Pop();
		const UPackage* CurrentPackage = CurrentNode->GetOutermost();
		OutPackages.AddUnique(CurrentPackage);

		if (IAssetReferenceCameraNode* AssetReferencer = Cast<IAssetReferenceCameraNode>(CurrentNode))
		{
			AssetReferencer->GatherPackages(OutPackages);
		}

		FCameraNodeChildrenView CurrentChildren = CurrentNode->GetChildren();
		for (UCameraNode* CurrentChild : ReverseIterate(CurrentChildren))
		{
			if (CurrentChild)
			{
				NodeStack.Add(CurrentChild);
			}
		}
	}
}

void UCameraRigAsset::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	if (InGraphName == NodeTreeGraphName)
	{
		NodePosX = NodeGraphNodePos.X;
		NodePosY = NodeGraphNodePos.Y;
	}
	else if (InGraphName == TransitionsGraphName)
	{
		NodePosX = TransitionGraphNodePos.X;
		NodePosY = TransitionGraphNodePos.Y;
	}
}

void UCameraRigAsset::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	if (InGraphName == NodeTreeGraphName)
	{
		NodeGraphNodePos.X = NodePosX;
		NodeGraphNodePos.Y = NodePosY;
	}
	else if (InGraphName == TransitionsGraphName)
	{
		TransitionGraphNodePos.X = NodePosX;
		TransitionGraphNodePos.Y = NodePosY;
	}
}

EObjectTreeGraphObjectSupportFlags UCameraRigAsset::GetSupportFlags(FName InGraphName) const
{
	return EObjectTreeGraphObjectSupportFlags::CommentText;
}

const FString& UCameraRigAsset::GetGraphNodeCommentText(FName InGraphName) const
{
	if (InGraphName == NodeTreeGraphName)
	{
		return NodeGraphNodeComment;
	}
	else if (InGraphName == TransitionsGraphName)
	{
		return TransitionGraphNodeComment;
	}

	static FString InvalidString;
	return InvalidString;
}

void UCameraRigAsset::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	if (InGraphName == NodeTreeGraphName)
	{
		NodeGraphNodeComment = NewComment;
	}
	else if (InGraphName == TransitionsGraphName)
	{
		TransitionGraphNodeComment = NewComment;
	}
}

void UCameraRigAsset::GetGraphNodeName(FName InGraphName, FText& OutName) const
{
	OutName = FText::FromString(GetName());
}

void UCameraRigAsset::GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const
{
	if (InGraphName == NodeTreeGraphName)
	{
		OutObjects.Append(AllNodeTreeObjects);
	}
	else if (InGraphName == TransitionsGraphName)
	{
		OutObjects.Append(AllTransitionsObjects);
	}
}

void UCameraRigAsset::AddConnectableObject(FName InGraphName, UObject* InObject)
{
	using namespace UE::Cameras;

	Modify();

	if (InGraphName == NodeTreeGraphName)
	{
		const int32 Index = AllNodeTreeObjects.AddUnique(InObject);
		ensure(Index == AllNodeTreeObjects.Num() - 1);
		EventHandlers.Notify(&ICameraRigAssetEventHandler::OnObjectAddedToGraph, NodeTreeGraphName, InObject);
	}
	else if (InGraphName == TransitionsGraphName)
	{
		const int32 Index = AllTransitionsObjects.AddUnique(InObject);
		ensure(Index == AllTransitionsObjects.Num() - 1);
		EventHandlers.Notify(&ICameraRigAssetEventHandler::OnObjectAddedToGraph, TransitionsGraphName, InObject);
	}
}

void UCameraRigAsset::RemoveConnectableObject(FName InGraphName, UObject* InObject)
{
	using namespace UE::Cameras;

	Modify();

	if (InGraphName == NodeTreeGraphName)
	{
		const int32 NumRemoved = AllNodeTreeObjects.Remove(InObject);
		ensure(NumRemoved == 1);
		EventHandlers.Notify(&ICameraRigAssetEventHandler::OnObjectRemovedFromGraph, NodeTreeGraphName, InObject);
	}
	else if (InGraphName == TransitionsGraphName)
	{
		const int32 NumRemoved = AllTransitionsObjects.Remove(InObject);
		ensure(NumRemoved == 1);
		EventHandlers.Notify(&ICameraRigAssetEventHandler::OnObjectRemovedFromGraph, TransitionsGraphName, InObject);
	}
}

#endif  // WITH_EDITOR

void UCameraRigAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	const UEnum* VariableTypeEnum = StaticEnum<ECameraVariableType>();
	for (const UCameraObjectInterfaceBlendableParameter* BlendableParameter : Interface.BlendableParameters)
	{
		if (BlendableParameter)
		{
			FAssetRegistryTag BlendableParameterTag;
			BlendableParameterTag.Name = FName(BlendableParameter->InterfaceParameterName);
			BlendableParameterTag.Value = VariableTypeEnum->GetNameStringByValue((int64)BlendableParameter->VariableType);
			BlendableParameterTag.Type = FAssetRegistryTag::TT_Alphabetical;
			Context.AddTag(BlendableParameterTag);
		}
	}

	const UEnum* ContextDataTypeEnum = StaticEnum<ECameraContextDataType>();
	for (const UCameraObjectInterfaceDataParameter* DataParameter : Interface.DataParameters)
	{
		if (DataParameter)
		{
			FAssetRegistryTag DataParameterTag;
			DataParameterTag.Name = FName(DataParameter->InterfaceParameterName);
			DataParameterTag.Value = ContextDataTypeEnum->GetNameStringByValue((int64)DataParameter->DataType);
			if (DataParameter->DataContainerType == ECameraContextDataContainerType::Array)
			{
				DataParameterTag.Value += TEXT("[]");
			}
			DataParameterTag.Type = FAssetRegistryTag::TT_Alphabetical;
			Context.AddTag(DataParameterTag);
		}
	}

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR

void UCameraRigAsset::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);
}

#endif  // WITH_EDITOR

