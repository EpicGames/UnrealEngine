// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigAssetReference.h"

#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraNodeEvaluator.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigAssetReference)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FCameraRigAssetReference::FCameraRigAssetReference()
{
}

FCameraRigAssetReference::FCameraRigAssetReference(UCameraRigAsset* InCameraRig)
	: CameraRig(InCameraRig)
{
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

const UBaseCameraObject* FCameraRigAssetReference::GetCameraObject() const
{
	return CameraRig;
}

void FCameraRigAssetReference::EnsureAllocationInfo(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const
{
	if (CameraRig)
	{
		const FCameraObjectAllocationInfo& AllocationInfo = CameraRig->AllocationInfo;
		OutResult.VariableTable.EnsureVariables(AllocationInfo.VariableTableInfo);
		OutResult.ContextDataTable.EnsureData(AllocationInfo.ContextDataTableInfo);
	}
}

void FCameraRigAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutVariableTable, nullptr, bDrivenOnly, false);
}

void FCameraRigAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, UE::Cameras::FCameraContextDataTable& OutContextDataTable, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutVariableTable, &OutContextDataTable, bDrivenOnly, false);
}

void FCameraRigAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutResult.VariableTable, &OutResult.ContextDataTable, bDrivenOnly, false);
}

void FCameraRigAssetReference::ApplyParameterOverridesAndDefaults(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const
{
	ApplyParameterOverridesImpl(&OutResult.VariableTable, &OutResult.ContextDataTable, false, true);
}

void FCameraRigAssetReference::ApplyParameterOverridesImpl(UE::Cameras::FCameraVariableTable* OutVariableTable, UE::Cameras::FCameraContextDataTable* OutContextDataTable, bool bDrivenOnly, bool bApplyUnwrittenDefaults) const
{
	using namespace UE::Cameras;

	if (CameraRig)
	{
		FCameraObjectInterfaceParameterOverrideHelper Helper(OutVariableTable, OutContextDataTable);
		Helper.bDrivenOnly = bDrivenOnly;
		Helper.ApplyParameterOverrides(CameraRig, Parameters);
		if (bApplyUnwrittenDefaults)
		{
			Helper.ApplyParameterDefaults(CameraRig, true);
		}
	}
}

bool FCameraRigAssetReference::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_SoftObjectProperty)
	{
		FSoftObjectPtr CameraRigPath;
		Slot << CameraRigPath;
		CameraRig = Cast<UCameraRigAsset>(CameraRigPath.Get());
		return true;
	}
	return false;
}

void FCameraRigAssetReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		// Make a property bag with the legacy overrides, and then set the values in it.
		bool bHasAnyLegacyOverride = false;
		TArray<FPropertyBagPropertyDesc> LegacyParameterProperties;
		TArray<FCameraObjectInterfaceParameterMetaData> LegacyParameterMetaData;

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		for (F##ValueName##CameraRigParameterOverride& ParameterOverride : ParameterOverrides_DEPRECATED.ValueName##Overrides)\
		{\
			FName PropertyName(ParameterOverride.InterfaceParameterName);\
			EPropertyBagPropertyType PropertyType = EPropertyBagPropertyType::Struct;\
			const UObject* PropertyTypeObject = F##ValueName##CameraParameter::StaticStruct();\
			FPropertyBagPropertyDesc LegacyParameterProperty(PropertyName, PropertyType, PropertyTypeObject);\
			LegacyParameterProperty.ID = ParameterOverride.InterfaceParameterGuid;\
			LegacyParameterProperties.Add(LegacyParameterProperty);\
			FCameraObjectInterfaceParameterMetaData MetaData;\
			MetaData.ParameterGuid = ParameterOverride.InterfaceParameterGuid;\
			MetaData.bIsOverridden_DEPRECATED = true;\
			LegacyParameterMetaData.Add(MetaData);\
			bHasAnyLegacyOverride = true;\
		}
		UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

		if (bHasAnyLegacyOverride)
		{
			Parameters = FInstancedOverridablePropertyBag();
			Parameters.AddProperties(LegacyParameterProperties);

			ParameterMetaData = LegacyParameterMetaData;

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			for (F##ValueName##CameraRigParameterOverride& ParameterOverride : ParameterOverrides_DEPRECATED.ValueName##Overrides)\
			{\
				FName PropertyName(ParameterOverride.InterfaceParameterName);\
				Parameters.SetValueStruct<F##ValueName##CameraParameter>(PropertyName, ParameterOverride.Value);\
			}
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

			ParameterOverrides_DEPRECATED = FCameraRigParameterOverrides();
		}

		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!ParameterOverrideGuids_DEPRECATED.IsEmpty())
		{
			for (const FGuid& Guid : ParameterOverrideGuids_DEPRECATED)
			{
				FCameraObjectInterfaceParameterMetaData MetaData;
				MetaData.ParameterGuid = Guid;
				MetaData.bIsOverridden_DEPRECATED = true;
				ParameterMetaData.Add(MetaData);
			}

			ParameterOverrideGuids_DEPRECATED.Reset();
		}
	}

	FBaseCameraObjectReference::PostSerialize(Ar);
}

