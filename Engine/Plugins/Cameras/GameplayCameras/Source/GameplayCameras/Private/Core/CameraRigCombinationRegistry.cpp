// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigCombinationRegistry.h"

#include "Core/CameraRigAsset.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigCombinationRegistry)

namespace UE::Cameras
{

FCameraRigCombinationRegistry::FCameraRigCombinationKey::FCameraRigCombinationKey(TArrayView<const UCameraRigAsset*> InCombination)
{
	CachedHash = 0;
	for (const UCameraRigAsset* CameraRig : InCombination)
	{
		ensure(CameraRig);
		TWeakObjectPtr<const UCameraRigAsset> WeakCameraRig(CameraRig);
		CachedHash = HashCombine(CachedHash, GetTypeHash(WeakCameraRig));
		Combination.Add(WeakCameraRig);
	}
}

const UCameraRigAsset* FCameraRigCombinationRegistry::FindOrCreateCombination(TArrayView<const UCameraRigAsset*> InCombination)
{
	FCameraRigCombinationKey Key(InCombination);
	int32& Index = Combinations.FindOrAdd(Key, INDEX_NONE);
	if (Index == INDEX_NONE)
	{
		Index = CombinedCameraRigs.Find(nullptr);
		if (Index == INDEX_NONE)
		{
			Index = CombinedCameraRigs.Add(nullptr);
		}
		check(CombinedCameraRigs.IsValidIndex(Index));

		UCameraRigAsset* NewCombinedCameraRig = NewObject<UCameraRigAsset>();
		UCombinedCameraRigsCameraNode* PrefabNode = NewObject<UCombinedCameraRigsCameraNode>(NewCombinedCameraRig);
		{
			for (const UCameraRigAsset* IndividualCameraRig : InCombination)
			{
				PrefabNode->CameraRigReferences.Add(const_cast<UCameraRigAsset*>(IndividualCameraRig));

				NewCombinedCameraRig->AllocationInfo.Append(IndividualCameraRig->AllocationInfo);
			}
		}
		NewCombinedCameraRig->RootNode = PrefabNode;

		CombinedCameraRigs[Index] = NewCombinedCameraRig;
	}
	return CombinedCameraRigs[Index];
}

void FCameraRigCombinationRegistry::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(CombinedCameraRigs);
}

class FCombinedCameraRigsCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCombinedCameraRigsCameraNodeEvaluator)

protected:

	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void ApplyParameterOverrides(FCameraVariableTable& OutVariableTable, bool bDrivenOnly);

private:

	TArray<FCameraNodeEvaluator*> CameraRigRootEvaluators;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCombinedCameraRigsCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FCombinedCameraRigCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(int32, CameraRigIndex);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, CameraRigName);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FCombinedCameraRigCameraDebugBlock)

FCameraNodeEvaluatorChildrenView FCombinedCameraRigsCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView(CameraRigRootEvaluators);
}

void FCombinedCameraRigsCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UCombinedCameraRigsCameraNode* CombinedRigsNode = GetCameraNodeAs<UCombinedCameraRigsCameraNode>();
	for (const FCameraRigAssetReference& IndividualCameraRigReference : CombinedRigsNode->CameraRigReferences)
	{
		if (const UCameraRigAsset* CameraRig = IndividualCameraRigReference.GetCameraRig())
		{
			if (CameraRig->RootNode)
			{
				FCameraNodeEvaluator* CameraRigRootEvaluator = Params.BuildEvaluator(CameraRig->RootNode);
				if (CameraRigRootEvaluator)
				{
					CameraRigRootEvaluators.Add(CameraRigRootEvaluator);
				}
			}
		}
	}
}

void FCombinedCameraRigsCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Apply overrides right away.
	ApplyParameterOverrides(OutResult.VariableTable, false);
}

void FCombinedCameraRigsCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Keep applying overrides in case they are driven by a variable.
	ApplyParameterOverrides(OutResult.VariableTable, true);

	for (FCameraNodeEvaluator* CameraRigRootEvaluator : CameraRigRootEvaluators)
	{
		CameraRigRootEvaluator->Run(Params, OutResult);
	}
}

void FCombinedCameraRigsCameraNodeEvaluator::ApplyParameterOverrides(FCameraVariableTable& OutVariableTable, bool bDrivenOnly)
{
	const UCombinedCameraRigsCameraNode* CombinedRigsNode = GetCameraNodeAs<UCombinedCameraRigsCameraNode>();
	for (const FCameraRigAssetReference& IndividualCameraRigReference : CombinedRigsNode->CameraRigReferences)
	{
		IndividualCameraRigReference.ApplyParameterOverrides(OutVariableTable, bDrivenOnly);
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCombinedCameraRigsCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	// Wrap our children inside debug blocks that show their camera rig name.
	for (int32 Index = 0; Index < CameraRigRootEvaluators.Num(); ++Index)
	{
		FCameraNodeEvaluator* CameraRigRootEvaluator(CameraRigRootEvaluators[Index]);
		if (!CameraRigRootEvaluator)
		{
			continue;
		}

		FString CameraRigName;
		if (const UCameraNode* CameraRigRootNode = CameraRigRootEvaluator->GetCameraNode())
		{
			if (const UCameraRigAsset* CameraRig = CameraRigRootNode->GetTypedOuter<UCameraRigAsset>())
			{
				CameraRigName = CameraRig->GetName();
			}
		}

		FCombinedCameraRigCameraDebugBlock& ChildBlock = Builder.StartChildDebugBlock<FCombinedCameraRigCameraDebugBlock>();
		ChildBlock.CameraRigIndex = Index;
		ChildBlock.CameraRigName = CameraRigName;
		{
			CameraRigRootEvaluator->BuildDebugBlocks(Params, Builder);
		}
		Builder.EndChildDebugBlock();
	}

	Builder.SkipChildren();
}

void FCombinedCameraRigCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("{cam_passive}[%d] {cam_notice}%s{cam_default}\n"), CameraRigIndex + 1, *CameraRigName);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

void UCombinedCameraRigsCameraNode::GetAllCombinationCameraRigs(const UCameraRigAsset* InCameraRig, TArray<const UCameraRigAsset*>& OutCameraRigs)
{
	if (!InCameraRig)
	{
		return;
	}

	if (const UCombinedCameraRigsCameraNode* CameraRigCombinationNode = Cast<UCombinedCameraRigsCameraNode>(InCameraRig->RootNode))
	{
		for (const FCameraRigAssetReference& CameraRigReference : CameraRigCombinationNode->CameraRigReferences)
		{
			if (const UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig())
			{
				OutCameraRigs.Add(CameraRig);
			}
		}
	}
	else
	{
		OutCameraRigs.Add(InCameraRig);
	}
}

FCameraNodeEvaluatorPtr UCombinedCameraRigsCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCombinedCameraRigsCameraNodeEvaluator>();
}

