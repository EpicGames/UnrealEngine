// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Core/CameraNode.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"

#include "CameraRigCombinationRegistry.generated.h"

namespace UE::Cameras
{

class FCameraRigCombinationRegistry
{
public:

	const UCameraRigAsset* FindOrCreateCombination(TArrayView<const UCameraRigAsset*> InCombination);

	void AddReferencedObjects(FReferenceCollector& Collector);

private:

	using FCameraRigCombination = TArray<TWeakObjectPtr<const UCameraRigAsset>, TInlineAllocator<4>>;

	struct FCameraRigCombinationKey
	{
		FCameraRigCombinationKey(TArrayView<const UCameraRigAsset*> InCombination);
		FCameraRigCombination Combination;
		uint32 CachedHash = 0;
	};

	friend inline uint32 GetTypeHash(const FCameraRigCombinationKey& Key)
	{
		return Key.CachedHash;
	}

	friend inline bool operator==(const FCameraRigCombinationKey& A, const FCameraRigCombinationKey& B)
	{
		return A.Combination == B.Combination;
	}

	using FCameraRigCombinationMap = TMap<FCameraRigCombinationKey, int32>;
	FCameraRigCombinationMap Combinations;

	TArray<TObjectPtr<const UCameraRigAsset>> CombinedCameraRigs;
};

}  // namespace UE::Cameras
 
UCLASS(MinimalAPI, Hidden)
class UCombinedCameraRigsCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	static void GetAllCombinationCameraRigs(const UCameraRigAsset* InCameraRig, TArray<const UCameraRigAsset*>& OutCameraRigs);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The camera rigs to run. */
	UPROPERTY()
	TArray<FCameraRigAssetReference> CameraRigReferences;
};

