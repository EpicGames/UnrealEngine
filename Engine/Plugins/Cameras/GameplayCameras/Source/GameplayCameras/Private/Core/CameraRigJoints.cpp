// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigJoints.h"

#include "Core/CameraVariableTableAllocationInfo.h"
#include "Core/BuiltInCameraVariables.h"

namespace UE::Cameras
{

FArchive& operator<< (FArchive& Ar, FCameraRigJoint& RigJoint)
{
	Ar << RigJoint.VariableID;
	Ar << RigJoint.Transform;
	return Ar;
}

void FCameraRigJoints::AddJoint(const FCameraRigJoint& InJoint)
{
	Joints.Add(InJoint);
}

void FCameraRigJoints::AddJoint(const FCameraVariableDefinition& InVariableDefinition, const FTransform3d& InTransform)
{
	AddJoint(FCameraRigJoint{ InVariableDefinition.VariableID, InTransform });
}

void FCameraRigJoints::AddYawPitchJoint(const FTransform3d& InTransform)
{
	AddJoint(FBuiltInCameraVariables::Get().YawPitchDefinition, InTransform);
}

void FCameraRigJoints::Reset()
{
	Joints.Reset();
}

void FCameraRigJoints::Serialize(FArchive& Ar)
{
	Ar << Joints;
}

void FCameraRigJoints::OverrideAll(const FCameraRigJoints& OtherJoints)
{
	Joints = OtherJoints.Joints;
}

void FCameraRigJoints::LerpAll(const FCameraRigJoints& ToJoints, float BlendFactor)
{
	const bool bFlipOldNewJoints = (BlendFactor >= 0.5f);

	// Gather our existing joints. Flag them as "keep" if we are still below 50% blend, which means
	// that we are not switching over to the other joints yet.
	using FFlaggedJoints = TTuple<FCameraRigJoint, bool>;
	TMap<FCameraVariableID, FFlaggedJoints> JointsPerVariable;
	const bool bKeepExistingJoints = !bFlipOldNewJoints;
	for (const FCameraRigJoint& Joint : Joints)
	{
		JointsPerVariable.Add(Joint.VariableID, { Joint, bKeepExistingJoints });
	}

	// Look at the other joints... if we have a similar joint, blend the transform. If this is a
	// new joint, only add it if we are over 50% blend (i.e. if we are switching over).
	for (const FCameraRigJoint& OtherJoint : ToJoints.Joints)
	{
		if (FFlaggedJoints* FlaggedJoint = JointsPerVariable.Find(OtherJoint.VariableID))
		{
			FCameraRigJoint& Joint = FlaggedJoint->Key;

			FTransform3d OutTransform;
			OutTransform.SetLocation(FMath::Lerp(Joint.Transform.GetLocation(), OtherJoint.Transform.GetLocation(), BlendFactor));
			OutTransform.SetRotation(FMath::Lerp(Joint.Transform.GetRotation(), OtherJoint.Transform.GetRotation(), BlendFactor));
			OutTransform.SetScale3D(FMath::Lerp(Joint.Transform.GetScale3D(), OtherJoint.Transform.GetScale3D(), BlendFactor));

			Joint.Transform = OutTransform;

			FlaggedJoint->Value = true;
		}
		else if (bFlipOldNewJoints)
		{
			JointsPerVariable.Add(OtherJoint.VariableID, { OtherJoint, true });
		}
	}

	// Reset our list of joints to whatever joints we have in our map that have been flagged
	// for keeping.
	Joints.Reset();
	for (auto& Pair : JointsPerVariable)
	{
		const FFlaggedJoints& FlaggedJoint(Pair.Value);
		if (FlaggedJoint.Value)
		{
			Joints.Add(FlaggedJoint.Key);
		}
	}
}

}  // namespace UE::Cameras

