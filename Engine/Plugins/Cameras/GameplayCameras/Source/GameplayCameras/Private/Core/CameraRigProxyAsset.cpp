// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigProxyAsset.h"

#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigProxyAsset)

void UCameraRigProxyAsset::PostLoad()
{
	Super::PostLoad();

	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UCameraRigProxyAsset::PostInitProperties()
{
	Super::PostInitProperties();
	
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && 
			!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UCameraRigProxyAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		Guid = FGuid::NewGuid();
	}
}

