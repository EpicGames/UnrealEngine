// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigProxyRedirectTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigProxyRedirectTable)

UCameraRigAsset* FCameraRigProxyRedirectTable::ResolveProxy(const FCameraRigProxyResolveParams& InParams) const
{
	ensure(InParams.CameraRigProxy);

	for (const FCameraRigProxyRedirectTableEntry& Entry : Entries)
	{
		if (InParams.CameraRigProxy == Entry.CameraRigProxy)
		{
			return Entry.CameraRig;
		}
	}
	return nullptr;
}

