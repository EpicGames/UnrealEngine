// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigTransition.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Core/BlendCameraNode.h"
#include "Logging/TokenizedMessage.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigTransition)

#define LOCTEXT_NAMESPACE "CameraRigTransition"

void UCameraRigTransitionCondition::PostLoad()
{
#if WITH_EDITORONLY_DATA

	if (GraphNodePosX_DEPRECATED != 0 || GraphNodePosY_DEPRECATED != 0)
	{
		GraphNodePos = FIntVector2(GraphNodePosX_DEPRECATED, GraphNodePosY_DEPRECATED);

		GraphNodePosX_DEPRECATED = 0;
		GraphNodePosY_DEPRECATED = 0;
	}

#endif

	Super::PostLoad();
}

bool UCameraRigTransitionCondition::TransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const
{
	return OnTransitionMatches(Params);
}

void UCameraRigTransitionCondition::Build(FCameraObjectBuildContext& BuildContext)
{
	OnBuild(BuildContext);
}

#if WITH_EDITOR

void UCameraRigTransitionCondition::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UCameraRigTransitionCondition::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

const FString& UCameraRigTransitionCondition::GetGraphNodeCommentText(FName InGraphName) const
{
	return GraphNodeComment;
}

void UCameraRigTransitionCondition::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	GraphNodeComment = NewComment;
}

#endif

void UCameraRigTransition::PostLoad()
{
#if WITH_EDITORONLY_DATA

	if (GraphNodePosX_DEPRECATED != 0 || GraphNodePosY_DEPRECATED != 0)
	{
		GraphNodePos = FIntVector2(GraphNodePosX_DEPRECATED, GraphNodePosY_DEPRECATED);

		GraphNodePosX_DEPRECATED = 0;
		GraphNodePosY_DEPRECATED = 0;
	}

#endif

	Super::PostLoad();
}

bool UCameraRigTransition::AllConditionsMatch(const FCameraRigTransitionConditionMatchParams& Params) const
{
	for (const UCameraRigTransitionCondition* Condition : Conditions)
	{
		if (Condition && !Condition->TransitionMatches(Params))
		{
			return false;
		}
	}

	return true;
}

void UCameraRigTransition::Build(FCameraObjectBuildContext& BuildContext)
{
	if (Blend)
	{
		Blend->Build(BuildContext);
	}
	else
	{
		BuildContext.BuildLog.AddMessage(
				EMessageSeverity::Error, this,
				LOCTEXT("NullBlendError", "No blend defined on transition. To make a straight-cut transition, use the Pop blend."));
	}

	for (UCameraRigTransitionCondition* Condition : Conditions)
	{
		if (Condition)
		{
			Condition->Build(BuildContext);
		}
		else
		{
			BuildContext.BuildLog.AddMessage(
					EMessageSeverity::Error, this,
					LOCTEXT("NullConditionError", "Found an invalid transition condition."));
		}
	}
}

#if WITH_EDITOR

void UCameraRigTransition::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UCameraRigTransition::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

const FString& UCameraRigTransition::GetGraphNodeCommentText(FName InGraphName) const
{
	return GraphNodeComment;
}

void UCameraRigTransition::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	GraphNodeComment = NewComment;
}

#endif  // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

