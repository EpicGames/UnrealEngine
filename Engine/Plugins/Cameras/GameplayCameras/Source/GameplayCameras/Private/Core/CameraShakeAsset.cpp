// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraShakeAsset.h"

#include "Build/CameraBuildContext.h"
#include "Build/CameraShakeAssetBuilder.h"
#include "Core/ShakeCameraNode.h"  // IWYU pragma: keep
#include "Misc/EngineVersionComparison.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeAsset)

void UCameraShakeAsset::BuildCameraShake()
{
	using namespace UE::Cameras;

	FCameraBuildLog BuildLog;
	BuildLog.SetForwardMessagesToLogging(true);
	FCameraBuildContext BuildContext(BuildLog);
	BuildCameraShake(BuildContext);
}

void UCameraShakeAsset::BuildCameraShake(UE::Cameras::FCameraBuildContext& InBuildContext)
{
	using namespace UE::Cameras;

	FCameraShakeAssetBuilder Builder(InBuildContext);
	Builder.BuildCameraShake(this);
}

UCameraNode* UCameraShakeAsset::GetRootNode()
{
	return RootNode;
}

void UCameraShakeAsset::DirtyBuildStatus()
{
	BuildStatus = ECameraBuildStatus::Dirty;
}

void UCameraShakeAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Upgrade the old parameter connections.
	UpgradeInterfaceConnections(this, NAME_None);
#endif
}

void UCameraShakeAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR

	const bool bIsUserObject = !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
	const bool bIsEditorAutoSave = ObjectSaveContext.IsFromAutoSave();
#else
	const bool bIsEditorAutoSave = ((ObjectSaveContext.GetSaveFlags() & SAVE_FromAutosave) != 0);
#endif  // UE >= 5.7.0
	if (bIsUserObject && !bIsEditorAutoSave)
	{
		// Build when saving/cooking.
		BuildCameraShake();
	}

#endif

	Super::PreSave(ObjectSaveContext);
}

#if WITH_EDITOR

void UCameraShakeAsset::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UCameraShakeAsset::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

EObjectTreeGraphObjectSupportFlags UCameraShakeAsset::GetSupportFlags(FName InGraphName) const
{
	return EObjectTreeGraphObjectSupportFlags::CommentText;
}

const FString& UCameraShakeAsset::GetGraphNodeCommentText(FName InGraphName) const
{
	return GraphNodeComment;
}

void UCameraShakeAsset::OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment)
{
	Modify();

	GraphNodeComment = NewComment;
}

void UCameraShakeAsset::GetGraphNodeName(FName InGraphName, FText& OutName) const
{
	OutName = FText::FromString(GetName());
}

void UCameraShakeAsset::GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const
{
	OutObjects.Append(AllNodeObjects);
}

void UCameraShakeAsset::AddConnectableObject(FName InGraphName, UObject* InObject)
{
	using namespace UE::Cameras;

	Modify();

	const int32 Index = AllNodeObjects.AddUnique(InObject);
	ensure(Index == AllNodeObjects.Num() - 1);
}

void UCameraShakeAsset::RemoveConnectableObject(FName InGraphName, UObject* InObject)
{
	using namespace UE::Cameras;

	Modify();

	const int32 NumRemoved = AllNodeObjects.Remove(InObject);
	ensure(NumRemoved == 1);
}

#endif  // WITH_EDITOR

