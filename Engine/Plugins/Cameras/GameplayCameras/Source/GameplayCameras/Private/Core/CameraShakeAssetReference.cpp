// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraShakeAssetReference.h"

#include "Core/CameraShakeAsset.h"
#include "Core/CameraVariableTable.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeAssetReference)

FCameraShakeAssetReference::FCameraShakeAssetReference()
{
}

FCameraShakeAssetReference::FCameraShakeAssetReference(UCameraShakeAsset* InCameraShake)
	: CameraShake(InCameraShake)
{
}

const UBaseCameraObject* FCameraShakeAssetReference::GetCameraObject() const
{
	return CameraShake;
}

void FCameraShakeAssetReference::EnsureAllocationInfo(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const
{
	if (CameraShake)
	{
		const FCameraObjectAllocationInfo& AllocationInfo = CameraShake->AllocationInfo;
		OutResult.VariableTable.EnsureVariables(AllocationInfo.VariableTableInfo);
		OutResult.ContextDataTable.EnsureData(AllocationInfo.ContextDataTableInfo);
	}
}

void FCameraShakeAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutVariableTable, nullptr, bDrivenOnly, false);
}

void FCameraShakeAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, UE::Cameras::FCameraContextDataTable& OutContextDataTable, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutVariableTable, &OutContextDataTable, bDrivenOnly, false);
}

void FCameraShakeAssetReference::ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const
{
	ApplyParameterOverridesImpl(&OutResult.VariableTable, &OutResult.ContextDataTable, bDrivenOnly, false);
}

void FCameraShakeAssetReference::ApplyParameterOverridesAndDefaults(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const
{
	ApplyParameterOverridesImpl(&OutResult.VariableTable, &OutResult.ContextDataTable, false, true);
}

void FCameraShakeAssetReference::ApplyParameterOverridesImpl(UE::Cameras::FCameraVariableTable* OutVariableTable, UE::Cameras::FCameraContextDataTable* OutContextDataTable, bool bDrivenOnly, bool bApplyUnwrittenDefaults) const
{
	using namespace UE::Cameras;
	
	if (CameraShake)
	{
		FCameraObjectInterfaceParameterOverrideHelper Helper(OutVariableTable, OutContextDataTable);
		Helper.bDrivenOnly = bDrivenOnly;
		Helper.ApplyParameterOverrides(CameraShake, Parameters);
		if (bApplyUnwrittenDefaults)
		{
			Helper.ApplyParameterDefaults(CameraShake, true);
		}
	}
}

