// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraSystemEvaluator.h"

#include "Camera/CameraTypes.h"
#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraRigCombinationRegistry.h"
#include "Core/DefaultRootCameraNode.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraSystemTrace.h"
#include "Debug/RootCameraDebugBlock.h"
#include "GameplayCamerasSettings.h"
#include "Math/ColorList.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Services/CameraActionService.h"
#include "Services/CameraModifierService.h"
#include "Services/CameraParameterSetterService.h"
#include "Services/CameraShakeService.h"
#include "Services/OrientationInitializationService.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

DECLARE_CYCLE_STAT(TEXT("Camera System Eval"), CameraSystemEval_Total, STATGROUP_CameraSystem);

namespace UE::Cameras
{

extern bool GGameplayCamerasDebugEnable;

void FCameraSystemEvaluationResult::Reset()
{
	CameraPose.ClearAllChangedFlags();
	VariableTable.ClearAllWrittenThisFrameFlags();
	ContextDataTable.ClearAllWrittenThisFrameFlags();
	bIsCameraCut = false;
	bIsValid = false;
}

void FCameraSystemEvaluationResult::Reset(const FCameraNodeEvaluationResult& NodeResult)
{
	Reset();

	// Make the camera poses actually equal, so that we get the exact same changed-flags.
	CameraPose = NodeResult.CameraPose;

	VariableTable.OverrideAll(NodeResult.VariableTable);
	ContextDataTable.OverrideAll(NodeResult.ContextDataTable);
	PostProcessSettings.OverrideAll(NodeResult.PostProcessSettings);

	bIsCameraCut = NodeResult.bIsCameraCut;
	bIsValid = true;
}

FCameraSystemEvaluator::FCameraSystemEvaluator()
{
}

void FCameraSystemEvaluator::Initialize(TObjectPtr<UObject> InOwner)
{
	FCameraSystemEvaluatorCreateParams Params;
	Params.Owner = InOwner;
	Initialize(Params);
}

void FCameraSystemEvaluator::Initialize(const FCameraSystemEvaluatorCreateParams& Params)
{
	UObject* Owner = Params.Owner;
	if (!Owner)
	{
		Owner = GetTransientPackage();
	}
	WeakOwner = Owner;

	Role = Params.Role;

	if (Params.RootNodeFactory)
	{
		RootNode = Params.RootNodeFactory();
	}
	else
	{
		RootNode = NewObject<UDefaultRootCameraNode>(Owner, TEXT("RootNode"));
	}

	ContextStack.Initialize(*this);

	FCameraNodeEvaluatorTreeBuildParams BuildParams;
	BuildParams.RootCameraNode = RootNode;
	RootEvaluator = static_cast<FRootCameraNodeEvaluator*>(RootEvaluatorStorage.BuildEvaluatorTree(BuildParams));

	RegisterEvaluationService(MakeShared<FCameraModifierService>());
	RegisterEvaluationService(MakeShared<FCameraParameterSetterService>());
	RegisterEvaluationService(MakeShared<FCameraShakeService>());
	RegisterEvaluationService(MakeShared<FOrientationInitializationService>());
	RegisterEvaluationService(MakeShared<FCameraActionService>());

	CameraRigCombinationRegistry = MakeShared<FCameraRigCombinationRegistry>();

	if (ensure(RootEvaluator))
	{
		FCameraNodeEvaluatorInitializeParams InitParams;
		InitParams.Evaluator = this;
		RootEvaluator->Initialize(InitParams, RootNodeResult);
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	if (!DebugID.IsValid())
	{
		DebugID = FCameraSystemDebugRegistry::Get().RegisterCameraSystemEvaluator(SharedThis(this));
	}
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

FCameraSystemEvaluator::~FCameraSystemEvaluator()
{
	ContextStack.OnStackChanged().Clear();
	ContextStack.Reset();

	{
		FCameraEvaluationServiceTeardownParams TeardownParams;
		TeardownParams.Evaluator = this;
		for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
		{
			EvaluationService->Teardown(TeardownParams);
		}
		EvaluationServices.Reset();
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	if (DebugID.IsValid())
	{
		FCameraSystemDebugRegistry::Get().UnregisterCameraSystemEvaluator(DebugID);
		DebugID = FCameraSystemDebugID();
	}
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
}

void FCameraSystemEvaluator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(RootNode);
	ContextStack.AddReferencedObjects(Collector);
	RootNodeResult.AddReferencedObjects(Collector);
	if (RootEvaluator)
	{
		RootEvaluator->AddReferencedObjects(Collector);
	}
	for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
	{
		EvaluationService->AddReferencedObjects(Collector);
	}
	if (CameraRigCombinationRegistry)
	{
		CameraRigCombinationRegistry->AddReferencedObjects(Collector);
	}
}

void FCameraSystemEvaluator::PushEvaluationContext(TSharedRef<FCameraEvaluationContext> EvaluationContext)
{
	ContextStack.PushContext(EvaluationContext);
}

void FCameraSystemEvaluator::RemoveEvaluationContext(TSharedRef<FCameraEvaluationContext> EvaluationContext)
{
	ContextStack.RemoveContext(EvaluationContext);
}

void FCameraSystemEvaluator::PopEvaluationContext()
{
	ContextStack.PopContext();
}

void FCameraSystemEvaluator::RegisterEvaluationService(TSharedRef<FCameraEvaluationService> EvaluationService)
{
	EvaluationServices.Add(EvaluationService);
	{
		FCameraEvaluationServiceInitializeParams InitParams;
		InitParams.Evaluator = this;
		EvaluationService->Initialize(InitParams);
	}
}

void FCameraSystemEvaluator::UnregisterEvaluationService(TSharedRef<FCameraEvaluationService> EvaluationService)
{
	{
		FCameraEvaluationServiceTeardownParams TeardownParams;
		TeardownParams.Evaluator = this;
		EvaluationService->Teardown(TeardownParams);
	}
	EvaluationServices.Remove(EvaluationService);
}

void FCameraSystemEvaluator::GetEvaluationServices(TArray<TSharedPtr<FCameraEvaluationService>>& OutEvaluationServices) const
{
	OutEvaluationServices = EvaluationServices;
}

TSharedPtr<FCameraEvaluationService> FCameraSystemEvaluator::FindEvaluationService(const FCameraObjectTypeID& TypeID) const
{
	for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
	{
		if (EvaluationService.Get()->IsKindOf(TypeID))
		{
			return EvaluationService;
		}
	}
	return nullptr;
}

void FCameraSystemEvaluator::Update(const FCameraSystemEvaluationParams& Params)
{
	UpdateImpl(Params.DeltaTime, ECameraNodeEvaluationType::Standard);
}

void FCameraSystemEvaluator::UpdateImpl(float DeltaTime, ECameraNodeEvaluationType EvaluationType)
{
	SCOPE_CYCLE_COUNTER(CameraSystemEval_Total);

	// Reset our result' flags. Don't reset the result itself yet, since we want to return
	// last frame's values when we don't have anything to run.
	RootNodeResult.ResetFrameFlags();

	// Reset variables and data.
	RootNodeResult.VariableTable.AutoResetValues();
	RootNodeResult.ContextDataTable.AutoResetValues();

	// Pre-update all services.
	PreUpdateServices(DeltaTime, EvaluationType);

	// Get the active evaluation context.
	TSharedPtr<FCameraEvaluationContext> ActiveContext = ContextStack.GetActiveContext();
	if (UNLIKELY(!ActiveContext.IsValid()))
	{
		Result.bIsValid = false;
		PreVisualResult.bIsValid = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
		BuildDebugBlocksIfNeeded();
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

		ContextStack.OnEndCameraSystemUpdate();

		return;
	}

	// Run the camera director, and activate any camera rig(s) it returns to us.
	FCameraDirectorEvaluator* ActiveDirectorEvaluator = ActiveContext->GetDirectorEvaluator();
	if (ActiveDirectorEvaluator)
	{
		UpdateCameraDirector(DeltaTime, ActiveDirectorEvaluator);
	}

	PostCameraDirectorUpdateServices(DeltaTime, EvaluationType);

	// Run the camera node tree.
	{
		FCameraNodeEvaluationParams NodeParams;
		NodeParams.Evaluator = this;
		NodeParams.DeltaTime = DeltaTime;
		NodeParams.EvaluationType = EvaluationType;

		RootNodeResult.Reset();

		RootEvaluator->Run(NodeParams, RootNodeResult);

		RootNodeResult.bIsValid = true;
	}

	// Post-update all services.
	PostUpdateServices(DeltaTime, EvaluationType);

	// Harvest the result.
	PreVisualResult.Reset(RootEvaluator->GetPreVisualLayerResult());
	Result.Reset(RootNodeResult);

	// Generate debug information if needed.
#if UE_GAMEPLAY_CAMERAS_DEBUG
	BuildDebugBlocksIfNeeded();
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// End of update things...
	ContextStack.OnEndCameraSystemUpdate();
}

void FCameraSystemEvaluator::UpdateCameraDirector(float DeltaTime, FCameraDirectorEvaluator* CameraDirectorEvaluator)
{
	check(CameraDirectorEvaluator);

	FCameraDirectorEvaluationResult DirectorResult;
	{
		FCameraDirectorEvaluationParams DirectorParams;
		DirectorParams.DeltaTime = DeltaTime;

		CameraDirectorEvaluator->Run(DirectorParams, DirectorResult);
	}

	TArray<FCameraRigActivationDeactivationRequest, TInlineAllocator<2>> MainLayerActivations;
	TArray<FCameraRigActivationDeactivationRequest, TInlineAllocator<2>> MainLayerDeactivations;

	for (FCameraRigActivationDeactivationRequest& Request : DirectorResult.Requests)
	{
		Request.ResolveCameraRigProxyIfNeeded(CameraDirectorEvaluator);
		if (!Request.IsValid())
		{
			continue;
		}

		// Put the main layer requests aside while we handle the other requests.
		if (Request.Layer == ECameraRigLayer::Main)
		{
			if (Request.RequestType == ECameraRigActivationDeactivationRequestType::Activate)
			{
				MainLayerActivations.Add(Request);
			}
			else if (Request.RequestType == ECameraRigActivationDeactivationRequestType::Deactivate)
			{
				MainLayerDeactivations.Add(Request);
			}
		}
		else
		{
			RootEvaluator->ExecuteCameraDirectorRequest(Request);
		}
	}

	if (MainLayerActivations.Num() == 1)
	{
		RootEvaluator->ExecuteCameraDirectorRequest(MainLayerActivations[0]);
	}
	else if (MainLayerActivations.Num() > 1)
	{
		FCameraRigActivationDeactivationRequest CombinedRequest;
		CombinedRequest.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
		GetCombinedCameraRigRequest(MainLayerActivations, CombinedRequest);

		RootEvaluator->ExecuteCameraDirectorRequest(CombinedRequest);
	}

	if (MainLayerDeactivations.Num() == 1)
	{
		RootEvaluator->ExecuteCameraDirectorRequest(MainLayerDeactivations[0]);
	}
	else if (MainLayerDeactivations.Num() > 1)
	{
		FCameraRigActivationDeactivationRequest CombinedRequest;
		CombinedRequest.RequestType = ECameraRigActivationDeactivationRequestType::Deactivate;
		GetCombinedCameraRigRequest(MainLayerDeactivations, CombinedRequest);

		RootEvaluator->ExecuteCameraDirectorRequest(CombinedRequest);
	}
}

void FCameraSystemEvaluator::GetCombinedCameraRigRequest(TConstArrayView<FCameraRigActivationDeactivationRequest> Requests, FCameraRigActivationDeactivationRequest& OutCombinedRequest)
{
	// We have a combination of camera rigs to activate. Let's dynamically generate a new camera rig
	// asset that combines them.
#if WITH_EDITOR
	const UGameplayCamerasSettings* Settings = GetDefault<UGameplayCamerasSettings>();
	if (Requests.Num() > Settings->CombinedCameraRigNumThreshold)
	{
		UE_LOGF(LogCameraSystem, Warning, 
				"Activating %d camera rigs combined! Is the camera director doing this on purpose? "
					"If so, raise the CombinedCameraRigNumThreshold setting to remove this warning.",
				Requests.Num());
	}
#endif

	// All combined camera rigs must belong to the same evaluation context, and we can't have more
	// than one transition override.
	TArray<const UCameraRigAsset*> Combination;
	TSharedPtr<const FCameraEvaluationContext> CommonContext = Requests[0].EvaluationContext;
	const UCameraRigTransition* FirstTransitionOverride = nullptr;
	bool bAnyForceActivationDeactivation = false;
	for (const FCameraRigActivationDeactivationRequest& Request : Requests)
	{
		Combination.Add(Request.CameraRig);
		ensureMsgf(Request.EvaluationContext == CommonContext,
				TEXT("All combined camera rigs must be activated from the same evaluation context."));

		if (Request.TransitionOverride)
		{
			if (ensureMsgf(FirstTransitionOverride == nullptr || FirstTransitionOverride == Request.TransitionOverride,
					TEXT("Only one transition override can be specified when activating/deactivating multiple main-layer rigs.")))
			{
				FirstTransitionOverride = Request.TransitionOverride;
			}
		}

		bAnyForceActivationDeactivation |= Request.bForceActivateDeactivate;
	}

	const UCameraRigAsset* CombinedCameraRig = CameraRigCombinationRegistry->FindOrCreateCombination(Combination);

	OutCombinedRequest.EvaluationContext = CommonContext;
	OutCombinedRequest.CameraRig = CombinedCameraRig;
	OutCombinedRequest.TransitionOverride = FirstTransitionOverride;
	OutCombinedRequest.bForceActivateDeactivate = bAnyForceActivationDeactivation;
}

void FCameraSystemEvaluator::PreUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType)
{
	FCameraEvaluationServiceUpdateParams ServiceUpdateParams;
	ServiceUpdateParams.Evaluator = this;
	ServiceUpdateParams.DeltaTime = DeltaTime;
	ServiceUpdateParams.EvaluationType = EvaluationType;

	FCameraEvaluationServiceUpdateResult ServiceUpdateResult(RootNodeResult);

	for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
	{
		if (EvaluationService->HasAllEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPreUpdate))
		{
			EvaluationService->PreUpdate(ServiceUpdateParams, ServiceUpdateResult);
		}
	}
}

void FCameraSystemEvaluator::PostCameraDirectorUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType)
{
	FCameraEvaluationServiceUpdateParams ServiceUpdateParams;
	ServiceUpdateParams.Evaluator = this;
	ServiceUpdateParams.DeltaTime = DeltaTime;
	ServiceUpdateParams.EvaluationType = EvaluationType;

	FCameraEvaluationServiceUpdateResult ServiceUpdateResult(RootNodeResult);

	for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
	{
		if (EvaluationService->HasAllEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPostCameraDirectorUpdate))
		{
			EvaluationService->PostCameraDirectorUpdate(ServiceUpdateParams, ServiceUpdateResult);
		}
	}
}

void FCameraSystemEvaluator::PostUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType)
{
	FCameraEvaluationServiceUpdateParams ServiceUpdateParams;
	ServiceUpdateParams.Evaluator = this;
	ServiceUpdateParams.DeltaTime = DeltaTime;
	ServiceUpdateParams.EvaluationType = EvaluationType;

	FCameraEvaluationServiceUpdateResult ServiceUpdateResult(RootNodeResult);

	for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
	{
		if (EvaluationService->HasAllEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPostUpdate))
		{
			EvaluationService->PostUpdate(ServiceUpdateParams, ServiceUpdateResult);
		}
	}
}

void FCameraSystemEvaluator::ViewRotationPreviewUpdate(const FCameraSystemEvaluationParams& Params, FCameraSystemViewRotationEvaluationResult& OutResult)
{
	SCOPE_CYCLE_COUNTER(CameraSystemEval_Total);

	EvaluatorSnapshot.Reset();

	FMemoryWriter Writer(EvaluatorSnapshot);
	FCameraNodeEvaluatorHierarchy CameraSystemHierarchy(RootEvaluator);

	FCameraNodeEvaluatorSerializeParams SerializeParams;
	CameraSystemHierarchy.CallSerialize(SerializeParams, Writer);

	{
		FCameraNodeEvaluationParams NodeParams;
		NodeParams.Evaluator = this;
		NodeParams.DeltaTime = Params.DeltaTime;
		NodeParams.EvaluationType = ECameraNodeEvaluationType::ViewRotationPreview;

		RootNodeResult.Reset();

		RootEvaluator->Run(NodeParams, RootNodeResult);

		const FRotator3d& PreviewRotation = RootNodeResult.CameraPose.GetRotation();
		OutResult.DeltaRotation += (PreviewRotation - OutResult.ViewRotation).GetNormalized();
	}

	FMemoryReader Reader(EvaluatorSnapshot);
	CameraSystemHierarchy.CallSerialize(SerializeParams, Reader);
}

void FCameraSystemEvaluator::GetEvaluatedCameraView(FMinimalViewInfo& DesiredView)
{
	RootNodeResult.GetViewInfo(DesiredView);
}

void FCameraSystemEvaluator::ExecuteOperation(FCameraOperation& Operation)
{
	FCameraOperationParams Params;
	Params.Evaluator = this;
	RootEvaluator->ExecuteOperation(Params, Operation);
}

#if WITH_EDITOR

void FCameraSystemEvaluator::EditorPreviewUpdate(const FCameraSystemEvaluationParams& Params)
{
	UpdateImpl(Params.DeltaTime, ECameraNodeEvaluationType::EditorPreview);
}

void FCameraSystemEvaluator::DrawEditorPreview(const FCameraSystemEditorPreviewParams& Params)
{	
	FCameraEditorPreviewDrawParams NodeParams;

	UObject* Owner = WeakOwner.Get();
	UWorld* OwnerWorld = (Owner && Params.bDrawWorldDebug) ? Owner->GetWorld() : nullptr;
	FCameraDebugRenderer Renderer(OwnerWorld, Params.SceneView, Params.Canvas, !Params.bIsLockedToCamera);

	Renderer.BeginDrawing();

	if (RootDebugBlock)
	{
		FRootCameraDebugDrawParams RootParams;
		RootDebugBlock->RootDebugDraw(RootParams, Renderer);
	}
	else
	{
		RootEvaluator->DrawEditorPreview(NodeParams, Renderer);

		if (!Params.bIsLockedToCamera)
		{
			const FLinearColor TrailColor(FColorList::LightBlue);
			TConstArrayView<FVector3d> Trail = RootNodeResult.GetCameraPoseLocationTrail();
			for (int32 Index = 1; Index < Trail.Num(); ++Index)
			{
				const FVector3d& PrevPoint(Trail[Index - 1]);
				const FVector3d& NextPoint(Trail[Index]);
				Renderer.DrawLine(PrevPoint, NextPoint, TrailColor, 1.f);
			}
		}
	}
	Renderer.EndDrawing();
}

#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG

bool FCameraSystemEvaluator::IsDebugTraceEnabled()
{
#if UE_GAMEPLAY_CAMERAS_TRACE
	return FCameraSystemTrace::IsTraceEnabled();
#else
	return false;
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
}

bool FCameraSystemEvaluator::ShouldBuildOrDrawDebugBlocks()
{
	const bool bTraceEnabled = IsDebugTraceEnabled();
	return bTraceEnabled || GGameplayCamerasDebugEnable;
}

void FCameraSystemEvaluator::BuildDebugBlocksIfNeeded()
{
	if (!ShouldBuildOrDrawDebugBlocks())
	{
		RootDebugBlock = nullptr;
		return;
	}

	// Clear previous frame's debug info and make room for this frame's.
	DebugBlockStorage.DestroyDebugBlocks();

	// Create the root debug block and start building more.
	RootDebugBlock = DebugBlockStorage.BuildDebugBlock<FRootCameraDebugBlock>();

	FCameraDebugBlockBuildParams BuildParams;
	FCameraDebugBlockBuilder DebugBlockBuilder(DebugBlockStorage, *RootDebugBlock);
	RootDebugBlock->BuildDebugBlocks(*this, BuildParams, DebugBlockBuilder);
}

void FCameraSystemEvaluator::DebugUpdate(const FCameraSystemDebugUpdateParams& Params)
{
	if (!ShouldBuildOrDrawDebugBlocks() || !RootDebugBlock)
	{
		return;
	}

	UObject* Owner = WeakOwner.Get();
	UWorld* OwnerWorld = Owner ? Owner->GetWorld() : nullptr;

#if UE_GAMEPLAY_CAMERAS_TRACE
	if (IsDebugTraceEnabled())
	{
		FCameraSystemTrace::TraceEvaluation(OwnerWorld, Result, *RootDebugBlock);
	}
#endif

	FRootCameraDebugDrawParams RootParams;
	RootParams.bFromDefaultCameraSystem = Params.bIsDefaultCameraSystem;
	RootParams.bForceDraw = Params.bForceDraw;
	FCameraDebugRenderer Renderer(OwnerWorld, Params.CanvasObject, Params.bIsDebugCameraEnabled);
	RootDebugBlock->RootDebugDraw(RootParams, Renderer);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

