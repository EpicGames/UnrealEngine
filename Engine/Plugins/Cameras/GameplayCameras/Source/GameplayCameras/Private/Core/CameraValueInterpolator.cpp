// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraValueInterpolator)

TUniquePtr<UE::Cameras::TCameraValueInterpolator<double>> UCameraValueInterpolator::BuildDoubleInterpolator() const
{
	return OnBuildDoubleInterpolator();
}

TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector2d>> UCameraValueInterpolator::BuildVector2dInterpolator() const
{
	return OnBuildVector2dInterpolator();
}

TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector3d>> UCameraValueInterpolator::BuildVector3dInterpolator() const
{
	return OnBuildVector3dInterpolator();
}

UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(UPopValueInterpolator, UE::Cameras::TPopValueInterpolator)

