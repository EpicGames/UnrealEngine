// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableAssets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableAssets)

UCameraVariableAsset::UCameraVariableAsset(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

FCameraVariableID  UCameraVariableAsset::GetVariableID() const
{
	ensure(Guid.IsValid());
	return FCameraVariableID::FromHashValue(GetTypeHash(Guid));
}

FCameraVariableDefinition UCameraVariableAsset::GetVariableDefinition() const
{
	FCameraVariableDefinition VariableDefinition;
	VariableDefinition.VariableID = GetVariableID();
	VariableDefinition.VariableType = GetVariableType();
	VariableDefinition.bIsPrivate = bIsPrivate;
	VariableDefinition.bIsInput = bIsPreBlended;
	VariableDefinition.bAutoReset = bAutoReset;
#if WITH_EDITORONLY_DATA
	VariableDefinition.VariableName = GetDisplayName();
#endif
	return VariableDefinition;
}

void UCameraVariableAsset::PostLoad()
{
	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}

	Super::PostLoad();
}

void UCameraVariableAsset::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && 
			!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}

void UCameraVariableAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		Guid = FGuid::NewGuid();
	}
}

#if WITH_EDITORONLY_DATA

FString UCameraVariableAsset::GetDisplayName() const
{
	if (!DisplayName.IsEmpty())
	{
		return DisplayName;
	}
	return GetName();
}

#endif  // WITH_EDITORONLY_DATA

#if WITH_EDITOR

FText UCameraVariableAsset::GetDisplayText() const
{
	if (!DisplayName.IsEmpty())
	{
		return FText::FromString(DisplayName);
	}
	return FText::FromName(GetFName());
}

#endif  // WITH_EDITOR

