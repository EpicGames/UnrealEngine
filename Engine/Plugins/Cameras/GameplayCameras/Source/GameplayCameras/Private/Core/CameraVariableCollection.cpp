// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableCollection.h"

#include "Core/CameraVariableAssets.h"
#include "GameplayCameras.h"
#include "UObject/ObjectRedirector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableCollection)

UCameraVariableCollection::UCameraVariableCollection(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UCameraVariableCollection::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	for (UCameraVariableAsset* Variable : Variables)
	{
		if (!Variable->HasAnyFlags(RF_Public))
		{
			UE_LOGF(LogCameraSystem, Warning, "Adding missing RF_Public flag on variable '%ls'.", *GetPathNameSafe(Variable));
			Variable->SetFlags(RF_Public);
		}
	}

	CleanUpStrayObjects();
#endif  // WITH_EDITOR
}

#if WITH_EDITOR

void UCameraVariableCollection::CleanUpStrayObjects()
{
	UPackage* CollectionPackage = GetOutermost();
	if (!CollectionPackage || CollectionPackage == GetTransientPackage())
	{
		return;
	}

	TSet<UObject*> StrayObjects;
	TSet<UCameraVariableAsset*> KnownVariables(Variables);

	TArray<UObject*> ObjectsInPackage;
	GetObjectsWithPackage(CollectionPackage, ObjectsInPackage);
	for (UObject* Object : ObjectsInPackage)
	{
		UCameraVariableAsset* Variable = Cast<UCameraVariableAsset>(Object);
		if (!Variable)
		{
			continue;
		}
		if (KnownVariables.Contains(Variable))
		{
			continue;
		}

		Variable->ClearFlags(RF_Public | RF_Standalone);
		StrayObjects.Add(Variable);
	}

	if (StrayObjects.Num() > 0)
	{
		// Also clean-up any redirectors to these objects.
		for (UObject* Object : ObjectsInPackage)
		{
			if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
			{
				if (StrayObjects.Contains(Redirector->DestinationObject))
				{
					Redirector->ClearFlags(RF_Public | RF_Standalone);
					Redirector->DestinationObject = nullptr;
				}
			}
		}

		UE_LOGF(LogCameraSystem, Warning,
				"Cleaned up %d stray camera variables in camera variable collection '%ls'. Please resave the asset.",
				StrayObjects.Num(), *GetPathNameSafe(this));
	}
}

#endif  // WITH_EDITOR
