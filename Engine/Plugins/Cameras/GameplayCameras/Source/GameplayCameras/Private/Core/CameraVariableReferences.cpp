// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableReferences.h"

#include "Core/CameraVariableTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableReferences)

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
const ValueType* F##ValueName##CameraVariableReference::GetValue(const UE::Cameras::FCameraVariableTable& VariableTable) const\
{\
	if (VariableID)\
	{\
		if (const ValueType* ActualValue = VariableTable.FindValue<ValueType>(VariableID))\
		{\
			return ActualValue;\
		}\
	}\
	return nullptr;\
}
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

