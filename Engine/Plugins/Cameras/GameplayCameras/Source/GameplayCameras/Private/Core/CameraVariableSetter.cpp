// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableSetter.h"

#include "Math/Interpolation.h"
#include "Math/UnrealMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableSetter)

namespace UE::Cameras
{

void FCameraVariableSetter::Update(float DeltaTime)
{
	UpdateState(DeltaTime);
}

void FCameraVariableSetter::Stop(bool bImmediately)
{
	EState PrevState = State;
	if (PrevState != EState::Inactive)
	{
		if (!bImmediately)
		{
			State = EState::BlendOut;
			if (PrevState == EState::Full)
			{
				CurrentTime = 0.f;
			}
			else if (PrevState == EState::BlendIn)
			{
				// TODO: this assumes a symmetrical blend curve, which may not always be true.
				const float BlendInPercent = (BlendInTime > 0.f) ? FMath::Clamp(CurrentTime / BlendInTime, 0.f, 1.f) : 1.f;
				CurrentTime = (1.f - BlendInPercent) * BlendOutTime;
			}
			// else: was blending out already.
		}
		else
		{
			State = EState::Inactive;
		}
	}
	// else: we were already stopped.
}

void FCameraVariableSetter::UpdateState(float DeltaTime)
{
	if (State == EState::BlendIn || State == EState::BlendOut)
	{
		const float NewTime = CurrentTime + DeltaTime;
		if (State == EState::BlendIn)
		{
			if (NewTime < BlendInTime)
			{
				CurrentTime = NewTime;
			}
			else
			{
				State = EState::Full;
				CurrentTime = BlendInTime;
			}
		}
		else if (State == EState::BlendOut)
		{
			if (NewTime < BlendOutTime)
			{
				CurrentTime = NewTime;
			}
			else
			{
				State = EState::Inactive;
				CurrentTime = BlendOutTime;
			}
		}
	}
}

float FCameraVariableSetter::GetBlendFactor()
{
	float BlendPercent = 0.f;
	switch (State)
	{
		case EState::BlendIn:
			BlendPercent = (BlendInTime > 0.f) ? (CurrentTime / BlendInTime) : 1.f;
			break;
		case EState::Full:
			BlendPercent = 1.f;
			break;
		case EState::BlendOut:
			BlendPercent = 1.f - ((BlendOutTime > 0.f) ? (CurrentTime / BlendOutTime) : 1.f);
			break;
		case EState::Inactive:
		default:
			break;
	}

	BlendPercent = FMath::Clamp(BlendPercent, 0.f, 1.f);
	switch (BlendType)
	{
		case ECameraVariableSetterBlendType::None:
			return BlendPercent >= 1.f ? 1.f : 0.f;
		case ECameraVariableSetterBlendType::Linear:
			return BlendPercent;
		case ECameraVariableSetterBlendType::SmoothStep:
			return SmoothStep(BlendPercent);
		case ECameraVariableSetterBlendType::SmootherStep:
			return SmootherStep(BlendPercent);
		default:
			return 1.f;
	}
}

}  // namespace UE::Cameras

