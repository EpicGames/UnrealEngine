// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableTable.h"

#include "HAL/UnrealMemory.h"
#include "IGameplayCamerasModule.h"
#include "Math/UnrealMath.h"

namespace UE::Cameras
{

namespace Private
{

static const FString GUnavailableVariableDebugName(TEXT("<no debug info>"));

bool IsVariableInMask(FCameraVariableID VariableID, const FCameraVariableTableFlags* InMask, bool bInvertMask)
{
	if (InMask)
	{
		const bool bIsInMask = InMask->VariableIDs.Contains(VariableID);
		return bInvertMask ? !bIsInMask : bIsInMask;
	}
	return true;
}

}  // namespace Private

const uint32 FCameraVariableTable::GDefaultCapacity = 64;
const uint32 FCameraVariableTable::GDefaultAlignment = alignof(void*);

TArray<FBlendableStructInfo> FCameraVariableTable::CachedBlendableStructs;
bool FCameraVariableTable::bCachedBlendableStructs(false);

FCameraVariableTable::FCameraVariableTable()
{
}

FCameraVariableTable::FCameraVariableTable(FCameraVariableTable&& Other)
	: Entries(MoveTemp(Other.Entries))
	, EntryLookup(MoveTemp(Other.EntryLookup))
	, Memory(Other.Memory)
	, Capacity(Other.Capacity)
	, Alignment(Other.Alignment)
	, Used(Other.Used)
{
	Other.Memory = nullptr;
	Other.Capacity = 0;
	Other.Alignment = 0;
	Other.Used = 0;
}

FCameraVariableTable& FCameraVariableTable::operator=(FCameraVariableTable&& Other)
{
	Entries = MoveTemp(Other.Entries);
	EntryLookup = MoveTemp(Other.EntryLookup);
	Memory = Other.Memory;
	Capacity = Other.Capacity;
	Alignment = Other.Alignment;
	Used = Other.Used;

	Other.Memory = nullptr;
	Other.Capacity = 0;
	Other.Alignment = 0;
	Other.Used = 0;

	return *this;
}

FCameraVariableTable::~FCameraVariableTable()
{
	if (Memory)
	{
		FMemory::Free(Memory);
		Memory = nullptr;
		Capacity = Alignment = Used = 0;
	}
}

void FCameraVariableTable::Initialize(const FCameraVariableTableAllocationInfo& AllocationInfo)
{
	// Reset any previous state.
	Entries.Reset();
	EntryLookup.Reset();
	if (Memory)
	{
		FMemory::Free(Memory);
		Memory = nullptr;
	}

	// Compute the total buffer size we need, and create our entries as we go.
	uint32 TotalSizeOf = 0;
	uint32 MaxAlignOf = 0;
	uint32 CurSizeOf, CurAlignOf;
	for (const FCameraVariableDefinition& VariableDefinition : AllocationInfo.VariableDefinitions)
	{
		GetVariableTypeAllocationInfo(VariableDefinition.VariableType, VariableDefinition.BlendableStructType, CurSizeOf, CurAlignOf);
		const uint32 NewEntryOffset = Align(TotalSizeOf, CurAlignOf);
		TotalSizeOf = NewEntryOffset + CurSizeOf;
		MaxAlignOf = FMath::Max(MaxAlignOf, CurAlignOf);

		FEntry NewEntry;
		CreateEntry(VariableDefinition, NewEntry);
		NewEntry.Offset = NewEntryOffset;
		Entries.Add(NewEntry);
		EntryLookup.Add(NewEntry.ID, Entries.Num() - 1);
	}

	// Allocate the memory buffer.
	MaxAlignOf = FMath::Max(GDefaultAlignment, MaxAlignOf);
	Memory = reinterpret_cast<uint8*>(FMemory::Malloc(TotalSizeOf, MaxAlignOf));
	Capacity = TotalSizeOf;
	Alignment = MaxAlignOf;
	Used = TotalSizeOf;

	// Go back to our entries and initialize each entry to the default value for that variable type.
	InitializeEntries();
}

void FCameraVariableTable::CreateEntry(const FCameraVariableDefinition& VariableDefinition, FEntry& OutEntry)
{
	OutEntry.ID = VariableDefinition.VariableID;
	OutEntry.Type = VariableDefinition.VariableType;
	OutEntry.StructType = VariableDefinition.BlendableStructType;
	OutEntry.Offset = 0;
	OutEntry.Flags = EEntryFlags::None;
	if (VariableDefinition.bIsPrivate)
	{
		OutEntry.Flags |= EEntryFlags::Private;
	}
	if (VariableDefinition.bIsInput)
	{
		OutEntry.Flags |= EEntryFlags::Input;
	}
	if (VariableDefinition.bAutoReset)
	{
		OutEntry.Flags |= EEntryFlags::AutoReset;
	}
#if WITH_EDITORONLY_DATA
	OutEntry.DebugName = VariableDefinition.VariableName;
#endif
}

void FCameraVariableTable::InitializeEntries(int32 StartIndex, int32 EndIndex)
{
	EndIndex = (EndIndex >= 0 ? EndIndex : Entries.Num());
	for (int32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		const FEntry& Entry = Entries[Index];
		uint8* ValuePtr = Memory + Entry.Offset;
		switch (Entry.Type)
		{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			case ECameraVariableType::ValueName:\
				{\
					ValueType* TypedValuePtr = reinterpret_cast<ValueType*>(ValuePtr);\
					*TypedValuePtr = ValueType();\
				}\
				break;
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			case ECameraVariableType::BlendableStruct:
				{
					check(Entry.StructType);
					Entry.StructType->InitializeStruct(ValuePtr);
				}
				break;
		}
	}
}

bool FCameraVariableTable::EnsureVariables(const FCameraVariableTableAllocationInfo& AllocationInfo)
{
	uint32 TotalSizeOf = Used;
	uint32 MaxAlignOf = 0;
	uint32 CurSizeOf, CurAlignOf;

	bool bAllExist = true;
	int32 FirstNewEntry = Entries.Num();

	for (const FCameraVariableDefinition& VariableDefinition : AllocationInfo.VariableDefinitions)
	{
		if (EntryLookup.Contains(VariableDefinition.VariableID))
		{
			continue;
		}

		GetVariableTypeAllocationInfo(VariableDefinition.VariableType, VariableDefinition.BlendableStructType, CurSizeOf, CurAlignOf);
		const uint32 NewEntryOffset = Align(TotalSizeOf, CurAlignOf);
		TotalSizeOf = NewEntryOffset + CurSizeOf;
		MaxAlignOf = FMath::Max(MaxAlignOf, CurAlignOf);

		FEntry NewEntry;
		CreateEntry(VariableDefinition, NewEntry);
		NewEntry.Offset = NewEntryOffset;
		Entries.Add(NewEntry);
		EntryLookup.Add(NewEntry.ID, Entries.Num() - 1);

		bAllExist = false;
	}

	if (!bAllExist)
	{
		// Alignment may not have been initialized yet.
		Alignment = FMath::Max(Alignment, MaxAlignOf);

		if (TotalSizeOf > Capacity)
		{
			ReallocateBuffer(TotalSizeOf);
		}

		Used = TotalSizeOf;

		InitializeEntries(FirstNewEntry);
	}

	return bAllExist;
}

void FCameraVariableTable::CacheBlendableStructs()
{
	if (bCachedBlendableStructs)
	{
		return;
	}

	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	CachedBlendableStructs = GameplayCamerasModule.GetBlendableStructs();
	bCachedBlendableStructs = true;
}

FBlendableStructTypeErasedInterpolator FCameraVariableTable::GetBlendableStructInterpolator(const UScriptStruct* StructType)
{
	CacheBlendableStructs();

	for (const FBlendableStructInfo& Info : CachedBlendableStructs)
	{
		if (Info.StructType == StructType)
		{
			return Info.Interpolator;
		}
	}
	return nullptr;
}

void FCameraVariableTable::AddVariable(const FCameraVariableDefinition& VariableDefinition)
{
	uint32 SizeOf, AlignOf;
	GetVariableTypeAllocationInfo(VariableDefinition.VariableType, VariableDefinition.BlendableStructType, SizeOf, AlignOf);

	uint8* VariablePtr = Align(Memory + Used, AlignOf);
	uint32 NewUsed = (VariablePtr + SizeOf) - Memory;

	if (NewUsed > Capacity)
	{
		// Alignment may not have been initialized yet.
		Alignment = FMath::Max(AlignOf, GDefaultAlignment);

		ReallocateBuffer(NewUsed);

		VariablePtr = Align(Memory + Used, AlignOf);
	}

	Used = NewUsed;

	FEntry NewEntry;
	CreateEntry(VariableDefinition, NewEntry);
	NewEntry.Offset = VariablePtr - Memory;
	Entries.Add(NewEntry);
	EntryLookup.Add(VariableDefinition.VariableID, Entries.Num() - 1);

	InitializeEntries(Entries.Num() - 1);
}

void FCameraVariableTable::ReallocateBuffer(uint32 MinRequired)
{
	uint32 NewCapacity = Capacity <= 0 ? GDefaultCapacity : Capacity * 2;
	if (MinRequired > 0)
	{
		NewCapacity = FMath::Max(NewCapacity, MinRequired);
	}

	uint8* OldMemory = Memory;
	uint8* NewMemory = reinterpret_cast<uint8*>(FMemory::Malloc(NewCapacity, Alignment));

	if (OldMemory)
	{
		FMemory::Memmove(NewMemory, OldMemory, Capacity);
		FMemory::Free(OldMemory);
	}

	Memory = NewMemory;
	Capacity = NewCapacity;
}

FCameraVariableTable::FEntry* FCameraVariableTable::FindEntry(FCameraVariableID VariableID)
{
	const int32* IndexPtr = EntryLookup.Find(VariableID);
	return IndexPtr ? &Entries[*IndexPtr] : nullptr;
}

const FCameraVariableTable::FEntry* FCameraVariableTable::FindEntry(FCameraVariableID VariableID) const
{
	const int32* IndexPtr = EntryLookup.Find(VariableID);
	return IndexPtr ? &Entries[*IndexPtr] : nullptr;
}

bool FCameraVariableTable::GetVariableTypeAllocationInfo(ECameraVariableType VariableType, const UScriptStruct* StructType, uint32& OutSizeOf, uint32& OutAlignOf)
{
	switch (VariableType)
	{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		case ECameraVariableType::ValueName:\
			OutSizeOf = sizeof(ValueType);\
			OutAlignOf = alignof(ValueType);\
			return true;
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
		case ECameraVariableType::BlendableStruct:
			{
				check(StructType);
				const UScriptStruct::ICppStructOps* StructOps = StructType->GetCppStructOps();
				OutSizeOf = StructOps->GetSize();
				OutAlignOf = StructOps->GetAlignment();
			}
			return true;
	}
	return false;
}

bool FCameraVariableTable::ContainsValue(FCameraVariableID VariableID) const
{
	return EntryLookup.Contains(VariableID);
}

const uint8* FCameraVariableTable::GetValue(
		FCameraVariableID VariableID,
		ECameraVariableType ExpectedVariableType,
		const UScriptStruct* ExpectedBlendableStructType,
		bool bOnlyIfWritten) const
{
	const uint8* Value = TryGetValue(VariableID, ExpectedVariableType, ExpectedBlendableStructType, bOnlyIfWritten);
	ensureMsgf(Value, TEXT("Can't get camera variable (ID '%d') because it doesn't exist in the table."), VariableID.GetValue());
	return Value;
}

const uint8* FCameraVariableTable::TryGetValue(
		FCameraVariableID VariableID,
		ECameraVariableType ExpectedVariableType,
		const UScriptStruct* ExpectedBlendableStructType,
		bool bOnlyIfWritten) const
{
	const FEntry* Entry = FindEntry(VariableID);
	if (Entry)
	{
		ensure(Entry->Type == ExpectedVariableType && Entry->StructType == ExpectedBlendableStructType);
		if (!bOnlyIfWritten || EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written))
		{
			return Memory + Entry->Offset;
		}
	}

	return nullptr;
}

uint8* FCameraVariableTable::TryGetMutableValue(
		FCameraVariableID VariableID,
		ECameraVariableType ExpectedVariableType,
		const UScriptStruct* ExpectedBlendableStructType,
		bool bOnlyIfWritten)
{
	FEntry* Entry = FindEntry(VariableID);
	if (Entry)
	{
		ensure(Entry->Type == ExpectedVariableType && Entry->StructType == ExpectedBlendableStructType);
		if (!bOnlyIfWritten || EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written))
		{
			return Memory + Entry->Offset;
		}
	}

	return nullptr;
}

void FCameraVariableTable::SetValue(
		FCameraVariableID VariableID, 
		ECameraVariableType ExpectedVariableType, 
		const UScriptStruct* ExpectedBlendableStructType,
		const uint8* InRawValuePtr,
		bool bMarkAsWrittenThisFrame)
{
	FEntry* Entry = FindEntry(VariableID);
	if (ensureMsgf(Entry, TEXT("Can't set camera variable (ID '%d') because it doesn't exist in the table."), VariableID.GetValue()))
	{
		check(ExpectedVariableType == Entry->Type && ExpectedBlendableStructType == Entry->StructType);
		uint32 SizeOf, AlignOf;
		GetVariableTypeAllocationInfo(Entry->Type, Entry->StructType, SizeOf, AlignOf);
		uint8* ValuePtr = Memory + Entry->Offset;
		FMemory::Memcpy(ValuePtr, InRawValuePtr, SizeOf);
		Entry->Flags |= EEntryFlags::Written;
		if (bMarkAsWrittenThisFrame)
		{
			Entry->Flags |= EEntryFlags::WrittenThisFrame;
		}
	}
}

bool FCameraVariableTable::TrySetValue(
		FCameraVariableID VariableID,
		ECameraVariableType ExpectedVariableType,
		const UScriptStruct* ExpectedBlendableStructType,
		const uint8* InRawValuePtr,
		bool bMarkAsWrittenThisFrame)
{
	if (FEntry* Entry = FindEntry(VariableID))
	{
		check(ExpectedVariableType == Entry->Type && ExpectedBlendableStructType == Entry->StructType);
		uint32 SizeOf, AlignOf;
		GetVariableTypeAllocationInfo(Entry->Type, Entry->StructType, SizeOf, AlignOf);
		uint8* ValuePtr = Memory + Entry->Offset;
		FMemory::Memcpy(ValuePtr, InRawValuePtr, SizeOf);
		Entry->Flags |= EEntryFlags::Written;
		if (bMarkAsWrittenThisFrame)
		{
			Entry->Flags |= EEntryFlags::WrittenThisFrame;
		}
		return true;
	}
	return false;
}

bool FCameraVariableTable::IsValueWritten(FCameraVariableID VariableID) const
{
	if (const FEntry* Entry = FindEntry(VariableID))
	{
		return EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written);
	}
	return false;
}

void FCameraVariableTable::UnsetValue(FCameraVariableID VariableID)
{
	if (FEntry* Entry = FindEntry(VariableID))
	{
		EnumRemoveFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

void FCameraVariableTable::UnsetAllValues()
{
	for (FEntry& Entry : Entries)
	{
		EnumRemoveFlags(Entry.Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
	}
}

bool FCameraVariableTable::IsValueWrittenThisFrame(FCameraVariableID VariableID) const
{
	if (const FEntry* Entry = FindEntry(VariableID))
	{
		return EnumHasAnyFlags(Entry->Flags, EEntryFlags::WrittenThisFrame);
	}
	return false;
}

void FCameraVariableTable::ClearAllWrittenThisFrameFlags()
{
	for (FEntry& Entry : Entries)
	{
		EnumRemoveFlags(Entry.Flags, EEntryFlags::WrittenThisFrame);
	}
}

void FCameraVariableTable::AutoResetValues()
{
	for (FEntry& Entry : Entries)
	{
		if (EnumHasAnyFlags(Entry.Flags, EEntryFlags::AutoReset))
		{
			EnumRemoveFlags(Entry.Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
		}
	}
}

bool FCameraVariableTable::TryGetVariableDefinition(FCameraVariableID VariableID, FCameraVariableDefinition& OutVariableDefinition) const
{
	if (const FEntry* Entry = FindEntry(VariableID))
	{
		OutVariableDefinition.VariableID = Entry->ID;
		OutVariableDefinition.VariableType = Entry->Type;
		OutVariableDefinition.BlendableStructType = Entry->StructType;
		OutVariableDefinition.bIsPrivate = EnumHasAnyFlags(Entry->Flags, EEntryFlags::Private);
		OutVariableDefinition.bIsInput = EnumHasAnyFlags(Entry->Flags, EEntryFlags::Input);
#if WITH_EDITORONLY_DATA
		OutVariableDefinition.VariableName = Entry->DebugName;
#endif  // WITH_EDITORONLY_DATA
		return true;
	}
	return false;
}

void FCameraVariableTable::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		Ar << Capacity;
		Ar << Used;
		Ar.Serialize(Memory, Used);

		int32 NumEntries = Entries.Num();
		Ar << NumEntries;
		Ar.Serialize(Entries.GetData(), Entries.Num() * sizeof(FEntry));
	}

	if (Ar.IsLoading())
	{
		// Load the data from the saved buffer. It should have a lesser-or-equal size to our buffer,
		// because we might have grown since we were saved, but we wouldn't have shrunk (variable tables
		// don't shrink). So we keep our present capacity and ignore the saved one, except to confirm
		// our expectations.
		uint32 LoadedCapacity = 0;
		Ar << LoadedCapacity;

		uint32 LoadedUsed = 0;
		Ar << LoadedUsed;

		ensureMsgf(LoadedCapacity <= Capacity, 
				TEXT("Loading camera variable table data from a bigger, and probably different, table."));

		Ar.Serialize(Memory, LoadedUsed);
		Used = LoadedUsed;

		// The number of entries may have changed, if new entries were created since we were saved.
		// Resize down to what we originally had.
		int32 LoadedNumEntries = 0;
		Ar << LoadedNumEntries;
		
		ensureMsgf(LoadedNumEntries <= Entries.Num(), 
				TEXT("Loading camera variable table entries from a bigger, and probably different, table."));
		Entries.SetNum(LoadedNumEntries, EAllowShrinking::No);
		Ar.Serialize(Entries.GetData(), LoadedNumEntries * sizeof(FEntry));

		// Rebuild the entry lookup table.
		EntryLookup.Reset();
		for (auto It = Entries.CreateConstIterator(); It; ++It)
		{
			EntryLookup.Add(It->ID, It.GetIndex());
		}
	}
}

void FCameraVariableTable::OverrideAll(const FCameraVariableTable& OtherTable, bool bIncludePrivateValues)
{
	const ECameraVariableTableFilter Filter = !bIncludePrivateValues ? 
		ECameraVariableTableFilter::PublicOnly : ECameraVariableTableFilter::None;
	InternalOverride(OtherTable, Filter, nullptr, false, nullptr);
}

void FCameraVariableTable::Override(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter)
{
	InternalOverride(OtherTable, Filter, nullptr, false, nullptr);
}

void FCameraVariableTable::Override(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter, const FCameraVariableTableFlags& InMask, bool bInvertMask, FCameraVariableTableFlags& OutMask)
{
	InternalOverride(OtherTable, Filter, &InMask, bInvertMask, &OutMask);
}

void FCameraVariableTable::InternalOverride(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter, const FCameraVariableTableFlags* InMask, bool bInvertMask, FCameraVariableTableFlags* OutMask)
{
	using namespace UE::Cameras::Private;

	const bool bPublicOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::PublicOnly);
	const bool bInputOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::InputOnly);
	const bool bKnownOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::KnownOnly);
	const bool bChangedOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::ChangedOnly);

	for (const FEntry& OtherEntry : OtherTable.Entries)
	{
		// Look for entries in the other table that have been written to, and aren't private.
		const EEntryFlags OtherFlags = OtherEntry.Flags;
		if (EnumHasAnyFlags(OtherFlags, EEntryFlags::Written)
				&& (!bPublicOnly || !EnumHasAnyFlags(OtherFlags, EEntryFlags::Private))
				&& (!bInputOnly || EnumHasAnyFlags(OtherFlags, EEntryFlags::Input))
				&& (!bChangedOnly || EnumHasAnyFlags(OtherFlags, EEntryFlags::WrittenThisFrame))
				&& IsVariableInMask(OtherEntry.ID, InMask, bInvertMask))
		{
			// See if we know this variable.
			FEntry* ThisEntry = FindEntry(OtherEntry.ID);
			if (ThisEntry)
			{
				// We already have the other table's variable in our table. Let's check
				// that the types match, and then copy the memory.
#if WITH_EDITORONLY_DATA
				ensureMsgf(ThisEntry->DebugName == OtherEntry.DebugName,
						TEXT("Camera variable name collision! Expected variable '%d' to be named '%s', but other table has '%s'!"),
						ThisEntry->ID.GetValue(), *ThisEntry->DebugName, *OtherEntry.DebugName);
#endif

#if WITH_EDITORONLY_DATA
				const FString& DebugName = ThisEntry->DebugName;
#else
				const FString& DebugName = GUnavailableVariableDebugName;
#endif
				if (!ensureMsgf(ThisEntry->Type == OtherEntry.Type && ThisEntry->StructType == OtherEntry.StructType, 
						TEXT("Camera variable name collision! Expected '%d' (%s) to be of type '%s' but other table has type '%s'!"),
						ThisEntry->ID.GetValue(), *DebugName,
						*UEnum::GetValueAsString(ThisEntry->Type), *UEnum::GetValueAsString(OtherEntry.Type)))
				{
					// Unexpected situation: two variables of different types sharing the same ID.
					continue;
				}
			}
			else if (!bKnownOnly)
			{
				// We don't have this variable in our table. Let's add it.
				FCameraVariableDefinition NewVariableDefinition;
				NewVariableDefinition.VariableID = OtherEntry.ID;
				NewVariableDefinition.VariableType = OtherEntry.Type;
				NewVariableDefinition.BlendableStructType = OtherEntry.StructType;
				NewVariableDefinition.bIsInput = EnumHasAllFlags(OtherEntry.Flags, EEntryFlags::Input);
#if WITH_EDITORONLY_DATA
				NewVariableDefinition.VariableName = OtherEntry.DebugName;
#endif
				AddVariable(NewVariableDefinition);

				ThisEntry = FindEntry(OtherEntry.ID);
			}
			else
			{
				// We don't have this variable in our table but we only want to override
				// known entries.
				continue;
			}

			if (ensure(ThisEntry))
			{
				uint32 ValueSize = 0, ValueAlignment = 0;
				GetVariableTypeAllocationInfo(ThisEntry->Type, ThisEntry->StructType, ValueSize, ValueAlignment);
				check(ValueSize != 0);

				uint8* ThisValuePtr = Memory + ThisEntry->Offset;
				const uint8* OtherValuePtr = OtherTable.Memory + OtherEntry.Offset;
				FMemory::Memcpy(ThisValuePtr, OtherValuePtr, ValueSize);
				EnumAddFlags(ThisEntry->Flags, EEntryFlags::Written | (OtherFlags & EEntryFlags::WrittenThisFrame));

				if (OutMask)
				{
					OutMask->VariableIDs.Add(ThisEntry->ID);
				}
			}
		}
	}
}

void FCameraVariableTable::LerpAll(const FCameraVariableTable& ToTable, float Factor, bool bIncludePrivateValues)
{
	const ECameraVariableTableFilter Filter = !bIncludePrivateValues ? 
		ECameraVariableTableFilter::PublicOnly : ECameraVariableTableFilter::None;
	InternalLerp(ToTable, Filter, Factor, nullptr, false, nullptr);
}

void FCameraVariableTable::Lerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor)
{
	InternalLerp(ToTable, Filter, Factor, nullptr, false, nullptr);
}

void FCameraVariableTable::Lerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor, const FCameraVariableTableFlags& InMask, bool bInvertMask, FCameraVariableTableFlags& OutMask)
{
	InternalLerp(ToTable, Filter, Factor, &InMask, false, &OutMask);
}

void FCameraVariableTable::InternalLerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor, const FCameraVariableTableFlags* InMask, bool bInvertMask, FCameraVariableTableFlags* OutMask)
{
	using namespace UE::Cameras::Private;

	const bool bPublicOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::PublicOnly);
	const bool bInputOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::InputOnly);
	const bool bKnownOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::KnownOnly);
	const bool bChangedOnly = EnumHasAnyFlags(Filter, ECameraVariableTableFilter::ChangedOnly);

	for (const FEntry& ToEntry : ToTable.Entries)
	{
		// Look for entries in the other table that have been written to, and aren't private.
		const EEntryFlags ToFlags = ToEntry.Flags;
		if (EnumHasAnyFlags(ToFlags, EEntryFlags::Written)
				&& (!bPublicOnly || !EnumHasAnyFlags(ToFlags, EEntryFlags::Private))
				&& (!bInputOnly || EnumHasAnyFlags(ToFlags, EEntryFlags::Input))
				&& (!bChangedOnly || EnumHasAnyFlags(ToFlags, EEntryFlags::WrittenThisFrame))
				&& IsVariableInMask(ToEntry.ID, InMask, bInvertMask))
		{
			// See if we know this variable.
			FEntry* FromEntry = FindEntry(ToEntry.ID);
			if (FromEntry)
			{
				// We already have the other table's variable in our table. Let's check
				// that the types match, and then interpolate the values.
#if WITH_EDITORONLY_DATA
				ensureMsgf(FromEntry->DebugName == ToEntry.DebugName,
						TEXT("Camera variable name collision! Expected variable '%d' to be named '%s', but other table has '%s'!"),
						FromEntry->ID.GetValue(), *FromEntry->DebugName, *ToEntry.DebugName);
#endif

#if WITH_EDITORONLY_DATA
				const FString& DebugName = FromEntry->DebugName;
#else
				const FString& DebugName = GUnavailableVariableDebugName;
#endif
				if (!ensureMsgf(FromEntry->Type == ToEntry.Type, 
						TEXT("Camera variable name collision! Expected '%d' (%s) to be of type '%s' but other table has type '%s'!"),
						FromEntry->ID.GetValue(), *DebugName,
						*UEnum::GetValueAsString(FromEntry->Type), *UEnum::GetValueAsString(ToEntry.Type)))
				{
					// Unexpected situation: two variables of different types sharing the same ID.
					continue;
				}

				const EEntryFlags FromFlags = FromEntry->Flags;
				ensureMsgf(EnumHasAllFlags(FromFlags, EEntryFlags::Written),
						TEXT("Camera variable '%d' (%s) is LERP'ing from an uninitialized value!"),
						FromEntry->ID.GetValue(), *DebugName);

				uint8* FromValuePtr = Memory + FromEntry->Offset;
				const uint8* ToValuePtr = ToTable.Memory + ToEntry.Offset;
				switch (FromEntry->Type)
				{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
					case ECameraVariableType::ValueName:\
						{\
							ValueType& FromValue = *reinterpret_cast<ValueType*>(FromValuePtr);\
							const ValueType& ToValue = *reinterpret_cast<const ValueType*>(ToValuePtr);\
							ValueType InterpValue = TCameraVariableInterpolation<ValueType>::Interpolate(*FromEntry, FromValue, ToValue, Factor);\
							FromValue = InterpValue;\
						}\
						break;
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
					case ECameraVariableType::BlendableStruct:
						{
							check(FromEntry->StructType);
							FBlendableStructTypeErasedInterpolator Interpolator = GetBlendableStructInterpolator(FromEntry->StructType);
							if (ensure(Interpolator))
							{
								Interpolator(FromValuePtr, ToValuePtr, Factor);
							}
						}
						break;
				}
				// We consider this variable "written to this frame" if it was written in either variable tables this frame.
				// If the value interpolates because the from/to values are different, but neither was written this frame, we
				// don't consider the interpolated value written this frame either.
				EnumAddFlags(FromEntry->Flags, EEntryFlags::Written | (FromFlags & EEntryFlags::WrittenThisFrame) | (ToFlags & EEntryFlags::WrittenThisFrame));
			}
			else if (!bKnownOnly)
			{
				// We don't have this variable in our table. Let's add it.
				FCameraVariableDefinition NewVariableDefinition;
				NewVariableDefinition.VariableID = ToEntry.ID;
				NewVariableDefinition.VariableType = ToEntry.Type;
				NewVariableDefinition.BlendableStructType = ToEntry.StructType;
				NewVariableDefinition.bIsInput = EnumHasAllFlags(ToEntry.Flags, EEntryFlags::Input);
#if WITH_EDITORONLY_DATA
				NewVariableDefinition.VariableName = ToEntry.DebugName;
#endif
				AddVariable(NewVariableDefinition);

				FromEntry = FindEntry(ToEntry.ID);
				check(FromEntry);

				uint32 ValueSize = 0, ValueAlignment = 0;
				GetVariableTypeAllocationInfo(FromEntry->Type, FromEntry->StructType, ValueSize, ValueAlignment);
				check(ValueSize != 0);

				uint8* FromValuePtr = Memory + FromEntry->Offset;
				const uint8* ToValuePtr = ToTable.Memory + ToEntry.Offset;
				FMemory::Memcpy(FromValuePtr, ToValuePtr, ValueSize);
				EnumAddFlags(FromEntry->Flags, EEntryFlags::Written | (ToFlags & EEntryFlags::WrittenThisFrame));
			}
			else
			{
				// We don't have this variable in our table but we only want to override
				// known entries.
				continue;
			}

			if (OutMask)
			{
				OutMask->VariableIDs.Add(FromEntry->ID);
			}
		}
	}
}

}  // namespace UE::Cameras

