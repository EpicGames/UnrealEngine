// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraVariableTableAllocationInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableTableAllocationInfo)

namespace UE::Cameras::Private
{

void GetKnownVariableDefinitions(
		const FCameraVariableTableAllocationInfo& AllocationInfo,
		TMap<FCameraVariableID, int32>& OutKnownIDs)
{
	for (auto It = AllocationInfo.VariableDefinitions.CreateConstIterator(); It; ++It)
	{
		const FCameraVariableDefinition& VariableDefinition(*It);
		OutKnownIDs.Add(VariableDefinition.VariableID, It.GetIndex());
	}
}

}  // namespace UE::Cameras::Private

void FCameraVariableTableAllocationInfo::Combine(const FCameraVariableTableAllocationInfo& OtherInfo)
{
	TMap<FCameraVariableID, int32> KnownIDs;
	UE::Cameras::Private::GetKnownVariableDefinitions(*this, KnownIDs);

	for (const FCameraVariableDefinition& OtherVariableDefinition : OtherInfo.VariableDefinitions)
	{
		const int32 KnownIndex = KnownIDs.FindRef(OtherVariableDefinition.VariableID, INDEX_NONE);
		if (KnownIndex == INDEX_NONE)
		{
			VariableDefinitions.Add(OtherVariableDefinition);
		}
		else
		{
			const FCameraVariableDefinition& KnownVariableDefinition(VariableDefinitions[KnownIndex]);
			ensure(KnownVariableDefinition == OtherVariableDefinition);
		}
	}
}

bool FCameraVariableTableAllocationInfo::Contains(const FCameraVariableTableAllocationInfo& OtherInfo) const
{
	TMap<FCameraVariableID, int32> KnownIDs;
	UE::Cameras::Private::GetKnownVariableDefinitions(*this, KnownIDs);

	for (const FCameraVariableDefinition& OtherVariableDefinition : OtherInfo.VariableDefinitions)
	{
		const int32 KnownIndex = KnownIDs.FindRef(OtherVariableDefinition.VariableID, INDEX_NONE);
		if (KnownIndex == INDEX_NONE)
		{
			return false;
		}
	}

	return true;
}

