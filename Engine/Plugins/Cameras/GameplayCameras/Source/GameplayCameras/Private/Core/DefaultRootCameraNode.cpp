// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/DefaultRootCameraNode.h"

#include "Core/BlendStackCameraNode.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/PersistentBlendStackCameraNode.h"
#include "Core/RootCameraNodeCameraRigEvent.h"
#include "Core/TransientBlendStackCameraNode.h"
#include "Debug/BlendStacksCameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/RootCameraDebugBlock.h"
#include "Math/ColorList.h"
#include "Services/CameraParameterSetterService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultRootCameraNode)

namespace UE::Cameras::Private
{

TObjectPtr<UBlendStackCameraNode> CreateBlendStack(
		UObject* This, const FObjectInitializer& ObjectInit,
		const FName& Name, ECameraBlendStackType BlendStackType, ECameraRigLayer Layer)
{
	TObjectPtr<UBlendStackCameraNode> NewBlendStack = ObjectInit.CreateDefaultSubobject<UBlendStackCameraNode>(
			This, Name);
	NewBlendStack->BlendStackType = BlendStackType;
	NewBlendStack->Layer = Layer;
	return NewBlendStack;
}

}  // namespace UE::Cameras::Private

UDefaultRootCameraNode::UDefaultRootCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	using namespace UE::Cameras::Private;

	BaseLayer = CreateBlendStack(this, ObjectInit, TEXT("BaseLayer"), ECameraBlendStackType::AdditivePersistent, ECameraRigLayer::Base);
	MainLayer = CreateBlendStack(this, ObjectInit, TEXT("MainLayer"), ECameraBlendStackType::IsolatedTransient, ECameraRigLayer::Main);
	GlobalLayer = CreateBlendStack(this, ObjectInit, TEXT("GlobalLayer"), ECameraBlendStackType::AdditivePersistent, ECameraRigLayer::Global);
	VisualLayer = CreateBlendStack(this, ObjectInit, TEXT("VisualLayer"), ECameraBlendStackType::AdditivePersistent, ECameraRigLayer::Visual);
}

FCameraNodeEvaluatorPtr UDefaultRootCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FDefaultRootCameraNodeEvaluator>();
}

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FDefaultRootCameraNodeEvaluator)

void FDefaultRootCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UDefaultRootCameraNode* Data = GetCameraNodeAs<UDefaultRootCameraNode>();
	BaseLayer = BuildBlendStackEvaluator<FPersistentBlendStackCameraNodeEvaluator>(Params, Data->BaseLayer);
	MainLayer = BuildBlendStackEvaluator<FTransientBlendStackCameraNodeEvaluator>(Params, Data->MainLayer);
	GlobalLayer = BuildBlendStackEvaluator<FPersistentBlendStackCameraNodeEvaluator>(Params, Data->GlobalLayer);
	VisualLayer = BuildBlendStackEvaluator<FPersistentBlendStackCameraNodeEvaluator>(Params, Data->VisualLayer);
}

template<typename EvaluatorType>
EvaluatorType* FDefaultRootCameraNodeEvaluator::BuildBlendStackEvaluator(const FCameraNodeEvaluatorBuildParams& Params, UBlendStackCameraNode* BlendStackNode)
{
	EvaluatorType* BlendStackEvaluator = Params.BuildEvaluatorAs<EvaluatorType>(BlendStackNode);
	BlendStackEvaluator->OnCameraRigEvent().AddRaw(this, &FDefaultRootCameraNodeEvaluator::OnBlendStackEvent);
	return BlendStackEvaluator;
}

FCameraNodeEvaluatorChildrenView FDefaultRootCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ BaseLayer, MainLayer, GlobalLayer, VisualLayer });
}

void FDefaultRootCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	ParameterSetterService = Params.Evaluator->FindEvaluationService<FCameraParameterSetterService>();
}

void FDefaultRootCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	BaseLayer->Run(Params, OutResult);
	MainLayer->Run(Params, OutResult);
	GlobalLayer->Run(Params, OutResult);
	SetPreVisualLayerResult(OutResult);
	if (Params.EvaluationType != ECameraNodeEvaluationType::IK && 
			Params.EvaluationType != ECameraNodeEvaluationType::ViewRotationPreview)
	{
		VisualLayer->Run(Params, OutResult);
	}
}

void FDefaultRootCameraNodeEvaluator::OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	MainLayer->ExecuteOperation(Params, Operation);
}

FCameraRigInstanceID FDefaultRootCameraNodeEvaluator::OnActivateCameraRig(const FActivateCameraRigParams& Params)
{
	if (Params.Layer == ECameraRigLayer::Main)
	{
		ensure(Params.OrderKey == 0);
		FBlendStackCameraPushParams PushParams;
		PushParams.EvaluationContext = Params.EvaluationContext;
		PushParams.CameraRig = Params.CameraRig;
		PushParams.TransitionOverride = Params.TransitionOverride;
		PushParams.bForcePush = Params.bForceActivate;
		
		const FBlendStackEntryID EntryID = MainLayer->Push(PushParams);
		return FCameraRigInstanceID::FromBlendStackEntryID(EntryID, ECameraRigLayer::Main);
	}
	else
	{
		FPersistentBlendStackCameraNodeEvaluator* TargetLayer = nullptr;
		switch (Params.Layer)
		{
			case ECameraRigLayer::Base:
				TargetLayer = BaseLayer;
				break;
			case ECameraRigLayer::Global:
				TargetLayer = GlobalLayer;
				break;
			case ECameraRigLayer::Visual:
				TargetLayer = VisualLayer;
				break;
		}
		if (ensure(TargetLayer))
		{
			FBlendStackCameraInsertParams InsertParams;
			InsertParams.EvaluationContext = Params.EvaluationContext;
			InsertParams.CameraRig = Params.CameraRig;
			InsertParams.TransitionOverride = Params.TransitionOverride;
			InsertParams.StackOrder = Params.OrderKey;
			InsertParams.bForceInsert = Params.bForceActivate;

			const FBlendStackEntryID EntryID = TargetLayer->Insert(InsertParams);
			return FCameraRigInstanceID::FromBlendStackEntryID(EntryID, Params.Layer);
		}
	}

	return FCameraRigInstanceID();
}

void FDefaultRootCameraNodeEvaluator::OnDeactivateCameraRig(const FDeactivateCameraRigParams& Params)
{
	ECameraRigLayer Layer = (Params.InstanceID.IsValid() ? Params.InstanceID.GetLayer() : Params.Layer);
	if (Layer == ECameraRigLayer::Main)
	{
		FBlendStackCameraFreezeParams FreezeParams;
		FreezeParams.EntryID = Params.InstanceID.ToBlendStackEntryID();
		FreezeParams.CameraRig = Params.CameraRig;
		FreezeParams.EvaluationContext = Params.EvaluationContext;
		MainLayer->Freeze(FreezeParams);
	}
	else
	{
		FPersistentBlendStackCameraNodeEvaluator* TargetLayer = nullptr;
		switch (Layer)
		{
			case ECameraRigLayer::Base:
				TargetLayer = BaseLayer;
				break;
			case ECameraRigLayer::Global:
				TargetLayer = GlobalLayer;
				break;
			case ECameraRigLayer::Visual:
				TargetLayer = VisualLayer;
				break;
		}
		if (ensure(TargetLayer))
		{
			FBlendStackCameraRemoveParams RemoveParams;
			RemoveParams.EntryID = Params.InstanceID.ToBlendStackEntryID();
			RemoveParams.EvaluationContext = Params.EvaluationContext;
			RemoveParams.CameraRig = Params.CameraRig;
			RemoveParams.TransitionOverride = Params.TransitionOverride;
			RemoveParams.bRemoveImmediately = Params.bDeactiveImmediately;
			TargetLayer->Remove(RemoveParams);
		}
	}
}

void FDefaultRootCameraNodeEvaluator::OnDeactivateAllCameraRigs(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately)
{
	BaseLayer->RemoveAll(InContext, bImmediately);
	MainLayer->FreezeAll(InContext);
	GlobalLayer->RemoveAll(InContext, bImmediately);
	VisualLayer->RemoveAll(InContext, bImmediately);
}

void FDefaultRootCameraNodeEvaluator::OnGetActiveCameraRigInfo(FCameraRigEvaluationInfo& OutCameraRigInfo) const
{
	OutCameraRigInfo = MainLayer->GetActiveCameraRigEvaluationInfo();
}

bool FDefaultRootCameraNodeEvaluator::OnHasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const
{
	return MainLayer->HasAnyRunningCameraRig(InContext);
}

void FDefaultRootCameraNodeEvaluator::OnGetCameraRigInfo(const FCameraRigInstanceID InstanceID, FCameraRigEvaluationInfo& OutCameraRigInfo) const
{
	FPersistentBlendStackCameraNodeEvaluator* TargetLayer = nullptr;
	switch (InstanceID.GetLayer())
	{
		case ECameraRigLayer::Base:
			TargetLayer = BaseLayer;
			break;
		case ECameraRigLayer::Global:
			TargetLayer = GlobalLayer;
			break;
		case ECameraRigLayer::Visual:
			TargetLayer = VisualLayer;
			break;
	}
	if (ensure(TargetLayer))
	{
		OutCameraRigInfo = TargetLayer->GetCameraRigEvaluationInfo(InstanceID.ToBlendStackEntryID());
	}
}

const FCameraVariableTable* FDefaultRootCameraNodeEvaluator::OnGetBlendedParameters() const
{
	return &MainLayer->GetBlendedParameters();
}

void FDefaultRootCameraNodeEvaluator::OnBuildSingleCameraRigHierarchy(const FSingleCameraRigHierarchyBuildParams& Params, FCameraNodeEvaluatorHierarchy& OutHierarchy)
{
	OutHierarchy.Build(BaseLayer);
	{
		OutHierarchy.AppendTagged(Params.CameraRigRangeName, Params.CameraRigInfo.RootEvaluator);
	}
	OutHierarchy.Append(GlobalLayer);
}

void FDefaultRootCameraNodeEvaluator::OnRunSingleCameraRig(const FSingleCameraRigEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	ensure(Params.EvaluationParams.EvaluationContext == Params.CameraRigInfo.EvaluationContext);
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext = Params.EvaluationParams.EvaluationContext;

	BaseLayer->Run(Params.EvaluationParams, OutResult);

	FCameraNodeEvaluator* RootEvaluator = Params.CameraRigInfo.RootEvaluator;

	// Emulate what the main blend stack does.

	{
		const FCameraNodeEvaluationResult& InitialResult(EvaluationContext->GetInitialResult());
		OutResult.VariableTable.OverrideAll(InitialResult.VariableTable, true);
		OutResult.ContextDataTable.OverrideAll(InitialResult.ContextDataTable);

		if (Params.EvaluationParams.bIsActiveCameraRig)
		{
			const FCameraNodeEvaluationResult* ActiveOnlyResult = EvaluationContext->GetConditionalResult(ECameraEvaluationDataCondition::ActiveCameraRig);
			if (ActiveOnlyResult)
			{
				OutResult.VariableTable.OverrideAll(ActiveOnlyResult->VariableTable, true);
				OutResult.ContextDataTable.OverrideAll(ActiveOnlyResult->ContextDataTable);
			}
		}

		if (ParameterSetterService)
		{
			ParameterSetterService->ApplyCameraVariableSetters(OutResult.VariableTable);
		}

		const FCameraNodeEvaluationResult* CameraRigResult = Params.CameraRigInfo.LastResult;
		FCameraBlendedParameterUpdateParams InputParams(Params.EvaluationParams, CameraRigResult->CameraPose);
		FCameraBlendedParameterUpdateResult InputResult(OutResult.VariableTable);

		FCameraNodeEvaluatorHierarchy Hierarchy(RootEvaluator);
		Hierarchy.CallUpdateParameters(InputParams, InputResult);
	}

	// No parameter blending: we are running this camera rig in isolation.

	{
		const FCameraNodeEvaluationResult& InitialResult(EvaluationContext->GetInitialResult());
		OutResult.CameraPose.OverrideChanged(InitialResult.CameraPose);
		OutResult.bIsCameraCut = (OutResult.bIsCameraCut || InitialResult.bIsCameraCut);
		OutResult.bIsValid = true;

		if (RootEvaluator)
		{
			RootEvaluator->Run(Params.EvaluationParams, OutResult);
		}
	}

	GlobalLayer->Run(Params.EvaluationParams, OutResult);
	// Don't run the visual layer.

	OutResult.bIsValid = true;
}

TSharedPtr<FCameraActionScope> FDefaultRootCameraNodeEvaluator::OnGetActiveCameraRigActionScope(bool bCreateIfNeeded)
{
	return MainLayer->GetActiveCameraRigActionScope(bCreateIfNeeded);
}

void FDefaultRootCameraNodeEvaluator::OnBlendStackEvent(const FBlendStackCameraRigEvent& InEvent)
{
	if (InEvent.EventType == EBlendStackCameraRigEventType::Pushed ||
			InEvent.EventType == EBlendStackCameraRigEventType::Popped)
	{
		FRootCameraNodeCameraRigEvent RootEvent;
		RootEvent.CameraRigInfo = InEvent.CameraRigInfo;
		RootEvent.Transition = InEvent.Transition;

		switch (InEvent.EventType)
		{
		case EBlendStackCameraRigEventType::Pushed:
			RootEvent.EventType = ERootCameraNodeCameraRigEventType::Activated;
			break;
		case EBlendStackCameraRigEventType::Popped:
			RootEvent.EventType = ERootCameraNodeCameraRigEventType::Deactivated;
			break;
		}

		if (InEvent.BlendStackEvaluator == BaseLayer)
		{
			RootEvent.EventLayer = ECameraRigLayer::Base;
		}
		else if (InEvent.BlendStackEvaluator == MainLayer)
		{
			RootEvent.EventLayer = ECameraRigLayer::Main;
		}
		else if (InEvent.BlendStackEvaluator == GlobalLayer)
		{
			RootEvent.EventLayer = ECameraRigLayer::Global;
		}
		else if (InEvent.BlendStackEvaluator == VisualLayer)
		{
			RootEvent.EventLayer = ECameraRigLayer::Visual;
		}

		BroadcastCameraRigEvent(RootEvent);
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FDefaultRootCameraNodeEvaluatorDebugBlock)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDefaultRootCameraNodeEvaluatorDebugBlock)

void FDefaultRootCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	// Create the debug block that shows the overall blend stack layers.
	FBlendStacksCameraDebugBlock& DebugBlock = Builder.BuildDebugBlock<FBlendStacksCameraDebugBlock>();
	{
		DebugBlock.AddBlendStack(TEXT("Base Layer"), BaseLayer->BuildDetailedDebugBlock(Params, FColorList::Grey, FColorList::LightGrey, Builder));
		DebugBlock.AddBlendStack(TEXT("Main Layer"), MainLayer->BuildDetailedDebugBlock(Params, FColorList::LightBlue, FColorList::SlateBlue, Builder));
		DebugBlock.AddBlendStack(TEXT("Global Layer"), GlobalLayer->BuildDetailedDebugBlock(Params, FColorList::Orange, FColorList::OrangeRed, Builder));
		DebugBlock.AddBlendStack(TEXT("Visual Layer"), VisualLayer->BuildDetailedDebugBlock(Params, FColorList::Pink, FColorList::NeonPink, Builder));
	}

	Builder.GetRootDebugBlock().AddChild(&DebugBlock);
}

void FDefaultRootCameraNodeEvaluatorDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

