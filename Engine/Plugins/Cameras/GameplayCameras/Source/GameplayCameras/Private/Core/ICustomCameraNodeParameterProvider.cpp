// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ICustomCameraNodeParameterProvider.h"

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableReferences.h"
#include "GameplayCamerasDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ICustomCameraNodeParameterProvider)

namespace UE::Cameras
{

void FCameraNodeParameterInfos::AddBlendableParameter(
		FName ParameterName, 
		ECameraVariableType VariableType, 
		const UScriptStruct* BlendableStructType,
		const uint8* DefaultValue,
		FCameraVariableID* OverrideVariableID,
		UCameraVariableAsset* OverrideVariable)
{
	BlendableParameters.Add({ ParameterName, VariableType, BlendableStructType, DefaultValue, OverrideVariableID, OverrideVariable });
}

void FCameraNodeParameterInfos::AddBlendableParameter(FCustomCameraNodeBlendableParameter& Parameter, const uint8* DefaultValue)
{
	AddBlendableParameter(
			Parameter.ParameterName,
			Parameter.VariableType,
			Parameter.BlendableStructType,
			DefaultValue,
			&Parameter.OverrideVariableID,
			Parameter.OverrideVariable);
}

void FCameraNodeParameterInfos::AddDataParameter(
		FName ParameterName, 
		ECameraContextDataType DataType,
		ECameraContextDataContainerType DataContainerType,
		const UObject* DataTypeObject,
		const uint8* DefaultValue,
		FCameraContextDataID* OverrideDataID)
{
	DataParameters.Add({ ParameterName, DataType, DataContainerType, DataTypeObject, DefaultValue, OverrideDataID });
}

void FCameraNodeParameterInfos::AddDataParameter(FCustomCameraNodeDataParameter& Parameter, const uint8* DefaultValue)
{
	AddDataParameter(
			Parameter.ParameterName,
			Parameter.DataType,
			Parameter.DataContainerType,
			Parameter.DataTypeObject,
			DefaultValue,
			&Parameter.OverrideDataID);
}

void FCameraNodeParameterInfos::Reset()
{
	BlendableParameters.Reset();
	DataParameters.Reset();
}

const FCameraNodeBlendableParameterInfo* FCameraNodeParameterInfos::FindBlendableParameter(FName ParameterName) const
{
	for (const FBlendableParameterInfo& BlendableParameter : BlendableParameters)
	{
		if (BlendableParameter.ParameterName == ParameterName)
		{
			return &BlendableParameter;
		}
	}
	return nullptr;
}

const FCameraNodeDataParameterInfo* FCameraNodeParameterInfos::FindDataParameter(FName ParameterName) const
{
	for (const FDataParameterInfo& DataParameter : DataParameters)
	{
		if (DataParameter.ParameterName == ParameterName)
		{
			return &DataParameter;
		}
	}
	return nullptr;
}

void FCameraNodeParameterInfos::BuildFrom(UCameraNode* InCameraNode)
{
	Reset();

	if (!ensure(InCameraNode))
	{
		return;
	}

	UClass* CameraNodeClass = InCameraNode->GetClass();

	for (TFieldIterator<FProperty> It(CameraNodeClass); It; ++It)
	{
		FProperty* Property(*It);

		// First look for some built-in blendable parameters.
		if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			bool bIsCameraParameterProperty = true;
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			if (StructProperty->Struct == F##ValueName##CameraParameter::StaticStruct())\
			{\
				auto* CameraParameterPtr = StructProperty->ContainerPtrToValuePtr<F##ValueName##CameraParameter>(InCameraNode);\
				AddBlendableParameter(\
						StructProperty->GetFName(),\
						ECameraVariableType::ValueName,\
						nullptr,\
						reinterpret_cast<uint8*>(&CameraParameterPtr->Value),\
						&CameraParameterPtr->VariableID,\
						CameraParameterPtr->Variable);\
			}\
			else if (StructProperty->Struct == F##ValueName##CameraVariableReference::StaticStruct())\
			{\
				auto* VariableReferencePtr = StructProperty->ContainerPtrToValuePtr<F##ValueName##CameraVariableReference>(InCameraNode);\
				AddBlendableParameter(\
						StructProperty->GetFName(),\
						ECameraVariableType::ValueName,\
						nullptr,\
						nullptr,\
						&VariableReferencePtr->VariableID,\
						VariableReferencePtr->Variable);\
			}\
			else
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			{
				// Other struct type...
				bIsCameraParameterProperty = false;
			}

			if (bIsCameraParameterProperty)
			{
				continue;
			}
		}

		// Look for a custom blendable parameter.
		const FName BlendableIDPropertyName = FName(It->GetName() + TEXT("BlendableID"));
		FStructProperty* BlendableIDStructProperty = CastField<FStructProperty>(CameraNodeClass->FindPropertyByName(BlendableIDPropertyName));
		if (BlendableIDStructProperty && BlendableIDStructProperty->Struct == FCameraVariableID::StaticStruct())
		{
			bool bIsCameraParameterProperty = true;
			ECameraVariableType VariableType = ECameraVariableType::Boolean;
			UScriptStruct* VariableTypeObject = nullptr;
			if (Property->IsA<FBoolProperty>())
			{
				VariableType = ECameraVariableType::Boolean;
			}
			else if (Property->IsA<FIntProperty>())
			{
				VariableType = ECameraVariableType::Integer32;
			}
			else if (Property->IsA<FFloatProperty>())
			{
				VariableType = ECameraVariableType::Float;
			}
			else if (Property->IsA<FDoubleProperty>())
			{
				VariableType = ECameraVariableType::Double;
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct == TVariantStructure<FVector2f>::Get())
				{
					VariableType = ECameraVariableType::Vector2f;
				}
				else if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
				{
					VariableType = ECameraVariableType::Vector2d;
				}
				else if (StructProperty->Struct == TVariantStructure<FVector3f>::Get())
				{
					VariableType = ECameraVariableType::Vector3f;
				}
				else if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					VariableType = ECameraVariableType::Vector3d;
				}
				else if (StructProperty->Struct == TVariantStructure<FVector4f>::Get())
				{
					VariableType = ECameraVariableType::Vector4f;
				}
				else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
				{
					VariableType = ECameraVariableType::Vector4d;
				}
				else if (StructProperty->Struct == TVariantStructure<FRotator3f>::Get())
				{
					VariableType = ECameraVariableType::Rotator3f;
				}
				else if (StructProperty->Struct == TBaseStructure<FRotator>::Get())
				{
					VariableType = ECameraVariableType::Rotator3d;
				}
				else if (StructProperty->Struct == TVariantStructure<FTransform3f>::Get())
				{
					VariableType = ECameraVariableType::Transform3f;
				}
				else if (StructProperty->Struct == TBaseStructure<FTransform>::Get())
				{
					VariableType = ECameraVariableType::Transform3d;
				}
				else
				{
					// TODO: check that the structure is registered as blendable
					VariableType = ECameraVariableType::BlendableStruct;
					VariableTypeObject = StructProperty->Struct;
				}
			}
			else
			{
				bIsCameraParameterProperty = false;
			}

			if (bIsCameraParameterProperty)
			{
				void* DefaultValue = Property->ContainerPtrToValuePtr<void>(InCameraNode);
				FCameraVariableID* BlendableID = BlendableIDStructProperty->ContainerPtrToValuePtr<FCameraVariableID>(InCameraNode);
				AddBlendableParameter(
						Property->GetFName(),
						VariableType,
						VariableTypeObject,
						static_cast<uint8*>(DefaultValue),
						BlendableID,
						nullptr);
				continue;
			}
		}

		// Look for a data parameter.
		const FName DataIDPropertyName = FName(It->GetName() + TEXT("DataID"));
		FStructProperty* DataIDStructProperty = CastField<FStructProperty>(CameraNodeClass->FindPropertyByName(DataIDPropertyName));
		if (DataIDStructProperty && DataIDStructProperty->Struct == FCameraContextDataID::StaticStruct())
		{
			bool bIsDataProperty = true;
			ECameraContextDataType DataPropertyType = ECameraContextDataType::Name;
			ECameraContextDataContainerType DataPropertyContainerType = ECameraContextDataContainerType::None;
			UObject* DataPropertyTypeObject = nullptr;

			bool bHasDataPropertyContainer = false;
			FProperty* ActualProperty = Property;
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				bHasDataPropertyContainer = true;
				ActualProperty = ArrayProperty->Inner;
				DataPropertyContainerType = ECameraContextDataContainerType::Array;
			}

			if (ActualProperty->IsA<FNameProperty>())
			{
				DataPropertyType = ECameraContextDataType::Name;
			}
			else if (ActualProperty->IsA<FStrProperty>())
			{
				DataPropertyType = ECameraContextDataType::String;
			}
			else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(ActualProperty))
			{
				DataPropertyType = ECameraContextDataType::Enum;
				DataPropertyTypeObject = EnumProperty->GetEnum();
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(ActualProperty))
			{
				DataPropertyType = ECameraContextDataType::Struct;
				DataPropertyTypeObject = StructProperty->Struct;
			}
			else if (FClassProperty* ClassProperty = CastField<FClassProperty>(ActualProperty))
			{
				DataPropertyType = ECameraContextDataType::Class;
				DataPropertyTypeObject = ClassProperty->MetaClass;
			}
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ActualProperty))
			{
				DataPropertyType = ECameraContextDataType::Object;
				DataPropertyTypeObject = ObjectProperty->PropertyClass;
			}
			else
			{
				bIsDataProperty = false;
			}

			if (bIsDataProperty)
			{
				void* DefaultValue = !bHasDataPropertyContainer ? 
					Property->ContainerPtrToValuePtr<void>(InCameraNode) : nullptr;
				FCameraContextDataID* DataID = DataIDStructProperty->ContainerPtrToValuePtr<FCameraContextDataID>(InCameraNode);
				AddDataParameter(
						Property->GetFName(),
						DataPropertyType,
						DataPropertyContainerType,
						DataPropertyTypeObject,
						static_cast<uint8*>(DefaultValue),
						DataID);
				continue;
			}
		}
	}

	// Add any custom parameters the node may declare on its own.
	if (ICustomCameraNodeParameterProvider* CustomParameterProvider = Cast<ICustomCameraNodeParameterProvider>(InCameraNode))
	{
		CustomParameterProvider->GetCustomCameraNodeParameters(*this);
	}
}

}  // namespace UE::Cameras

void ICustomCameraNodeParameterProvider::OnCustomCameraNodeParametersChanged(const UCameraNode* ThisAsCameraNode) const
{
	using namespace UE::Cameras;

	FGameplayCamerasDelegates::OnCustomCameraNodeParametersChanged().Broadcast(ThisAsCameraNode);
}

