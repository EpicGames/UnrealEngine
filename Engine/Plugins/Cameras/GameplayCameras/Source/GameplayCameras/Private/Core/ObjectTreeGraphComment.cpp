// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ObjectTreeGraphComment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTreeGraphComment)

UObjectTreeGraphComment::UObjectTreeGraphComment(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

#if WITH_EDITOR

void UObjectTreeGraphComment::GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const
{
	NodePosX = GraphNodePos.X;
	NodePosY = GraphNodePos.Y;
}

void UObjectTreeGraphComment::OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty)
{
	Modify(bMarkDirty);

	GraphNodePos.X = NodePosX;
	GraphNodePos.Y = NodePosY;
}

void UObjectTreeGraphComment::GetGraphNodeName(FName InGraphName, FText& OutName) const
{
	OutName = FText::FromString(CommentText);
}

void UObjectTreeGraphComment::OnRenameGraphNode(FName InGraphName, const FString& NewName)
{
	Modify();

	CommentText = NewName;
}

#endif  // WITH_EDITOR

