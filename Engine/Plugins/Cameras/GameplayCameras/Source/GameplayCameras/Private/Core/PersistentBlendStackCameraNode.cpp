// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/PersistentBlendStackCameraNode.h"

#include "Algo/BinarySearch.h"
#include "Core/BlendCameraNode.h"
#include "Core/BlendStackCameraRigEvent.h"
#include "Core/BlendStackRootCameraNode.h"
#include "Core/CameraEvaluationContext.h"
#include "Helpers/CameraRigTransitionFinder.h"
#include "Nodes/Blends/InterruptedBlendCameraNode.h"
#include "Nodes/Blends/PopBlendCameraNode.h"
#include "Nodes/Blends/ReverseBlendCameraNode.h"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FPersistentBlendStackCameraNodeEvaluator)

FBlendStackEntryID FPersistentBlendStackCameraNodeEvaluator::Insert(const FBlendStackCameraInsertParams& Params)
{
	// See if we already have this camera rig and evaluation context in the stack.
	if (!Params.bForceInsert)
	{
		for (int32 Index = 0; Index < Entries.Num(); ++Index)
		{
			const FCameraRigEntry& Entry(Entries[Index]);
			const FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[Index]);

			if (!Entry.Flags.bIsFrozen &&
					Entry.CameraRig == Params.CameraRig &&
					Entry.EvaluationContext == Params.EvaluationContext &&
					EntryExtraInfo.StackOrder == Params.StackOrder)
			{
				return FBlendStackEntryID();
			}
		}
	}

	UObject* Outer = const_cast<UObject*>((UObject*)GetCameraNode());
	UBlendStackRootCameraNode* EntryRootNode = NewObject<UBlendStackRootCameraNode>(Outer, NAME_None);
	{
		EntryRootNode->RootNode = Params.CameraRig->RootNode;

		// Find a transition to blend in. If no transition is found, use a pop blend.
		UBlendCameraNode* ModeBlend = nullptr;
		if (const UCameraRigTransition* Transition = FindEnterTransition(Params))
		{
			ModeBlend = Transition->Blend;
		}
		if (!ModeBlend)
		{
			ModeBlend = NewObject<UPopBlendCameraNode>(EntryRootNode, NAME_None);
		}
		EntryRootNode->Blend = ModeBlend;
	}

	FCameraRigEntry NewEntry;
	InitializeEntry(
			NewEntry, 
			Params.CameraRig,
			Params.EvaluationContext,
			EntryRootNode,
			false);

	FCameraRigEntryExtraInfo NewExtraInfo;
	NewExtraInfo.StackOrder = Params.StackOrder;
	NewExtraInfo.BlendStatus = EBlendStatus::BlendIn;

#if WITH_EDITOR
	AddPackageListeners(NewEntry);
#endif  // WITH_EDITOR

	const FBlendStackEntryID AddedEntryID(NewEntry.EntryID);

	ensure(Entries.Num() == EntryExtraInfos.Num());

	const int32 AddedIndex = Algo::UpperBoundBy(EntryExtraInfos, Params.StackOrder, &FCameraRigEntryExtraInfo::StackOrder);
	Entries.Insert(MoveTemp(NewEntry), AddedIndex);
	EntryExtraInfos.Insert(MoveTemp(NewExtraInfo), AddedIndex);

	if (OnCameraRigEventDelegate.IsBound())
	{
		BroadcastCameraRigEvent(EBlendStackCameraRigEventType::Pushed, Entries[AddedIndex], nullptr);
	}

	return AddedEntryID;
}

void FPersistentBlendStackCameraNodeEvaluator::Remove(const FBlendStackCameraRemoveParams& Params)
{
	TArray<int32, TInlineAllocator<4>> EntriesToRemove;

	if (Params.EntryID.IsValid())
	{
		// Remove the entry by ID.
		const int32 EntryIndex = IndexOfEntry(Params.EntryID);
		if (EntryIndex != INDEX_NONE)
		{
			EntriesToRemove.Add(EntryIndex);
		}
	}
	else
	{
		// Remove any entries matching the given context and rig asset.
		for (int32 Index = Entries.Num() - 1; Index >= 0; --Index)
		{
			FCameraRigEntry& Entry(Entries[Index]);
			if (Entry.CameraRig == Params.CameraRig &&
					Entry.EvaluationContext == Params.EvaluationContext)
			{
				EntriesToRemove.Add(Index);
			}
		}
	}

	for (int32 Index : EntriesToRemove)
	{
		RemoveEntry(Index, Params.TransitionOverride, Params.bRemoveImmediately);
	}
}

void FPersistentBlendStackCameraNodeEvaluator::RemoveAll(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately)
{
	TArray<int32, TInlineAllocator<4>> EntriesToRemove;
	for (int32 Index = Entries.Num() - 1; Index >= 0; --Index)
	{
		FCameraRigEntry& Entry(Entries[Index]);
		if (Entry.EvaluationContext == InContext)
		{
			EntriesToRemove.Add(Index);
		}
	}

	for (int32 Index : EntriesToRemove)
	{
		RemoveEntry(Index, nullptr, bImmediately);
	}
}

void FPersistentBlendStackCameraNodeEvaluator::RemoveEntry(int32 EntryIndex, const UCameraRigTransition* TransitionOverride, bool bImmediately)
{
	ensure(Entries.IsValidIndex(EntryIndex));

	// If we need to remove the camera rig immediately, simply pop out its entries.
	// Else, we need to start blending out that entry.
	if (bImmediately)
	{
		PopEntry(EntryIndex);
		EntryExtraInfos.RemoveAt(EntryIndex);
	}
	else
	{
		FCameraRigEntry& Entry(Entries[EntryIndex]);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[EntryIndex]);
		const UCameraRigTransition* Transition = FindExitTransition(Entry, TransitionOverride);
		if (Transition && Transition->Blend)
		{
			// Swap the blend-in evaluator on this entry with a blend-out one.
			if (EntryExtraInfo.BlendStatus != EBlendStatus::BlendOut)
			{
				FCameraNodeEvaluatorBuilder BlendOutBuilder(Entry.EvaluatorStorage);
				FCameraNodeEvaluatorBuildParams BlendOutBuildParams(BlendOutBuilder);
				FBlendCameraNodeEvaluator* BlendOutEvaluator = BlendOutBuildParams.BuildEvaluatorAs<FBlendCameraNodeEvaluator>(Transition->Blend);

				FCameraNodeEvaluatorInitializeParams BlendOutInitParams;
				BlendOutInitParams.Evaluator = OwningEvaluator;
				BlendOutInitParams.EvaluationContext = Entry.EvaluationContext.Pin();
				BlendOutInitParams.Layer = Layer;
				BlendOutEvaluator->Initialize(BlendOutInitParams, Entry.Result);

				// Reverse this blend so it plays as a blend-out. Also, see if we are going to 
				// interrupt an ongoing blend-in... if so, give a chance for the blend-out to
				// start at an "equivalent spot".
				if (!BlendOutEvaluator->SetReversed(true))
				{
					BlendOutEvaluator = Entry.EvaluatorStorage.BuildEvaluator<FReverseBlendCameraNodeEvaluator>(BlendOutEvaluator);
				}
				if (EntryExtraInfo.BlendStatus == EBlendStatus::BlendIn)
				{
					FBlendCameraNodeEvaluator* OngoingBlend = Entry.RootEvaluator->GetBlendEvaluator();

					FCameraNodeBlendInterruptionParams InterruptionParams;
					InterruptionParams.InterruptedBlend = OngoingBlend;
					if (!BlendOutEvaluator->InitializeFromInterruption(InterruptionParams))
					{
						BlendOutEvaluator = Entry.EvaluatorStorage.BuildEvaluator<FInterruptedBlendCameraNodeEvaluator>(BlendOutEvaluator, OngoingBlend);
					}
				}
				// Note: neither the reverse or interrupted blends need initialization, but
				// technically we're missing calling it on them.
				Entry.RootEvaluator->SetBlendEvaluator(BlendOutEvaluator);

				EntryExtraInfo.BlendStatus = EBlendStatus::BlendOut;
				EntryExtraInfo.bIsBlendFinished = false;
				EntryExtraInfo.bIsBlendFull = false;
			}
			// else: we were already blending out, so let this continue.
		}
		else
		{
			// No transition found... just cut.
			PopEntry(EntryIndex);
			EntryExtraInfos.RemoveAt(EntryIndex);
		}
	}
}

void FPersistentBlendStackCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	ensure(Entries.Num() == EntryExtraInfos.Num());

	// Validate our entries and resolve evaluation context weak pointers.
	TArray<FResolvedEntry> ResolvedEntries;
	ResolveEntries(Params, ResolvedEntries);

	// Run the stack!
	InternalUpdate(ResolvedEntries, Params, OutResult);

	// Tidy things up.
	OnRunFinished(OutResult);
}

void FPersistentBlendStackCameraNodeEvaluator::InternalUpdate(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	TArray<int32, TInlineAllocator<4>> EntriesToRemove;

	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[ResolvedEntry.EntryIndex]);

		if (!Entry.Flags.bIsFrozen && ResolvedEntry.Context.IsValid())
		{
			FCameraNodeEvaluationParams CurParams(Params);
			CurParams.EvaluationContext = ResolvedEntry.Context;
			CurParams.bIsFirstFrame = Entry.Flags.bIsFirstFrame;

			FCameraNodeEvaluationResult& CurResult(Entry.Result);

			// Update the current entry's result.
			{
				CurResult.Reset();

				// Start with the input given to us.
				CurResult.CameraPose = OutResult.CameraPose;
				CurResult.VariableTable.OverrideAll(OutResult.VariableTable);
				CurResult.CameraRigJoints.OverrideAll(OutResult.CameraRigJoints);
				CurResult.PostProcessSettings.OverrideAll(OutResult.PostProcessSettings);

				// Override it with whatever the evaluation context has set on its result.
				// NOTE: don't override the camera pose, because additive/persistent blend stacks are designed to be
				//       used for "modifier rigs". As such they shouldn't reset the camera pose to wherever their 
				//       context is located. They should only modify it.
				const FCameraNodeEvaluationResult& ContextResult(Entry.ContextResult);
				CurResult.VariableTable.OverrideAll(ContextResult.VariableTable, true);
				CurResult.ContextDataTable.OverrideAll(ContextResult.ContextDataTable);

				// Setup flags.
				CurResult.bIsCameraCut = OutResult.bIsCameraCut || ContextResult.bIsCameraCut || Entry.Flags.bForceCameraCut;
				CurResult.bIsValid = true;
			}

			// Update pre-blended parameters.
			{
				FCameraBlendedParameterUpdateParams InputParams(CurParams, CurResult.CameraPose);
				FCameraBlendedParameterUpdateResult InputResult(CurResult.VariableTable);

				Entry.EvaluatorHierarchy.ForEachEvaluator(ECameraNodeEvaluatorFlags::NeedsParameterUpdate,
						[&InputParams, &InputResult](FCameraNodeEvaluator* ParameterEvaluator)
						{
							ParameterEvaluator->UpdateParameters(InputParams, InputResult);
						});
			}

			// Run the blend node.
			FBlendCameraNodeEvaluator* EntryBlendEvaluator = Entry.RootEvaluator->GetBlendEvaluator();
			if (EntryBlendEvaluator)
			{
				EntryBlendEvaluator->Run(CurParams, CurResult);
			}

			// Blend pre-blended parameters.
			if (EntryBlendEvaluator)
			{
				FCameraNodePreBlendParams PreBlendParams(CurParams, CurResult.CameraPose, CurResult.VariableTable);
				PreBlendParams.VariableTableFilter = ECameraVariableTableFilter::InputOnly;

				FCameraNodePreBlendResult PreBlendResult(OutResult.VariableTable);

				EntryBlendEvaluator->BlendParameters(PreBlendParams, PreBlendResult);
			}
			else
			{
				OutResult.VariableTable.Override(CurResult.VariableTable, ECameraVariableTableFilter::InputOnly);
			}

			// Run the camera rig's root node.
			FCameraNodeEvaluator* RootEvaluator = Entry.RootEvaluator->GetRootEvaluator();
			if (RootEvaluator)
			{
				RootEvaluator->Run(CurParams, CurResult);
			}

			// Blend the results.
			if (EntryBlendEvaluator)
			{
				FCameraNodeBlendParams BlendParams(CurParams, CurResult);
				FCameraNodeBlendResult BlendResult(OutResult);

				EntryBlendEvaluator->BlendResults(BlendParams, BlendResult);

				EntryExtraInfo.bIsBlendFinished = BlendResult.bIsBlendFinished;
				EntryExtraInfo.bIsBlendFull = BlendResult.bIsBlendFull;
			}
			else
			{
				OutResult.OverrideAll(CurResult);
			}

			// Update blend state.
			if (EntryExtraInfo.BlendStatus == EBlendStatus::BlendIn)
			{
				if (EntryExtraInfo.bIsBlendFull && EntryExtraInfo.bIsBlendFinished)
				{
					EntryExtraInfo.BlendStatus = EBlendStatus::None;
				}
			}
			else if (EntryExtraInfo.BlendStatus == EBlendStatus::BlendOut)
			{
				if (EntryExtraInfo.bIsBlendFull && EntryExtraInfo.bIsBlendFinished)
				{
					EntriesToRemove.Add(ResolvedEntry.EntryIndex);
				}
			}
		}
		else
		{
			FCameraNodeEvaluationResult& CurResult(Entry.Result);

			OutResult.VariableTable.Override(CurResult.VariableTable, ECameraVariableTableFilter::None);
			OutResult.OverrideAll(CurResult);
		}
	}

	if (!Params.IsStatelessEvaluation())
	{
		for (int32 Index = EntriesToRemove.Num() - 1; Index >= 0; --Index)
		{
			PopEntry(EntriesToRemove[Index]);
			EntryExtraInfos.RemoveAt(EntriesToRemove[Index]);
		}
	}
}

const UCameraRigTransition* FPersistentBlendStackCameraNodeEvaluator::FindEnterTransition(const FBlendStackCameraInsertParams& Params) const
{
	// If we are forced to use a specific transition, our search is over.
	if (Params.TransitionOverride)
	{
		return Params.TransitionOverride;
	}

	// Find a transition that works for blending the given camera rig in.
	return FCameraRigTransitionFinder::FindTransition(
			Params.CameraRig->EnterTransitions,
			nullptr, nullptr, nullptr, false,
			Params.EvaluationContext, Params.CameraRig, nullptr);
}

const UCameraRigTransition* FPersistentBlendStackCameraNodeEvaluator::FindExitTransition(const FCameraRigEntry& Entry, const UCameraRigTransition* TransitionOverride) const
{
	// If we are forced to use a specific transition, our search is over.
	if (TransitionOverride)
	{
		return TransitionOverride;
	}

	// Find a transition that works for blending the given camera rig out.
	return FCameraRigTransitionFinder::FindTransition(
			Entry.CameraRig->ExitTransitions,
			Entry.EvaluationContext.Pin(), Entry.CameraRig, nullptr, Entry.Flags.bIsFrozen,
			nullptr, nullptr, nullptr);
}

#if WITH_EDITOR

void FPersistentBlendStackCameraNodeEvaluator::OnEntryReinitialized(int32 EntryIndex)
{
	if (!ensure(EntryExtraInfos.IsValidIndex(EntryIndex)))
	{
		return;
	}

	FCameraRigEntryExtraInfo& ExtraInfo = EntryExtraInfos[EntryIndex];
	{
		// When hot-reloading camera rigs, the base class replaces the blend node with pop so
		// let's update our own extra info accordingly.
		ExtraInfo.bIsBlendFull = true;
		ExtraInfo.bIsBlendFinished = true;
		ExtraInfo.BlendStatus = EBlendStatus::None;
	}
}

#endif

}  // namespace UE::Cameras

