// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/PostProcessSettingsCollection.h"

#include "Misc/EngineVersionComparison.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
#include "Engine/PostProcessUtils.h"
#else
#include "Compat/PostProcessUtils.h"
#endif

namespace UE::Cameras
{

void FPostProcessSettingsCollection::Reset()
{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	PostProcessSettings = FPostProcessSettings::GetDefault();
#else
	PostProcessSettings = FPostProcessSettings();
#endif
}

void FPostProcessSettingsCollection::OverrideAll(const FPostProcessSettingsCollection& OtherCollection)
{
	PostProcessSettings = OtherCollection.PostProcessSettings;
}

void FPostProcessSettingsCollection::OverrideChanged(const FPostProcessSettingsCollection& OtherCollection)
{
	OverrideChanged(OtherCollection.PostProcessSettings);
}

void FPostProcessSettingsCollection::OverrideChanged(const FPostProcessSettings& OtherPostProcessSettings)
{
	FPostProcessSettings& ThisPP = PostProcessSettings;
	const FPostProcessSettings& OtherPP = OtherPostProcessSettings;

	FPostProcessUtils::OverridePostProcessSettings(ThisPP, OtherPP);
}

void FPostProcessSettingsCollection::LerpAll(const FPostProcessSettingsCollection& ToCollection, float BlendFactor)
{
	LerpAll(ToCollection.PostProcessSettings, BlendFactor);
}

void FPostProcessSettingsCollection::LerpAll(const FPostProcessSettings& ToPostProcessSettings, float BlendFactor)
{
	InternalLerp(ToPostProcessSettings, BlendFactor);
}

void FPostProcessSettingsCollection::InternalLerp(const FPostProcessSettings& ToPostProcessSettings, float BlendFactor)
{
	FPostProcessSettings& ThisPP = PostProcessSettings;
	const FPostProcessSettings& ToPP = ToPostProcessSettings;

	FPostProcessUtils::BlendPostProcessSettings(ThisPP, ToPP, BlendFactor);
}

void FPostProcessSettingsCollection::Serialize(FArchive& Ar)
{
	UScriptStruct* PostProcessSettingsStruct = FPostProcessSettings::StaticStruct();
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	PostProcessSettingsStruct->SerializeItem(Ar, &PostProcessSettings, &FPostProcessSettings::GetDefault());
#else
	FPostProcessSettings DefaultSettings;
	PostProcessSettingsStruct->SerializeItem(Ar, &PostProcessSettings, &DefaultSettings);
#endif
}

}  // namespace UE::Cameras

