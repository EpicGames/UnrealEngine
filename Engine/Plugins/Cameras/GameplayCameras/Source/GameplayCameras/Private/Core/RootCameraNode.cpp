// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/RootCameraNode.h"

#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNodeCameraRigEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootCameraNode)

namespace UE::Cameras
{

void FRootCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	OwningEvaluator = Params.Evaluator;
}

FCameraRigInstanceID FRootCameraNodeEvaluator::ActivateCameraRig(const FActivateCameraRigParams& Params)
{
	return OnActivateCameraRig(Params);
}

void FRootCameraNodeEvaluator::DeactivateCameraRig(const FDeactivateCameraRigParams& Params)
{
	OnDeactivateCameraRig(Params);
}

void FRootCameraNodeEvaluator::DeactivateAllCameraRigs(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately)
{
	OnDeactivateAllCameraRigs(InContext, bImmediately);
}

void FRootCameraNodeEvaluator::ExecuteCameraDirectorRequest(const FCameraRigActivationDeactivationRequest& Request)
{
	if (Request.RequestType == ECameraRigActivationDeactivationRequestType::Activate)
	{
		FActivateCameraRigParams CameraRigParams;
		CameraRigParams.EvaluationContext = Request.EvaluationContext;
		CameraRigParams.CameraRig = Request.CameraRig;
		CameraRigParams.Layer = Request.Layer;
		CameraRigParams.OrderKey = Request.OrderKey;
		CameraRigParams.TransitionOverride = Request.TransitionOverride;
		CameraRigParams.bForceActivate = Request.bForceActivateDeactivate;
		ActivateCameraRig(CameraRigParams);
	}
	else if (Request.RequestType == ECameraRigActivationDeactivationRequestType::Deactivate)
	{
		FDeactivateCameraRigParams CameraRigParams;
		CameraRigParams.EvaluationContext = Request.EvaluationContext;
		CameraRigParams.CameraRig = Request.CameraRig;
		CameraRigParams.Layer = Request.Layer;
		CameraRigParams.TransitionOverride = Request.TransitionOverride;
		CameraRigParams.bDeactiveImmediately = Request.bForceActivateDeactivate;
		DeactivateCameraRig(CameraRigParams);
	}
}

void FRootCameraNodeEvaluator::GetActiveCameraRigInfo(FCameraRigEvaluationInfo& OutCameraRigInfo) const
{
	OnGetActiveCameraRigInfo(OutCameraRigInfo);
}

void FRootCameraNodeEvaluator::GetCameraRigInfo(const FCameraRigInstanceID InstanceID, FCameraRigEvaluationInfo& OutCameraRigInfo) const
{
	OnGetCameraRigInfo(InstanceID, OutCameraRigInfo);
}

bool FRootCameraNodeEvaluator::HasAnyActiveCameraRig() const
{
	FCameraRigEvaluationInfo RigInfo;
	GetActiveCameraRigInfo(RigInfo);
	return RigInfo.RootEvaluator != nullptr;
}

bool FRootCameraNodeEvaluator::HasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const
{
	return OnHasAnyRunningCameraRig(InContext);
}

const FCameraVariableTable* FRootCameraNodeEvaluator::GetBlendedParameters() const
{
	return OnGetBlendedParameters();
}

void FRootCameraNodeEvaluator::BuildSingleCameraRigHierarchy(const FSingleCameraRigHierarchyBuildParams& Params, FCameraNodeEvaluatorHierarchy& OutHierarchy)
{
	OnBuildSingleCameraRigHierarchy(Params, OutHierarchy);
}

void FRootCameraNodeEvaluator::RunSingleCameraRig(const FSingleCameraRigEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Before we do the actual evaluation we need to auto-reset any camera variable 
	// that needs auto-resetting. Otherwise, we might end up with an update result 
	// that isn't representative of what would happen normally.
	// TODO: we might have to reset variables on the context's initial result too?
	OutResult.VariableTable.AutoResetValues();

	OnRunSingleCameraRig(Params, OutResult);
}

TSharedPtr<FCameraActionScope> FRootCameraNodeEvaluator::GetActiveCameraRigActionScope(bool bCreateIfNeeded)
{
	return OnGetActiveCameraRigActionScope(bCreateIfNeeded);
}

const FCameraNodeEvaluationResult& FRootCameraNodeEvaluator::GetPreVisualLayerResult() const
{
	return PreVisualResult;
}

void FRootCameraNodeEvaluator::SetPreVisualLayerResult(const FCameraNodeEvaluationResult& InResult)
{
	PreVisualResult.OverrideAll(InResult, true);
}

void FRootCameraNodeEvaluator::BroadcastCameraRigEvent(const FRootCameraNodeCameraRigEvent& InEvent) const
{
	OnCameraRigEventDelegate.Broadcast(InEvent);
}

}  // namespace UE::Cameras

