// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ShakeCameraNode.h"

#include "Core/CameraPose.h"
#include "Core/PostProcessSettingsCollection.h"
#include "Engine/Engine.h"
#include "IXRTrackingSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShakeCameraNode)

namespace UE::Cameras
{

void FCameraNodeShakeDelta::Combine(const FCameraNodeShakeDelta& Other, float OtherScale)
{
	Location += OtherScale * Other.Location;
	Rotation += OtherScale * Other.Rotation;
	FieldOfView += OtherScale * Other.FieldOfView;
}

void FCameraNodeShakeResult::ApplyDelta(const FCameraNodeShakeParams& Params)
{
	FVector3d FinalDeltaLocation = ShakeDelta.Location;
	FRotator3d FinalDeltaRotation = ShakeDelta.Rotation;
	float FinalDeltaFieldOfView = ShakeDelta.FieldOfView;

	// Apply the shake scale.
	if (Params.ShakeScale != 1.f)
	{
		FinalDeltaLocation *= Params.ShakeScale;
		FinalDeltaRotation *= Params.ShakeScale;
		FinalDeltaFieldOfView *= Params.ShakeScale;
	}

	// Apply the same limits as the vanilla Blueprint camera shakes:
	// - Don't allow shake to flip pitch past vertical, if not using a headset.
	// - If using a headset, we can't limit the camera locked to the player's head.
	if (!FinalDeltaRotation.IsZero() && (!GEngine->XRSystem.IsValid() || !GEngine->XRSystem->IsHeadTrackingAllowed()))
	{
		// Find normalized result when combined, and remove any offset that would push it past the limit.
		const float NormalizedInputPitch = FRotator::NormalizeAxis(ShakenResult.CameraPose.GetRotation().Pitch);
		const float NormalizedOutputPitchOffset = FRotator::NormalizeAxis(FinalDeltaRotation.Pitch);
		FinalDeltaRotation.Pitch = FMath::ClampAngle(NormalizedInputPitch + NormalizedOutputPitchOffset, -89.9f, 89.9f) - NormalizedInputPitch;
	}

	// Apply location and rotation.
	FCameraPose& CameraPose = ShakenResult.CameraPose;
	const FRotationMatrix CameraRotation(CameraPose.GetRotation());
	if (Params.PlaySpace == ECameraShakePlaySpace::CameraLocal)
	{
		// Convert the delta rotation/location into the camera's rotation space.
		const FVector3d ShakenPoseLocation = CameraPose.GetLocation() + CameraRotation.TransformVector(FinalDeltaLocation);
		CameraPose.SetLocation(ShakenPoseLocation);

		const FRotator3d ShakenPoseRotation = (FRotationMatrix(FinalDeltaRotation) * CameraRotation).Rotator();
		CameraPose.SetRotation(ShakenPoseRotation);
	}
	else if (Params.PlaySpace == ECameraShakePlaySpace::UserDefined)
	{
		const FMatrix& UserPlaySpaceMatrix = Params.UserPlaySpaceMatrix;

		const FVector3d ShakenPoseLocation = CameraPose.GetLocation() + UserPlaySpaceMatrix.TransformVector(FinalDeltaLocation);
		CameraPose.SetLocation(ShakenPoseLocation);

		const FMatrix WorldDeltaRotation = UserPlaySpaceMatrix.Inverse() * FRotationMatrix(FinalDeltaRotation) * UserPlaySpaceMatrix;
		const FMatrix LocalDeltaRotation = CameraRotation.Inverse() * WorldDeltaRotation;
		const FRotator3d ShakenPoseRotation = (LocalDeltaRotation * CameraRotation).Rotator();
		CameraPose.SetRotation(ShakenPoseRotation);
	}

	// Apply field of view.
	if (FinalDeltaFieldOfView != 0.f)
	{
		const double EffectiveFieldOfView = CameraPose.GetEffectiveFieldOfView();
		CameraPose.SetFieldOfView(EffectiveFieldOfView + FinalDeltaFieldOfView);
		CameraPose.SetFocalLength(-1.f);
	}
}

UE_DEFINE_CAMERA_NODE_EVALUATOR(FShakeCameraNodeEvaluator)

void FShakeCameraNodeEvaluator::ShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult)
{
	OnShakeResult(Params, OutResult);
}

void FShakeCameraNodeEvaluator::RestartShake(const FCameraNodeShakeRestartParams& Params)
{
	OnRestartShake(Params);
}

}  // namespace UE::Cameras

