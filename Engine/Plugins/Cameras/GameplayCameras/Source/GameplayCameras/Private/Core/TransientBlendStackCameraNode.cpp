// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/TransientBlendStackCameraNode.h"

#include "Algo/AnyOf.h"
#include "Core/BlendCameraNode.h"
#include "Core/BlendStackCameraRigEvent.h"
#include "Core/BlendStackRootCameraNode.h"
#include "Core/CameraAsset.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigCombinationRegistry.h"
#include "Core/CameraRigTransition.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Helpers/CameraRigTransitionFinder.h"
#include "Nodes/Blends/PopBlendCameraNode.h"
#include "Services/CameraActionEvaluator.h"
#include "Services/CameraActionScope.h"
#include "Services/CameraParameterSetterService.h"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FTransientBlendStackCameraNodeEvaluator)

FBlendStackEntryID FTransientBlendStackCameraNodeEvaluator::Push(const FBlendStackCameraPushParams& Params)
{
	bool bSearchedForTransition = false;
	const UCameraRigTransition* Transition = nullptr;

	if (!Entries.IsEmpty())
	{
		FCameraRigEntry& TopEntry(Entries.Top());
		if (!TopEntry.Flags.bIsFrozen 
				&& TopEntry.EvaluationContext == Params.EvaluationContext)
		{
			// Don't push anything if what is being requested is already the active 
			// camera rig.
			if (!Params.bForcePush && TopEntry.CameraRig == Params.CameraRig)
			{
				return FBlendStackEntryID();
			}

			// See if we can merge the new camera rig onto the active camera rig.
			const ECameraRigMergingEligibility Eligibility = TopEntry.RootEvaluator->CompareCameraRigForMerging(Params.CameraRig);

			if (!Params.bForcePush && Eligibility == ECameraRigMergingEligibility::Active)
			{
				// This camera rig is already the active one on the merged stack.
				return FBlendStackEntryID();
			}

			if (Eligibility == ECameraRigMergingEligibility::EligibleForMerge)
			{
				// This camera rig can be merged with the one current running. However, we
				// only do it if the transition explicitly allows it.
				bSearchedForTransition = true;
				Transition = FindTransition(Params);

				if (Transition && Transition->bAllowCameraRigMerging)
				{
					return PushMergedEntry(Params, Transition);
				}
			}
		}
	}

	// It's a legitimate new entry in the blend stack.
	if (!bSearchedForTransition)
	{
		bSearchedForTransition = true;
		Transition = FindTransition(Params);
	}

	return PushNewEntry(Params, Transition);
}

FBlendStackEntryID FTransientBlendStackCameraNodeEvaluator::PushNewEntry(const FBlendStackCameraPushParams& Params, const UCameraRigTransition* Transition)
{
	// Freeze all previous entries if requested.
	if (Transition && Transition->bFreezePreviousCameraRigs)
	{
		for (FCameraRigEntry& Entry : Entries)
		{
			FreezeEntry(Entry);
		}
	}

	// Create the new root node to wrap the new camera rig's root node, and the specific
	// blend node for this transition.
	// We need to const-cast here to be able to use our own blend stack node as the outer
	// of the new node.
	const UCameraRigTransition* UsedTransition = nullptr;
	UObject* Outer = const_cast<UObject*>((UObject*)GetCameraNode());
	UBlendStackRootCameraNode* EntryRootNode = NewObject<UBlendStackRootCameraNode>(Outer, NAME_None);
	{
		EntryRootNode->RootNode = Params.CameraRig->RootNode;

		// Find a transition and use its blend. If no transition is found,
		// make a camera cut transition.
		UBlendCameraNode* ModeBlend = nullptr;
		if (Transition)
		{
			ModeBlend = Transition->Blend;
			UsedTransition = Transition;
		}
		if (!ModeBlend)
		{
			ModeBlend = NewObject<UPopBlendCameraNode>(EntryRootNode, NAME_None);
		}
		EntryRootNode->Blend = ModeBlend;
	}

	// Make the new stack entry, and use its storage buffer to build the tree of evaluators.
	FCameraRigEntry NewEntry;
	InitializeEntry(
			NewEntry, 
			Params.CameraRig,
			Params.EvaluationContext,
			EntryRootNode,
			true);
	
#if WITH_EDITOR
	// Listen to changes to the packages inside which this camera rig is defined. We will hot-reload the
	// camera node evaluators for this camera rig when we detect changes.
	AddPackageListeners(NewEntry);
#endif  // WITH_EDITOR

	const FBlendStackEntryID AddedEntryID(NewEntry.EntryID);

	// Important: we need to move the new entry here because copying evaluator storage
	// is disabled.
	Entries.Add(MoveTemp(NewEntry));
	EntryExtraInfos.Emplace();

	if (OnCameraRigEventDelegate.IsBound())
	{
		BroadcastCameraRigEvent(EBlendStackCameraRigEventType::Pushed, Entries.Last(), UsedTransition);
	}

	return AddedEntryID;
}

FBlendStackEntryID FTransientBlendStackCameraNodeEvaluator::PushMergedEntry(const FBlendStackCameraPushParams& PushParams, const UCameraRigTransition* Transition)
{
	// Freeze all previous entries if requested, except the one we're merging into.
	if (Transition && Transition->bFreezePreviousCameraRigs)
	{
		for (int32 Index = 0; Index < Entries.Num() - 1; ++Index)
		{
			FCameraRigEntry& Entry = Entries[Index];
			FreezeEntry(Entry);
		}
	}

	const UBlendCameraNode* Blend = Transition ? Transition->Blend.Get() : nullptr;

	FCameraRigEntry& TopEntry = Entries.Top();
	FCameraNodeEvaluatorBuilder Builder(TopEntry.EvaluatorStorage);
	FCameraNodeEvaluatorBuildParams BuildParams(Builder);

	FCameraNodeEvaluatorInitializeParams InitParams;
	InitParams.Evaluator = OwningEvaluator;
	InitParams.EvaluationContext = TopEntry.EvaluationContext.Pin();
	InitParams.Layer = Layer;

	TopEntry.RootEvaluator->MergeCameraRig(BuildParams, InitParams, TopEntry.Result, PushParams.CameraRig, Blend);

	// Merging a camera rig the first time changes the evaluator tree by removing any prefab nodes
	// near the root... so we need to rebuild our hierarchy cache.
	TopEntry.EvaluatorHierarchy.Build(TopEntry.RootEvaluator);

	// Swap out the camera rig registered as "active" for this entry.
#if WITH_EDITOR
	RemoveListenedPackages(TopEntry);
#endif
	{
		TopEntry.CameraRig = PushParams.CameraRig;
	}
#if WITH_EDITOR
	AddPackageListeners(TopEntry);
#endif

	return TopEntry.EntryID;
}

void FTransientBlendStackCameraNodeEvaluator::Freeze(const FBlendStackCameraFreezeParams& Params)
{
	if (Params.EntryID.IsValid())
	{
		// Freeze the entry by ID.
		const int32 EntryIndex = IndexOfEntry(Params.EntryID);
		if (EntryIndex != INDEX_NONE)
		{
			FreezeEntry(Entries[EntryIndex]);
		}
	}
	else
	{
		// Freeze any entries matching the given context and rig asset.
		for (FCameraRigEntry& Entry : Entries)
		{
			if (!Entry.Flags.bIsFrozen && 
					Entry.CameraRig == Params.CameraRig &&
					Entry.EvaluationContext == Params.EvaluationContext)
			{
				FreezeEntry(Entry);
			}
		}
	}
}

void FTransientBlendStackCameraNodeEvaluator::FreezeAll(TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	for (FCameraRigEntry& Entry : Entries)
	{
		if (!Entry.Flags.bIsFrozen && Entry.EvaluationContext == EvaluationContext)
		{
			FreezeEntry(Entry);
		}
	}
}

TSharedPtr<FCameraActionScope> FTransientBlendStackCameraNodeEvaluator::GetActiveCameraRigActionScope(bool bCreateIfNeeded)
{
	if (!EntryExtraInfos.IsEmpty())
	{
		FCameraRigEntryExtraInfo& ActiveEntryExtraInfo = EntryExtraInfos.Last();
		if (bCreateIfNeeded && !ActiveEntryExtraInfo.ActionScope.IsValid())
		{
			ActiveEntryExtraInfo.ActionScope = MakeShared<FCameraActionScope>();

			FCameraRigEvaluationInfo CameraRigInfo = GetActiveCameraRigEvaluationInfo();
			ActiveEntryExtraInfo.ActionScope->Initialize(CameraRigInfo);
		}
		return ActiveEntryExtraInfo.ActionScope;
	}
	return nullptr;
}

void FTransientBlendStackCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	ParameterSetterService = Params.Evaluator->FindEvaluationService<FCameraParameterSetterService>();
}

void FTransientBlendStackCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	ensure(Entries.Num() == EntryExtraInfos.Num());

	// Validate our entries and resolve evaluation context weak pointers.
	TArray<FResolvedEntry> ResolvedEntries;
	ResolveEntries(Params, ResolvedEntries);

	// Gather parameters to pre-blend, and evaluate blend nodes.
	InternalPreBlendPrepare(ResolvedEntries, Params, OutResult);

	// Blend input variables.
	InternalPreBlendExecute(ResolvedEntries, Params, OutResult);

	// Run the root nodes. They will use the pre-blended inputs from the last step.
	// Frozen entries are skipped, since they only ever use the last result they produced.
	InternalUpdate(ResolvedEntries, Params, OutResult);

	// Now blend all the results, keeping track of blends that have reached 100% so
	// that we can remove any camera rigs below (since they would have been completely
	// blended out by that).
	InternalPostBlendExecute(ResolvedEntries, Params, OutResult);

	// Tidy up.
	OnRunFinished(OutResult);
	InternalRunFinished(OutResult);
}

void FTransientBlendStackCameraNodeEvaluator::InternalPreBlendPrepare(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	constexpr ECameraVariableTableFilter VariableTableFilter = ECameraVariableTableFilter::ChangedOnly;
	constexpr ECameraContextDataTableFilter ContextDataTableFilter = ECameraContextDataTableFilter::ChangedOnly;

	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[ResolvedEntry.EntryIndex]);

		if (UNLIKELY(Entry.Flags.bIsFrozen || !ResolvedEntry.Context.IsValid()))
		{
			continue;
		}

		FCameraNodeEvaluationParams CurParams(Params);
		CurParams.EvaluationContext = ResolvedEntry.Context;
		CurParams.bIsFirstFrame = Entry.Flags.bIsFirstFrame;
		CurParams.bIsActiveCameraRig = ResolvedEntry.bIsActiveEntry;

		FCameraNodeEvaluationResult& CurResult(Entry.Result);

		// Start with the input given to us.
		CurResult.VariableTable.OverrideAll(OutResult.VariableTable);

		// Override it with whatever the evaluation context has set on its result this frame.
		const FCameraNodeEvaluationResult& ContextResult(Entry.ContextResult);
		CurResult.VariableTable.OverrideAll(ContextResult.VariableTable, true);
		CurResult.ContextDataTable.OverrideAll(ContextResult.ContextDataTable);

		// Override it with variable setters.
		if (ParameterSetterService)
		{
			ParameterSetterService->ApplyCameraVariableSetters(CurResult.VariableTable);
		}

		// Gather input parameters if needed (and remember if it was indeed needed).
		if (!EntryExtraInfo.bInputRunThisFrame)
		{
			bool bHasPreBlendedParameters = false;
			FCameraBlendedParameterUpdateParams InputParams(CurParams, CurResult.CameraPose);
			FCameraBlendedParameterUpdateResult InputResult(CurResult.VariableTable);

			Entry.EvaluatorHierarchy.ForEachEvaluator(ECameraNodeEvaluatorFlags::NeedsParameterUpdate,
					[&bHasPreBlendedParameters, &InputParams, &InputResult](FCameraNodeEvaluator* ParameterEvaluator)
					{
						ParameterEvaluator->UpdateParameters(InputParams, InputResult);
						bHasPreBlendedParameters = true;
					});

			EntryExtraInfo.bHasPreBlendedParameters = bHasPreBlendedParameters;
			EntryExtraInfo.bInputRunThisFrame = true;
		}

		// Run blends.
		// Note that we pass last frame's camera pose to the Run() method. This may change.
		// Blends aren't expected to use the camera pose to do any logic until BlendResults().
		if (!EntryExtraInfo.bBlendRunThisFrame)
		{
			FBlendCameraNodeEvaluator* EntryBlendEvaluator = Entry.RootEvaluator->GetBlendEvaluator();
			if (EntryBlendEvaluator)
			{
				EntryBlendEvaluator->Run(CurParams, CurResult);
			}

			EntryExtraInfo.bBlendRunThisFrame = true;
		}
	}
}

void FTransientBlendStackCameraNodeEvaluator::InternalPreBlendExecute(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Blend all the camera rigs' input variables (include private variables such as camera rig parameters).
	// The result of this pre-blending goes into PreBlendVariableTable, which will then have a big mix
	// of all the variables.
	PreBlendVariableTable.ClearAllWrittenThisFrameFlags();

	if (!Algo::AnyOf(EntryExtraInfos, [](const FCameraRigEntryExtraInfo& Item) { return Item.bHasPreBlendedParameters; }))
	{
		return;
	}

	constexpr ECameraVariableTableFilter VariableTableFilter = ECameraVariableTableFilter::InputOnly;

	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[ResolvedEntry.EntryIndex]);

		FCameraNodeEvaluationResult& CurResult(Entry.Result);

		// Running entries pre-blend the values that were put in the variable table in InternalPreBlendPrepare.
		// Frozen entries still contribute by blending their last evaluated values.
		FCameraNodeEvaluationParams CurParams(Params);
		CurParams.EvaluationContext = ResolvedEntry.Context;
		CurParams.bIsFirstFrame = Entry.Flags.bIsFirstFrame;

		FCameraNodePreBlendParams PreBlendParams(CurParams, CurResult.CameraPose, CurResult.VariableTable);
		PreBlendParams.VariableTableFilter = VariableTableFilter;

		FCameraNodePreBlendResult PreBlendResult(PreBlendVariableTable);

		FBlendCameraNodeEvaluator* EntryBlendEvaluator = Entry.RootEvaluator ? Entry.RootEvaluator->GetBlendEvaluator() : nullptr;
		if (EntryBlendEvaluator)
		{
			EntryBlendEvaluator->BlendParameters(PreBlendParams, PreBlendResult);
			EntryExtraInfo.bIsPreBlendFull = (PreBlendResult.bIsBlendFinished && PreBlendResult.bIsBlendFull);
		}
		else
		{
			PreBlendVariableTable.Override(CurResult.VariableTable, PreBlendParams.VariableTableFilter);
			EntryExtraInfo.bIsPreBlendFull = true;
		}
	}

	// Write the values back to each entry table, so that each of these camera rigs will run with
	// the pre-blended values. We limit this writing to the variables each of them knows, since
	// there's no need to add entries they don't use to their variable table.
	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		if (!Entry.Flags.bIsFrozen && ResolvedEntry.Context.IsValid())
		{
			FCameraNodeEvaluationResult& CurResult(Entry.Result);
			CurResult.VariableTable.Override(PreBlendVariableTable, ECameraVariableTableFilter::KnownOnly);
		}
	}
}

void FTransientBlendStackCameraNodeEvaluator::InternalUpdate(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[ResolvedEntry.EntryIndex]);

		if (UNLIKELY(Entry.Flags.bIsFrozen || !ResolvedEntry.Context.IsValid()))
		{
			continue;
		}

		FCameraNodeEvaluationResult& CurResult(Entry.Result);

		FCameraNodeEvaluationParams CurParams(Params);
		CurParams.EvaluationContext = ResolvedEntry.Context;
		CurParams.bIsFirstFrame = Entry.Flags.bIsFirstFrame;
		CurParams.bIsActiveCameraRig = ResolvedEntry.bIsActiveEntry;

		// Start with the input given to us.
		CurResult.Reset();
		CurResult.CameraPose = OutResult.CameraPose;
		CurResult.CameraRigJoints.OverrideAll(OutResult.CameraRigJoints);
		CurResult.PostProcessSettings.OverrideAll(OutResult.PostProcessSettings);

		// Override it with whatever the evaluation context has set on its result.
		const FCameraNodeEvaluationResult& ContextResult(Entry.ContextResult);
		CurResult.CameraPose.OverrideChanged(ContextResult.CameraPose);
		CurResult.bIsCameraCut = OutResult.bIsCameraCut || ContextResult.bIsCameraCut || Entry.Flags.bForceCameraCut;
		
		CurResult.bIsValid = true;

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
		CurResult.AddCameraPoseTrailPointIfNeeded(ContextResult.CameraPose.GetLocation());
#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

		const bool bHasActionScope = EntryExtraInfo.ActionScope.IsValid();
		if (bHasActionScope)
		{
			EntryExtraInfo.ActionScope->PreScopeRun(CurParams, CurResult);
		}

		// Run the camera rig's root node.
		FCameraNodeEvaluator* RootEvaluator = Entry.RootEvaluator->GetRootEvaluator();
		if (RootEvaluator)
		{
			RootEvaluator->Run(CurParams, CurResult);
		}

		if (bHasActionScope)
		{
			EntryExtraInfo.ActionScope->PostScopeRun(CurParams, CurResult);
		}
	}
}

void FTransientBlendStackCameraNodeEvaluator::InternalPostBlendExecute(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	int32 PopEntriesBelow = INDEX_NONE;
	for (FResolvedEntry& ResolvedEntry : ResolvedEntries)
	{
		FCameraRigEntry& Entry(ResolvedEntry.Entry);
		FCameraRigEntryExtraInfo& EntryExtraInfo(EntryExtraInfos[ResolvedEntry.EntryIndex]);

		FCameraNodeEvaluationResult& CurResult(Entry.Result);

		FCameraNodeEvaluationParams CurParams(Params);
		CurParams.EvaluationContext = ResolvedEntry.Context;
		CurParams.bIsFirstFrame = Entry.Flags.bIsFirstFrame;
		FCameraNodeBlendParams BlendParams(CurParams, CurResult);

		FCameraNodeBlendResult BlendResult(OutResult);

		FBlendCameraNodeEvaluator* EntryBlendEvaluator = Entry.RootEvaluator ? Entry.RootEvaluator->GetBlendEvaluator() : nullptr;
		if (EntryBlendEvaluator)
		{
			EntryBlendEvaluator->BlendResults(BlendParams, BlendResult);

			if (BlendResult.bIsBlendFull && BlendResult.bIsBlendFinished)
			{
				PopEntriesBelow = ResolvedEntry.EntryIndex;
			}
		}
		else
		{
			OutResult.OverrideAll(CurResult);

			PopEntriesBelow = ResolvedEntry.EntryIndex;
		}
	}

	// Pop out camera rigs that have been blended out.
	if (!Params.IsStatelessEvaluation() && PopEntriesBelow != INDEX_NONE)
	{
		PopEntries(PopEntriesBelow);

		EntryExtraInfos.RemoveAt(0, PopEntriesBelow);
	}
}

void FTransientBlendStackCameraNodeEvaluator::InternalRunFinished(FCameraNodeEvaluationResult& OutResult)
{
	for (FCameraRigEntryExtraInfo& ExtraInfo : EntryExtraInfos)
	{
		ExtraInfo.bInputRunThisFrame = false;
		ExtraInfo.bBlendRunThisFrame = false;
		ExtraInfo.bHasPreBlendedParameters = false;
		ExtraInfo.bIsPreBlendFull = false;
	}
}

void FTransientBlendStackCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	for (FCameraRigEntryExtraInfo& ExtraInfo : EntryExtraInfos)
	{
		if (ExtraInfo.ActionScope)
		{
			FCameraActionEvaluatorSerializeParams ActionScopeParams;
			ExtraInfo.ActionScope->Serialize(ActionScopeParams, Ar);
		}

		Ar << ExtraInfo.bInputRunThisFrame;
		Ar << ExtraInfo.bBlendRunThisFrame;
		Ar << ExtraInfo.bHasPreBlendedParameters;
		Ar << ExtraInfo.bIsPreBlendFull;
	}
}

void FTransientBlendStackCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Super::OnAddReferencedObjects(Collector);

	for (FCameraRigEntryExtraInfo& ExtraInfo : EntryExtraInfos)
	{
		if (ExtraInfo.ActionScope)
		{
			ExtraInfo.ActionScope->AddReferencedObjects(Collector);
		}
	}
}

const UCameraRigTransition* FTransientBlendStackCameraNodeEvaluator::FindTransition(const FBlendStackCameraPushParams& Params) const
{
	// If we are forced to use a specific transition, our search is over.
	if (Params.TransitionOverride)
	{
		return Params.TransitionOverride;
	}

	// Find a transition that works for blending towards ToCameraRig.
	// If the stack isn't empty, we need to find a transition that works between the previous and 
	// next camera rigs. If the stack is empty, we blend the new camera rig in from nothing if
	// appropriate.
	if (!Entries.IsEmpty())
	{
		// Grab information about the new entry to push.
		TSharedPtr<const FCameraEvaluationContext> ToContext = Params.EvaluationContext;
		const UCameraAsset* ToCameraAsset = ToContext ? ToContext->GetCameraAsset() : nullptr;

		// Grab information about the top entry (i.e. the currently active camera rig).
		const FCameraRigEntry& TopEntry = Entries.Top();
		TSharedPtr<const FCameraEvaluationContext> FromContext = TopEntry.EvaluationContext.Pin();
		const UCameraAsset* FromCameraAsset = FromContext ? FromContext->GetCameraAsset() : nullptr;

		// If the new or current top entries are a combination, look for transitions on all 
		// their combined camera rigs.
		TArray<const UCameraRigAsset*> ToCombinedCameraRigs;
		UCombinedCameraRigsCameraNode::GetAllCombinationCameraRigs(Params.CameraRig, ToCombinedCameraRigs);

		TArray<const UCameraRigAsset*> FromCombinedCameraRigs;
		UCombinedCameraRigsCameraNode::GetAllCombinationCameraRigs(TopEntry.CameraRig, FromCombinedCameraRigs);

		const bool bFromFrozen = TopEntry.Flags.bIsFrozen;
		const UCameraRigTransition* TransitionToUse = nullptr;

		// Start by looking at exit transitions on the last active (top) camera rig.
		for (const UCameraRigAsset* FromCameraRig : FromCombinedCameraRigs)
		{
			if (!FromCameraRig->ExitTransitions.IsEmpty())
			{
				// Look for exit transitions on the last active camera rig itself.
				for (const UCameraRigAsset* ToCameraRig : ToCombinedCameraRigs)
				{
					TransitionToUse = FCameraRigTransitionFinder::FindTransition(
							FromCameraRig->ExitTransitions,
							FromContext, FromCameraRig, FromCameraAsset, bFromFrozen,
							ToContext, ToCameraRig, ToCameraAsset);
					if (TransitionToUse)
					{
						return TransitionToUse;
					}
				}
			}
		}
		for (const UCameraRigAsset* FromCameraRig : FromCombinedCameraRigs)
		{
			if (FromCameraAsset && !FromCameraAsset->GetExitTransitions().IsEmpty())
			{
				// Look for exit transitions on its parent camera asset.
				for (const UCameraRigAsset* ToCameraRig : ToCombinedCameraRigs)
				{
					TransitionToUse = FCameraRigTransitionFinder::FindTransition(
							FromCameraAsset->GetExitTransitions(),
							FromContext, FromCameraRig, FromCameraAsset, bFromFrozen,
							ToContext, ToCameraRig, ToCameraAsset);
					if (TransitionToUse)
					{
						return TransitionToUse;
					}
				}
			}
		}

		// Now look at enter transitions on the new camera rig.
		for (const UCameraRigAsset* ToCameraRig : ToCombinedCameraRigs)
		{
			if (!ToCameraRig->EnterTransitions.IsEmpty())
			{
				// Look for enter transitions on the new camera rig itself.
				for (const UCameraRigAsset* FromCameraRig : FromCombinedCameraRigs)
				{
					TransitionToUse = FCameraRigTransitionFinder::FindTransition(
							ToCameraRig->EnterTransitions,
							FromContext, FromCameraRig, FromCameraAsset, bFromFrozen,
							ToContext, ToCameraRig, ToCameraAsset);
					if (TransitionToUse)
					{
						return TransitionToUse;
					}
				}
			}
		}
		for (const UCameraRigAsset* ToCameraRig : ToCombinedCameraRigs)
		{
			if (ToCameraAsset && !ToCameraAsset->GetEnterTransitions().IsEmpty())
			{
				// Look at enter transitions on its parent camera asset.
				for (const UCameraRigAsset* FromCameraRig : FromCombinedCameraRigs)
				{
					TransitionToUse = FCameraRigTransitionFinder::FindTransition(
							ToCameraAsset->GetEnterTransitions(),
							FromContext, FromCameraRig, FromCameraAsset, bFromFrozen,
							ToContext, ToCameraRig, ToCameraAsset);
					if (TransitionToUse)
					{
						return TransitionToUse;
					}
				}
			}
		}
	}
	// else: make the first camera rig in the stack start at 100% blend immediately.

	return nullptr;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FActionScopesCameraDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(, FActionScopesCameraDebugBlock)

public:

	FActionScopesCameraDebugBlock() = default;

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
};

void FTransientBlendStackCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	Super::OnBuildDebugBlocks(Params, Builder);

	FActionScopesCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FActionScopesCameraDebugBlock>();

	for (const FCameraRigEntryExtraInfo& EntryExtraInfo : EntryExtraInfos)
	{
		if (EntryExtraInfo.ActionScope.IsValid())
		{
			EntryExtraInfo.ActionScope->BuildDebugBlocks(Params, Builder);
		}
	}
}

UE_DEFINE_CAMERA_DEBUG_BLOCK(FActionScopesCameraDebugBlock)

void FActionScopesCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
}

#endif

#if WITH_EDITOR

void FTransientBlendStackCameraNodeEvaluator::OnEntryReinitialized(int32 EntryIndex)
{
	// Empty our pre-blend variable table in case it has variables that have been changed or are
	// not valid anymore, such as the user changing the type of a camera rig parameter.
	FCameraVariableTableAllocationInfo EmptyAllocationInfo;
	PreBlendVariableTable.Initialize(EmptyAllocationInfo);
}

#endif

}  // namespace UE::Cameras

