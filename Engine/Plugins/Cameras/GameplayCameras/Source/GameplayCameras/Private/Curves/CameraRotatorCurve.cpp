// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/CameraRotatorCurve.h"

#include "Math/Rotator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRotatorCurve)

FRotator FCameraRotatorCurve::GetValue(float InTime) const
{
	FRotator Result;
	Result.Pitch = Curves[0].Eval(InTime);
	Result.Yaw = Curves[1].Eval(InTime);
	Result.Roll = Curves[2].Eval(InTime);
	return Result;
}

bool FCameraRotatorCurve::HasAnyData() const
{
	return Curves[0].HasAnyData() || Curves[1].HasAnyData() || Curves[2].HasAnyData();
}

