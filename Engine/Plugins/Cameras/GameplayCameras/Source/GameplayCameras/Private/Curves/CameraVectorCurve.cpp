// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curves/CameraVectorCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVectorCurve)

FVector FCameraVectorCurve::GetValue(float InTime) const
{
	FVector Result;
	Result.X = Curves[0].Eval(InTime);
	Result.Y = Curves[1].Eval(InTime);
	Result.Z = Curves[2].Eval(InTime);
	return Result;
}

bool FCameraVectorCurve::HasAnyData() const
{
	return Curves[0].HasAnyData() || Curves[1].HasAnyData() || Curves[2].HasAnyData();
}

