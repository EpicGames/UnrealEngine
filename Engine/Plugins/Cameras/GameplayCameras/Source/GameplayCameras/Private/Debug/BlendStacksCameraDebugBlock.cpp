// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/BlendStacksCameraDebugBlock.h"

#include "Core/BlendStackCameraNode.h"
#include "Debug/CameraDebugCategories.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

FString GGameplayCamerasDebugBlendStacksFilter;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBlendStacksFilter(
	TEXT("GameplayCameras.Debug.BlendStacks.Filter"),
	GGameplayCamerasDebugBlendStacksFilter,
	TEXT(""));

UE_DEFINE_CAMERA_DEBUG_BLOCK(FBlendStacksCameraDebugBlock)

FBlendStacksCameraDebugBlock::FBlendStacksCameraDebugBlock()
{
}

void FBlendStacksCameraDebugBlock::AddBlendStack(const FString& InBlendStackName, FBlendStackCameraDebugBlock* InDebugBlock)
{
	BlendStackNames.Add(InBlendStackName);
	AddChild(InDebugBlock);
}

void FBlendStacksCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (!Params.IsCategoryActive(FCameraDebugCategories::BlendStacks))
	{
		Renderer.SkipAllBlocks();
		return;
	}
	
	const FCameraDebugColors& Colors = FCameraDebugColors::Get();

	Renderer.SetTextColor(Colors.Title);
	Renderer.AddText(TEXT("Blend Stacks\n\n"));
	Renderer.SetTextColor(Colors.Default);

	TArrayView<FCameraDebugBlock*> BlendStackBlocks(GetChildren());

	const int32 MaxNum = FMath::Max(BlendStackBlocks.Num(), BlendStackNames.Num());

	for (int32 Index = 0; Index < MaxNum; ++Index)
	{
		const FString CurrentName = BlendStackNames.IsValidIndex(Index) ? BlendStackNames[Index] : TEXT("<unnamed blend stack>");

		Renderer.AddText(CurrentName);
		Renderer.NewLine();

		const bool bDoDebugDraw = (
				GGameplayCamerasDebugBlendStacksFilter.IsEmpty() ||
				CurrentName.Contains(GGameplayCamerasDebugBlendStacksFilter));
		if (bDoDebugDraw)
		{
			if (BlendStackBlocks.IsValidIndex(Index))
			{
				FCameraDebugBlock* BlendStackBlock = BlendStackBlocks[Index];
				BlendStackBlock->DebugDraw(Params, Renderer);
			}
			else
			{
				Renderer.AddText(TEXT("<missing blend stack>"));
			}
		}
	}

	// We've already manually renderered our children blocks.
	Renderer.SkipAllBlocks();
}

void FBlendStacksCameraDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << BlendStackNames;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

