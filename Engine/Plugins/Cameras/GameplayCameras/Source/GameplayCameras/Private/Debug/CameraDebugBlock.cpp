// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugBlock.h"

#include "Debug/CameraDebugRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

bool FCameraDebugBlockDrawParams::IsCategoryActive(const FString& InCategory) const
{
	return ActiveCategories.Contains(InCategory);
}

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraDebugBlock)

void FCameraDebugBlock::Attach(FCameraDebugBlock* InAttachment)
{
	Attachments.Add(InAttachment);
}

void FCameraDebugBlock::AddChild(FCameraDebugBlock* InChild)
{
	Children.Add(InChild);
}


void FCameraDebugBlock::DebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.ResetVisitFlags();

	OnDebugDraw(Params, Renderer);
	
	ECameraDebugDrawVisitFlags VisitFlags = Renderer.GetVisitFlags();

	// Attachments can render on the same line as this debug block so we call DebugDraw directly on them.
	if (!EnumHasAnyFlags(VisitFlags, ECameraDebugDrawVisitFlags::SkipAttachedBlocks) && !Attachments.IsEmpty())
	{
		for (FCameraDebugBlock* Attachment : Attachments)
		{
			Attachment->DebugDraw(Params, Renderer);
		}
	}

	// Children should always render on lines below so we need to make sure we're on a new line for
	// the remainder of this function. The call to AddIndent() below will flush any pending text and 
	// add this new line automatically, but we need to add the optional new line ourselves if we
	// happen to skip this next section.
	//
	if (!EnumHasAnyFlags(VisitFlags, ECameraDebugDrawVisitFlags::SkipChildrenBlocks) && !Children.IsEmpty())
	{
		Renderer.AddIndent();

		for (FCameraDebugBlock* Child : Children)
		{
			Child->DebugDraw(Params, Renderer);
		}

		Renderer.RemoveIndent();
	}
	else
	{
		Renderer.NewLine(true);
	}

	OnPostDebugDraw(Params, Renderer);
}

void FCameraDebugBlock::Serialize(FArchive& Ar)
{
	OnSerialize(Ar);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

