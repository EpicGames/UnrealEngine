// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugBlockBuilder.h"

#include "Debug/CameraDebugBlock.h"
#include "Debug/RootCameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

FCameraDebugBlockBuilder::FCameraDebugBlockBuilder(FCameraDebugBlockStorage& InStorage, FRootCameraDebugBlock& InRootBlock)
	: Storage(InStorage)
	, RootBlock(InRootBlock)
{
	// We should always have the root block in the working hierarchy.
	CurrentHierarchy.Add(&InRootBlock);
}

void FCameraDebugBlockBuilder::OnAttachDebugBlock(FCameraDebugBlock* InNewBlock)
{
	if (ensureMsgf(CurrentHierarchy.Num() > 0, TEXT("Can't attach block, no current block defined!")))
	{
		CurrentHierarchy.Last()->Attach(InNewBlock);
	}
}

void FCameraDebugBlockBuilder::OnStartChildDebugBlock(FCameraDebugBlock* InNewBlock)
{
	if (ensureMsgf(CurrentHierarchy.Num() > 0, TEXT("Can't add child block, no current block defined!")))
	{
		CurrentHierarchy.Last()->AddChild(InNewBlock);
	}
	CurrentHierarchy.Add(InNewBlock);
}

void FCameraDebugBlockBuilder::EndChildDebugBlock()
{
	if (ensureMsgf(CurrentHierarchy.Num() > 0, TEXT("Can't end block, no current block defined!")))
	{
		CurrentHierarchy.Pop();
	}
}

void FCameraDebugBlockBuilder::SkipChildren()
{
	VisitFlags |= ECameraDebugBlockBuildVisitFlags::SkipChildren;
}

void FCameraDebugBlockBuilder::ResetVisitFlags()
{
	VisitFlags = ECameraDebugBlockBuildVisitFlags::None;
}

void FCameraDebugBlockBuilder::StartParentDebugBlockOverride(FCameraDebugBlock& InNewParent)
{
	if (ensureMsgf(HierarchyOverrideStart == INDEX_NONE, TEXT("Can't override parenting, an override is already in progress.")))
	{
		HierarchyOverrideStart = CurrentHierarchy.Num();
		CurrentHierarchy.Add(&InNewParent);
	}
}

void FCameraDebugBlockBuilder::EndParentDebugBlockOverride()
{
	if (ensureMsgf(HierarchyOverrideStart != INDEX_NONE, TEXT("No parenting override active.")))
	{
		ensureMsgf(
				HierarchyOverrideStart == CurrentHierarchy.Num() - 1,
				TEXT("Mismatch between expected parenting override and actual hierarchy level. "
					 "Did you forget to end some child blocks?"));
		while (CurrentHierarchy.Num() > HierarchyOverrideStart)
		{
			CurrentHierarchy.Pop();
		}
		HierarchyOverrideStart = INDEX_NONE;
	}
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

