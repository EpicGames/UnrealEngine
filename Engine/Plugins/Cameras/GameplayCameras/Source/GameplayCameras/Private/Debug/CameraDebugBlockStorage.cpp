// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugBlockStorage.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

void FCameraDebugBlockStorage::DestroyDebugBlocks(bool bFreeAllocations)
{
	Super::DestroyObjects(bFreeAllocations);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

