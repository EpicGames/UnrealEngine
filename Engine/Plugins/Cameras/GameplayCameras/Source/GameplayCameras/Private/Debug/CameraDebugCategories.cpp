// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugCategories.h"

#include "Containers/UnrealString.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

const FString FCameraDebugCategories::NodeTree("nodetree");
const FString FCameraDebugCategories::DirectorTree("directortree");
const FString FCameraDebugCategories::BlendStacks("blendstacks");
const FString FCameraDebugCategories::Services("services");
const FString FCameraDebugCategories::PoseStats("posestats");
const FString FCameraDebugCategories::Viewfinder("viewfinder");

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

