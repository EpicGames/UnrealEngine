// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugClock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

#include "Debug/CameraDebugColors.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "CameraDebugClock"

namespace UE::Cameras
{
	
float GGameplayCamerasDebugClockPadding = 10.f;
FAutoConsoleVariableRef CVarGameplayCamerasDebugClockPadding(
		TEXT("GameplayCameras.DebugClock.Margin"),
		GGameplayCamerasDebugClockPadding,
		TEXT("Default: 10px. The uniform padding inside the debug clock card."));

float GGameplayCamerasDebugClockArrowThickness = 3.f;
FAutoConsoleVariableRef CVarGameplayCamerasDebugClockArrowThickness(
		TEXT("GameplayCameras.DebugClock.ArrowThickness"),
		GGameplayCamerasDebugClockArrowThickness,
		TEXT("Default: 3px. The thickness of the clock arrow."));

extern float GGameplayCamerasDebugBackgroundOpacity;

FCameraDebugClockDrawParams::FCameraDebugClockDrawParams()
{
	const FCameraDebugColors& ColorScheme = FCameraDebugColors::Get();
	ClockBackgroundColor = ColorScheme.Background.WithAlpha((uint8)(GGameplayCamerasDebugBackgroundOpacity * 255));
	ClockNameColor = ColorScheme.Title;
	ClockFaceColor = ColorScheme.Passive;
	ClockValueLineColor = ColorScheme.Warning;
}

namespace Internal
{

FCameraDebugClockRenderer::FCameraDebugClockRenderer(FCanvas* InCanvas, const FCameraDebugClockDrawParams& InDrawParams)
	: Canvas(InCanvas)
	, DrawParams(InDrawParams)
{
}

void FCameraDebugClockRenderer::DrawVectorClock(const FVector2d& Value, double MaxLength) const
{
	DrawFrame();

	DrawCurrentValue(FText::Format(LOCTEXT("CurrentValueFmt", "{0}"), FText::FromString(Value.ToString())));

	double ClockRadius;
	FVector2f ClockCenter;
	GetClockFaceParams(ClockCenter, ClockRadius);

	const double ValueToPixels = MaxLength != 0 ? ClockRadius / MaxLength : 1.f;
	FVector2f ScreenValue(Value * ValueToPixels);
	ScreenValue.Y = -ScreenValue.Y;  // Flip Y for UI-space coordinates.

	FCanvasLineItem ValueLineItem(FVector2D(ClockCenter), FVector2D(ClockCenter + ScreenValue));
	ValueLineItem.SetColor(DrawParams.ClockValueLineColor);
	ValueLineItem.LineThickness = GGameplayCamerasDebugClockArrowThickness;
	Canvas->DrawItem(ValueLineItem);
}

void FCameraDebugClockRenderer::DrawAngleClock(double Angle) const
{
	DrawFrame();

	DrawCurrentValue(FText::Format(LOCTEXT("CurrentValueFmt", "{0}"), Angle));

	double ClockRadius;
	FVector2f ClockCenter;
	GetClockFaceParams(ClockCenter, ClockRadius);

	FVector2f ScreenValue(ClockRadius * FMath::Cos(Angle), ClockRadius * FMath::Sin(Angle));
	ScreenValue.Y = -ScreenValue.Y;  // Flip Y for UI-space coordinates.

	FCanvasLineItem ValueLineItem(FVector2D(ClockCenter), FVector2D(ClockCenter + ScreenValue));
	ValueLineItem.SetColor(DrawParams.ClockValueLineColor);
	ValueLineItem.LineThickness = 3.f;
	Canvas->DrawItem(ValueLineItem);
}

void FCameraDebugClockRenderer::DrawFrame() const
{
	// Draw the background tile.
	{
		FCanvasTileItem TileItem(
				FVector2D(DrawParams.ClockPosition),
				FVector2D(DrawParams.ClockSize),
				DrawParams.ClockBackgroundColor);
		TileItem.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem(TileItem);
	}

	// Draw the clock name.
	if (!DrawParams.ClockName.IsEmpty())
	{
		UFont* SmallFont = GEngine->GetSmallFont();
		const float MaxSmallFontCharHeight = SmallFont->GetMaxCharHeight();

		const FVector2f ClockNamePosition(DrawParams.ClockPosition + 
				FVector2f(
					GGameplayCamerasDebugClockPadding, 
					DrawParams.ClockSize.Y - GGameplayCamerasDebugClockPadding - MaxSmallFontCharHeight));
		FCanvasTextItem ClockNameItem(
				FVector2D(ClockNamePosition), 
				DrawParams.ClockName, 
				SmallFont, 
				DrawParams.ClockNameColor);
		Canvas->DrawItem(ClockNameItem);
	}

	// Draw the clock face.
	double ClockRadius;
	FVector2f ClockCenter;
	GetClockFaceParams(ClockCenter, ClockRadius);
	{
		const int32 NumSides = FMath::Max(20, (int)ClockRadius / 25);
		const float	AngleDelta = 2.0f * UE_PI / NumSides;
		const FVector2f AxisX(1.f, 0.f);
		const FVector2f AxisY(0.f, -1.f);
		FVector2f LastVertex = ClockCenter + AxisX * ClockRadius;

		for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
		{
			const float CurAngle = AngleDelta * (SideIndex + 1);
			const FVector2f Vertex = ClockCenter + (AxisX * FMath::Cos(CurAngle) + AxisY * FMath::Sin(CurAngle)) * ClockRadius;

			FCanvasLineItem LineItem{ FVector2D(LastVertex), FVector2D(Vertex) };
			LineItem.SetColor(DrawParams.ClockFaceColor);
			Canvas->DrawItem(LineItem);

			LastVertex = Vertex;
		}
	}
}

void FCameraDebugClockRenderer::DrawCurrentValue(const FText& CurrentValueStr) const
{
	UFont* TinyFont = GEngine->GetTinyFont();
	const float MaxTinyFontCharHeight = TinyFont->GetMaxCharHeight();

	const FVector2f CurrentValuePosition = DrawParams.ClockPosition + FVector2f(GGameplayCamerasDebugClockPadding);
	FCanvasTextItem TextItem(
			FVector2D(CurrentValuePosition),
			CurrentValueStr,
			TinyFont,
			DrawParams.ClockValueLineColor);
	Canvas->DrawItem(TextItem);
}

void FCameraDebugClockRenderer::GetClockFaceParams(FVector2f& OutClockCenter, double& OutClockRadius) const
{
	UFont* SmallFont = GEngine->GetSmallFont();
	const float MaxSmallFontCharHeight = SmallFont->GetMaxCharHeight();
	const double ClockAreaHeight = DrawParams.ClockSize.Y - 3 * GGameplayCamerasDebugClockPadding - MaxSmallFontCharHeight;

	OutClockRadius = (FMath::Min(DrawParams.ClockSize.X, DrawParams.ClockSize.Y) - 2 * GGameplayCamerasDebugClockPadding) / 2.f;
	OutClockCenter = DrawParams.ClockPosition + FVector2f(
			GGameplayCamerasDebugClockPadding + OutClockRadius,
			GGameplayCamerasDebugClockPadding + ClockAreaHeight / 2.f);
}

}  // namespace Internal

void FCameraDebugClock::Update(double InAngle)
{
	Value = FVariant(TInPlaceType<FAngleValue>(), FAngleValue{ InAngle });
}

void FCameraDebugClock::Update(const FVector2d& InValue)
{
	if (Value.IsType<FVectorValue>())
	{
		const FVectorValue PreviousValue = Value.Get<FVectorValue>();
		const double NextMaxLength = FMath::Max(PreviousValue.CurrentMaxLength, InValue.Length());
		Value = FVariant(TInPlaceType<FVectorValue>(), FVectorValue{ InValue, NextMaxLength });
	}
}

void FCameraDebugClock::Draw(FCanvas* Canvas, const FCameraDebugClockDrawParams& DrawParams)
{
	using namespace Internal;

	FCameraDebugClockRenderer Renderer(Canvas, DrawParams);
	switch (Value.GetIndex())
	{
		case FVariant::IndexOfType<FVectorValue>():
			{
				const FVectorValue& VectorValue = Value.Get<FVectorValue>();
				Renderer.DrawVectorClock(VectorValue.Vector, VectorValue.CurrentMaxLength);
			}
			break;
		case FVariant::IndexOfType<FAngleValue>():
			{
				const FAngleValue& AngleValue = Value.Get<FAngleValue>();
				Renderer.DrawAngleClock(AngleValue.Angle);
			}
			break;
	}
}

void FCameraDebugClock::Serialize(FArchive& Ar)
{
	Ar << Value;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

#endif

