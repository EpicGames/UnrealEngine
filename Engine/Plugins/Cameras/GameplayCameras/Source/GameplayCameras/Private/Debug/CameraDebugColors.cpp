// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugColors.h"

#include "Misc/Optional.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

FString FCameraDebugColors::CurrentColorSchemeName;
FCameraDebugColors FCameraDebugColors::CurrentColorScheme;
TMap<FString, FColor> FCameraDebugColors::ColorMap;
TMap<FString, FCameraDebugColors> FCameraDebugColors::ColorSchemes;

const FCameraDebugColors& FCameraDebugColors::Get()
{
	return CurrentColorScheme;
}

const FString& FCameraDebugColors::GetName()
{
	return CurrentColorSchemeName;
}

void FCameraDebugColors::Set(const FString& InColorSchemeName)
{
	if (InColorSchemeName != CurrentColorSchemeName)
	{
		const FCameraDebugColors* ColorScheme = ColorSchemes.Find(InColorSchemeName);
		if (ensureMsgf(ColorScheme, TEXT("No such color scheme: %s"), *InColorSchemeName))
		{
			CurrentColorSchemeName = InColorSchemeName;
			CurrentColorScheme = *ColorScheme;
			UpdateColorMap(CurrentColorScheme);
		}
	}
}

void FCameraDebugColors::Set(const FCameraDebugColors& InColorScheme)
{
	CurrentColorSchemeName = TEXT("<Custom>");
	CurrentColorScheme = InColorScheme;
	UpdateColorMap(CurrentColorScheme);
}

void FCameraDebugColors::RegisterColorScheme(const FString& InColorSchemeName, const FCameraDebugColors& InColorScheme)
{
	ColorSchemes.Add(InColorSchemeName, InColorScheme);
}

void FCameraDebugColors::GetColorSchemeNames(TArray<FString>& OutColorSchemeNames)
{
	ColorSchemes.GetKeys(OutColorSchemeNames);
}

void FCameraDebugColors::UpdateColorMap(const FCameraDebugColors& Instance)
{
	ColorMap.Reset();
	ColorMap.Add(TEXT("cam_title"), Instance.Title);
	ColorMap.Add(TEXT("cam_default"), Instance.Default);
	ColorMap.Add(TEXT("cam_passive"), Instance.Passive);
	ColorMap.Add(TEXT("cam_verypassive"), Instance.VeryPassive);
	ColorMap.Add(TEXT("cam_highlighted"), Instance.Hightlighted);
	ColorMap.Add(TEXT("cam_notice"), Instance.Notice);
	ColorMap.Add(TEXT("cam_notice2"), Instance.Notice2);
	ColorMap.Add(TEXT("cam_good"), Instance.Good);
	ColorMap.Add(TEXT("cam_warning"), Instance.Warning);
	ColorMap.Add(TEXT("cam_error"), Instance.Error);
	ColorMap.Add(TEXT("cam_background"), Instance.Background);
}

TOptional<FColor> FCameraDebugColors::GetFColorByName(const FString& InColorName)
{
	if (ColorMap.IsEmpty())
	{
		UpdateColorMap(FCameraDebugColors::Get());
	}
	if (const FColor* Color = ColorMap.Find(InColorName))
	{
		return TOptional<FColor>(*Color);
	}
	return TOptional<FColor>();
}

void FCameraDebugColors::RegisterBuiltinColorSchemes()
{
	// Colors inspired by the Solarized palette.
	//
	//    SOLARIZED HEX     RGB        
    //    --------- ------- -----------
    //    base03    #002b36   0  43  54
    //    base02    #073642   7  54  66
    //    base01    #586e75  88 110 117
    //    base00    #657b83 101 123 131
    //    base0     #839496 131 148 150
    //    base1     #93a1a1 147 161 161
    //    base2     #eee8d5 238 232 213
    //    base3     #fdf6e3 253 246 227
    //    yellow    #b58900 181 137   0
    //    orange    #cb4b16 203  75  22
    //    red       #dc322f 220  50  47
    //    magenta   #d33682 211  54 130
    //    violet    #6c71c4 108 113 196
    //    blue      #268bd2  38 139 210
    //    cyan      #2aa198  42 161 152
    //    green     #859900 133 153   0
	//
    const FColor Base03(   0,  43,  54);
    const FColor Base02(   7,  54,  66);
    const FColor Base01(  88, 110, 117);
    const FColor Base00( 101, 123, 131);
    const FColor Base0(  131, 148, 150);
    const FColor Base1(  147, 161, 161);
    const FColor Base2(  238, 232, 213);
    const FColor Base3(  253, 246, 227);
    const FColor Yellow( 181, 137,   0);
    const FColor Orange( 203,  75,  22);
    const FColor Red(    220,  50,  47);
    const FColor Magenta(211,  54, 130);
    const FColor Violet( 108, 113, 196);
    const FColor Blue(    38, 139, 210);
    const FColor Cyan(    42, 161, 152);
    const FColor Green(  133, 153,   0);
	{
		FCameraDebugColors SolarizedDark;
		SolarizedDark.Title = Blue;
		SolarizedDark.Default = Base2;
		SolarizedDark.Passive = Base1;
		SolarizedDark.VeryPassive = Base0;
		SolarizedDark.Hightlighted = Base3;
		SolarizedDark.Notice = Cyan;
		SolarizedDark.Notice2 = Magenta;
		SolarizedDark.Good = Green;
		SolarizedDark.Warning = Yellow;
		SolarizedDark.Error = Red;
		SolarizedDark.Background = Base03;

		RegisterColorScheme(TEXT("SolarizedDark"), SolarizedDark);
	}
	{
		FCameraDebugColors SolarizedLight;
		SolarizedLight.Title = Blue;
		SolarizedLight.Default = Base01;
		SolarizedLight.Passive = Base00;
		SolarizedLight.VeryPassive = Base0;
		SolarizedLight.Hightlighted = Base03;
		SolarizedLight.Notice = Cyan;
		SolarizedLight.Notice2 = Magenta;
		SolarizedLight.Good = Green;
		SolarizedLight.Warning = Yellow;
		SolarizedLight.Error = Red;
		SolarizedLight.Background = Base3;

		RegisterColorScheme(TEXT("SolarizedLight"), SolarizedLight);
	}

	Set(TEXT("SolarizedDark"));
}

FLinearColor LerpLinearColorUsingHSV(
	const FLinearColor& Start, const FLinearColor& End,
	int32 Increment, int32 TotalIncrements)
{
	if (TotalIncrements == 1)
	{
		return End;
	}
	else
	{
		const float Alpha = (float)Increment / (float)(TotalIncrements - 1);
		return FLinearColor::LerpUsingHSV(Start, End, Alpha);
	}
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

