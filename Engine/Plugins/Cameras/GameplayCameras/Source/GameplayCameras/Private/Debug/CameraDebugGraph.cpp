// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugGraph.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "CameraDebugGraph"

namespace UE::Cameras
{

float GGameplayCamerasDebugGraphPadding = 10.f;
FAutoConsoleVariableRef CVarGameplayCamerasDebugGraphPadding(
		TEXT("GameplayCameras.DebugGraph.Padding"),
		GGameplayCamerasDebugGraphPadding,
		TEXT("Default: 10px. The uniform padding inside the debug graph card."));

float GGameplayCamerasDebugGraphDefaultHistoryTime = 3.f;
FAutoConsoleVariableRef CVarGameplayCamerasDebugGraphDefaultHistoryTime(
		TEXT("GameplayCameras.DebugGraph.DefaultHistoryTime"),
		GGameplayCamerasDebugGraphDefaultHistoryTime,
		TEXT("Default: 2s. The default amount of the time in seconds displayed by a debug graph."));

FCameraDebugGraphDrawParams::FCameraDebugGraphDrawParams()
	: HistoryTime(0.f)
{
	const FCameraDebugColors& ColorScheme = FCameraDebugColors::Get();
	GraphBackgroundColor = ColorScheme.Background.WithAlpha((uint8)(GGameplayCamerasDebugBackgroundOpacity * 255));
	GraphNameColor = ColorScheme.Title;
	GraphLineColors = { ColorScheme.Notice };
	// Leave history time at zero so that we can dynamically adjust the history time with the CVar.
}

float FCameraDebugGraphDrawParams::GetDefaultMaxHistoryTime()
{
	return GGameplayCamerasDebugGraphDefaultHistoryTime;
}

namespace Internal
{

FCameraDebugGraphRenderer::FCameraDebugGraphRenderer(FCanvas* InCanvas, const FCameraDebugGraphDrawParams& InDrawParams)
	: Canvas(InCanvas)
	, DrawParams(InDrawParams)
{
}
	
void FCameraDebugGraphRenderer::DrawEmptyFrame() const
{
	DrawFrameImpl();
}

void FCameraDebugGraphRenderer::DrawFrame(TArrayView<float> CurrentValues) const
{
	DrawFrameImpl();

	// Draw the current values on top of each other in the top-left corner of the card.
	UFont* TinyFont = GEngine->GetTinyFont();
	const float MaxTinyFontCharHeight = TinyFont->GetMaxCharHeight();

	FVector2f CurrentValuePosition = DrawParams.GraphPosition +
		FVector2f(GGameplayCamerasDebugGraphPadding, GGameplayCamerasDebugGraphPadding);
	for (int32 Index = 0; Index < CurrentValues.Num(); ++Index)
	{
		FCanvasTextItem TextItem(
				FVector2D(CurrentValuePosition), 
				FText::Format(LOCTEXT("CurrentValueFmt", "{0}"), CurrentValues[Index]),
				TinyFont, 
				DrawParams.GraphLineColors[Index]);
		Canvas->DrawItem(TextItem);

		CurrentValuePosition.Y += MaxTinyFontCharHeight + 2.f;
	}
}

void FCameraDebugGraphRenderer::DrawFrameImpl() const
{
	// Draw the card's background tile.
	{
		FCanvasTileItem TileItem(
				FVector2D(DrawParams.GraphPosition), 
				FVector2D(DrawParams.GraphSize), 
				DrawParams.GraphBackgroundColor);
		TileItem.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem(TileItem);
	}

	// Add the graph name at the bottom of the card.
	if (!DrawParams.GraphName.IsEmpty())
	{
		UFont* SmallFont = GEngine->GetSmallFont();
		const float MaxSmallFontCharHeight = SmallFont->GetMaxCharHeight();

		const FVector2D GraphNamePosition(
				DrawParams.GraphPosition + 
				FVector2f(
					GGameplayCamerasDebugGraphPadding, 
					DrawParams.GraphSize.Y - GGameplayCamerasDebugGraphPadding - MaxSmallFontCharHeight));
		FCanvasTextItem GraphNameItem(GraphNamePosition, DrawParams.GraphName, SmallFont, DrawParams.GraphNameColor);
		Canvas->DrawItem(GraphNameItem);
	}
}

void FCameraDebugGraphRenderer::DrawGraphLine(const FLineDrawParams& LineDrawParams, TStridedView<float> Times, TStridedView<float> Values) const
{
	UFont* SmallFont = GEngine->GetSmallFont();
	const float MaxSmallFontCharHeight = SmallFont->GetMaxCharHeight();

	// Compute the actual area for the graph inside the card, based on the current margins and paddings.
	const FVector2D GraphAreaSize(
			DrawParams.GraphSize.X - 2 * GGameplayCamerasDebugGraphPadding,
			DrawParams.GraphSize.Y - 3 * GGameplayCamerasDebugGraphPadding - MaxSmallFontCharHeight);
	const FVector2D GraphAreaPosition(DrawParams.GraphPosition + FVector2f(GGameplayCamerasDebugGraphPadding));
	const double GraphAreaRight = GraphAreaPosition.X + GraphAreaSize.X;
	const double GraphAreaBottom = GraphAreaPosition.Y + GraphAreaSize.Y;

	// Figure out how much history we're showing, and compute the conversion ratio between seconds
	// and pixels.
	float HistoryTime = DrawParams.HistoryTime;
	if (HistoryTime <= 0)
	{
		HistoryTime = FCameraDebugGraphDrawParams::GetDefaultMaxHistoryTime();
	}
	const float TimeToPixels = GraphAreaSize.X / HistoryTime;
	const float LatestTime = Times[Times.Num() - 1];

	// Figure out how much vertical range we're showing, and compute the conversion ratio between
	// value units and pixels.
	float ValueRange = LineDrawParams.MaxValue - LineDrawParams.MinValue;
	if (ValueRange <= 0)
	{
		ValueRange = 1.f;
	}
	const float ValueToPixels = GraphAreaSize.Y / ValueRange;
	const float LowestValue = LineDrawParams.MinValue;

	// Draw the lines! We start drawing from the right-side of the card, so that the latest value is
	// always exactly on the edge of the card's graph area. The oldest values, on the left-side, may
	// overflow into the padding area, so we have to possibly cut that short.
	const int32 Num = FMath::Min(Times.Num(), Values.Num());
	for (int32 Index = Num - 1; Index > 0; --Index)
	{
		const float NextTime = Times[Index];
		const float PrevTime = Times[Index - 1];

		const float NextValue = Values[Index];
		const float PrevValue = Values[Index - 1];

		const FVector2D NextGraphPoint(
				GraphAreaRight - (LatestTime - NextTime) * TimeToPixels,
				GraphAreaBottom + (LowestValue - NextValue) * ValueToPixels);
		const FVector2D PrevGraphPoint(
				GraphAreaRight - (LatestTime - PrevTime) * TimeToPixels,
				GraphAreaBottom + (LowestValue - PrevValue) * ValueToPixels);

		if (PrevGraphPoint.X >= GraphAreaPosition.X)
		{
			FCanvasLineItem LineItem(NextGraphPoint, PrevGraphPoint);
			LineItem.SetColor(LineDrawParams.LineColor);
			Canvas->DrawItem(LineItem);
		}
		else
		{
			const float SegmentTimeRange = (NextGraphPoint.X - PrevGraphPoint.X);
			if (SegmentTimeRange > 0)
			{
				const float Factor = (GraphAreaPosition.X - PrevGraphPoint.X) / SegmentTimeRange;
				const FVector2D EdgePoint = FMath::Lerp(PrevGraphPoint, NextGraphPoint, Factor);

				FCanvasLineItem LineItem(NextGraphPoint, EdgePoint);
				LineItem.SetColor(LineDrawParams.LineColor);
				Canvas->DrawItem(LineItem);
			}
			break;
		}
	}
}

}  // namespace Internal

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

#endif

