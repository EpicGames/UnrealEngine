// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDebugRenderer.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Components/LineBatchComponent.h"
#include "Core/CameraPose.h"
#include "Debug/CameraDebugClock.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/DebugTextRenderer.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Math/Box2D.h"
#include "Misc/EngineVersionComparison.h"
#include "SceneView.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

int32 GGameplayCamerasDebugLeftMargin = 10;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugLeftMargin(
	TEXT("GameplayCameras.Debug.LeftMargin"),
	GGameplayCamerasDebugLeftMargin,
	TEXT("(Default: 10px. The left margin for rendering Gameplay Cameras debug text."));

int32 GGameplayCamerasDebugTopMargin = 10;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugTopMargin(
	TEXT("GameplayCameras.Debug.TopMargin"),
	GGameplayCamerasDebugTopMargin,
	TEXT("(Default: 10px. The top margin for rendering Gameplay Cameras debug text."));

int32 GGameplayCamerasDebugRightMargin = 10;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugRightMargin(
	TEXT("GameplayCameras.Debug.RightMargin"),
	GGameplayCamerasDebugRightMargin,
	TEXT("(Default: 10px. The right margin for rendering Gameplay Cameras debug text."));

int32 GGameplayCamerasDebugInnerMargin = 5;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugInnerMargin(
	TEXT("GameplayCameras.Debug.InnerMargin"),
	GGameplayCamerasDebugInnerMargin,
	TEXT("(Default: 10px. The inner margin for rendering Gameplay Cameras debug text."));

int32 GGameplayCamerasDebugIndent = 20;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugIndent(
	TEXT("GameplayCameras.Debug.Indent"),
	GGameplayCamerasDebugIndent,
	TEXT("(Default: 20px. The indent for rendering Gameplay Cameras debug text."));

int32 GGameplayCamerasDebugBackgroundDepthSortKey = 1;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBackgroundDepthSortKey(
	TEXT("GameplayCameras.Debug.BackgroundDepthSortKey"),
	GGameplayCamerasDebugBackgroundDepthSortKey,
	TEXT("Default: 1. The sort key for drawing the background behind debug text and debug cards."));

int32 GGameplayCamerasDebugCardWidth = 200;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugCardWidth(
	TEXT("GameplayCameras.Debug.CardWidth"),
	GGameplayCamerasDebugCardWidth,
	TEXT("Default: 200px. The width of the debug cards (e.g. graphs, clocks, etc.)"));

int32 GGameplayCamerasDebugCardHeight = 250;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugCardHeight(
	TEXT("GameplayCameras.Debug.CardHeight"),
	GGameplayCamerasDebugCardHeight,
	TEXT("Default: 250px. The height of the debug cards (e.g. graphs, clocks, etc.)"));

int32 GGameplayCamerasDebugCardGap = 10;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugCardGap(
	TEXT("GameplayCameras.Debug.CardGap"),
	GGameplayCamerasDebugCardGap,
	TEXT("Default: 10px. The gap between the debug cards (e.g. graphs, clocks, etc.)"));

int32 GGameplayCamerasDebugMaxCardColumns = 2;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugMaxCardColumns(
	TEXT("GameplayCameras.Debug.MaxCardColumns"),
	GGameplayCamerasDebugMaxCardColumns,
	TEXT("Default: 2. The number of columns to layout the debug cards (e.g. graphs, clocks, etc.)"));

float GGameplayCamerasDebugDefaultCameraSize = 50.f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugDefaultCameraSize(
	TEXT("GameplayCameras.Debug.DefaultCameraSize"),
	GGameplayCamerasDebugDefaultCameraSize,
	TEXT("Default: 50. The default size of debug cameras."));

float GGameplayCamerasDebugDefaultCoordinateSystemAxesLength = 100.f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugDefaultCoordinateSystemAxesLength(
	TEXT("GameplayCameras.Debug.DefaultCoordinateSystemAxesLength"),
	GGameplayCamerasDebugDefaultCoordinateSystemAxesLength,
	TEXT("Default: 100. The default length of coordinate system axes."));

bool GGameplayCamerasDebugDrawBackground = true;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugDrawBackground(
	TEXT("GameplayCameras.Debug.DrawBackground"),
	GGameplayCamerasDebugDrawBackground,
	TEXT(""));

float GGameplayCamerasDebugBackgroundOpacity = 0.6f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugBackgroundOpacity(
	TEXT("GameplayCameras.Debug.BackgroundOpacity"),
	GGameplayCamerasDebugBackgroundOpacity,
	TEXT(""));

FString GGameplayCamerasDebugColorScheme = TEXT("SolarizedDark");
static FAutoConsoleVariableRef CVarGameplayCamerasDebugColorScheme(
	TEXT("GameplayCameras.Debug.ColorScheme"),
	GGameplayCamerasDebugColorScheme,
	TEXT(""));

FCameraDebugRenderer::FCameraDebugRenderer(UWorld* InWorld, UCanvas* InCanvasObject, bool bInIsExternalRendering)
{
	FCanvas* InCanvas = nullptr;
	const FSceneView* InSceneView = nullptr;
	if (InCanvasObject)
	{
		InCanvas = InCanvasObject->Canvas;
		InSceneView = InCanvasObject->SceneView;
	}

	Initialize(InWorld, InSceneView, InCanvas, bInIsExternalRendering);
}

FCameraDebugRenderer::FCameraDebugRenderer(UWorld* InWorld, const FSceneView* InSceneView, FCanvas* InCanvas, bool bInIsExternalRendering)
{
	Initialize(InWorld, InSceneView, InCanvas, bInIsExternalRendering);
}

void FCameraDebugRenderer::Initialize(UWorld* InWorld, const FSceneView* InSceneView, FCanvas* InCanvas, bool bInIsExternalRendering)
{
	World = InWorld;
	bIsExternalRendering = bInIsExternalRendering;
	DrawColor = FColor::White;

	RenderFont = GEngine->GetSmallFont();
	MaxCharHeight = RenderFont->GetMaxCharHeight();

	NextDrawPosition = FVector2f{ (float)GGameplayCamerasDebugLeftMargin, (float)GGameplayCamerasDebugTopMargin };

	NextCardPosition = FVector2f::ZeroVector;
	NextCardColumn = 0;

	CanvasSize = FVector2D(EForceInit::ForceInit);

	if (InSceneView && InCanvas)
	{
		SceneView = InSceneView;
		Canvas = InCanvas;

		FIntRect ViewRect = InCanvas->GetViewRect();
		if (ViewRect.Width() == 0 || ViewRect.Height() == 0)
		{
			ViewRect = InSceneView->UnconstrainedViewRect;
		}

		CanvasSize = FVector2d(ViewRect.Width(), ViewRect.Height()) / InCanvas->GetDPIScale();

		NextCardPosition = FVector2f{ 
			(float)CanvasSize.X - (float)GGameplayCamerasDebugCardWidth - (float)GGameplayCamerasDebugRightMargin,
			(float)GGameplayCamerasDebugTopMargin };
	}
}

FCameraDebugRenderer::~FCameraDebugRenderer()
{
	FlushText();
}

void FCameraDebugRenderer::BeginDrawing()
{
	// Update the color scheme in case it changed.
	FCameraDebugColors::Set(GGameplayCamerasDebugColorScheme);
}

void FCameraDebugRenderer::EndDrawing()
{
	// Render a translucent background to help readability.
	if (GGameplayCamerasDebugDrawBackground)
	{
		DrawTextBackgroundTile(GGameplayCamerasDebugBackgroundOpacity);
	}
}

void FCameraDebugRenderer::AddText(const FString& InString)
{
	AddTextImpl(*InString);
}

void FCameraDebugRenderer::AddText(const TCHAR* Fmt, ...)
{
	va_list Args;
	va_start(Args, Fmt);
	AddTextFmtImpl(Fmt, Args);
	va_end(Args);
}

void FCameraDebugRenderer::AddTextFmtImpl(const TCHAR* Fmt, va_list Args)
{
	Formatter.Reset();
	Formatter.AppendV(Fmt, Args);
	const TCHAR* Message = Formatter.ToString();
	AddTextImpl(Message);
}

void FCameraDebugRenderer::AddTextImpl(const TCHAR* Buffer)
{
	LineBuilder.Append(Buffer);
}

bool FCameraDebugRenderer::NewLine(bool bSkipIfEmptyLine)
{
	FlushText();

	const float IndentMargin = GetIndentMargin();
	const bool bIsLineEmpty = FMath::IsNearlyEqual(NextDrawPosition.X, IndentMargin);
	if (!bIsLineEmpty || !bSkipIfEmptyLine)
	{
		NextDrawPosition.X = IndentMargin;
		NextDrawPosition.Y += MaxCharHeight;
		return true;
	}
	return false;
}

FColor FCameraDebugRenderer::GetTextColor() const
{
	return DrawColor;
}

FColor FCameraDebugRenderer::SetTextColor(const FColor& Color)
{
	FlushText();
	FColor ReturnColor = DrawColor;
	DrawColor = Color;
	return ReturnColor;
}

float FCameraDebugRenderer::GetIndentMargin() const
{
	return (float)(GGameplayCamerasDebugLeftMargin + IndentLevel * GGameplayCamerasDebugIndent);
}

void FCameraDebugRenderer::FlushText()
{
	if (LineBuilder.Len() > 0)
	{
		int32 ViewHeight = CanvasSize.Y;
		if (NextDrawPosition.Y < ViewHeight)
		{
			FDebugTextRenderer TextRenderer(Canvas, DrawColor, RenderFont);
			TextRenderer.LeftMargin = GetIndentMargin();
			TextRenderer.RenderText(NextDrawPosition, LineBuilder.ToView());

			NextDrawPosition = TextRenderer.GetEndDrawPosition();
			RightMargin = FMath::Max(RightMargin, TextRenderer.GetRightMargin());
		}
		// else: text is going off-screen.

		LineBuilder.Reset();
	}
}

void FCameraDebugRenderer::AddIndent()
{
	// Flush any remaining text we have on the current indent level and move
	// to a new line, unless the current line was empty.
	NewLine(true);

	++IndentLevel;

	// The next draw position is at the beginning of a new line (or the beginning
	// of an old line if it was empty). Either way, it's left at the previous
	// indent level, so we need to bump it to the right.
	NextDrawPosition.X = GetIndentMargin();
}

void FCameraDebugRenderer::RemoveIndent()
{
	// Flush any remaining text we have on the current indent level and move
	// to a new line, unless the current line was empty.
	NewLine(true);

	if (ensureMsgf(IndentLevel > 0, TEXT("Can't go into negative indenting!")))
	{
		--IndentLevel;

		// See comment in AddIndent().
		NextDrawPosition.X = GetIndentMargin();
	}
}

void FCameraDebugRenderer::DrawTextBackgroundTile(float Opacity)
{
	const float IndentMargin = GetIndentMargin();
	const bool bIsLastLineEmpty = FMath::IsNearlyEqual(NextDrawPosition.X, IndentMargin);
	const float TextBottom = bIsLastLineEmpty ? NextDrawPosition.Y : NextDrawPosition.Y + MaxCharHeight;

	const float InnerMargin = GGameplayCamerasDebugInnerMargin;
	const FVector2D TopLeft(GGameplayCamerasDebugLeftMargin - InnerMargin, GGameplayCamerasDebugTopMargin - InnerMargin);
	const FVector2D BottomRight(RightMargin + InnerMargin, TextBottom + InnerMargin);
	const FVector2D TileSize(BottomRight.X - TopLeft.X, BottomRight.Y - TopLeft.Y);

	const FColor BackgroundColor = FCameraDebugColors::Get().Background.WithAlpha((uint8)(Opacity * 255));

	// Draw the background behind the text, if any.
	if (Canvas && TextBottom > GGameplayCamerasDebugTopMargin)
	{
		Canvas->PushDepthSortKey(GGameplayCamerasDebugBackgroundDepthSortKey);
		{
			FCanvasTileItem BackgroundTile(TopLeft, TileSize, BackgroundColor);
			BackgroundTile.BlendMode = SE_BLEND_Translucent;
			Canvas->DrawItem(BackgroundTile);
		}
		Canvas->PopDepthSortKey();
	}
}

void FCameraDebugRenderer::DrawClock(FCameraDebugClock& InClock, const FText& InClockName)
{
	FCameraDebugClockDrawParams DrawParams;
	DrawParams.ClockName = InClockName;
	DrawParams.ClockPosition = GetNextCardPosition();
	DrawParams.ClockSize = FVector2f(GGameplayCamerasDebugCardWidth, GGameplayCamerasDebugCardHeight);
	InClock.Draw(Canvas, DrawParams);
}

void FCameraDebugRenderer::DrawCameraPose(const FCameraPose& InCameraPose, const FLinearColor& LineColor, float CameraSize)
{
	const FTransform3d Transform = InCameraPose.GetTransform();
	const float EffectiveFieldOfView = InCameraPose.GetEffectiveFieldOfView();
	const float AspectRatio = InCameraPose.GetSensorAspectRatio();
	const double TargetDistance = InCameraPose.GetTargetDistance();
	DrawCamera(Transform, EffectiveFieldOfView, AspectRatio, TargetDistance, LineColor, CameraSize, 1.f);
}

FVector2f FCameraDebugRenderer::GetNextCardPosition()
{
	const FVector2f Result(NextCardPosition);

	++NextCardColumn;
	if (NextCardColumn >= GGameplayCamerasDebugMaxCardColumns)
	{
		// We went over the number of columns we're supposed to stick to.
		// Place the next card below the previous cards, at the right-side edge of the canvas.
		NextCardColumn = 0;
		NextCardPosition.X = CanvasSize.X - (float)GGameplayCamerasDebugCardWidth - (float)GGameplayCamerasDebugRightMargin;
		NextCardPosition.Y += GGameplayCamerasDebugCardHeight + GGameplayCamerasDebugCardGap;
	}
	else
	{
		// We can go to the next column. Place the next card to the left of the previous card.
		NextCardPosition.X -= (float)GGameplayCamerasDebugCardWidth + (float)GGameplayCamerasDebugCardGap;
	}

	return Result;
}

void FCameraDebugRenderer::GetNextDrawGraphParams(FCameraDebugGraphDrawParams& OutDrawParams, const FText& InGraphName)
{
	OutDrawParams.GraphName = InGraphName;
	OutDrawParams.GraphPosition = GetNextCardPosition();
	OutDrawParams.GraphSize = FVector2f(GGameplayCamerasDebugCardWidth, GGameplayCamerasDebugCardHeight);
}

void FCameraDebugRenderer::Draw2DPointCross(const FVector2D& Location, float CrossSize, const FLinearColor& LineColor, float LineThickness)
{
	if (Canvas)
	{
		const float HalfCrossSize = CrossSize / 2.f;

		FCanvasLineItem Horizontal(Location - FVector2D(HalfCrossSize, 0), Location + FVector2D(HalfCrossSize, 0));
		Horizontal.SetColor(LineColor);
		Horizontal.LineThickness = LineThickness;
		Canvas->DrawItem(Horizontal);

		FCanvasLineItem Vertical(Location - FVector2D(0, HalfCrossSize), Location + FVector2D(0, HalfCrossSize));
		Vertical.SetColor(LineColor);
		Vertical.LineThickness = LineThickness;
		Canvas->DrawItem(Vertical);
	}
}

void FCameraDebugRenderer::Draw2DLine(const FVector2D& Start, const FVector2D& End, const FLinearColor& LineColor, float LineThickness)
{
	if (Canvas)
	{
		FCanvasLineItem LineItem(Start, End);
		LineItem.SetColor(LineColor);
		LineItem.LineThickness = LineThickness;
		Canvas->DrawItem(LineItem);
	}
}

void FCameraDebugRenderer::Draw2DBox(const FBox2D& Box, const FLinearColor& LineColor, float LineThickness)
{
	if (Canvas)
	{
		FCanvasBoxItem BoxItem(Box.Min, Box.GetSize());
		BoxItem.SetColor(LineColor);
		BoxItem.LineThickness = LineThickness;
		Canvas->DrawItem(BoxItem);
	}
}

void FCameraDebugRenderer::Draw2DBox(const FVector2D& BoxPosition, const FVector2D& BoxSize, const FLinearColor& LineColor, float LineThickness)
{
	if (Canvas)
	{
		FCanvasBoxItem BoxItem(BoxPosition, BoxSize);
		BoxItem.SetColor(LineColor);
		BoxItem.LineThickness = LineThickness;
		Canvas->DrawItem(BoxItem);
	}
}

void FCameraDebugRenderer::Draw2DCircle(const FVector2D& Center, float Radius, const FLinearColor& LineColor, float LineThickness, int32 NumSides)
{
	if (NumSides <= 0)
	{
		NumSides = FMath::Max(6, (int)Radius / 25);
	}

	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	const FVector2D AxisX(1.f, 0.f);
	const FVector2D AxisY(0.f, -1.f);
	FVector2D LastVertex = Center + AxisX * Radius;

	for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const float CurAngle = AngleDelta * (SideIndex + 1);
		const FVector2D Vertex = Center + (AxisX * FMath::Cos(CurAngle) + AxisY * FMath::Sin(CurAngle)) * Radius;
		Draw2DLine(LastVertex, Vertex, LineColor, LineThickness);
		LastVertex = Vertex;
	}
}

void FCameraDebugRenderer::DrawPoint(const FVector3d& Location, float PointSize, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		LineBatcher->DrawPoint(Location, LineColor, PointSize, SDPG_Foreground);
	}
}

void FCameraDebugRenderer::DrawLine(const FVector3d& Start, const FVector3d& End, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		LineBatcher->DrawLine(Start, End, LineColor, SDPG_Foreground, LineThickness);
	}
}

void FCameraDebugRenderer::DrawBox(const FVector3d& Center, const FVector3d& Size, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		LineBatcher->DrawBox(Center, Size, LineColor, 0.f, SDPG_Foreground, LineThickness);
	}
}

void FCameraDebugRenderer::DrawBox(const FTransform3d& Transform, const FVector3d& Size, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		// Create all box corners in world space.
		FVector3d TopCorners[4] = {
			FVector3d(Size.X, Size.Y, Size.Z),
			FVector3d(-Size.X, Size.Y, Size.Z),
			FVector3d(-Size.X, -Size.Y, Size.Z),
			FVector3d(Size.X, -Size.Y, Size.Z)
		};
		FVector3d BottomCorners[4] = {
			FVector3d(Size.X, Size.Y, -Size.Z),
			FVector3d(-Size.X, Size.Y, -Size.Z),
			FVector3d(-Size.X, -Size.Y, -Size.Z),
			FVector3d(Size.X, -Size.Y, -Size.Z)
		};
		for (int32 Index = 0; Index < 4; ++Index)
		{
			TopCorners[Index] = Transform.TransformVectorNoScale(TopCorners[Index]);
			BottomCorners[Index] = Transform.TransformVectorNoScale(BottomCorners[Index]);
		}

		TArray<FBatchedLine> Lines;
		const FVector3d Center = Transform.GetLocation();

		// Draw the top and bottom squares, and the lines in between.
		for (int32 Index = 0; Index < 4; ++Index)
		{
			Lines.Emplace(
					Center + TopCorners[Index], Center + TopCorners[(Index + 1) % 4], 
					LineColor, 0.f, LineThickness, SDPG_Foreground);
			Lines.Emplace(
					Center + BottomCorners[Index], Center + BottomCorners[(Index + 1) % 4], 
					LineColor, 0.f, LineThickness, SDPG_Foreground);
			Lines.Emplace(
					Center + TopCorners[Index], Center + BottomCorners[Index], 
					LineColor, 0.f, LineThickness, SDPG_Foreground);
		}

		LineBatcher->DrawLines(Lines);
	}
}

void FCameraDebugRenderer::DrawSphere(const FVector3d& Center, float Radius, int32 Segments, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		LineBatcher->DrawSphere(Center, Radius, Segments, LineColor, 0.f, SDPG_Foreground, LineThickness);
	}
}

void FCameraDebugRenderer::DrawDirectionalArrow(const FVector3d& Start, const FVector3d& End, float ArrowSize, const FLinearColor& LineColor, float LineThickness)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		LineBatcher->DrawDirectionalArrow(Start, End, ArrowSize, LineColor, 0.f, SDPG_Foreground, LineThickness);
	}
}

void FCameraDebugRenderer::DrawCamera(const FTransform3d& Transform, float HorizontalFieldOfView, float AspectRatio, float TargetDistance, const FLinearColor& LineColor, float CameraSize, float LineThickness)
{
	if (AspectRatio <= 0.f)
	{
		AspectRatio = 1.f;
	}
	if (CameraSize <= 0.f)
	{
		CameraSize = GGameplayCamerasDebugDefaultCameraSize;
	}

	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		// We draw a pyramid representing the camera's FOV and aspect ratio. So we only need the origin
		// point and the four corner points of the base.
		const float TanHalfHFOV = FMath::Tan(FMath::DegreesToRadians(HorizontalFieldOfView / 2.f));
		const float BaseHalfWidth = TanHalfHFOV * CameraSize;
		const float BaseHalfHeight = BaseHalfWidth / AspectRatio;

		const FVector3d ForwardDir(FVector3d::ForwardVector);
		const FVector3d UpDir(FVector3d::UpVector);
		const FVector3d RightDir(FVector3d::RightVector);

		// Upper right, bottom right, bottom left, upper left.
		FVector3d BaseCorners[4];
		BaseCorners[0] = (ForwardDir * CameraSize) + (UpDir * BaseHalfHeight) + (RightDir * BaseHalfWidth);
		BaseCorners[1] = (ForwardDir * CameraSize) - (UpDir * BaseHalfHeight) + (RightDir * BaseHalfWidth);
		BaseCorners[2] = (ForwardDir * CameraSize) - (UpDir * BaseHalfHeight) - (RightDir * BaseHalfWidth);
		BaseCorners[3] = (ForwardDir * CameraSize) + (UpDir * BaseHalfHeight) - (RightDir * BaseHalfWidth);

		const FVector3d Location = Transform.GetLocation();
		for (FVector3d& BaseCorner : BaseCorners)
		{
			BaseCorner = Location + Transform.TransformVectorNoScale(BaseCorner);
		}

		TArray<FBatchedLine> BatchedLines;
		// Pyramid corners.
		BatchedLines.Emplace(Location, BaseCorners[0], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(Location, BaseCorners[1], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(Location, BaseCorners[2], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(Location, BaseCorners[3], LineColor, 0.f, LineThickness, SDPG_Foreground);
		// Base edges.
		BatchedLines.Emplace(BaseCorners[0], BaseCorners[1], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(BaseCorners[1], BaseCorners[2], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(BaseCorners[2], BaseCorners[3], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(BaseCorners[3], BaseCorners[0], LineColor, 0.f, LineThickness, SDPG_Foreground);
		// Base cross.
		BatchedLines.Emplace(BaseCorners[0], BaseCorners[2], LineColor, 0.f, LineThickness, SDPG_Foreground);
		BatchedLines.Emplace(BaseCorners[1], BaseCorners[3], LineColor, 0.f, LineThickness, SDPG_Foreground);

		// Optional target distance line.
		if (TargetDistance > 0.f)
		{
			const FVector3d AimDir = Transform.GetRotation().GetForwardVector();
			BatchedLines.Emplace(
					Location + AimDir * CameraSize, Location + AimDir * TargetDistance, 
					LineColor, 0.f, LineThickness, SDPG_Foreground);
		}

		LineBatcher->DrawLines(BatchedLines);
	}
}

void FCameraDebugRenderer::DrawCoordinateSystem(const FVector3d& Location, const FRotator3d& Rotation, float AxesLength)
{
	if (ULineBatchComponent* LineBatcher = GetDebugLineBatcher())
	{
		if (AxesLength <= 0.f)
		{
			AxesLength = GGameplayCamerasDebugDefaultCoordinateSystemAxesLength;
		}

		LineBatcher->DrawLine(
				Location, 
				Location + Rotation.RotateVector(FVector3d::ForwardVector * AxesLength),
				FLinearColor::Red,
				SDPG_Foreground,
				0.f);
		LineBatcher->DrawLine(
				Location, 
				Location + Rotation.RotateVector(FVector3d::RightVector * AxesLength),
				FLinearColor::Green,
				SDPG_Foreground,
				0.f);
		LineBatcher->DrawLine(
				Location, 
				Location + Rotation.RotateVector(FVector3d::UpVector * AxesLength),
				FLinearColor::Blue,
				SDPG_Foreground,
				0.f);
	}
}

void FCameraDebugRenderer::DrawCoordinateSystem(const FTransform3d& Transform, float AxesLength)
{
	DrawCoordinateSystem(Transform.GetLocation(), Transform.GetRotation().Rotator(), AxesLength);
}

void FCameraDebugRenderer::DrawText(const FVector3d& WorldPosition, const FString& Text, const FLinearColor& TextColor, UFont* TextFont, float TextScale)
{
	DrawText(WorldPosition, FVector2d::ZeroVector, Text, TextColor, TextFont, TextScale);
}

void FCameraDebugRenderer::DrawTextView(const FVector3d& WorldPosition, FStringView Text, const FLinearColor& TextColor, UFont* TextFont, float TextScale)
{
	DrawTextView(WorldPosition, FVector2d::ZeroVector, Text, TextColor, TextFont, TextScale);
}

void FCameraDebugRenderer::DrawText(const FVector3d& WorldPosition, const FVector2d& ScreenOffset, const FString& Text, const FLinearColor& TextColor, UFont* TextFont, float TextScale)
{
	if (Canvas && SceneView)
	{
		UFont* ActualTextFont = TextFont ? TextFont : GEngine->GetSmallFont();

		FVector2d PixelOffset(FVector2d::ZeroVector);
		if (ScreenOffset.X != 0.0 || ScreenOffset.Y != 0.0)
		{
			const double ScreenWidth = SceneView->UnscaledViewRect.Width();
			const double ScreenHeight = SceneView->UnscaledViewRect.Height();
			PixelOffset = FVector2d(ScreenOffset.X / ScreenWidth, ScreenOffset.Y / ScreenHeight);
		}

		FVector2d PixelPosition;
		SceneView->WorldToPixel(WorldPosition, PixelPosition);

		FCanvasTextItem TextItem(
				FVector2D(PixelPosition.X + PixelOffset.X, PixelPosition.Y + PixelOffset.Y),
				FText::FromString(Text),
				ActualTextFont,
				TextColor);
		TextItem.BlendMode = SE_BLEND_Translucent;
		TextItem.Scale = FVector2D(TextScale, TextScale);
		Canvas->DrawItem(TextItem);	
	}
}

void FCameraDebugRenderer::DrawTextView(const FVector3d& WorldPosition, const FVector2d& ScreenOffset, FStringView Text, const FLinearColor& TextColor, UFont* TextFont, float TextScale)
{
	if (Canvas && SceneView)
	{
		UFont* ActualTextFont = TextFont ? TextFont : GEngine->GetSmallFont();

		FVector2d PixelOffset(FVector2d::ZeroVector);
		if (ScreenOffset.X != 0.0 || ScreenOffset.Y != 0.0)
		{
			const double ScreenWidth = SceneView->UnscaledViewRect.Width();
			const double ScreenHeight = SceneView->UnscaledViewRect.Height();
			PixelOffset = FVector2d(ScreenOffset.X / ScreenWidth, ScreenOffset.Y / ScreenHeight);
		}

		FVector2d PixelPosition;
		SceneView->WorldToPixel(WorldPosition, PixelPosition);

		FCanvasTextStringViewItem TextItem(
				FVector2D(PixelPosition.X + PixelOffset.X, PixelPosition.Y + PixelOffset.Y),
				Text,
				ActualTextFont,
				TextColor);
		TextItem.BlendMode = SE_BLEND_Translucent;
		TextItem.Scale = FVector2D(TextScale, TextScale);
		Canvas->DrawItem(TextItem);	
	}
}

ULineBatchComponent* FCameraDebugRenderer::GetDebugLineBatcher() const
{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	return World ? World->GetLineBatcher(UWorld::ELineBatcherType::Foreground) : nullptr;
#else
	return World ? World->ForegroundLineBatcher : nullptr;
#endif
}

void FCameraDebugRenderer::SkipAttachedBlocks()
{
	VisitFlags |= ECameraDebugDrawVisitFlags::SkipAttachedBlocks;
}

void FCameraDebugRenderer::SkipChildrenBlocks()
{
	VisitFlags |= ECameraDebugDrawVisitFlags::SkipChildrenBlocks;
}

void FCameraDebugRenderer::SkipAllBlocks()
{
	VisitFlags |= (ECameraDebugDrawVisitFlags::SkipAttachedBlocks | ECameraDebugDrawVisitFlags::SkipChildrenBlocks);
}

ECameraDebugDrawVisitFlags FCameraDebugRenderer::GetVisitFlags() const
{
	return VisitFlags;
}

void FCameraDebugRenderer::ResetVisitFlags()
{
	VisitFlags = ECameraDebugDrawVisitFlags::None;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

