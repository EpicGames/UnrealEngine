// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDirectorDebugBlock.h"

#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraDirectorTreeDebugBlock.h"
#include "Debug/CameraNodeEvaluationResultDebugBlock.h"
#include "Debug/CameraPoseDebugBlock.h"
#include "Debug/ContextDataTableDebugBlock.h"
#include "Debug/VariableTableDebugBlock.h"
#include "HAL/IConsoleManager.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

bool GGameplayCamerasDebugContextInitialResultShowUnchanged = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugContextInitialResultShowUnchanged(
	TEXT("GameplayCameras.Debug.ContextInitialResult.ShowUnchanged"),
	GGameplayCamerasDebugContextInitialResultShowUnchanged,
	TEXT(""));

bool GGameplayCamerasDebugContextInitialResultShowVariableIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugContextInitialResultShowVariableIDs(
	TEXT("GameplayCameras.Debug.ContextInitialResult.ShowVariableIDs"),
	GGameplayCamerasDebugContextInitialResultShowVariableIDs,
	TEXT(""));

bool GGameplayCamerasDebugContextInitialResultShowDataIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugContextInitialResultShowDataIDs(
	TEXT("GameplayCameras.Debug.ContextInitialResult.ShowDataIDs"),
	GGameplayCamerasDebugContextInitialResultShowDataIDs,
	TEXT(""));

bool GGameplayCamerasDebugContextInitialResultShowCoordinateSystem = true;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugContextInitialResultShowCoordinateSystem(
	TEXT("GameplayCameras.Debug.ContextInitialResult.ShowCoordinateSystem"),
	GGameplayCamerasDebugContextInitialResultShowCoordinateSystem,
	TEXT(""));

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraDirectorDebugBlock)

void FCameraDirectorDebugBlock::Initialize(TSharedPtr<FCameraEvaluationContext> Context, FCameraDebugBlockBuilder& Builder)
{
	if (Context)
	{
		const FCameraObjectTypeRegistry& TypeRegistry = FCameraObjectTypeRegistry::Get();
		const FName ContextTypeName = TypeRegistry.GetTypeNameSafe(Context->GetTypeID());

		const UObject* ContextOwner = Context->GetOwner();
		const FCameraDirectorEvaluator* DirectorEvaluator = Context->GetDirectorEvaluator();
		const UCameraDirector* CameraDirector = DirectorEvaluator->GetCameraDirector();

		ContextClassName = ContextTypeName;
		OwnerName = *GetPathNameSafe(ContextOwner);
		OwnerClassName = ContextOwner ? ContextOwner->GetClass()->GetFName() : NAME_None;
		CameraAssetName = GetNameSafe(Context->GetCameraAsset());
		CameraDirectorClassName = GetFNameSafe(CameraDirector);
		InitialContextTransform = Context->GetInitialResult().CameraPose.GetTransform();
		bIsValid = true;

		const FCameraNodeEvaluationResult& InitialResult = Context->GetInitialResult();
		FCameraNodeEvaluationResultDebugBlock& ResultBlock = Builder.BuildDebugBlock<FCameraNodeEvaluationResultDebugBlock>();
		ResultBlock.Initialize(InitialResult, Builder);
		AddChild(&ResultBlock);
		{
			ResultBlock.GetCameraPoseDebugBlock()
				->WithShowUnchangedCVar(TEXT("GameplayCameras.Debug.ContextInitialResult.ShowUnchanged"));
			ResultBlock.GetVariableTableDebugBlock()
				->WithShowVariableIDsCVar(TEXT("GameplayCameras.Debug.ContextInitialResult.ShowVariableIDs"));
			ResultBlock.GetContextDataTableDebugBlock()
				->WithShowDataIDsCVar(TEXT("GameplayCameras.Debug.ContextInitialResult.ShowDataIDs"));
		}

		TArrayView<const TSharedPtr<FCameraEvaluationContext>> ChildrenContexts = Context->GetChildrenContexts();
		if (ChildrenContexts.Num() > 0)
		{
			FCameraDirectorTreeDebugBlock& ChildBlock = Builder.StartChildDebugBlock<FCameraDirectorTreeDebugBlock>();
			{
				ChildBlock.Initialize(ChildrenContexts, Builder);
			}
			Builder.EndChildDebugBlock();
		}
	}
	else
	{
		bIsValid = false;
	}
}

void FCameraDirectorDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	// Basic debug info.
	if (bIsValid)
	{
		Renderer.AddText(TEXT("{cam_passive}[%s]{cam_default}"), *CameraDirectorClassName.ToString());
		Renderer.AddIndent();
		{
			Renderer.AddText(TEXT("Context {cam_passive}[%s]{cam_default}\n"), *ContextClassName.ToString());

			Renderer.AddText(TEXT("Owned by {cam_passive}[%s]{cam_default}\n"), *OwnerClassName.ToString());
			Renderer.AddIndent();
			{
				Renderer.AddText(*OwnerName);
			}
			Renderer.RemoveIndent();

			Renderer.AddText(TEXT("{cam_passive}From camera asset {cam_notice}%s{cam_default}\n"), *CameraAssetName);
		}
		Renderer.RemoveIndent();

		if (Renderer.IsExternalRendering() && GGameplayCamerasDebugContextInitialResultShowCoordinateSystem)
		{
			Renderer.DrawCoordinateSystem(InitialContextTransform);
		}
	}
	else
	{
		Renderer.AddText(TEXT("{cam_error}Invalid context!{cam_default}\n"));
	}
}

void FCameraDirectorDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << ContextClassName;
	Ar << OwnerClassName;
	Ar << OwnerName;
	Ar << CameraAssetName;
	Ar << CameraDirectorClassName;
	Ar << InitialContextTransform;
	Ar << bIsValid;
}
	
}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

