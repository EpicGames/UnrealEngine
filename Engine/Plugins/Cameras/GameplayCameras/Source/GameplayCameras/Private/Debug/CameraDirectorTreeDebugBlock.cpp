// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraDirectorTreeDebugBlock.h"

#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraDirectorDebugBlock.h"
#include "Debug/CameraPoseDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraDirectorTreeDebugBlock)

void FCameraDirectorTreeDebugBlock::Initialize(const FCameraEvaluationContextStack& ContextStack, FCameraDebugBlockBuilder& Builder)
{
	const int32 NumContexts = ContextStack.NumContexts();
	for (int32 Index = 0; Index < NumContexts; ++Index)
	{
		const FCameraEvaluationContextStack::FContextEntry& Entry(ContextStack.Entries[Index]);
		TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin();

		FCameraDirectorDebugBlock& Child = Builder.StartChildDebugBlock<FCameraDirectorDebugBlock>();
		{
			Child.Initialize(Context, Builder);
		}
		Builder.EndChildDebugBlock();
	}
}

void FCameraDirectorTreeDebugBlock::Initialize(TArrayView<const TSharedPtr<FCameraEvaluationContext>> Contexts, FCameraDebugBlockBuilder& Builder)
{
	for (TSharedPtr<FCameraEvaluationContext> Context : Contexts)
	{
		FCameraDirectorDebugBlock& Child = Builder.StartChildDebugBlock<FCameraDirectorDebugBlock>();
		{
			Child.Initialize(Context, Builder);
		}
		Builder.EndChildDebugBlock();
	}
}

void FCameraDirectorTreeDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const FCameraDebugColors& Colors = FCameraDebugColors::Get();

	// Separate the active director from the inactive ones.
	// The inactive ones are at the beginning (bottom) of the stack.
	Renderer.SetTextColor(Colors.Notice);
	Renderer.AddText("Inactive Directors\n");
	Renderer.SetTextColor(Colors.Default);
	Renderer.AddIndent();

	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());

	const int32 NumInactiveDirectors = ChildrenView.Num() - 1;

	for (int32 Index = 0; Index < ChildrenView.Num(); ++Index)
	{
		FLinearColor CameraPoseLineColor(FColorList::SlateBlue);

		// If we reached the top of the stack, display the active director separately,
		// and draw the initial camera pose in SlateBlue color. Otherwise, draw the
		// the initial camera pose somewhere between dark and light grey.
		if (Index == ChildrenView.Num() - 1)
		{
			Renderer.RemoveIndent();

			Renderer.SetTextColor(Colors.Notice);
			Renderer.AddText("Active Director\n");
			Renderer.SetTextColor(Colors.Default);

			Renderer.AddIndent();
		}
		else
		{
			CameraPoseLineColor = LerpLinearColorUsingHSV(
					FColorList::DimGrey, FColorList::LightGrey, Index, NumInactiveDirectors);
		}

		FString PoseLabel(LexToString(Index));
		FScopedGlobalCameraPoseRenderingParams ScopedPoseParams(PoseLabel, CameraPoseLineColor);

		Renderer.AddText(TEXT("{cam_passive}[%d]{cam_default} "), Index + 1);

		if (FCameraDirectorDebugBlock* EntryDebugBlock = ChildrenView[Index]->CastThisChecked<FCameraDirectorDebugBlock>())
		{
			EntryDebugBlock->DebugDraw(Params, Renderer);
		}

		Renderer.NewLine();
	}

	Renderer.RemoveIndent();
	Renderer.SetTextColor(Colors.Default);

	Renderer.SkipAllBlocks();
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

