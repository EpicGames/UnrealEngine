// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraEvaluationServiceDebugBlock.h"

#include "Core/CameraEvaluationService.h"
#include "Core/CameraObjectRtti.h"
#include "Debug/CameraDebugRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraEvaluationServiceDebugBlock)

FCameraEvaluationServiceDebugBlock::FCameraEvaluationServiceDebugBlock()
{
}

FCameraEvaluationServiceDebugBlock::FCameraEvaluationServiceDebugBlock(TSharedPtr<const FCameraEvaluationService> InEvaluationService)
{
	FCameraObjectTypeID TypeID = InEvaluationService->GetTypeID();
	const FCameraObjectTypeInfo* TypeInfo = FCameraObjectTypeRegistry::Get().GetTypeInfo(TypeID);
	ServiceClassName = TypeInfo ? TypeInfo->TypeName.ToString() : TEXT("<no type info>");
}

void FCameraEvaluationServiceDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("{cam_passive}[%s]{cam_default} "), *ServiceClassName);
}

void FCameraEvaluationServiceDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << ServiceClassName;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

