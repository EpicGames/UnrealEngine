// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraNodeEvaluationResultDebugBlock.h"

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraPoseDebugBlock.h"
#include "Debug/CameraRigJointsDebugBlock.h"
#include "Debug/ContextDataTableDebugBlock.h"
#include "Debug/DebugTextRenderer.h"
#include "Debug/PostProcessSettingsDebugBlock.h"
#include "Debug/VariableTableDebugBlock.h"
#include "Math/ColorList.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraNodeEvaluationResultDebugBlock)

FCameraNodeEvaluationResultDebugBlock::FCameraNodeEvaluationResultDebugBlock()
{
}

void FCameraNodeEvaluationResultDebugBlock::Initialize(const FCameraNodeEvaluationResult& InResult, FCameraDebugBlockBuilder& Builder)
{
	bIsCameraCut = InResult.bIsCameraCut;
	bIsValid = InResult.bIsValid;

	AddChild(&Builder.BuildDebugBlock<FCameraPoseDebugBlock>(InResult.CameraPose));
	AddChild(&Builder.BuildDebugBlock<FVariableTableDebugBlock>(InResult.VariableTable));
	AddChild(&Builder.BuildDebugBlock<FContextDataTableDebugBlock>(InResult.ContextDataTable));
	AddChild(&Builder.BuildDebugBlock<FPostProcessSettingsDebugBlock>(InResult.PostProcessSettings));
	AddChild(&Builder.BuildDebugBlock<FCameraRigJointsDebugBlock>(InResult.CameraRigJoints, InResult.VariableTable));
}

void FCameraNodeEvaluationResultDebugBlock::Initialize(const FCameraSystemEvaluationResult& InResult, FCameraDebugBlockBuilder& Builder)
{
	bIsCameraCut = InResult.bIsCameraCut;
	bIsValid = InResult.bIsValid;

	AddChild(&Builder.BuildDebugBlock<FCameraPoseDebugBlock>(InResult.CameraPose));
	AddChild(&Builder.BuildDebugBlock<FVariableTableDebugBlock>(InResult.VariableTable));
	AddChild(&Builder.BuildDebugBlock<FContextDataTableDebugBlock>(InResult.ContextDataTable));
	AddChild(&Builder.BuildDebugBlock<FPostProcessSettingsDebugBlock>(InResult.PostProcessSettings));
	// No rig joints exposed on the camera system result.
}

FCameraPoseDebugBlock* FCameraNodeEvaluationResultDebugBlock::GetCameraPoseDebugBlock()
{
	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());
	if (ChildrenView.IsValidIndex(0))
	{
		return ChildrenView[0]->CastThis<FCameraPoseDebugBlock>();
	}
	return nullptr;
}

FVariableTableDebugBlock* FCameraNodeEvaluationResultDebugBlock::GetVariableTableDebugBlock()
{
	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());
	if (ChildrenView.IsValidIndex(1))
	{
		return ChildrenView[1]->CastThis<FVariableTableDebugBlock>();
	}
	return nullptr;
}

FContextDataTableDebugBlock* FCameraNodeEvaluationResultDebugBlock::GetContextDataTableDebugBlock()
{
	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());
	if (ChildrenView.IsValidIndex(2))
	{
		return ChildrenView[2]->CastThis<FContextDataTableDebugBlock>();
	}
	return nullptr;
}

FPostProcessSettingsDebugBlock* FCameraNodeEvaluationResultDebugBlock::GetPostProcessSettingsDebugBlock()
{
	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());
	if (ChildrenView.IsValidIndex(3))
	{
		return ChildrenView[3]->CastThis<FPostProcessSettingsDebugBlock>();
	}
	return nullptr;
}

void FCameraNodeEvaluationResultDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const FCameraDebugColors& Colors = FCameraDebugColors::Get();

	if (bIsValid)
	{
		Renderer.AddText(TEXT("Valid: {cam_good}YES"));
	}
	else
	{
		Renderer.AddText(TEXT("Valid: {cam_error}NO"));
	}

	if (bIsCameraCut)
	{
		Renderer.AddText(TEXT("  {cam_warning}IsCameraCut"));
	}

	Renderer.NewLine();
	Renderer.SetTextColor(Colors.Default);

	TArrayView<FCameraDebugBlock*> ChildrenView(GetChildren());
	if (ChildrenView.IsValidIndex(0))
	{
		Renderer.AddText(TEXT("{cam_title}Camera Pose:"));
		Renderer.AddIndent();
		Renderer.SetTextColor(Colors.Default);
		ChildrenView[0]->DebugDraw(Params, Renderer);
		Renderer.RemoveIndent();
	}
	if (ChildrenView.IsValidIndex(1))
	{
		Renderer.AddText(TEXT("{cam_title}Variable Table:"));
		Renderer.AddIndent();
		Renderer.SetTextColor(Colors.Default);
		ChildrenView[1]->DebugDraw(Params, Renderer);
		Renderer.RemoveIndent();
	}
	if (ChildrenView.IsValidIndex(2))
	{
		Renderer.AddText(TEXT("{cam_title}Context Data Table:"));
		Renderer.AddIndent();
		Renderer.SetTextColor(Colors.Default);
		ChildrenView[2]->DebugDraw(Params, Renderer);
		Renderer.RemoveIndent();
	}
	if (ChildrenView.IsValidIndex(3))
	{
		Renderer.AddText(TEXT("{cam_title}Post-Process Settings:"));
		Renderer.AddIndent();
		Renderer.SetTextColor(Colors.Default);
		ChildrenView[3]->DebugDraw(Params, Renderer);
		Renderer.RemoveIndent();
	}
	if (ChildrenView.IsValidIndex(4))
	{
		Renderer.AddText(TEXT("{cam_title}Camera Rig Joints:"));
		Renderer.AddIndent();
		Renderer.SetTextColor(Colors.Default);
		ChildrenView[4]->DebugDraw(Params, Renderer);
		Renderer.RemoveIndent();
	}

	// We already manually rendered our children.
	Renderer.SkipAllBlocks();
}

void FCameraNodeEvaluationResultDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << bIsCameraCut;
	Ar << bIsValid;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

