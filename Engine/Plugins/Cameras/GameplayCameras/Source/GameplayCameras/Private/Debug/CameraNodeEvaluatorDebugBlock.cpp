// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraNodeEvaluatorDebugBlock.h"

#include "Core/CameraNode.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

FString GGameplayCamerasDebugNodeTreeFilter;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugNodeTreeFilter(
	TEXT("GameplayCameras.Debug.NodeTree.Filter"),
	GGameplayCamerasDebugNodeTreeFilter,
	TEXT("(Default: "". Filters the debug camera node tree by node name/type."));

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraNodeEvaluatorDebugBlock)

FCameraNodeEvaluatorDebugBlock::FCameraNodeEvaluatorDebugBlock()
{
}

FCameraNodeEvaluatorDebugBlock::FCameraNodeEvaluatorDebugBlock(TObjectPtr<const UCameraNode> InCameraNode)
{
	NodeClassName = (InCameraNode ? InCameraNode->GetClass()->GetName() : TEXT("<no node>"));
}

void FCameraNodeEvaluatorDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	// Only show the name/type of this node, along with extra info (in the attached blocks) if we
	// are in the "nodetree" debug mode, and we're not filtering this node out.
	const bool bDoDebugDraw = (
			GGameplayCamerasDebugNodeTreeFilter.IsEmpty() ||
			NodeClassName.Contains(GGameplayCamerasDebugNodeTreeFilter));
	if (bDoDebugDraw)
	{
		Renderer.AddText(TEXT("{cam_passive}[%s]{cam_default} "), *NodeClassName);
	}
	else
	{
		Renderer.SkipAttachedBlocks();
	}
}

void FCameraNodeEvaluatorDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << NodeClassName;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

