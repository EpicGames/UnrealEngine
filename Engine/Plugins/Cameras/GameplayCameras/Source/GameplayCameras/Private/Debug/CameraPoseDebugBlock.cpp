// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraPoseDebugBlock.h"

#include "Containers/UnrealString.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/DebugTextRenderer.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "Math/ColorList.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

float GGameplayCamerasDebugCameraPoseLabelWorldOffset = 20.f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugCameraPoseLabelWorldOffset(
	TEXT("GameplayCameras.Debug.CameraPose.LabelWorldOffset"),
	GGameplayCamerasDebugCameraPoseLabelWorldOffset,
	TEXT(""));

namespace Private
{

template<typename FieldType>
void DebugDrawCameraPoseField(FCameraDebugRenderer& Renderer, const TCHAR* FieldName, typename TCallTraits<FieldType>::ParamType FieldValue, const FColor& Color)
{
	Renderer.SetTextColor(Color);
	Renderer.AddText(TEXT("%s  : %s\n"), FieldName, *ToDebugString(FieldValue));
}

}  // namespace Private

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraPoseDebugBlock)

FLinearColor FCameraPoseDebugBlock::GlobalCameraPoseLineColor(FColorList::SlateBlue);
FString FCameraPoseDebugBlock::GlobalCameraPoseLabel;

FScopedGlobalCameraPoseRenderingParams::FScopedGlobalCameraPoseRenderingParams(const FString& Label)
	: FScopedGlobalCameraPoseRenderingParams(Label, FCameraPoseDebugBlock::GlobalCameraPoseLineColor)
{
}

FScopedGlobalCameraPoseRenderingParams::FScopedGlobalCameraPoseRenderingParams(const FLinearColor& LineColor)
	: FScopedGlobalCameraPoseRenderingParams(FCameraPoseDebugBlock::GlobalCameraPoseLabel, LineColor)
{
}

FScopedGlobalCameraPoseRenderingParams::FScopedGlobalCameraPoseRenderingParams(const FString& Label, const FLinearColor& LineColor)
{
	PreviousLabel = FCameraPoseDebugBlock::GlobalCameraPoseLabel;
	PreviousLineColor = FCameraPoseDebugBlock::GlobalCameraPoseLineColor;

	FCameraPoseDebugBlock::GlobalCameraPoseLabel = Label;
	FCameraPoseDebugBlock::GlobalCameraPoseLineColor = LineColor;
}

FScopedGlobalCameraPoseRenderingParams::~FScopedGlobalCameraPoseRenderingParams()
{
	FCameraPoseDebugBlock::GlobalCameraPoseLabel = PreviousLabel;
	FCameraPoseDebugBlock::GlobalCameraPoseLineColor = PreviousLineColor;
}

FCameraPoseDebugBlock::FCameraPoseDebugBlock()
{
}

FCameraPoseDebugBlock::FCameraPoseDebugBlock(const FCameraPose& InCameraPose)
	: CameraPose(InCameraPose)
{
}

void FCameraPoseDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bDrawText)
	{
		bool bShowUnchanged = false;
		if (!ShowUnchangedCVarName.IsEmpty())
		{
			IConsoleVariable* ShowUnchangedCVar = IConsoleManager::Get().FindConsoleVariable(*ShowUnchangedCVarName, false);
			if (ensureMsgf(ShowUnchangedCVar, TEXT("No such console variable: %s"), *ShowUnchangedCVarName))
			{
				bShowUnchanged = ShowUnchangedCVar->GetBool();
			}
		}

		const FCameraDebugColors& Colors = FCameraDebugColors::Get();
		const FColor ChangedColor = Colors.Default;
		const FColor UnchangedColor = Colors.Passive;

		const FCameraPoseFlags& ChangedFlags = CameraPose.GetChangedFlags();

#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
		if (bShowUnchanged || ChangedFlags.PropName)\
		{\
			const FColor& PropColor = ChangedFlags.PropName ? ChangedColor : UnchangedColor;\
			Private::DebugDrawCameraPoseField<PropType>(Renderer, TEXT(#PropName), CameraPose.Get##PropName(), PropColor);\
		}
		UE_CAMERA_POSE_FOR_ALL_PROPERTIES()
#undef UE_CAMERA_POSE_FOR_PROPERTY

		Renderer.SetTextColor(Colors.Default);
		Renderer.AddText(TEXT("Effective FOV  : %f\n"), CameraPose.GetEffectiveFieldOfView());
		Renderer.AddText(TEXT("Effective Aspect Ratio  : %f\n"), CameraPose.GetSensorAspectRatio());
	}

	if (bDrawInExternalRendering && Renderer.IsExternalRendering())
	{
		Renderer.DrawCameraPose(CameraPose, GlobalCameraPoseLineColor);

		if (!GlobalCameraPoseLabel.IsEmpty())
		{
			const FVector3d TextWorldOffset(0, 0, GGameplayCamerasDebugCameraPoseLabelWorldOffset);
			UFont* LargeFont = GEngine->GetLargeFont();
			Renderer.DrawText(
					CameraPose.GetLocation() + TextWorldOffset,
					GlobalCameraPoseLabel,
					GlobalCameraPoseLineColor,
					LargeFont);
		}
	}
}

void FCameraPoseDebugBlock::OnSerialize(FArchive& Ar)
{
	FCameraPose::SerializeWithFlags(Ar, CameraPose);
	Ar << ShowUnchangedCVarName;
	Ar << bDrawText;
	Ar << bDrawInExternalRendering;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

