// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraPoseLocationTrailDebugBlock.h"

#include "Core/CameraNodeEvaluator.h"
#include "Debug/CameraDebugRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraPoseLocationTrailDebugBlock)

FCameraPoseLocationTrailDebugBlock::FCameraPoseLocationTrailDebugBlock()
{
}

FCameraPoseLocationTrailDebugBlock::FCameraPoseLocationTrailDebugBlock(const FCameraNodeEvaluationResult& InResult)
{
	Trail = InResult.GetCameraPoseLocationTrail();
}

void FCameraPoseLocationTrailDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (Renderer.IsExternalRendering())
	{
		DrawCameraPoseLocationTrail(Renderer);
	}
}

void FCameraPoseLocationTrailDebugBlock::DrawCameraPoseLocationTrail(FCameraDebugRenderer& Renderer) const
{
	const FLinearColor TrailColor(FColorList::LightBlue);
	for (int32 Index = 1; Index < Trail.Num(); ++Index)
	{
		const FVector3d& PrevPoint(Trail[Index - 1]);
		const FVector3d& NextPoint(Trail[Index]);

		if (!PrevPoint.IsZero() && FVector3d::Distance(PrevPoint, NextPoint) > UE_SMALL_NUMBER)
		{
			Renderer.DrawLine(PrevPoint, NextPoint, TrailColor, 0.3f);
		}
	}
}

void FCameraPoseLocationTrailDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << Trail;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

