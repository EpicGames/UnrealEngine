// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraRigJointsDebugBlock.h"

#include "Core/CameraVariableTable.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraRigJointsDebugBlock)

FCameraRigJointsDebugBlock::FCameraRigJointsDebugBlock()
{
}

FCameraRigJointsDebugBlock::FCameraRigJointsDebugBlock(const FCameraRigJoints& InCameraRigJoints, const FCameraVariableTable& InVariableTable)
{
	for (const FCameraRigJoint& Joint : InCameraRigJoints.GetJoints())
	{
		FEntry Entry;
		Entry.Location = Joint.Transform.GetLocation();
		Entry.Rotation = Joint.Transform.Rotator();
		Entry.VariableID = Joint.VariableID;

#if WITH_EDITORONLY_DATA
		FCameraVariableDefinition VariableDefinition;
		if (InVariableTable.TryGetVariableDefinition(Joint.VariableID, VariableDefinition))
		{
			Entry.VariableName = VariableDefinition.VariableName;
		}
		else
		{
			Entry.VariableName = TEXT("<Unknown>");
		}
#else
		Entry.VariableName = TEXT("<Unknown>");
#endif  // WITH_EDITORONLY_DATA
	}
}

void FCameraRigJointsDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("%d joints"), Entries.Num());
	Renderer.AddIndent();
	{
		for (const FEntry& Entry : Entries)
		{
			Renderer.AddText(
					TEXT("location {cam_notice}%s{cam_default}  "
						"rotation {cam_notice}%s{cam_default}  "
						"variable {cam_notice2}%s{cam_passive} [%d]{cam_default}"),
					*Entry.Location.ToString(),
					*Entry.Rotation.ToString(),
					*Entry.VariableName,
					Entry.VariableID.GetValue());
		}
	}
	Renderer.RemoveIndent();
}

void FCameraRigJointsDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << Entries;
}

FArchive& operator<< (FArchive& Ar, FCameraRigJointsDebugBlock::FEntry& Entry)
{
	Ar << Entry.Location;
	Ar << Entry.Rotation;
	Ar << Entry.VariableID;
	Ar << Entry.VariableName;
	return Ar;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

