// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraSystemDebugRegistry.h"

#include "Core/CameraSystemEvaluator.h"
#include "Serialization/Archive.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

FArchive& operator <<(FArchive& Ar, FCameraSystemDebugID& InDebugID)
{
	Ar << InDebugID.Value;
	return Ar;
}

FCameraSystemDebugRegistry& FCameraSystemDebugRegistry::Get()
{
	static FCameraSystemDebugRegistry Instance;
	return Instance;
}

FCameraSystemDebugID FCameraSystemDebugRegistry::RegisterCameraSystemEvaluator(TSharedRef<FCameraSystemEvaluator> InEvaluator)
{
	const FString OwnerName = GetNameSafe(InEvaluator->GetOwner());
	FEntry NewEntry{ InEvaluator, OwnerName };
	const int32 NewIndex = Entries.Add(NewEntry);
	return FCameraSystemDebugID(NewIndex + 1);
}

void FCameraSystemDebugRegistry::UnregisterCameraSystemEvaluator(FCameraSystemDebugID InDebugID)
{
	if (ensure(InDebugID.IsValid() && !InDebugID.IsAny() && !InDebugID.IsAuto()))
	{
		Entries.RemoveAt(InDebugID.Value - 1);
	}
}

void FCameraSystemDebugRegistry::GetRegisteredCameraSystemEvaluators(FRegisteredCameraSystems& OutEvaluators) const
{
	for (auto It = Entries.CreateConstIterator(); It; ++It)
	{
		const FEntry& Entry(*It);
		if (TSharedPtr<FCameraSystemEvaluator> Evaluator = Entry.WeakEvaluator.Pin())
		{
			OutEvaluators.Add(Evaluator);
		}
	}
}

int32 FCameraSystemDebugRegistry::NumRegisteredCameraSystemEvaluators() const
{
	return Entries.Num();
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

