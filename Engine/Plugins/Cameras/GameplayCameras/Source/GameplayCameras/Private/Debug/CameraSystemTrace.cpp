// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CameraSystemTrace.h"

#include "Containers/UnrealString.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/RootCameraDebugBlock.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "ObjectTrace.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

bool GGameplayCamerasDebugTrace = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugTrace(
	TEXT("GameplayCameras.Debug.Trace"),
	GGameplayCamerasDebugTrace,
	TEXT("(Default: false. Enables background tracing of GamplayCameras system debug info."));

// Channel "CameraSystemChannel".
UE_TRACE_CHANNEL(CameraSystemChannel, "Per-frame Gameplay Cameras evaluation data (pose, FOV, evaluator/blend hierarchy).");

// Log name "CameraSystem", event name "CameraSystemEvaluation".
UE_TRACE_EVENT_BEGIN(CameraSystem, CameraSystemEvaluation)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(int32, CameraSystemDebugID)
	UE_TRACE_EVENT_FIELD(double, EvaluatedLocationX)
	UE_TRACE_EVENT_FIELD(double, EvaluatedLocationY)
	UE_TRACE_EVENT_FIELD(double, EvaluatedLocationZ)
	UE_TRACE_EVENT_FIELD(double, EvaluatedRotationPitch)
	UE_TRACE_EVENT_FIELD(double, EvaluatedRotationYaw)
	UE_TRACE_EVENT_FIELD(double, EvaluatedRotationRoll)
	UE_TRACE_EVENT_FIELD(float, EvaluatedFieldOfView)
	UE_TRACE_EVENT_FIELD(uint8[], SerializedBlocks)
UE_TRACE_EVENT_END()

class FCameraDebugBlockSerializer
{
protected:

	static uint8 TokenBufferStart;
	static uint8 TokenSerializerVersion;
	static uint8 TokenBlockStart;
	static uint8 TokenRelatedIndices;
	static uint8 TokenBlockEnd;
	static uint8 TokenBufferEnd;
};

uint8 FCameraDebugBlockSerializer::TokenBufferStart = 0x11;
uint8 FCameraDebugBlockSerializer::TokenSerializerVersion = 0x01;
uint8 FCameraDebugBlockSerializer::TokenBlockStart = 0x22;
uint8 FCameraDebugBlockSerializer::TokenRelatedIndices = 0x33;
uint8 FCameraDebugBlockSerializer::TokenBlockEnd = 0x44;
uint8 FCameraDebugBlockSerializer::TokenBufferEnd = 0x44;

class FCameraDebugBlockWriter : public FCameraDebugBlockSerializer
{
public:

	FCameraDebugBlockWriter(FBufferArchive& InArchive)
		: Archive(InArchive)
		, TypeRegistry(FCameraObjectTypeRegistry::Get())
	{}
	
	void Write(FCameraDebugBlock& InRootDebugBlock)
	{
		check(Archive.IsSaving());

		Archive << TokenBufferStart;
		Archive << TokenSerializerVersion;

		NextBlockIndex = 0;
		BlockIndices.Add(&InRootDebugBlock, NextBlockIndex++);

		TArray<FCameraDebugBlock*> WriteStack;
		WriteStack.Add(&InRootDebugBlock);
		while (!WriteStack.IsEmpty())
		{
			FCameraDebugBlock* CurBlock(WriteStack.Pop());
			WriteImpl(CurBlock, WriteStack);
		}

		Archive << TokenBufferEnd;

		ensure(BlockIndices.IsEmpty());
	}

private:
	
	void WriteImpl(FCameraDebugBlock* InBlock, TArray<FCameraDebugBlock*>& InWriteStack)
	{
		Archive << TokenBlockStart;

		int32 BlockIndex = BlockIndices.FindChecked(InBlock);
		Archive << BlockIndex;
		BlockIndices.Remove(InBlock);

		// We can't serialize type IDs because they're not stable, so we serialize
		// the type name directly. This isn't very optimized, as it writes lots of
		// strings in the buffer, but it's good enough as a first implementation.
		FCameraObjectTypeID BlockTypeID = InBlock->GetTypeID();
		const FCameraObjectTypeInfo* BlockTypeInfo = TypeRegistry.GetTypeInfo(BlockTypeID);
		check(BlockTypeInfo);
		FName BlockTypeName = BlockTypeInfo->TypeName;
		Archive << BlockTypeName;

		InBlock->Serialize(Archive);

		Archive << TokenRelatedIndices;

		TArray<int32> AttachmentIndices;
		for (FCameraDebugBlock* Attachment : InBlock->GetAttachments())
		{
			int32 AttachmentIndex = BlockIndices.Add(Attachment, NextBlockIndex++);
			AttachmentIndices.Add(AttachmentIndex);
			InWriteStack.Add(Attachment);
		}
		Archive << AttachmentIndices;

		TArray<int32> ChildrenIndices;
		for (FCameraDebugBlock* Child : InBlock->GetChildren())
		{
			int32 ChildIndex = BlockIndices.Add(Child, NextBlockIndex++);
			ChildrenIndices.Add(ChildIndex);
			InWriteStack.Add(Child);
		}
		Archive << ChildrenIndices;

		Archive << TokenBlockEnd;
	}

private:

	FBufferArchive& Archive;
	FCameraObjectTypeRegistry& TypeRegistry;

	TMap<FCameraDebugBlock*, int32> BlockIndices;
	int32 NextBlockIndex;
};

class FCameraDebugBlockReader : public FCameraDebugBlockSerializer
{
public:

	FCameraDebugBlockReader(FArchive& InArchive, FCameraDebugBlockStorage& InStorage)
		: Archive(InArchive)
		, Storage(InStorage)
		, TypeRegistry(FCameraObjectTypeRegistry::Get())
	{
	}

	FCameraDebugBlock* Read()
	{
		check(Archive.IsLoading());

		uint8 Token;

		Archive << Token;
		check(Token == TokenBufferStart);
		Archive << Token;
		check(Token == TokenSerializerVersion);
		
		bool bIsRootBlock = true;
		while (true)
		{
			Archive << Token;
			if (Token == TokenBufferEnd)
			{
				break;
			}
			else if (Token == TokenBlockStart)
			{
				ReadImpl();
			}
			else
			{
				ensure(false);
				break;
			}
		}

		SetupRelatedBlocks();

		ensure(BlocksByIndex.IsValidIndex(0));
		return BlocksByIndex[0];
	}

private:

	void ReadImpl()
	{
		int32 BlockIndex;
		Archive << BlockIndex;

		FName BlockTypeName;
		Archive << BlockTypeName;
		check(BlockTypeName != NAME_None);

		FCameraObjectTypeID BlockTypeID = TypeRegistry.FindTypeByName(BlockTypeName);
		check(BlockTypeID.IsValid());
		const FCameraObjectTypeInfo* BlockTypeInfo = TypeRegistry.GetTypeInfo(BlockTypeID);
		check(BlockTypeInfo);

		void* NewBlockPtr = Storage.BuildDebugBlockUninitialized(BlockTypeInfo->Sizeof, BlockTypeInfo->Alignof);
		check(NewBlockPtr);
		BlockTypeInfo->Constructor(NewBlockPtr);

		// This isn't quite correct for complicated inheritance configurations, but we don't
		// expect those sorts of setups for debug blocks... hopefully.
		FCameraDebugBlock* NewBlock = reinterpret_cast<FCameraDebugBlock*>(NewBlockPtr);

		NewBlock->Serialize(Archive);

		uint8 Token;

		Archive << Token;
		check(Token == TokenRelatedIndices);

		FRelatedIndices CurRelatedIndices;
		Archive << CurRelatedIndices.AttachmentIndices;
		Archive << CurRelatedIndices.ChildrenIndices;
		RelatedIndices.Add(NewBlock, MoveTemp(CurRelatedIndices));

		Archive << Token;
		check(Token == TokenBlockEnd);

		BlocksByIndex.Insert(BlockIndex, NewBlock);
	}

	void SetupRelatedBlocks()
	{
		for (const TPair<FCameraDebugBlock*, FRelatedIndices>& Pair : RelatedIndices)
		{
			FCameraDebugBlock* CurBlock = Pair.Key;

			for (uint32 AttachmentIndex : Pair.Value.AttachmentIndices)
			{
				check(BlocksByIndex.IsValidIndex(AttachmentIndex));
				FCameraDebugBlock* AttachedBlock = BlocksByIndex[AttachmentIndex];
				CurBlock->Attach(AttachedBlock);
			}

			for (uint32 ChildIndex : Pair.Value.ChildrenIndices)
			{
				check(BlocksByIndex.IsValidIndex(ChildIndex));
				FCameraDebugBlock* ChildBlock = BlocksByIndex[ChildIndex];
				CurBlock->AddChild(ChildBlock);
			}
		}
	}

private:

	struct FRelatedIndices
	{
		TArray<int32> AttachmentIndices;
		TArray<int32> ChildrenIndices;
	};

	FArchive& Archive;
	FCameraDebugBlockStorage& Storage;
	FCameraObjectTypeRegistry& TypeRegistry;

	TSparseArray<FCameraDebugBlock*> BlocksByIndex;
	TMap<FCameraDebugBlock*, FRelatedIndices> RelatedIndices;
};

class FCameraDebugBlockWriterArchive : public FBufferArchive
{
public:

	// FArchive interface.
	virtual FArchive& operator<<(UObject*& Res) override { return SaveObjectPtrImpl(Res); }
	FArchive& operator<<(FLazyObjectPtr& Value) override { return SaveObjectPtrImpl(Value.Get()); }
	FArchive& operator<<(FObjectPtr& Value) override { return SaveObjectPtrImpl(Value.Get()); }
	FArchive& operator<<(FSoftObjectPtr& Value) override { return SaveObjectPtrImpl(Value.Get()); }
	FArchive& operator<<(FSoftObjectPath& Value) override { return SaveObjectPtrImpl(Value.ResolveObject()); }
	FArchive& operator<<(FWeakObjectPtr& Value) override { return SaveObjectPtrImpl(Value.Get()); }

private:

	FArchive& SaveObjectPtrImpl(UObject* Obj)
	{
		if (ensure(IsSaving()))
		{
			// See FCameraDebugBlockReaderArchive for information.
			FString ResClassName = Obj ? Obj->GetClass()->GetName() : FString();
			*this << ResClassName;

			FObjectKey ResKey(Obj);
			*this << ResKey;
		}
		return *this;
	}
};

class FCameraDebugBlockReaderArchive : public FMemoryReader
{
public:

	explicit FCameraDebugBlockReaderArchive(const TArray<uint8>& InBytes)
		: FMemoryReader(InBytes)
	{}

	// FArchive interface.
	virtual FArchive& operator<<(UObject*& Res) override { return LoadObjectPtrImpl(Res); }
	FArchive& operator<<(FLazyObjectPtr& Value) override { return LoadObjectPtrImpl2(Value); }
	FArchive& operator<<(FObjectPtr& Value) override { return LoadObjectPtrImpl2(Value); }
	FArchive& operator<<(FSoftObjectPtr& Value) override { return LoadObjectPtrImpl2(Value); }
	FArchive& operator<<(FSoftObjectPath& Value) override { return LoadObjectPtrImpl2(Value); }
	FArchive& operator<<(FWeakObjectPtr& Value) override { return LoadObjectPtrImpl2(Value); }

private:

	template<typename PtrType>
	FArchive& LoadObjectPtrImpl2(PtrType& Value)
	{
		UObject* Res = nullptr;
		LoadObjectPtrImpl(Res);
		Value = PtrType(Res);
		return *this;
	}

	FArchive& LoadObjectPtrImpl(UObject*& Res)
	{
		if (ensure(IsLoading()))
		{
			// In 99.99% of cases, we are recording gameplay and playing it back immediately after. In this case, the 
			// object-key that we saved into the trace file will either be:
			// - A valid object that is still in memory (probably the most common case)
			// - An object that is gone, and so the object-key resolves to a null pointer.
			//
			// This means that we can't (and shouldn't) use UObjects inside debug blocks. And we generally don't. 
			// At this time, the only exception is the Post Process Settings debug block, which saves the 
			// FPostProcessSettings structure into the trace file. There is a small chance that it has some 
			// WeightedBlendables in it, and those are pointers to objects. We won't get debug info for those _if_ 
			// they are dynamic objects that are gone from memory by the time we replay. That's an acceptable 
			// limitation for now (especially since, again, in a majority of cases, these post-process materials are 
			// assets on disk so they're valid during replay).
			//
			// We add an extra protection by saving the object's class name. This is for the very unlikely case of
			// someone saving a trace file and replaying it later, in a different editor session. The object-key would
			// resolve to some completely unrelated object, so we skip that case. There is of course the even more
			// unlikely case that the object-key somehow resolves to an object of the right class, in which case we
			// will happily set it, but since it's the right class it should work, even though it's incorrect. At this
			// point let's just leave that as a known bug for now.
			//
			FString ResClassName;
			*this << ResClassName;

			FObjectKey ResKey;
			*this << ResKey;

			if (UObject* LoadedRes = ResKey.ResolveObjectPtr())
			{
				if (ResClassName == LoadedRes->GetClass()->GetName())
				{
					Res = LoadedRes;
				}
			}
			else
			{
				if (ResClassName.IsEmpty())
				{
					Res = nullptr;
				}
			}
		}
		return *this;
	}
};

// This name must match the one passed to UE_TRACE_CHANNEL above.
FString FCameraSystemTrace::ChannelName("CameraSystemChannel");
// These two names must match the names passed to UE_TRACE_EVENT_BEGIN above.
FString FCameraSystemTrace::LoggerName("CameraSystem");
FString FCameraSystemTrace::EvaluationEventName("CameraSystemEvaluation");

bool FCameraSystemTrace::IsTraceEnabled()
{
	return GGameplayCamerasDebugTrace || UE_TRACE_CHANNELEXPR_IS_ENABLED(CameraSystemChannel);
}

void FCameraSystemTrace::TraceEvaluation(UWorld* InWorld, const FCameraSystemEvaluationResult& InResult, FRootCameraDebugBlock& InRootDebugBlock)
{
	if (!IsTraceEnabled())
	{
		return;
	}

	FCameraDebugBlockWriterArchive BufferArchive;
	FCameraDebugBlockWriter Writer(BufferArchive);
	Writer.Write(InRootDebugBlock);

	// Names must match LoggerName, EvaluationEventName, ChannelName.
	UE_TRACE_LOG(CameraSystem, CameraSystemEvaluation, CameraSystemChannel)
		<< CameraSystemEvaluation.Cycle(FPlatformTime::Cycles64())
		<< CameraSystemEvaluation.RecordingTime(FObjectTrace::GetWorldElapsedTime(InWorld))
		<< CameraSystemEvaluation.CameraSystemDebugID(InRootDebugBlock.GetDebugID().GetValue())
		<< CameraSystemEvaluation.EvaluatedLocationX(InResult.CameraPose.GetLocation().X)
		<< CameraSystemEvaluation.EvaluatedLocationY(InResult.CameraPose.GetLocation().Y)
		<< CameraSystemEvaluation.EvaluatedLocationZ(InResult.CameraPose.GetLocation().Z)
		<< CameraSystemEvaluation.EvaluatedRotationYaw(InResult.CameraPose.GetRotation().Yaw)
		<< CameraSystemEvaluation.EvaluatedRotationPitch(InResult.CameraPose.GetRotation().Pitch)
		<< CameraSystemEvaluation.EvaluatedRotationRoll(InResult.CameraPose.GetRotation().Roll)
		<< CameraSystemEvaluation.EvaluatedFieldOfView(InResult.CameraPose.GetEffectiveFieldOfView())
    	<< CameraSystemEvaluation.SerializedBlocks(BufferArchive.GetData(), BufferArchive.Num());
}

FCameraDebugBlock* FCameraSystemTrace::ReadEvaluationTrace(TArray<uint8> InSerializedBlocks, FCameraDebugBlockStorage& InStorage)
{
	FCameraDebugBlockReaderArchive MemoryArchive(InSerializedBlocks);
	FCameraDebugBlockReader Reader(MemoryArchive, InStorage);
	return Reader.Read();
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

