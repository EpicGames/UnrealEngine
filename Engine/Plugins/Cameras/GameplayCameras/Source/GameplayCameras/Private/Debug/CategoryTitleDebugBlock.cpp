// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/CategoryTitleDebugBlock.h"

#include "Debug/CameraDebugRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCategoryTitleDebugBlock)

FCategoryTitleDebugBlock::FCategoryTitleDebugBlock()
{
}

FCategoryTitleDebugBlock::FCategoryTitleDebugBlock(const FString& InCategory, const FString& InTitle)
	: Category(InCategory)
	, Title(InTitle)
{
}

void FCategoryTitleDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (Params.IsCategoryActive(Category))
	{
		if (!Title.IsEmpty())
		{
			Renderer.AddText(TEXT("{cam_title}%s{cam_default}\n"), *Title);
		}
	}
	else
	{
		if (bSkipAttachedBlocksIfInactive)
		{
			Renderer.SkipAttachedBlocks();
		}
		if (bSkipChildrenBlocksIfInactive)
		{
			Renderer.SkipChildrenBlocks();
		}
	}
}

void FCategoryTitleDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << Category;
	Ar << Title;

	Ar << bSkipAttachedBlocksIfInactive;
	Ar << bSkipChildrenBlocksIfInactive;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

