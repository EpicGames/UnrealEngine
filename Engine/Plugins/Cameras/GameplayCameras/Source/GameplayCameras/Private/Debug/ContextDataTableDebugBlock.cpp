// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/ContextDataTableDebugBlock.h"

#include "Core/CameraContextDataTable.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/DebugTextRenderer.h"
#include "HAL/IConsoleManager.h"
#include "UObject/PropertyPortFlags.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FContextDataTableDebugBlock)

FContextDataTableDebugBlock::FContextDataTableDebugBlock()
{
}

FContextDataTableDebugBlock::FContextDataTableDebugBlock(const FCameraContextDataTable& InContextDataTable)
{
	Initialize(InContextDataTable);
}

void FContextDataTableDebugBlock::Initialize(const FCameraContextDataTable& InContextDataTable)
{
	for (const FCameraContextDataTable::FEntry& Entry : InContextDataTable.Entries)
	{
		FString EntryName;
#if WITH_EDITORONLY_DATA
		EntryName = Entry.DebugName;
#endif

		FName EntryTypeName;
		if (Entry.TypeObject)
		{
			EntryTypeName = Entry.TypeObject->GetFName();
		}

		const uint8* EntryData = InContextDataTable.Memory + Entry.Offset;
		FString EntryValueStr = GetDebugValueString(Entry.Type, Entry.ContainerType, Entry.TypeObject, EntryData);

		FEntryDebugInfo EntryDebugInfo{ Entry.ID.GetValue(), EntryName, EntryTypeName, EntryValueStr};
		EntryDebugInfo.bWritten = EnumHasAnyFlags(Entry.Flags, FCameraContextDataTable::EEntryFlags::Written);
		EntryDebugInfo.bWrittenThisFrame = EnumHasAnyFlags(Entry.Flags, FCameraContextDataTable::EEntryFlags::WrittenThisFrame);
		Entries.Add(EntryDebugInfo);
	}

	Entries.StableSort([](const FEntryDebugInfo& A, const FEntryDebugInfo& B) -> bool
			{
				return A.Name.Compare(B.Name) < 0;
			});
}

FString FContextDataTableDebugBlock::GetDebugValueString(
		ECameraContextDataType DataType, 
		ECameraContextDataContainerType DataContainerType,
		const UObject* DataTypeObject,
		const uint8* DataPtr)
{
	if (DataContainerType == ECameraContextDataContainerType::None)
	{
		return GetDebugValueString(DataType, DataTypeObject, DataPtr);
	}
	else if (DataContainerType == ECameraContextDataContainerType::Array)
	{
		FString ArrayStr;
		FCameraContextDataTable::FArrayEntryHelper Helper(DataType, DataTypeObject, const_cast<uint8*>(DataPtr));
		for (int32 Index = 0; Index < Helper.Num(); ++Index)
		{
			const uint8* ElementPtr = Helper.GetRawPtr(Index);
			ArrayStr += FString::Printf(TEXT("[%d] %s\n"), Index, *GetDebugValueString(DataType, DataTypeObject, ElementPtr));
		}
		return ArrayStr;
	}
	return FString();
}

FString FContextDataTableDebugBlock::GetDebugValueString(
		ECameraContextDataType DataType, 
		const UObject* DataTypeObject,
		const uint8* DataPtr)
{
	FString EntryValueStr;
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			EntryValueStr = reinterpret_cast<const FName*>(DataPtr)->ToString();
			break;
		case ECameraContextDataType::String:
			EntryValueStr = *reinterpret_cast<const FString*>(DataPtr);
			break;
		case ECameraContextDataType::Enum:
			{
				const UEnum* EnumType = CastChecked<const UEnum>(DataTypeObject);
				EntryValueStr = EnumType->GetNameStringByValue((int64)*reinterpret_cast<const uint8*>(DataPtr));
			}
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = CastChecked<const UScriptStruct>(DataTypeObject);
				const int32 ExportFlags = PPF_Delimited | PPF_IncludeTransient | PPF_ExternalEditor;
				StructType->ExportText(EntryValueStr, DataPtr, nullptr, nullptr, ExportFlags, nullptr);
			}
			break;
		case ECameraContextDataType::Object:
			EntryValueStr = reinterpret_cast<const TObjectPtr<UObject>*>(DataPtr)->GetPathName();
			break;
		case ECameraContextDataType::Class:
			EntryValueStr = reinterpret_cast<const TObjectPtr<UClass>*>(DataPtr)->GetPathName();
			break;
		default:
			ensure(false);
			break;
	}
	return EntryValueStr;
}

void FContextDataTableDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const FCameraDebugColors& Colors = FCameraDebugColors::Get();

#if WITH_EDITORONLY_DATA
	bool bShowDataIDs = false;
	if (!ShowDataIDsCVarName.IsEmpty())
	{
		IConsoleVariable* ShowDataIDsCVar = IConsoleManager::Get().FindConsoleVariable(*ShowDataIDsCVarName, false);
		if (ensureMsgf(ShowDataIDsCVar, TEXT("No such console variable: %s"), *ShowDataIDsCVarName))
		{
			bShowDataIDs = ShowDataIDsCVar->GetBool();
		}
	}
#endif

	for (const FEntryDebugInfo& Entry : Entries)
	{
#if WITH_EDITORONLY_DATA
		if (bShowDataIDs)
		{
			Renderer.AddText(TEXT("{cam_passive}[%d]{cam_default} "), Entry.ID);
		}

		if (!Entry.Name.IsEmpty())
		{
			Renderer.AddText(TEXT("%s [%s] "), *Entry.Name, *Entry.TypeName.ToString());
		}
		else
		{
			Renderer.AddText(TEXT("<no name data> [%s] "), *Entry.TypeName.ToString());
		}
#else
		Renderer.AddText(TEXT("[%d] <no name data> : "), Entry.ID);
#endif

		if (Entry.bWritten)
		{
			if (Entry.bWrittenThisFrame)
			{
				Renderer.AddText(TEXT(" {cam_passive}[WrittenThisFrame]"));
			}
			Renderer.NewLine();

			Renderer.AddIndent();
			{
				Renderer.AddText(Entry.Value);
			}
			Renderer.RemoveIndent();
		}
		else
		{
			Renderer.AddText("{cam_warning}[Uninitialized]\n");
		}

		Renderer.SetTextColor(Colors.Default);
	}
}

void FContextDataTableDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << Entries;
	Ar << ShowDataIDsCVarName;
}

FArchive& operator<< (FArchive& Ar, FContextDataTableDebugBlock::FEntryDebugInfo& EntryDebugInfo)
{
	Ar << EntryDebugInfo.Name;
	Ar << EntryDebugInfo.TypeName;
	Ar << EntryDebugInfo.Value;
	Ar << EntryDebugInfo.bWritten;
	Ar << EntryDebugInfo.bWrittenThisFrame;
	return Ar;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

