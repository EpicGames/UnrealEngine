// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/DebugTextRenderer.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Debug/CameraDebugColors.h"
#include "Engine/Font.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

void FDebugTextDrawCommand::Execute(FCanvas* Canvas, const FColor& DrawColor, const UFont* Font, FVector2f& InOutDrawPosition) const
{
	// Sadly we need to allocate a string here...
	const FText Text(FText::FromStringView(TextView));

	FCanvasTextItem TextItem(FVector2D(InOutDrawPosition), Text, Font, DrawColor);
	TextItem.BlendMode = SE_BLEND_Translucent;
	Canvas->DrawItem(TextItem);
	
	const float TextWidth = FDebugTextRenderer::GetStringViewSize(Font, TextView);
	InOutDrawPosition.X += TextWidth;
}

void FDebugTextNewLineCommand::Execute(FVector2f& InOutDrawPosition) const
{
	InOutDrawPosition.X = LeftMargin;
	InOutDrawPosition.Y += LineSpacing;
}

void FDebugTextSetColorCommand::Execute(FColor* OutDrawColor) const
{
	*OutDrawColor = DrawColor;
}

FDebugTextRenderer::FDebugTextRenderer(FCanvas* InCanvas, const FColor& InDrawColor, const UFont* InFont)
	: Canvas(InCanvas)
	, DrawColor(InDrawColor)
	, Font(InFont)
{
	LineSpacing = InFont->GetMaxCharHeight();
	LeftMargin = 0;
}

void FDebugTextRenderer::RenderText(float StartingDrawY, const TStringView<TCHAR> TextView)
{
	RenderText(FVector2f(LeftMargin, StartingDrawY), TextView);
}

void FDebugTextRenderer::RenderText(FVector2f StartingDrawPosition, const TStringView<TCHAR> TextView)
{
	FDebugTextCommandArray Commands;
	ParseText(TextView, Commands);

	NextDrawPosition = StartingDrawPosition;
	ExecuteCommands(Commands);
}

void FDebugTextRenderer::ParseText(const TStringView<TCHAR> TextView, FDebugTextCommandArray& OutCommands)
{
	const TCHAR* const Begin = TextView.GetData();
	const TCHAR* const End = Begin + TextView.Len();

	const TCHAR* RangeStart = TextView.GetData();
	const TCHAR* RangeCur = RangeStart;

	// Let's start parsing the given text and look for tokens of the form `{token}`.
	bool bIsInToken = false;
	while (true)
	{
		const bool bIsEOF = (RangeCur == End || *RangeCur == '\0');
		const bool bIsNL = !bIsEOF && (*RangeCur == '\n');
		if (!bIsInToken)
		{
			// Not in a token... let's see if we find an EOF or NL, in which case we need to
			// draw the text and either bail out or move to a new line. Otherwise, look for
			// the start of a token with `{`.
			if (bIsEOF || bIsNL)
			{
				// EOF or NL found... render the text so far and move to a new line.
				const bool bAddNewLine = bIsNL || (bIsEOF && bEndWithNewLine);
				AddDrawCommand(RangeStart, RangeCur, bAddNewLine, OutCommands);
				if (bIsEOF)
				{
					break;
				}
				else
				{
					++RangeCur;
					RangeStart = RangeCur;
				}
			}
			else if (*RangeCur == '{')
			{
				// We have a string to render up until the start of the new token.
				AddDrawCommand(RangeStart, RangeCur, false, OutCommands);
				++RangeCur;
				RangeStart = RangeCur;
				bIsInToken = true;
			}
			else
			{
				++RangeCur;
			}
		}
		else
		{
			if (*RangeCur == '}')
			{
				// We have a token!
				AddTokenCommand(RangeStart, RangeCur, OutCommands);
				++RangeCur;
				RangeStart = RangeCur;
				bIsInToken = false;
			}
			else if (bIsEOF || bIsNL)
			{
				// Unclosed token... just treat the whole thing as a string.
				const bool bAddNewLine = bIsNL || (bIsEOF && bEndWithNewLine);
				AddDrawCommand(RangeStart, RangeCur, bAddNewLine, OutCommands);
				break;
			}
			else
			{
				++RangeCur;
			}
		}
	}
}

void FDebugTextRenderer::ExecuteCommands(FDebugTextCommandArray& Commands)
{
	FColor OriginalDrawColor = DrawColor;

	for (FDebugTextCommand& Command : Commands)
	{
		switch (Command.GetIndex())
		{
			case FDebugTextCommand::IndexOfType<FDebugTextDrawCommand>():
				Command.Get<FDebugTextDrawCommand>().Execute(Canvas, DrawColor, Font, NextDrawPosition);
				UpdateRightMargin();
				break;
			case FDebugTextCommand::IndexOfType<FDebugTextNewLineCommand>():
				Command.Get<FDebugTextNewLineCommand>().Execute(NextDrawPosition);
				break;
			case FDebugTextCommand::IndexOfType<FDebugTextSetColorCommand>():
				Command.Get<FDebugTextSetColorCommand>().Execute(&DrawColor);
				break;
			default:
				ensureMsgf(false, TEXT("Unsupported command type!"));
				break;
		}
	}

	DrawColor = OriginalDrawColor;
}

void FDebugTextRenderer::UpdateRightMargin()
{
	RightMargin = FMath::Max(RightMargin, NextDrawPosition.X);
}

void FDebugTextRenderer::ExecuteCommands(float StartingDrawY, FDebugTextCommandArray& Commands)
{
	ExecuteCommands(FVector2f(LeftMargin, StartingDrawY), Commands);
}

void FDebugTextRenderer::ExecuteCommands(FVector2f StartingDrawPosition, FDebugTextCommandArray& Commands)
{
	NextDrawPosition = StartingDrawPosition;
	ExecuteCommands(Commands);
}

void FDebugTextRenderer::AddDrawCommand(const TCHAR* RangeStart, const TCHAR* RangeEnd, bool bNewLine, FDebugTextCommandArray& OutCommands)
{
	TStringView<TCHAR> CmdTextView(RangeStart, RangeEnd - RangeStart);
	if (!CmdTextView.IsEmpty())
	{
		OutCommands.Emplace(TInPlaceType<FDebugTextDrawCommand>(), FDebugTextDrawCommand{ CmdTextView });
	}
	// else: empty text, skip command entirely.

	if (bNewLine)
	{
		OutCommands.Emplace(TInPlaceType<FDebugTextNewLineCommand>(), FDebugTextNewLineCommand{ LineSpacing, LeftMargin });
	}
}

void FDebugTextRenderer::AddTokenCommand(const TCHAR* RangeStart, const TCHAR* RangeEnd, FDebugTextCommandArray& OutCommands)
{
	TStringView<TCHAR> TokenView(RangeStart, RangeEnd - RangeStart);
	// Sadly we need to allocate a string here too...
	FColor NewDrawColor = InterpretColor(FString(TokenView));
	OutCommands.Emplace(TInPlaceType<FDebugTextSetColorCommand>(), FDebugTextSetColorCommand{ NewDrawColor });
}

FColor FDebugTextRenderer::InterpretColor(const FString& ColorName)
{
	TOptional<FColor> CameraDebugColor = FCameraDebugColors::GetFColorByName(ColorName);
	if (CameraDebugColor.IsSet())
	{
		return CameraDebugColor.GetValue();
	}

	const bool bIsColorName = GColorList.IsValidColorName(*ColorName);
	if (bIsColorName)
	{
		return GColorList.GetFColorByName(*ColorName);
	}
	else
	{
		FColor OutColor;
		OutColor.InitFromString(ColorName);
		return OutColor;
	}
}

float FDebugTextRenderer::GetStringViewSize(const UFont* Font, TStringView<TCHAR> TextView)
{
	float TotalWidth = 0.0f;
	TCHAR PrevChar = '\0';
	for (TCHAR Char : TextView)
	{
		float TmpWidth, TmpHeight;
		Font->GetCharSize(Char, TmpWidth, TmpHeight);

		int8 CharKerning = 0;
		if (PrevChar != '\0')
		{
			CharKerning = Font->GetCharKerning(PrevChar, Char);
		}

		TotalWidth += TmpWidth + CharKerning;
		PrevChar = Char;
	}

	return FMath::CeilToInt(TotalWidth);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

