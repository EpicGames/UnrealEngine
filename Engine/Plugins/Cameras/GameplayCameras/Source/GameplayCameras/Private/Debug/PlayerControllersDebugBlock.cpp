// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/PlayerControllersDebugBlock.h"

#include "Camera/PlayerCameraManager.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/DebugTextRenderer.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FPlayerControllersDebugBlock)

FPlayerControllersDebugBlock::FPlayerControllersDebugBlock()
{
}

void FPlayerControllersDebugBlock::Initialize(UWorld* World)
{
	if (!World)
	{
		return;
	}

	bHadValidWorld = true;

	for (auto It = World->GetPlayerControllerIterator(); It; ++It)
	{
		APlayerController* PlayerController = It->Get();
		if (!PlayerController)
		{
			continue;
		}
		if (!PlayerController->IsLocalPlayerController())
		{
			continue;
		}

		APlayerCameraManager* CameraManager = PlayerController->PlayerCameraManager;
		if (!CameraManager)
		{
			continue;
		}

		AActor* ActiveViewTarget = CameraManager->GetViewTarget();
		const FMinimalViewInfo& ViewTargetPOV = CameraManager->GetCameraCacheView();

		FPlayerControllerDebugInfo PlayerControllerInfo;
		PlayerControllerInfo.PlayerControllerName = *GetNameSafe(PlayerController);
		PlayerControllerInfo.CameraManagerName = *GetNameSafe(CameraManager);

		int32 ViewportSizeX = 0, ViewportSizeY = 0;
		PlayerController->GetViewportSize(ViewportSizeX, ViewportSizeY);
		PlayerControllerInfo.ViewportSize = FIntPoint(ViewportSizeX, ViewportSizeY);

		PlayerControllerInfo.bHasDefaultAspectRatioAxisConstraint = false;
		if (const ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
		{
			PlayerControllerInfo.LocalPlayerName = LocalPlayer->GetName();
			PlayerControllerInfo.DefaultAspectRatioAxisConstraint = LocalPlayer->AspectRatioAxisConstraint;
			PlayerControllerInfo.bHasDefaultAspectRatioAxisConstraint = true;
		}

		PlayerControllerInfo.ViewTargetName = *GetNameSafe(ActiveViewTarget);
		PlayerControllerInfo.ViewTargetLocation = ViewTargetPOV.Location;
		PlayerControllerInfo.ViewTargetRotation = ViewTargetPOV.Rotation;
		PlayerControllerInfo.ViewTargetFOV = ViewTargetPOV.FOV;
		PlayerControllerInfo.ViewTargetAspectRatio = ViewTargetPOV.AspectRatio;
		PlayerControllerInfo.bViewTargetConstrainAspectRatio = ViewTargetPOV.bConstrainAspectRatio;
		PlayerControllerInfo.bViewTargetOverrideAspectRatioAxisConstraint = false;
		if (ViewTargetPOV.AspectRatioAxisConstraint.IsSet())
		{
			PlayerControllerInfo.ViewTargetAspectRatioAxisConstraint = ViewTargetPOV.AspectRatioAxisConstraint.GetValue();
			PlayerControllerInfo.bViewTargetOverrideAspectRatioAxisConstraint = true;
		}

		PlayerControllers.Add(PlayerControllerInfo);
	}
}

void FPlayerControllersDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText("{cam_title}Player Controllers:{cam_default}");
	Renderer.AddIndent();
	{
		Renderer.AddText(TEXT("%d active local player controller(s)\n"), PlayerControllers.Num());
		if (bHadValidWorld)
		{
			for (FPlayerControllerDebugInfo& PlayerController : PlayerControllers)
			{
				Renderer.AddText(TEXT("- {cam_notice}%s{cam_default}"), *PlayerController.PlayerControllerName);
				Renderer.AddIndent();
				{
					const FIntPoint ViewportSize = PlayerController.ViewportSize;
					const double AspectRatio = (ViewportSize.Y != 0 ? (double)ViewportSize.X / (double)ViewportSize.Y : 0.f);
					Renderer.AddText(TEXT("Local player: {cam_notice}%s{cam_default}\n"), *PlayerController.LocalPlayerName);
					Renderer.AddText(TEXT("Camera manager: {cam_notice}%s{cam_default}\n"), *PlayerController.CameraManagerName);
					Renderer.AddText(TEXT("Viewport: {cam_notice}%d x %d{cam_default} (aspect ratio %.3f)\n"), ViewportSize.X, ViewportSize.Y, AspectRatio);
					if (PlayerController.bHasDefaultAspectRatioAxisConstraint)
					{
						Renderer.AddText(TEXT("DefaultAspectRatioAxisConstraint: {cam_notice}%s{cam_default}\n"), *ToDebugString(PlayerController.DefaultAspectRatioAxisConstraint));
					}
					else
					{
						Renderer.AddText(TEXT("DefaultAspectRatioAxisConstraint: <no local player>\n"));
					}

					Renderer.AddText(TEXT("View target: {cam_notice}%s{cam_default}"), *PlayerController.ViewTargetName);
					Renderer.AddIndent();
					{
						Renderer.AddText(TEXT("Location: %s\n"), *ToDebugString(PlayerController.ViewTargetLocation));
						Renderer.AddText(TEXT("Rotation: %s\n"), *ToDebugString(PlayerController.ViewTargetRotation));
						Renderer.AddText(TEXT("FOV: %s\n"), *ToDebugString(PlayerController.ViewTargetFOV));
						Renderer.AddText(TEXT("AspectRatio: %s\n"), *ToDebugString(PlayerController.ViewTargetAspectRatio));
						Renderer.AddText(TEXT("ConstrainAspectRatio: %s\n"), *ToDebugString(PlayerController.bViewTargetConstrainAspectRatio));
						if (PlayerController.bViewTargetOverrideAspectRatioAxisConstraint)
						{
							Renderer.AddText(TEXT("AspectRatioAxisConstraint: %s\n"), *ToDebugString(PlayerController.ViewTargetAspectRatioAxisConstraint));
						}
						else
						{
							Renderer.AddText(TEXT("AspectRatioAxisConstraint: <no override>\n"));
						}
					}
					Renderer.RemoveIndent();
				}
				Renderer.RemoveIndent();
			}
		}
		else
		{
			Renderer.AddText(TEXT("<invalid world>"));
		}
	}
	Renderer.RemoveIndent();
}

void FPlayerControllersDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << PlayerControllers;
	Ar << bHadValidWorld;
}

FArchive& operator<< (FArchive& Ar, FPlayerControllersDebugBlock::FPlayerControllerDebugInfo& PlayerControllerInfo)
{
	Ar << PlayerControllerInfo.PlayerControllerName;
	Ar << PlayerControllerInfo.CameraManagerName;
	Ar << PlayerControllerInfo.ViewTargetName;
	Ar << PlayerControllerInfo.LocalPlayerName;
	Ar << PlayerControllerInfo.ViewportSize;
	Ar << PlayerControllerInfo.DefaultAspectRatioAxisConstraint;
	Ar << PlayerControllerInfo.ViewTargetLocation;
	Ar << PlayerControllerInfo.ViewTargetRotation;
	Ar << PlayerControllerInfo.ViewTargetFOV;
	Ar << PlayerControllerInfo.ViewTargetAspectRatio;
	Ar << PlayerControllerInfo.ViewTargetAspectRatioAxisConstraint;
	Ar << PlayerControllerInfo.bViewTargetConstrainAspectRatio;
	return Ar;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

