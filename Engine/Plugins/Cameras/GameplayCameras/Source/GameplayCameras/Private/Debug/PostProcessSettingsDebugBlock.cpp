// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/PostProcessSettingsDebugBlock.h"

#include "Containers/UnrealString.h"
#include "Curves/CurveFloat.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/DebugTextRenderer.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureCube.h"
#include "Math/ColorList.h"
#include "Misc/EngineVersionComparison.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FPostProcessSettingsDebugBlock)

FPostProcessSettingsDebugBlock::FPostProcessSettingsDebugBlock()
{
}

FPostProcessSettingsDebugBlock::FPostProcessSettingsDebugBlock(const FPostProcessSettingsCollection& InPostProcessSettings)
	: PostProcessSettings(InPostProcessSettings)
{
}

void FPostProcessSettingsDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const FPostProcessSettings& PPS = PostProcessSettings.Get();

#define UE_DRAW_PP(Name)\
	if (PPS.bOverride_##Name)\
	{\
		Renderer.AddText(TEXT("%s  : %s\n"), TEXT(#Name), *ToDebugString(PPS.Name));\
	}

#define UE_DRAW_PP_NAME(Name)\
	if (PPS.bOverride_##Name)\
	{\
		Renderer.AddText(TEXT("%s  : %s\n"), TEXT(#Name), *PPS.Name->GetName());\
	}

	{
		UE_DRAW_PP(TemperatureType);
		UE_DRAW_PP(WhiteTemp);
		UE_DRAW_PP(WhiteTint);

		UE_DRAW_PP(ColorSaturation);
		UE_DRAW_PP(ColorContrast);
		UE_DRAW_PP(ColorGamma);
		UE_DRAW_PP(ColorGain);
		UE_DRAW_PP(ColorOffset);

		UE_DRAW_PP(ColorSaturationShadows);
		UE_DRAW_PP(ColorContrastShadows);
		UE_DRAW_PP(ColorGammaShadows);
		UE_DRAW_PP(ColorGainShadows);
		UE_DRAW_PP(ColorOffsetShadows);

		UE_DRAW_PP(ColorSaturationMidtones);
		UE_DRAW_PP(ColorContrastMidtones);
		UE_DRAW_PP(ColorGammaMidtones);
		UE_DRAW_PP(ColorGainMidtones);
		UE_DRAW_PP(ColorOffsetMidtones);

		UE_DRAW_PP(ColorSaturationHighlights);
		UE_DRAW_PP(ColorContrastHighlights);
		UE_DRAW_PP(ColorGammaHighlights);
		UE_DRAW_PP(ColorGainHighlights);
		UE_DRAW_PP(ColorOffsetHighlights);

		UE_DRAW_PP(ColorCorrectionShadowsMax);
		UE_DRAW_PP(ColorCorrectionHighlightsMin);
		UE_DRAW_PP(ColorCorrectionHighlightsMax);

		UE_DRAW_PP(BlueCorrection);
		UE_DRAW_PP(ExpandGamut);
		UE_DRAW_PP(ToneCurveAmount);

		UE_DRAW_PP(FilmSlope);
		UE_DRAW_PP(FilmToe);
		UE_DRAW_PP(FilmShoulder);
		UE_DRAW_PP(FilmBlackClip);
		UE_DRAW_PP(FilmWhiteClip);

		UE_DRAW_PP(SceneColorTint);
		UE_DRAW_PP(SceneFringeIntensity);
		UE_DRAW_PP(ChromaticAberrationStartOffset);
		UE_DRAW_PP(BloomIntensity);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
		UE_DRAW_PP(BloomGaussianIntensity);
#endif
		UE_DRAW_PP(BloomThreshold);
		UE_DRAW_PP(Bloom1Tint);
		UE_DRAW_PP(BloomSizeScale);
		UE_DRAW_PP(Bloom1Size);
		UE_DRAW_PP(Bloom2Tint);
		UE_DRAW_PP(Bloom2Size);
		UE_DRAW_PP(Bloom3Tint);
		UE_DRAW_PP(Bloom3Size);
		UE_DRAW_PP(Bloom4Tint);
		UE_DRAW_PP(Bloom4Size);
		UE_DRAW_PP(Bloom5Tint);
		UE_DRAW_PP(Bloom5Size);
		UE_DRAW_PP(Bloom6Tint);
		UE_DRAW_PP(Bloom6Size);
		UE_DRAW_PP(BloomDirtMaskIntensity);
		UE_DRAW_PP(BloomDirtMaskTint);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
		UE_DRAW_PP(BloomConvolutionIntensity);
#endif
		UE_DRAW_PP(BloomConvolutionScatterDispersion);
		UE_DRAW_PP(BloomConvolutionSize);
		UE_DRAW_PP(BloomConvolutionCenterUV);
		UE_DRAW_PP(BloomConvolutionPreFilterMin);
		UE_DRAW_PP(BloomConvolutionPreFilterMax);
		UE_DRAW_PP(BloomConvolutionPreFilterMult);
		UE_DRAW_PP(AmbientCubemapIntensity);
		UE_DRAW_PP(AmbientCubemapTint);
		UE_DRAW_PP(CameraShutterSpeed);
		UE_DRAW_PP(CameraISO);
		UE_DRAW_PP(AutoExposureLowPercent);
		UE_DRAW_PP(AutoExposureHighPercent);
		UE_DRAW_PP(AutoExposureMinBrightness);
		UE_DRAW_PP(AutoExposureMaxBrightness);
		UE_DRAW_PP(AutoExposureSpeedUp);
		UE_DRAW_PP(AutoExposureSpeedDown);
		UE_DRAW_PP(AutoExposureBias);
		UE_DRAW_PP(HistogramLogMin);
		UE_DRAW_PP(HistogramLogMax);
		UE_DRAW_PP(LocalExposureMethod);
		UE_DRAW_PP(LocalExposureContrastScale_DEPRECATED);
		UE_DRAW_PP(LocalExposureHighlightContrastScale);
		UE_DRAW_PP(LocalExposureShadowContrastScale);
		UE_DRAW_PP(LocalExposureHighlightThreshold);
		UE_DRAW_PP(LocalExposureShadowThreshold);
		UE_DRAW_PP(LocalExposureDetailStrength);
		UE_DRAW_PP(LocalExposureBlurredLuminanceBlend);
		UE_DRAW_PP(LocalExposureBlurredLuminanceKernelSizePercent);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		UE_DRAW_PP(LocalExposureHighlightThresholdStrength);
		UE_DRAW_PP(LocalExposureShadowThresholdStrength);
#endif
		UE_DRAW_PP(LocalExposureMiddleGreyBias);
		UE_DRAW_PP(LensFlareIntensity);
		UE_DRAW_PP(LensFlareTint);
		UE_DRAW_PP(LensFlareBokehSize);
		UE_DRAW_PP(LensFlareThreshold);
		UE_DRAW_PP(VignetteIntensity);
		UE_DRAW_PP(VignetteType);
		UE_DRAW_PP(VignetteCenter);
		UE_DRAW_PP(VignetteColor);
		UE_DRAW_PP(VignetteSize);
		UE_DRAW_PP(VignetteSoftness);
		UE_DRAW_PP(Sharpen);
		UE_DRAW_PP(FilmGrainIntensity);
		UE_DRAW_PP(FilmGrainIntensityShadows);
		UE_DRAW_PP(FilmGrainIntensityMidtones);
		UE_DRAW_PP(FilmGrainIntensityHighlights);
		UE_DRAW_PP(FilmGrainShadowsMax);
		UE_DRAW_PP(FilmGrainHighlightsMin);
		UE_DRAW_PP(FilmGrainHighlightsMax);
		UE_DRAW_PP(FilmGrainScale);
		UE_DRAW_PP(AmbientOcclusionIntensity);
		UE_DRAW_PP(AmbientOcclusionStaticFraction);
		UE_DRAW_PP(AmbientOcclusionRadius);
		UE_DRAW_PP(AmbientOcclusionFadeDistance);
		UE_DRAW_PP(AmbientOcclusionFadeRadius);
		UE_DRAW_PP(AmbientOcclusionDistance_DEPRECATED);
		UE_DRAW_PP(AmbientOcclusionPower);
		UE_DRAW_PP(AmbientOcclusionBias);
		UE_DRAW_PP(AmbientOcclusionQuality);
		UE_DRAW_PP(AmbientOcclusionMipBlend);
		UE_DRAW_PP(AmbientOcclusionMipScale);
		UE_DRAW_PP(AmbientOcclusionMipThreshold);
		UE_DRAW_PP(AmbientOcclusionTemporalBlendWeight);
		UE_DRAW_PP(IndirectLightingColor);
		UE_DRAW_PP(IndirectLightingIntensity);
		UE_DRAW_PP(DepthOfFieldEnabled);
		UE_DRAW_PP(DepthOfFieldFocalDistance);
		UE_DRAW_PP(DepthOfFieldFstop);
		UE_DRAW_PP(DepthOfFieldMinFstop);
		UE_DRAW_PP(DepthOfFieldSensorWidth);
		UE_DRAW_PP(DepthOfFieldSqueezeFactor);
		UE_DRAW_PP(DepthOfFieldDepthBlurRadius);
		UE_DRAW_PP(DepthOfFieldUseHairDepth)
		UE_DRAW_PP(DepthOfFieldDepthBlurAmount);
		UE_DRAW_PP(DepthOfFieldFocalRegion);
		UE_DRAW_PP(DepthOfFieldNearTransitionRegion);
		UE_DRAW_PP(DepthOfFieldFarTransitionRegion);
		UE_DRAW_PP(DepthOfFieldScale);
		UE_DRAW_PP(DepthOfFieldNearBlurSize);
		UE_DRAW_PP(DepthOfFieldFarBlurSize);
		UE_DRAW_PP(DepthOfFieldOcclusion);
		UE_DRAW_PP(DepthOfFieldSkyFocusDistance);
		UE_DRAW_PP(DepthOfFieldVignetteSize);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		UE_DRAW_PP(DepthOfFieldAspectRatioScalar);
		UE_DRAW_PP(DepthOfFieldPetzvalBokeh);
		UE_DRAW_PP(DepthOfFieldPetzvalBokehFalloff);
		UE_DRAW_PP(DepthOfFieldPetzvalExclusionBoxExtents);
		UE_DRAW_PP(DepthOfFieldPetzvalExclusionBoxRadius);
		UE_DRAW_PP(DepthOfFieldBarrelRadius);
		UE_DRAW_PP(DepthOfFieldBarrelLength);
		if (PPS.bOverride_DepthOfFieldMatteBoxFlags)
		{
			Renderer.AddText(TEXT("DepthOfFieldMatteBoxFlags  : "));
			for (uint32 Index = 0; Index < UE_ARRAY_COUNT(PPS.DepthOfFieldMatteBoxFlags); ++Index)
			{
				const FMatteBoxFlag& Flag = PPS.DepthOfFieldMatteBoxFlags[Index];
				Renderer.AddText(TEXT(" %f;%f;%f"), Flag.Pitch, Flag.Roll, Flag.Length);
			}
			Renderer.AddText(TEXT("\n"));
		}
#endif

		UE_DRAW_PP(MotionBlurAmount);
		UE_DRAW_PP(MotionBlurMax);
		UE_DRAW_PP(MotionBlurPerObjectSize);
		UE_DRAW_PP(ScreenSpaceReflectionQuality);
		UE_DRAW_PP(ScreenSpaceReflectionIntensity);
		UE_DRAW_PP(ScreenSpaceReflectionMaxRoughness);

		UE_DRAW_PP(TranslucencyType);
		UE_DRAW_PP(RayTracingTranslucencyMaxRoughness);
		UE_DRAW_PP(RayTracingTranslucencyRefractionRays);
		UE_DRAW_PP(RayTracingTranslucencySamplesPerPixel);
		UE_DRAW_PP(RayTracingTranslucencyShadows);
		UE_DRAW_PP(RayTracingTranslucencyRefraction);

		UE_DRAW_PP(DynamicGlobalIlluminationMethod);
		UE_DRAW_PP(LumenSurfaceCacheResolution);
		UE_DRAW_PP(LumenSceneLightingQuality);
		UE_DRAW_PP(LumenSceneDetail);
		UE_DRAW_PP(LumenSceneViewDistance);
		UE_DRAW_PP(LumenSceneLightingUpdateSpeed);
		UE_DRAW_PP(LumenFinalGatherQuality);
		UE_DRAW_PP(LumenFinalGatherLightingUpdateSpeed);
		UE_DRAW_PP(LumenFinalGatherScreenTraces);
		UE_DRAW_PP(LumenMaxTraceDistance);

		UE_DRAW_PP(LumenDiffuseColorBoost);
		UE_DRAW_PP(LumenAmbientOcclusionIntensity);
		UE_DRAW_PP(LumenSkylightLeaking);
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		UE_DRAW_PP(LumenSkylightLeakingTint);
#endif
		UE_DRAW_PP(LumenFullSkylightLeakingDistance);

		UE_DRAW_PP(LumenRayLightingMode);
		UE_DRAW_PP(LumenReflectionsScreenTraces);
		UE_DRAW_PP(LumenFrontLayerTranslucencyReflections);
		UE_DRAW_PP(LumenMaxRoughnessToTraceReflections);
		UE_DRAW_PP(LumenMaxReflectionBounces);
		UE_DRAW_PP(LumenMaxRefractionBounces);
		UE_DRAW_PP(ReflectionMethod);
		UE_DRAW_PP(LumenReflectionQuality);
		UE_DRAW_PP(RayTracingAO);
		UE_DRAW_PP(RayTracingAOSamplesPerPixel);
		UE_DRAW_PP(RayTracingAOIntensity);
		UE_DRAW_PP(RayTracingAORadius);

		UE_DRAW_PP(PathTracingMaxBounces);
		UE_DRAW_PP(PathTracingSamplesPerPixel);
		UE_DRAW_PP(PathTracingMaxPathIntensity);
		UE_DRAW_PP(PathTracingEnableEmissiveMaterials);
		UE_DRAW_PP(PathTracingEnableReferenceDOF);
		UE_DRAW_PP(PathTracingEnableReferenceAtmosphere);
		UE_DRAW_PP(PathTracingEnableDenoiser);
		UE_DRAW_PP(PathTracingIncludeEmissive);
		UE_DRAW_PP(PathTracingIncludeDiffuse);
		UE_DRAW_PP(PathTracingIncludeIndirectDiffuse);
		UE_DRAW_PP(PathTracingIncludeSpecular);
		UE_DRAW_PP(PathTracingIncludeIndirectSpecular);
		UE_DRAW_PP(PathTracingIncludeVolume);
		UE_DRAW_PP(PathTracingIncludeIndirectVolume);

		UE_DRAW_PP(DepthOfFieldBladeCount);

		// This is no bOverride_AmbientCubemap so just see if it is set.
		if (PPS.AmbientCubemap)
		{
			Renderer.AddText(TEXT("AmbientCubemap  : %s\n"), *PPS.AmbientCubemap->GetName());
		}

		UE_DRAW_PP(ColorGradingIntensity);
		
		UE_DRAW_PP_NAME(ColorGradingLUT)
		UE_DRAW_PP_NAME(BloomDirtMask);
		UE_DRAW_PP(BloomMethod);
		UE_DRAW_PP_NAME(BloomConvolutionTexture)
		UE_DRAW_PP_NAME(FilmGrainTexture)

		UE_DRAW_PP(BloomConvolutionBufferScale);

		UE_DRAW_PP_NAME(AutoExposureBiasCurve);
		UE_DRAW_PP_NAME(AutoExposureMeterMask);
		UE_DRAW_PP_NAME(LocalExposureHighlightContrastCurve);
		UE_DRAW_PP_NAME(LocalExposureShadowContrastCurve);
		UE_DRAW_PP_NAME(LensFlareBokehShape);

		if (PPS.bOverride_LensFlareTints)
		{
			Renderer.AddText(TEXT("LensFlareTints  :"));
			for (uint32 i = 0; i < 8; ++i)
			{
				Renderer.AddText(TEXT(" %s"), *ToDebugString(PPS.LensFlareTints[i]));
			}
			Renderer.AddText(TEXT("\n"));
		}

		if (PPS.bOverride_MobileHQGaussian)
		{
			Renderer.AddText(TEXT("MobileHQGaussian  : %s\n"), *ToDebugString(PPS.bMobileHQGaussian));
		}

		UE_DRAW_PP(AutoExposureMethod);
		UE_DRAW_PP(AmbientOcclusionRadiusInWS);
		UE_DRAW_PP(MotionBlurTargetFPS);
		UE_DRAW_PP(AutoExposureApplyPhysicalCameraExposure);
		UE_DRAW_PP(UserFlags);

		if (PPS.WeightedBlendables.Array.Num() > 0)
		{
			Renderer.AddText(TEXT("WeightedBlendables  :"));
			for (int32 Index = 0; Index < PPS.WeightedBlendables.Array.Num(); ++Index)
			{
				const FWeightedBlendable& Blendable = PPS.WeightedBlendables.Array[Index];
				Renderer.AddText(TEXT(" [%d] %s (%d%%)"), Index, *GetNameSafe(Blendable.Object), (int32)(Blendable.Weight * 100.f));
			}
			Renderer.AddText(TEXT("\n"));
		}
	}

#undef UE_DRAW_PP
#undef UE_DRAW_PP_NAME
}

void FPostProcessSettingsDebugBlock::OnSerialize(FArchive& Ar)
{
	PostProcessSettings.Serialize(Ar);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

