// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/RootCameraDebugBlock.h"

#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugCategories.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraDirectorTreeDebugBlock.h"
#include "Debug/CameraEvaluationServiceDebugBlock.h"
#include "Debug/CameraNodeEvaluationResultDebugBlock.h"
#include "Debug/CameraPoseDebugBlock.h"
#include "Debug/ContextDataTableDebugBlock.h"
#include "Debug/CategoryTitleDebugBlock.h"
#include "Debug/PlayerControllersDebugBlock.h"
#include "Debug/VariableTableDebugBlock.h"
#include "Debug/ViewfinderDebugBlock.h"
#include "HAL/IConsoleManager.h"
#include "String/ParseTokens.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

bool GGameplayCamerasDebugEnable = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugEnable(
	TEXT("GameplayCameras.Debug.Enable"),
	GGameplayCamerasDebugEnable,
	TEXT("(Default: false. Enables debug drawing for the GameplayCameras system."));

int32 GGameplayCamerasDebugSystemID = INDEX_NONE;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugSystemID(
	TEXT("GameplayCameras.Debug.SystemID"),
	GGameplayCamerasDebugSystemID,
	TEXT("(Default: -1. Specifies the GameplayCameras system instance to show debug drawing for."));

FString GGameplayCamerasDebugCategories = "nodetree";
static FAutoConsoleVariableRef CVarGameplayCamerasDebugCategories(
	TEXT("GameplayCameras.Debug.Categories"),
	GGameplayCamerasDebugCategories,
	TEXT("(Default: nodes. Specifies which debug categories to display the GameplayCameras system."));

bool GGameplayCamerasDebugPoseStatsShowUnchanged = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugPoseStatsShowUnchanged(
	TEXT("GameplayCameras.Debug.PoseStats.ShowUnchanged"),
	GGameplayCamerasDebugPoseStatsShowUnchanged,
	TEXT(""));

bool GGameplayCamerasDebugPoseStatsShowVariableIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugPoseStatsShowVariableIDs(
	TEXT("GameplayCameras.Debug.PoseStats.ShowVariableIDs"),
	GGameplayCamerasDebugPoseStatsShowVariableIDs,
	TEXT(""));

bool GGameplayCamerasDebugPoseStatsShowDataIDs = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugPoseStatsShowDataIDs(
	TEXT("GameplayCameras.Debug.PoseStats.ShowDataIDs"),
	GGameplayCamerasDebugPoseStatsShowDataIDs,
	TEXT(""));

UE_DEFINE_CAMERA_DEBUG_BLOCK(FRootCameraDebugBlock)

void FRootCameraDebugBlock::BuildDebugBlocks(const FCameraSystemEvaluator& CameraSystem, const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	ensureMsgf(GetChildren().IsEmpty() && GetAttachments().IsEmpty(), TEXT("This root debug block has already been initialized!"));

	DebugID = CameraSystem.GetDebugID();

	// Debug block for showing the directors and context stack.
	FCategoryTitleDebugBlock& DirectorTreeCategory = Builder.StartChildDebugBlock<FCategoryTitleDebugBlock>();
	{
		DirectorTreeCategory.Title = TEXT("Camera Directors");
		DirectorTreeCategory.Category = FCameraDebugCategories::DirectorTree;

		const FCameraEvaluationContextStack& ContextStack = CameraSystem.GetEvaluationContextStack();
		FCameraDirectorTreeDebugBlock& DirectorTreeDebugBlock = Builder.StartChildDebugBlock<FCameraDirectorTreeDebugBlock>();
		DirectorTreeDebugBlock.Initialize(ContextStack, Builder);
		Builder.EndChildDebugBlock();
	}
	Builder.EndChildDebugBlock();

	// Debug block for showing the tree of camera nodes.
	FCategoryTitleDebugBlock& NodeTreeCategory = Builder.StartChildDebugBlock<FCategoryTitleDebugBlock>();
	{
		NodeTreeCategory.Title = TEXT("Camera Nodes");
		NodeTreeCategory.Category = FCameraDebugCategories::NodeTree;

		if (FRootCameraNodeEvaluator* RootNodeEvaluator = CameraSystem.GetRootNodeEvaluator())
		{
			RootNodeEvaluator->BuildDebugBlocks(Params, Builder);
		}

		// Draw the final camera pose in external rendering, but don't draw text.
		NodeTreeCategory.AddChild(
				&Builder.BuildDebugBlock<FCameraPoseDebugBlock>(CameraSystem.GetEvaluatedResult().CameraPose)
					.ShouldDrawText(false));
	}
	Builder.EndChildDebugBlock();

	// Debug block for the evaluation services.
	FCategoryTitleDebugBlock& ServicesCategory = Builder.StartChildDebugBlock<FCategoryTitleDebugBlock>();
	{
		ServicesCategory.Title = TEXT("Services");
		ServicesCategory.Category = FCameraDebugCategories::Services;

		TArray<TSharedPtr<FCameraEvaluationService>> EvaluationServices;
		CameraSystem.GetEvaluationServices(EvaluationServices);
		for (TSharedPtr<FCameraEvaluationService> EvaluationService : EvaluationServices)
		{
			FCameraEvaluationServiceDebugBlock& ServiceDebugBlock = Builder.StartChildDebugBlock<FCameraEvaluationServiceDebugBlock>(EvaluationService);
			{
				EvaluationService->BuildDebugBlocks(Params, Builder);
			}
			Builder.EndChildDebugBlock();
		}
	}
	Builder.EndChildDebugBlock();

	// Debug block for showing the final evaluated camera.
	FCategoryTitleDebugBlock& PoseStatsCategory = Builder.StartChildDebugBlock<FCategoryTitleDebugBlock>();
	{
		PoseStatsCategory.Title = TEXT("Evaluated Camera");
		PoseStatsCategory.Category = FCameraDebugCategories::PoseStats;

		FCameraNodeEvaluationResultDebugBlock& ResultDebugBlock = Builder.BuildDebugBlock<FCameraNodeEvaluationResultDebugBlock>();
		PoseStatsCategory.AddChild(&ResultDebugBlock);
		{
			ResultDebugBlock.Initialize(CameraSystem.GetEvaluatedResult(), Builder);
			ResultDebugBlock.GetCameraPoseDebugBlock()->WithShowUnchangedCVar(TEXT("GameplayCameras.Debug.PoseStats.ShowUnchanged"));
			ResultDebugBlock.GetVariableTableDebugBlock()->WithShowVariableIDsCVar(TEXT("GameplayCameras.Debug.PoseStats.ShowVariableIDs"));
			ResultDebugBlock.GetContextDataTableDebugBlock()->WithShowDataIDsCVar(TEXT("GameplayCameras.Debug.PoseStats.ShowDataIDs"));
		}
		FPlayerControllersDebugBlock& PlayerControllersDebugBlock = Builder.BuildDebugBlock<FPlayerControllersDebugBlock>();
		PoseStatsCategory.AddChild(&PlayerControllersDebugBlock);
		{
			UObject* CameraSystemOwner = CameraSystem.GetOwner();
			PlayerControllersDebugBlock.Initialize(CameraSystemOwner ? CameraSystemOwner->GetWorld() : nullptr);
		}
	}
	Builder.EndChildDebugBlock();
	
	// Debug block for rendering a viewfinder.
	AddChild(&Builder.BuildDebugBlock<FViewfinderDebugBlock>());
}

void FRootCameraDebugBlock::RootDebugDraw(const FRootCameraDebugDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (!ShouldDebugDraw(DebugID, Params.bFromDefaultCameraSystem))
	{
		return;
	}

	// Figure out what debug categories are active.
	FCameraDebugBlockDrawParams DrawParams;

	TArray<FStringView, TInlineAllocator<4>> ActiveCategories;
	UE::String::ParseTokens(GGameplayCamerasDebugCategories, ',', ActiveCategories);
	for (FStringView CategoryView : ActiveCategories)
	{
		DrawParams.ActiveCategories.Add(FString(CategoryView));
	}

	// Do the drawing!
	Renderer.BeginDrawing();
	FCameraDebugBlock::DebugDraw(DrawParams, Renderer);
	Renderer.EndDrawing();
}

void FRootCameraDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << DebugID;
}

bool FRootCameraDebugBlock::ShouldDebugDraw(FCameraSystemDebugID InDebugID, bool bIsDefaultCameraSystem)
{
	if (!GGameplayCamerasDebugEnable)
	{
		return false;
	}

	const FCameraSystemDebugID WantedDebugID(GGameplayCamerasDebugSystemID);
	const int32 NumCameraSystems = FCameraSystemDebugRegistry::Get().NumRegisteredCameraSystemEvaluators();

	// If the wanted debug ID is ours, we draw.
	// If the wanted debug ID is "any", we draw.
	// If the wanted debug ID is "auto", we draw if we are the view target or camera manager or
	// we are in charge of the "active" camera, whatever that means.
	const bool bDoDebugDraw = (WantedDebugID == InDebugID) 
		|| WantedDebugID.IsAny()
		|| (WantedDebugID.IsAuto() && (bIsDefaultCameraSystem || NumCameraSystems == 1));
	return bDoDebugDraw;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

