// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/VariableTableDebugBlock.h"

#include "Core/CameraVariableTable.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Debug/DebugTextRenderer.h"
#include "HAL/IConsoleManager.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FVariableTableDebugBlock)

FVariableTableDebugBlock::FVariableTableDebugBlock()
{
}

FVariableTableDebugBlock::FVariableTableDebugBlock(const FCameraVariableTable& InVariableTable)
{
	Initialize(InVariableTable);
}

void FVariableTableDebugBlock::Initialize(const FCameraVariableTable& InVariableTable)
{
	for (const FCameraVariableTable::FEntry& Entry : InVariableTable.Entries)
	{
		FString EntryName;
#if WITH_EDITORONLY_DATA
		EntryName = Entry.DebugName;
#endif

		FString EntryValueStr;
		const uint8* RawValuePtr = InVariableTable.Memory + Entry.Offset;

		switch (Entry.Type)
		{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			case ECameraVariableType::ValueName:\
				if (EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::Written))\
				{\
					const ValueType* EntryValue = reinterpret_cast<const ValueType*>(RawValuePtr);\
					EntryValueStr = ToDebugString(*EntryValue);\
				}\
				break;
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			case ECameraVariableType::BlendableStruct:
				if (EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::Written))
				{
					const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Entry.StructType);
					const int32 ExportFlags = PPF_Delimited | PPF_IncludeTransient | PPF_ExternalEditor;
					StructType->ExportText(EntryValueStr, RawValuePtr, nullptr, nullptr, ExportFlags, nullptr);
				}
				break;
		}

		FEntryDebugInfo EntryDebugInfo{ Entry.ID.GetValue(), EntryName, EntryValueStr };
		EntryDebugInfo.bIsInput = EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::Input);
		EntryDebugInfo.bIsPrivate = EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::Private);
		EntryDebugInfo.bWritten = EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::Written);
		EntryDebugInfo.bWrittenThisFrame = EnumHasAnyFlags(Entry.Flags, FCameraVariableTable::EEntryFlags::WrittenThisFrame);
		Entries.Add(EntryDebugInfo);
	}

	Entries.StableSort([](const FEntryDebugInfo& A, const FEntryDebugInfo& B) -> bool
			{
				if (!A.Name.IsEmpty() || !B.Name.IsEmpty())
				{
					return A.Name.Compare(B.Name) < 0;
				}
				return A.ID < B.ID;
			});
}

void FVariableTableDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
#if WITH_EDITORONLY_DATA
	bool bShowVariableIDs = false;
	if (!ShowVariableIDsCVarName.IsEmpty())
	{
		IConsoleVariable* ShowVariableIDsCVar = IConsoleManager::Get().FindConsoleVariable(*ShowVariableIDsCVarName, false);
		if (ensureMsgf(ShowVariableIDsCVar, TEXT("No such console variable: %s"), *ShowVariableIDsCVarName))
		{
			bShowVariableIDs = ShowVariableIDsCVar->GetBool();
		}
	}
#endif

	const FCameraDebugColors& Colors = FCameraDebugColors::Get();

	for (const FEntryDebugInfo& Entry : Entries)
	{
#if WITH_EDITORONLY_DATA
		if (bShowVariableIDs)
		{
			Renderer.AddText(TEXT("{cam_passive}[%d]{cam_default} "), Entry.ID);
		}

		if (!Entry.Name.IsEmpty())
		{
			Renderer.AddText(TEXT("%s : "), *Entry.Name);
		}
		else
		{
			Renderer.AddText(TEXT("<no name data> : "), Entry.ID);
		}
#else
		Renderer.AddText(TEXT("[%d] <no name data> : "), Entry.ID);
#endif

		if (Entry.bWritten)
		{
			Renderer.AddText(Entry.Value);
			Renderer.AddText(TEXT(" "));
			if (Entry.bWrittenThisFrame)
			{
				Renderer.AddText(TEXT("{cam_passive}[WrittenThisFrame]"));
			}
		}
		else
		{
			Renderer.AddText("{cam_warning}[Uninitialized]");
		}

		if (Entry.bIsInput)
		{
			Renderer.AddText("{cam_notice2}[Input]");
		}

		if (Entry.bIsPrivate)
		{
			Renderer.AddText("{cam_notice2}[Private]");
		}

		Renderer.NewLine();
		Renderer.SetTextColor(Colors.Default);
	}
}

void FVariableTableDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << Entries;
	Ar << ShowVariableIDsCVarName;
}

FArchive& operator<< (FArchive& Ar, FVariableTableDebugBlock::FEntryDebugInfo& EntryDebugInfo)
{
	Ar << EntryDebugInfo.ID;
	Ar << EntryDebugInfo.Name;
	Ar << EntryDebugInfo.Value;
	Ar << EntryDebugInfo.bIsInput;
	Ar << EntryDebugInfo.bIsPrivate;
	Ar << EntryDebugInfo.bWritten;
	Ar << EntryDebugInfo.bWrittenThisFrame;
	return Ar;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

