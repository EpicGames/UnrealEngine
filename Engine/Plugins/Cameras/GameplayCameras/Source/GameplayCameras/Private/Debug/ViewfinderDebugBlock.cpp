// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/ViewfinderDebugBlock.h"

#include "Debug/CameraDebugCategories.h"
#include "Debug/ViewfinderRenderer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

UE_DEFINE_CAMERA_DEBUG_BLOCK(FViewfinderDebugBlock)

FViewfinderDebugBlock::FViewfinderDebugBlock()
{
}

void FViewfinderDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (Params.IsCategoryActive(FCameraDebugCategories::Viewfinder))
	{
		FViewfinderRenderer::DrawViewfinder(Renderer, FViewfinderDrawElements::All);
	}
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

