// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/ViewfinderRenderer.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

float GGameplayCamerasDebugViewfinderReticleSizeFactor = 0.27f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugViewfinderReticleSizeFactor(
	TEXT("GameplayCameras.Debug.Viewfinder.ReticleSizeFactor"),
	GGameplayCamerasDebugViewfinderReticleSizeFactor,
	TEXT("Default: 0.1. The size of the viewfinder reticle, as a factor of the screen's vertical size."));

float GGameplayCamerasDebugViewfinderReticleInnerSizeFactor = 0.7f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugViewfinderReticleInnerSizeFactor(
	TEXT("GameplayCameras.Debug.Viewfinder.ReticleInnerSizeFactor"),
	GGameplayCamerasDebugViewfinderReticleInnerSizeFactor,
	TEXT(""));

int32 GGameplayCamerasDebugViewfinderReticleNumSides = 60;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugViewfinderReticleNumSides(
	TEXT("GameplayCameras.Debug.Viewfinder.ReticleNumSides"),
	GGameplayCamerasDebugViewfinderReticleNumSides,
	TEXT(""));

float GGameplayCamerasDebugViewfinderGuidesGapFactor = 0.02f;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugViewfinderGuidesGapFactor(
	TEXT("GameplayCameras.Debug.Viewfinder.GuidesGapFactor"),
	GGameplayCamerasDebugViewfinderGuidesGapFactor,
	TEXT(""));

void FViewfinderRenderer::DrawViewfinder(FCameraDebugRenderer& Renderer, FViewfinderDrawElements Elements)
{
	FCanvas* Canvas = Renderer.GetCanvas();

	const FVector2D CanvasSize = Renderer.GetCanvasSize();
	const float CanvasSizeX = CanvasSize.X;
	const float CanvasSizeY = CanvasSize.Y;

	const FVector2d CanvasCenter(CanvasSizeX / 2.f, CanvasSizeY / 2.f);

	if (EnumHasAnyFlags(Elements, FViewfinderDrawElements::FocusReticle))
	{
		// Draw the reticle.
		const float ReticleRadius = CanvasSizeY * GGameplayCamerasDebugViewfinderReticleSizeFactor / 2.f;
		const float ReticleInnerRadiusFactor = GGameplayCamerasDebugViewfinderReticleInnerSizeFactor;
		const int32 ReticleNumSides = GGameplayCamerasDebugViewfinderReticleNumSides;
		const FColor ReticleColor = FCameraDebugColors::Get().Passive;

		// ...outer reticle circle.
		Renderer.Draw2DCircle(CanvasCenter, ReticleRadius, ReticleColor, 1.f, ReticleNumSides);
		// ...inner reticle circle.
		const float ReticleInnerRadius = ReticleRadius * ReticleInnerRadiusFactor;
		const int32 ReticleInnerNumSides = (int32)(ReticleNumSides * ReticleInnerRadiusFactor);
		Renderer.Draw2DCircle(CanvasCenter, ReticleInnerRadius, ReticleColor, 1.f, ReticleInnerNumSides);
		// ...horizontal line inside reticle.
		Renderer.Draw2DLine(
				CanvasCenter - FVector2D(ReticleInnerRadius, 0), CanvasCenter + FVector2D(ReticleInnerRadius, 0), 
				ReticleColor);
	}

	if (EnumHasAnyFlags(Elements, FViewfinderDrawElements::RuleOfThirds))
	{
		// Draw the rule-of-thirds guides.
		const FColor GuideColor = FCameraDebugColors::Get().VeryPassive;
		const float GuidesGap = CanvasSizeY * GGameplayCamerasDebugViewfinderGuidesGapFactor;
		const FVector2D OneThird(CanvasSizeX / 3.f, CanvasSizeY / 3.f);
		const FVector2D TwoThirds(CanvasSizeX / 1.5f, CanvasSizeY / 1.5f);
		const FColor LineColor = FCameraDebugColors::Get().Passive;

		// ...top vertical guides
		Renderer.Draw2DLine(
				FVector2D(OneThird.X, 0), FVector2D(OneThird.X, OneThird.Y - GuidesGap),
				LineColor, 2.f);
		Renderer.Draw2DLine(
				FVector2D(TwoThirds.X, 0), FVector2D(TwoThirds.X, OneThird.Y - GuidesGap),
				LineColor, 2.f);
		// ...bottom vertical guides
		Renderer.Draw2DLine(
				FVector2D(OneThird.X, TwoThirds.Y + GuidesGap), FVector2D(OneThird.X, CanvasSizeY),
				LineColor, 2.f);
		Renderer.Draw2DLine(
				FVector2D(TwoThirds.X, TwoThirds.Y + GuidesGap), FVector2D(TwoThirds.X, CanvasSizeY),
				LineColor, 2.f);
		// ...left horizontal guides
		Renderer.Draw2DLine(
				FVector2D(0, OneThird.Y), FVector2D(OneThird.X - GuidesGap, OneThird.Y),
				LineColor, 2.f);
		Renderer.Draw2DLine(
				FVector2D(0, TwoThirds.Y), FVector2D(OneThird.X - GuidesGap, TwoThirds.Y),
				LineColor, 2.f);
		// ...right horizontal guides
		Renderer.Draw2DLine(
				FVector2D(TwoThirds.X + GuidesGap, OneThird.Y), FVector2D(CanvasSizeX, OneThird.Y),
				LineColor, 2.f);
		Renderer.Draw2DLine(
				FVector2D(TwoThirds.X + GuidesGap, TwoThirds.Y), FVector2D(CanvasSizeX, TwoThirds.Y),
				LineColor, 2.f);
	}
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG


