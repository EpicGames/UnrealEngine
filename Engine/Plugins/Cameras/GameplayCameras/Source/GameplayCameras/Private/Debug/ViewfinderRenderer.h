// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayCameras.h"
#include "Misc/EnumClassFlags.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraDebugRenderer;

enum class FViewfinderDrawElements
{
	None = 0,
	RuleOfThirds = 1 << 0,
	FocusReticle = 1 << 1,

	All = RuleOfThirds | FocusReticle
};

ENUM_CLASS_FLAGS(FViewfinderDrawElements)

class FViewfinderRenderer
{
public:

	static void DrawViewfinder(FCameraDebugRenderer& Renderer, FViewfinderDrawElements Elements);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

