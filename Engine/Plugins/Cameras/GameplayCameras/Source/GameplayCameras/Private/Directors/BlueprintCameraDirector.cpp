// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/BlueprintCameraDirector.h"

#include "Build/CameraBuildContext.h"
#include "Components/ActorComponent.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameFramework/ControllerGameplayCameraEvaluationComponent.h"
#include "GameplayCameras.h"
#include "GameplayCamerasSettings.h"
#include "Helpers/OutgoingReferenceFinder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintCameraDirector)

#define LOCTEXT_NAMESPACE "BlueprintCameraDirector"

namespace UE::Cameras
{

class FBlueprintCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FBlueprintCameraDirectorEvaluator)

public:
	
	~FBlueprintCameraDirectorEvaluator();

protected:

	virtual void OnInitialize(const FCameraDirectorInitializeParams& Params) override;
	virtual void OnActivate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnDeactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnAddChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) override;
	virtual void OnRemoveChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

private:

	FCameraSystemEvaluator* OwningEvaluator = nullptr;

	TObjectPtr<UBlueprintCameraDirectorEvaluator> EvaluatorBlueprint;
};

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FBlueprintCameraDirectorEvaluator)

FBlueprintCameraDirectorEvaluator::~FBlueprintCameraDirectorEvaluator()
{
	if (EvaluatorBlueprint)
	{
		EvaluatorBlueprint->NativeAbandonCameraDirector();
	}
}

void FBlueprintCameraDirectorEvaluator::OnInitialize(const FCameraDirectorInitializeParams& Params)
{
	const UBlueprintCameraDirector* Blueprint = GetCameraDirectorAs<UBlueprintCameraDirector>();
	if (!ensure(Blueprint))
	{
		return;
	}

	const UCameraAsset* CameraAsset = Params.OwnerContext->GetCameraAsset();
	if (!ensure(CameraAsset))
	{
		return;
	}

	if (Blueprint->CameraDirectorEvaluatorClass)
	{
		UObject* Outer = Params.OwnerContext->GetOwner();
		EvaluatorBlueprint = NewObject<UBlueprintCameraDirectorEvaluator>(Outer, Blueprint->CameraDirectorEvaluatorClass);
		EvaluatorBlueprint->NativeInitializeCameraDirector(this, Params);
	}
	else
	{
		UE_LOGF(LogCameraSystem, Error, "No Blueprint class set on camera director for '%ls'.", *CameraAsset->GetPathName());
	}
}

void FBlueprintCameraDirectorEvaluator::OnActivate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	OwningEvaluator = Params.Evaluator;

	if (EvaluatorBlueprint)
	{
		EvaluatorBlueprint->NativeActivateCameraDirector(Params);

		const FCameraDirectorEvaluationResult& BlueprintResult = EvaluatorBlueprint->GetEvaluationResult();
		OutResult = BlueprintResult;
	}
	else
	{
		UE_LOGF(LogCameraSystem, Error, "Can't activate Blueprint camera director, no Blueprint class was set!");
	}
}

void FBlueprintCameraDirectorEvaluator::OnDeactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	// We need to check a few more things here in case we're being deactivated while the owner object is getting GC'ed.
	UObject* ContextOwner = GetEvaluationContext()->GetOwner();
	const bool bIsValid = (EvaluatorBlueprint && ContextOwner && !ContextOwner->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed));
	if (bIsValid)
	{
		EvaluatorBlueprint->NativeDeactivateCameraDirector(Params);

		const FCameraDirectorEvaluationResult& BlueprintResult = EvaluatorBlueprint->GetEvaluationResult();
		OutResult = BlueprintResult;
	}

	OwningEvaluator = nullptr;
}

void FBlueprintCameraDirectorEvaluator::OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	if (EvaluatorBlueprint)
	{
		EvaluatorBlueprint->NativeRunCameraDirector(Params);

		const FCameraDirectorEvaluationResult& BlueprintResult = EvaluatorBlueprint->GetEvaluationResult();
		OutResult = BlueprintResult;
	}
}

void FBlueprintCameraDirectorEvaluator::OnAddChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result)
{
	if (EvaluatorBlueprint)
	{
		UObject* ChildContextOwner = Params.ChildContext->GetOwner();
		const bool bAdded = EvaluatorBlueprint->NativeAddChildEvaluationContext(ChildContextOwner);
		if (bAdded)
		{
			Result.Result = EChildContextManipulationResult::Success;
		}
	}
}

void FBlueprintCameraDirectorEvaluator::OnRemoveChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result)
{
	if (EvaluatorBlueprint)
	{
		UObject* ChildContextOwner = Params.ChildContext->GetOwner();
		const bool bRemoved = EvaluatorBlueprint->NativeRemoveChildEvaluationContext(ChildContextOwner);
		
		if (bRemoved)
		{
			Result.Result = EChildContextManipulationResult::Success;
		}
	}
}

void FBlueprintCameraDirectorEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EvaluatorBlueprint);
}

}  // namespace UE::Cameras

bool UBlueprintCameraDirectorEvaluator::RunChildCameraDirector(float DeltaTime, FName ChildSlotName)
{
	using namespace UE::Cameras;

	const int32 ChildIndex = ChildrenContextSlotNames.Find(ChildSlotName);
	TArrayView<const TSharedPtr<FCameraEvaluationContext>> ChildrenContexts = EvaluationContext->GetChildrenContexts();

	if (ChildIndex >= 0 && ensure(ChildrenContexts.IsValidIndex(ChildIndex)))
	{
		TSharedPtr<const FCameraEvaluationContext> ChildContext = ChildrenContexts[ChildIndex];

		if (FCameraDirectorEvaluator* ChildDirectorEvaluator = ChildContext->GetDirectorEvaluator())
		{
			FCameraDirectorEvaluationParams ChildParams;
			ChildParams.DeltaTime = DeltaTime;
			
			FCameraDirectorEvaluationResult ChildResult;
			
			ChildDirectorEvaluator->Run(ChildParams, ChildResult);

			if (!ChildResult.Requests.IsEmpty())
			{
				EvaluationResult = ChildResult;
				return true;
			}
		}
	}

	return false;
}

void UBlueprintCameraDirectorEvaluator::ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	Request.Layer = ECameraRigLayer::Base;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	Request.Layer = ECameraRigLayer::Global;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	Request.Layer = ECameraRigLayer::Visual;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::DeactivatePersistentBaseCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Deactivate;
	Request.Layer = ECameraRigLayer::Base;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::DeactivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Deactivate;
	Request.Layer = ECameraRigLayer::Global;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::DeactivatePersistentVisualCameraRig(UCameraRigAsset* CameraRigPrefab)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigPrefab);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Deactivate;
	Request.Layer = ECameraRigLayer::Visual;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::ActivateCameraRig(UCameraRigAsset* CameraRig, bool bForceNewInstance)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRig);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	Request.Layer = ECameraRigLayer::Main;
	Request.bForceActivateDeactivate = bForceNewInstance;
	EvaluationResult.Requests.Add(Request);
}

void UBlueprintCameraDirectorEvaluator::ActivateCameraRigViaProxy(UCameraRigProxyAsset* CameraRigProxy, bool bForceNewInstance)
{
	using namespace UE::Cameras;
	FCameraRigActivationDeactivationRequest Request(EvaluationContext, CameraRigProxy);
	Request.RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	Request.Layer = ECameraRigLayer::Main;
	Request.bForceActivateDeactivate = bForceNewInstance;
	EvaluationResult.Requests.Add(Request);
}

UCameraRigAsset* UBlueprintCameraDirectorEvaluator::ResolveCameraRigProxy(const UCameraRigProxyAsset* CameraRigProxy) const
{
	if (ensure(OwningDirectorEvaluator))
	{
		const UBlueprintCameraDirector* BlueprintCameraDirector = OwningDirectorEvaluator->GetCameraDirectorAs<UBlueprintCameraDirector>();
		if (ensure(BlueprintCameraDirector))
		{
			const FCameraRigProxyRedirectTable& ProxyTable = BlueprintCameraDirector->CameraRigProxyRedirectTable;

			FCameraRigProxyResolveParams ResolveParams;
			ResolveParams.CameraRigProxy = CameraRigProxy;
			return ProxyTable.ResolveProxy(ResolveParams);
		}
	}
	return nullptr;
}

AActor* UBlueprintCameraDirectorEvaluator::FindEvaluationContextOwnerActor(TSubclassOf<AActor> ActorClass) const
{
	if (EvaluationContext)
	{
		if (UActorComponent* ContextOwnerAsComponent = Cast<UActorComponent>(EvaluationContext->GetOwner()))
		{
			return ContextOwnerAsComponent->GetOwner();
		}
		else if (AActor* ContextOwnerAsActor = Cast<AActor>(EvaluationContext->GetOwner()))
		{
			return ContextOwnerAsActor;
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't access evaluation context outside of RunCameraDirector"), 
				ELogVerbosity::Error);
		return nullptr;
	}
}

FBlueprintCameraEvaluationDataRef UBlueprintCameraDirectorEvaluator::GetInitialContextResult() const
{
	if (EvaluationContext)
	{
		return FBlueprintCameraEvaluationDataRef::MakeExternalRef(&EvaluationContext->GetInitialResult());
	}
	else
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't access evaluation context's initial result outside of RunCameraDirector"), 
				ELogVerbosity::Error);
		return FBlueprintCameraEvaluationDataRef();
	}
}

FBlueprintCameraEvaluationDataRef UBlueprintCameraDirectorEvaluator::GetConditionalContextResult(ECameraEvaluationDataCondition Condition) const
{
	if (EvaluationContext)
	{
		return FBlueprintCameraEvaluationDataRef::MakeExternalRef(&EvaluationContext->GetOrAddConditionalResult(Condition));
	}
	else
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't access evaluation context's initial result outside of RunCameraDirector"), 
				ELogVerbosity::Error);
		return FBlueprintCameraEvaluationDataRef();
	}
}

UWorld* UBlueprintCameraDirectorEvaluator::GetWorld() const
{
	if (UWorld* CachedWorld = WeakCachedWorld.Get())
	{
		return CachedWorld;
	}

	if (HasAllFlags(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	UObject* Outer = GetOuter();
	while (Outer)
	{
		UWorld* World = Outer->GetWorld();
		if (World)
		{
			WeakCachedWorld = World;
			return World;
		}

		Outer = Outer->GetOuter();
	}

	return nullptr;
}

void UBlueprintCameraDirectorEvaluator::NativeInitializeCameraDirector(UE::Cameras::FCameraDirectorEvaluator* InOwningDirectorEvaluator, const UE::Cameras::FCameraDirectorInitializeParams& Params)
{
	using namespace UE::Cameras;

	ensure(OwningDirectorEvaluator == nullptr && InOwningDirectorEvaluator != nullptr);
	OwningDirectorEvaluator = InOwningDirectorEvaluator;
	EvaluationContext = Params.OwnerContext;
}

void UBlueprintCameraDirectorEvaluator::NativeAbandonCameraDirector()
{
	OwningDirectorEvaluator = nullptr;
}

void UBlueprintCameraDirectorEvaluator::NativeActivateCameraDirector(const UE::Cameras::FCameraDirectorActivateParams& Params)
{
	using namespace UE::Cameras;

	EvaluationResult.Reset();
	{
		UObject* EvaluationContextOwner = nullptr;
		if (EvaluationContext)
		{
			EvaluationContextOwner = EvaluationContext->GetOwner();
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FBlueprintCameraDirectorActivateParams OldParams;
		OldParams.EvaluationContextOwner = EvaluationContextOwner;
		ActivateCameraDirector(EvaluationContextOwner, OldParams);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UBlueprintCameraDirectorEvaluator::NativeDeactivateCameraDirector(const UE::Cameras::FCameraDirectorDeactivateParams& Params)
{
	EvaluationResult.Reset();
	{
		UObject* EvaluationContextOwner = nullptr;
		if (EvaluationContext)
		{
			EvaluationContextOwner = EvaluationContext->GetOwner();
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FBlueprintCameraDirectorDeactivateParams OldParams;
		OldParams.EvaluationContextOwner = EvaluationContextOwner;
		DeactivateCameraDirector(EvaluationContextOwner, OldParams);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UBlueprintCameraDirectorEvaluator::NativeRunCameraDirector(const UE::Cameras::FCameraDirectorEvaluationParams& Params)
{
	EvaluationResult.Reset();
	{
		UObject* EvaluationContextOwner = nullptr;
		if (EvaluationContext)
		{
			EvaluationContextOwner = EvaluationContext->GetOwner();
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FBlueprintCameraDirectorEvaluationParams OldParams;
		OldParams.DeltaTime = Params.DeltaTime;
		OldParams.EvaluationContextOwner = EvaluationContextOwner;
		RunCameraDirector(Params.DeltaTime, EvaluationContextOwner, OldParams);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

bool UBlueprintCameraDirectorEvaluator::NativeAddChildEvaluationContext(UObject* ChildEvaluationContextOwner)
{
	const FName ChildSlotName = AddChildEvaluationContext(ChildEvaluationContextOwner);
	
	if (!ChildSlotName.IsNone())
	{
		const int32 NewChildIndex = EvaluationContext->GetChildrenContexts().Num();

		ensure(!ChildrenContextSlotNames.IsValidIndex(NewChildIndex) || ChildrenContextSlotNames[NewChildIndex].IsNone());
		if (ChildrenContextSlotNames.Num() <= NewChildIndex)
		{
			ChildrenContextSlotNames.SetNum(NewChildIndex + 1);
		}

		ChildrenContextSlotNames[NewChildIndex] = ChildSlotName;

		return true;
	}

	return false;
}

bool UBlueprintCameraDirectorEvaluator::NativeRemoveChildEvaluationContext(UObject* ChildEvaluationContextOwner)
{
	const int32 ChildIndex = EvaluationContext->GetChildrenContexts().IndexOfByPredicate(
			[ChildEvaluationContextOwner](TSharedPtr<FCameraEvaluationContext> Item)
			{
				return Item->GetOwner() == ChildEvaluationContextOwner;
			});
	if (ChildIndex < 0)
	{
		return false;
	}

	if (!ensure(ChildrenContextSlotNames.IsValidIndex(ChildIndex) && !ChildrenContextSlotNames[ChildIndex].IsNone()))
	{
		return false;
	}

	const FName ChildSlotName = ChildrenContextSlotNames[ChildIndex];
	const bool bRemoved = RemoveChildEvaluationContext(ChildEvaluationContextOwner, ChildSlotName);
	if (bRemoved)
	{
		ChildrenContextSlotNames[ChildIndex] = NAME_None;

		return true;
	}

	return false;
}

FCameraDirectorEvaluatorPtr UBlueprintCameraDirector::OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;

	return Builder.BuildEvaluator<FBlueprintCameraDirectorEvaluator>();
}

void UBlueprintCameraDirector::OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext)
{
	using namespace UE::Cameras;

	// Check that a camera director evaluator Blueprint was specified.
	if (!CameraDirectorEvaluatorClass)
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, LOCTEXT("MissingBlueprintClass", "No evaluator Blueprint class is set."));
		return;
	}
}

void UBlueprintCameraDirector::OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const
{
	using namespace UE::Cameras;

	if (!CameraDirectorEvaluatorClass)
	{
		return;
	}

#if WITH_EDITORONLY_DATA

	UBlueprint* EvaluatorBlueprint = Cast<UBlueprint>(CameraDirectorEvaluatorClass->ClassGeneratedBy);
	if (!ensure(EvaluatorBlueprint))
	{
		return;
	}

	TArray<UClass*> RefClasses { UCameraRigAsset::StaticClass(), UCameraRigProxyAsset::StaticClass() };
	FOutgoingReferenceFinder ReferenceFinder(EvaluatorBlueprint, RefClasses);
	ReferenceFinder.MaxDistance = GetDefault<UGameplayCamerasSettings>()->MaxUsageSearchDistance;
	// Find camera rigs and proxies, but don't look at *their* outgoing references, otherwise we might pick rig prefabs,
	// or rigs referenced in transition conditions.
	ReferenceFinder.ExcludeClasses = RefClasses;
	ReferenceFinder.CollectReferences();
	ReferenceFinder.GetReferencesOfClass<UCameraRigAsset>(UsageInfo.CameraRigs);
	ReferenceFinder.GetReferencesOfClass<UCameraRigProxyAsset>(UsageInfo.CameraRigProxies);

#endif  // WITH_EDITORONLY_DATA
}

void UBlueprintCameraDirector::OnExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (CameraDirectorEvaluatorClass && CameraDirectorEvaluatorClass->ClassGeneratedBy)
	{
		FAssetRegistryTag ExternalDirectorTag;
		ExternalDirectorTag.Type = FAssetRegistryTag::ETagType::TT_Hidden;
		ExternalDirectorTag.Name = TEXT("ExternalDirector");
		ExternalDirectorTag.Value = CameraDirectorEvaluatorClass->ClassGeneratedBy->GetPathName();
		Context.AddTag(ExternalDirectorTag);
	}
#endif  // WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE

