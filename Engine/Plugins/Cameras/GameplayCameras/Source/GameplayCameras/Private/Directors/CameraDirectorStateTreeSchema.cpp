// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/CameraDirectorStateTreeSchema.h"

#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeTaskBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraDirectorStateTreeSchema)

namespace UE::Cameras
{

const FName FStateTreeContextDataNames::ContextOwner = TEXT("ContextOwner");

}  // namespace UE::Cameras

UCameraDirectorStateTreeSchema::UCameraDirectorStateTreeSchema()
{
	using namespace UE::Cameras;

	ContextDataDescs.Append({
			// EvaluationContextOwner: {A474F4B3-A82F-45C2-9405-3535F711751B}
			FStateTreeExternalDataDesc(
					FStateTreeContextDataNames::ContextOwner,
					UObject::StaticClass(), 
					FGuid(0xA474F4B3, 0xA82F45C2, 0x94053535, 0xF711751B))
			});
}

bool UCameraDirectorStateTreeSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	return
		// Common structs.
		InScriptStruct->IsChildOf(FStateTreeConditionCommonBase::StaticStruct())
		|| InScriptStruct->IsChildOf(FStateTreeConsiderationCommonBase::StaticStruct())
		|| InScriptStruct->IsChildOf(FStateTreeEvaluatorCommonBase::StaticStruct())
		|| InScriptStruct->IsChildOf(FStateTreeTaskCommonBase::StaticStruct())
		// GameplayCameras structs.
		|| InScriptStruct->IsChildOf(FGameplayCamerasStateTreeTask::StaticStruct())
		|| InScriptStruct->IsChildOf(FGameplayCamerasStateTreeCondition::StaticStruct())
		;
}

bool UCameraDirectorStateTreeSchema::IsClassAllowed(const UClass* InClass) const
{
	return IsChildOfBlueprintBase(InClass);
}

bool UCameraDirectorStateTreeSchema::IsExternalItemAllowed(const UStruct& InStruct) const
{
	return true;
}

void FCameraDirectorStateTreeEvaluationData::Reset()
{
	ActiveCameraRigs.Reset();
	ActiveCameraRigProxies.Reset();
}

