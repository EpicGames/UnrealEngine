// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/PriorityQueueCameraDirector.h"

#include "Algo/StableSort.h"
#include "Core/CameraEvaluationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PriorityQueueCameraDirector)

namespace UE::Cameras
{

int32 FPriorityQueueCameraDirectorEvaluator::FEntry::GetPriority() const
{
	switch (PriorityGiver.GetIndex())
	{
		case FPriorityGiver::IndexOfType<int32>():
			return PriorityGiver.Get<int32>();
		case FPriorityGiver::IndexOfType<IPriorityQueueEntry*>():
			return PriorityGiver.Get<IPriorityQueueEntry*>()->GetPriority();
		default:
			return 0;
	}
}

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FPriorityQueueCameraDirectorEvaluator)

void FPriorityQueueCameraDirectorEvaluator::AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext, int32 Priority)
{
	Super::AddChildEvaluationContext(InContext);
	Entries.Last().PriorityGiver = FPriorityGiver(TInPlaceType<int32>(), Priority);
}

void FPriorityQueueCameraDirectorEvaluator::AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext, IPriorityQueueEntry* PriorityEntry)
{
	Super::AddChildEvaluationContext(InContext);
	Entries.Last().PriorityGiver = FPriorityGiver(TInPlaceType<IPriorityQueueEntry*>(), PriorityEntry);
}

void FPriorityQueueCameraDirectorEvaluator::OnAddChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result)
{
	Entries.Add(FEntry{ Params.ChildContext, FPriorityGiver(TInPlaceType<int32>(), 0) });
	Result.Result = EChildContextManipulationResult::Success;
}

void FPriorityQueueCameraDirectorEvaluator::OnRemoveChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result)
{
	Result.Result = EChildContextManipulationResult::Failure;

	for (auto It = Entries.CreateIterator(); It; ++It)
	{
		if (It->ChildContext == Params.ChildContext)
		{
			It.RemoveCurrent();
			Result.Result = EChildContextManipulationResult::Success;
			break;
		}
	}
}

void FPriorityQueueCameraDirectorEvaluator::OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	if (Entries.Num() > 0)
	{
		FEntryArray EntriesCopy(Entries);
		Algo::StableSortBy(EntriesCopy, &FEntry::GetPriority);

		const FEntry& HighestPriorityEntry = EntriesCopy.Last();
		if (ensure(HighestPriorityEntry.ChildContext))
		{
			if (FCameraDirectorEvaluator* DirectorEvaluator = HighestPriorityEntry.ChildContext->GetDirectorEvaluator())
			{
				FCameraDirectorEvaluationParams ChildParams(Params);
				DirectorEvaluator->Run(ChildParams, OutResult);

				if (OutResult.Requests.Num() > 0)
				{
					return;
				}
			}
		}
	}
}

}  // namespace UE::Cameras

UPriorityQueueCameraDirector::UPriorityQueueCameraDirector(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

FCameraDirectorEvaluatorPtr UPriorityQueueCameraDirector::OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPriorityQueueCameraDirectorEvaluator>();
}

