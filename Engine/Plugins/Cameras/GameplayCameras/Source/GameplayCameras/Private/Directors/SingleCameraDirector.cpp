// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/SingleCameraDirector.h"

#include "Build/CameraBuildContext.h"
#include "Core/CameraAsset.h"
#include "Logging/TokenizedMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleCameraDirector)

#define LOCTEXT_NAMESPACE "SingleCameraDirector"

namespace UE::Cameras
{

class FSingleCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FSingleCameraDirectorEvaluator)
protected:
	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override
	{
		const USingleCameraDirector* SingleDirector = GetCameraDirectorAs<USingleCameraDirector>();
		if (SingleDirector->CameraRig)
		{
			OutResult.Add(GetEvaluationContext(), SingleDirector->CameraRig);
		}
	}
};

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FSingleCameraDirectorEvaluator)

}  // namespace UE::Cameras

USingleCameraDirector::USingleCameraDirector(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

FCameraDirectorEvaluatorPtr USingleCameraDirector::OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSingleCameraDirectorEvaluator>();
}

void USingleCameraDirector::OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext)
{
	if (!CameraRig)
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, this, LOCTEXT("MissingCameraRig", "No camera rig is set."));
	}
}

void USingleCameraDirector::OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const
{
	if (CameraRig)
	{
		UsageInfo.CameraRigs.Add(CameraRig);
	}
}

#undef LOCTEXT_NAMESPACE

