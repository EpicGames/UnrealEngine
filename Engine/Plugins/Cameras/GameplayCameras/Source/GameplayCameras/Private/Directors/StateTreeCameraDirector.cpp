// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/StateTreeCameraDirector.h"

#include "Build/CameraBuildContext.h"
#include "Core/CameraAsset.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Directors/CameraDirectorStateTreeSchema.h"
#include "GameplayCameras.h"
#include "GameplayCamerasSettings.h"
#include "Helpers/OutgoingReferenceFinder.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/EngineVersionComparison.h"
#include "StateTree.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeInstanceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCameraDirector)

#define LOCTEXT_NAMESPACE "StateTreeCameraDirector"

namespace UE::Cameras
{

class FStateTreeCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FStateTreeCameraDirectorEvaluator)

public:

protected:

	// FCameraDirectorEvaluator interface.
	virtual void OnActivate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnDeactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

private:

	bool SetContextRequirements(TSharedPtr<const FCameraEvaluationContext> OwnerContext, FStateTreeExecutionContext& StateTreeContext);

private:

	FStateTreeInstanceData StateTreeInstanceData;

	FCameraDirectorStateTreeEvaluationData EvaluationData;
};

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FStateTreeCameraDirectorEvaluator)

void FStateTreeCameraDirectorEvaluator::OnActivate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	const UStateTreeCameraDirector* StateTreeDirector = GetCameraDirectorAs<UStateTreeCameraDirector>();
	const FStateTreeReference& StateTreeReference = StateTreeDirector->StateTreeReference;
	const UStateTree* StateTree = StateTreeReference.GetStateTree();

	if (!StateTree)
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't activate camera director '%ls': it doesn't have a valid StateTree asset specified.",
			*GetNameSafe(StateTreeDirector));
		return;
	}

	UObject* ContextOwner = GetEvaluationContext()->GetOwner();
	if (!ContextOwner)
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't activate camera director '%ls': the evaluation context doesn't have a valid owner.",
			*GetNameSafe(StateTreeDirector));
		return;
	}

	FStateTreeExecutionContext StateTreeContext(*ContextOwner, *StateTree, StateTreeInstanceData);

	if (!StateTreeContext.IsValid())
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't activate camera director '%ls': initialization of execution context for StateTree asset '%ls' "
				"and context owner '%ls' failed.",
			*GetNameSafe(StateTreeDirector), *GetNameSafe(StateTree), *GetNameSafe(ContextOwner));
		return;
	}

	// TODO: validate schema.
	
	if (!SetContextRequirements(GetEvaluationContext(), StateTreeContext))
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't activate camera director '%ls': failed to setup external data views for StateTree asset '%ls'.",
			*GetNameSafe(StateTreeDirector), *GetNameSafe(StateTree));
		return;
	}

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
	StateTreeContext.Start(StateTreeReference.GetGlobalParameters());
#else
	StateTreeContext.Start(&StateTreeReference.GetParameters());
#endif
}

void FStateTreeCameraDirectorEvaluator::OnDeactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	const UStateTreeCameraDirector* StateTreeDirector = GetCameraDirectorAs<UStateTreeCameraDirector>();
	const FStateTreeReference& StateTreeReference = StateTreeDirector->StateTreeReference;
	const UStateTree* StateTree = StateTreeReference.GetStateTree();

	UObject* ContextOwner = GetEvaluationContext()->GetOwner();
	if (!ContextOwner)
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't deactivate camera director '%ls': the evaluation context doesn't have a valid owner.",
			*GetNameSafe(StateTreeDirector));
		return;
	}

	if (!StateTree)
	{
		UE_LOGF(LogCameraSystem, Error,
			"Can't deactivate camera director '%ls': it doesn't have a valid StateTree asset specified.",
			*GetNameSafe(StateTreeDirector));
		return;
	}
	
	FStateTreeExecutionContext StateTreeContext(*ContextOwner, *StateTree, StateTreeInstanceData);

	if (SetContextRequirements(GetEvaluationContext(), StateTreeContext))
	{
		StateTreeContext.Stop();
	}
}

void FStateTreeCameraDirectorEvaluator::OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	const UStateTreeCameraDirector* StateTreeDirector = GetCameraDirectorAs<UStateTreeCameraDirector>();
	const FStateTreeReference& StateTreeReference = StateTreeDirector->StateTreeReference;
	const UStateTree* StateTree = StateTreeReference.GetStateTree();

	UObject* ContextOwner = GetEvaluationContext()->GetOwner();

	if (!StateTree || !ContextOwner)
	{
		// Fail silently... we already emitted errors during OnActivate.
		return;
	}
	
	FStateTreeExecutionContext StateTreeContext(*ContextOwner, *StateTree, StateTreeInstanceData);

	if (SetContextRequirements(GetEvaluationContext(), StateTreeContext))
	{
		StateTreeContext.Tick(Params.DeltaTime);

		for (const UCameraRigAsset* ActiveCameraRig : EvaluationData.ActiveCameraRigs)
		{
			if (ActiveCameraRig)
			{
				OutResult.Add(GetEvaluationContext(), ActiveCameraRig);
			}
			else
			{
				UE_LOGF(LogCameraSystem, Error, "Null camera rig specified in camera director '%ls'.", 
						*StateTree->GetPathName());
			}
		}

		for (const UCameraRigProxyAsset* ActiveCameraRigProxy : EvaluationData.ActiveCameraRigProxies)
		{
			if (ActiveCameraRigProxy)
			{
				OutResult.Add(GetEvaluationContext(), ActiveCameraRigProxy);
			}
			else
			{
				UE_LOGF(LogCameraSystem, Error, "Null camera rig proxy specified in camera director '%ls'.",
						*StateTree->GetPathName());
			}
		}

	}
}

bool FStateTreeCameraDirectorEvaluator::SetContextRequirements(TSharedPtr<const FCameraEvaluationContext> OwnerContext, FStateTreeExecutionContext& StateTreeContext)
{
	UObject* ContextOwner = OwnerContext->GetOwner();
	StateTreeContext.SetContextDataByName(
			FStateTreeContextDataNames::ContextOwner, 
			FStateTreeDataView(ContextOwner));

	EvaluationData.Reset();

	StateTreeContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda(
		[this](const FStateTreeExecutionContext& Context, const UStateTree* StateTree, 
			TArrayView<const FStateTreeExternalDataDesc> ExternalDescs, TArrayView<FStateTreeDataView> OutDataViews)
		{
			for (int32 Index = 0; Index < ExternalDescs.Num(); Index++)
			{
				const FStateTreeExternalDataDesc& ExternalDesc(ExternalDescs[Index]);
				if (ExternalDesc.Struct == FCameraDirectorStateTreeEvaluationData::StaticStruct())
				{
					OutDataViews[Index] = FStateTreeDataView(FStructView::Make(EvaluationData));
				}
			}
			return true;
		}));

	return true;
}

void FStateTreeCameraDirectorEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	StateTreeInstanceData.AddStructReferencedObjects(Collector);
}

}  // namespace UE::Cameras

UStateTreeCameraDirector::UStateTreeCameraDirector(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

FCameraDirectorEvaluatorPtr UStateTreeCameraDirector::OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FStateTreeCameraDirectorEvaluator>();
}

void UStateTreeCameraDirector::OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext)
{
	using namespace UE::Cameras;

	// Check that a state tree was specified.
	if (!StateTreeReference.IsValid())
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, LOCTEXT("MissingStateTree", "No state tree reference is set."));
		return;
	}
}

void UStateTreeCameraDirector::OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const
{
	using namespace UE::Cameras;

	const UStateTree* StateTree = StateTreeReference.GetStateTree();
	if (!StateTree)
	{
		return;
	}

	TArray<UClass*> RefClasses { UCameraRigAsset::StaticClass(), UCameraRigProxyAsset::StaticClass() };
	FOutgoingReferenceFinder ReferenceFinder(const_cast<UStateTree*>(StateTree), RefClasses);
	ReferenceFinder.MaxDistance = GetDefault<UGameplayCamerasSettings>()->MaxUsageSearchDistance;
	// Find camera rigs and proxies, but don't look at *their* outgoing references, otherwise we might pick rig prefabs,
	// or rigs referenced in transition conditions.
	ReferenceFinder.ExcludeClasses = RefClasses;
	ReferenceFinder.CollectReferences();
	ReferenceFinder.GetReferencesOfClass<UCameraRigAsset>(UsageInfo.CameraRigs);
	ReferenceFinder.GetReferencesOfClass<UCameraRigProxyAsset>(UsageInfo.CameraRigProxies);
}

void UStateTreeCameraDirector::OnExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (const UStateTree* StateTree = StateTreeReference.GetStateTree())
	{
		FAssetRegistryTag ExternalDirectorTag;
		ExternalDirectorTag.Type = FAssetRegistryTag::ETagType::TT_Hidden;
		ExternalDirectorTag.Name = TEXT("ExternalDirector");
		ExternalDirectorTag.Value = StateTree->GetPathName();
		Context.AddTag(ExternalDirectorTag);
	}
}

#undef LOCTEXT_NAMESPACE

