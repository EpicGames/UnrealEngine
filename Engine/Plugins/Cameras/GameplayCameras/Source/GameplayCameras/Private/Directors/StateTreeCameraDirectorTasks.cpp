// Copyright Epic Games, Inc. All Rights Reserved.

#include "Directors/StateTreeCameraDirectorTasks.h"

#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCameraDirectorTasks)

#define LOCTEXT_NAMESPACE "StateTreeCameraDirectorTasks"

bool FGameplayCamerasActivateCameraRigTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(CameraDirectorEvaluationDataHandle);
	return true;
}

EStateTreeRunStatus FGameplayCamerasActivateCameraRigTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	if (UpdateResult(Context))
	{
		return bRunOnce ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Running;
	}
	return EStateTreeRunStatus::Failed;
}

EStateTreeRunStatus FGameplayCamerasActivateCameraRigTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	UpdateResult(Context);

	return EStateTreeRunStatus::Running;
}

bool FGameplayCamerasActivateCameraRigTask::UpdateResult(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.CameraRig)
	{
		FCameraDirectorStateTreeEvaluationData& EvaluationData = Context.GetExternalData(CameraDirectorEvaluationDataHandle);
		EvaluationData.ActiveCameraRigs.Add(InstanceData.CameraRig);
		return true;
	}
	return false;
}

bool FGameplayCamerasActivateCameraRigViaProxyTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(CameraDirectorEvaluationDataHandle);
	return true;
}

EStateTreeRunStatus FGameplayCamerasActivateCameraRigViaProxyTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	if (UpdateResult(Context))
	{
		return bRunOnce ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Running;
	}
	return EStateTreeRunStatus::Failed;
}

EStateTreeRunStatus FGameplayCamerasActivateCameraRigViaProxyTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	UpdateResult(Context);

	return EStateTreeRunStatus::Running;
}

bool FGameplayCamerasActivateCameraRigViaProxyTask::UpdateResult(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.CameraRigProxy)
	{
		FCameraDirectorStateTreeEvaluationData& EvaluationData = Context.GetExternalData(CameraDirectorEvaluationDataHandle);
		EvaluationData.ActiveCameraRigProxies.Add(InstanceData.CameraRigProxy);
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

