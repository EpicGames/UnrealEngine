// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ActivateCameraRigFunctions.h"

#include "Core/CameraRigAsset.h"
#include "Core/RootCameraNode.h"
#include "GameFramework/ControllerGameplayCameraEvaluationComponent.h"
#include "GameFramework/IGameplayCameraSystemHost.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActivateCameraRigFunctions)

void UActivateCameraRigFunctions::ActivatePersistentBaseCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig)
{
	ActivateCameraRigImpl(WorldContextObject, PlayerController, CameraRig, ECameraRigLayer::Base);
}

void UActivateCameraRigFunctions::ActivatePersistentGlobalCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig)
{
	ActivateCameraRigImpl(WorldContextObject, PlayerController, CameraRig, ECameraRigLayer::Global);
}

void UActivateCameraRigFunctions::ActivatePersistentVisualCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig)
{
	ActivateCameraRigImpl(WorldContextObject, PlayerController, CameraRig, ECameraRigLayer::Visual);
}

void UActivateCameraRigFunctions::ActivateCameraRigImpl(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig, ECameraRigLayer EvaluationLayer)
{
	using namespace UE::Cameras;

	if (!PlayerController)
	{
		FFrame::KismetExecutionMessage(
				TEXT("No player controller was given to activate a camera rig!"),
				ELogVerbosity::Error);
		return;
	}

	if (!CameraRig)
	{
		FFrame::KismetExecutionMessage(
				TEXT("No camera rig was given to activate!"),
				ELogVerbosity::Error);
		return;
	}

	// Look for a camera system either under the player controller (such as with AGameplayCamerasPlayerCameraManager),
	// or under the current view target (such as with a UGameplayCameraComponentBase whose actor is the view target).
	UControllerGameplayCameraEvaluationComponent* CameraEvaluationComponent = nullptr;
	if (IGameplayCameraSystemHost* FoundHost = IGameplayCameraSystemHost::FindActiveHost(PlayerController))
	{
		UObject* FoundHostObject = FoundHost->GetAsObject();
		AActor* HostOwningActor = Cast<AActor>(FoundHostObject);
		if (!HostOwningActor)
		{
			HostOwningActor = FoundHostObject->GetTypedOuter<AActor>();
		}
		ensure(HostOwningActor);

		bool bComponentCreated = false;
		CameraEvaluationComponent = UControllerGameplayCameraEvaluationComponent::FindOrAddComponent(HostOwningActor, &bComponentCreated);
		if (bComponentCreated)
		{
			CameraEvaluationComponent->Initialize(FoundHost->GetAsScriptInterface(), PlayerController);
		}
	}

	if (CameraEvaluationComponent)
	{
		CameraEvaluationComponent->ActivateCameraRig(CameraRig, EvaluationLayer);
	}
	else
	{
		UE_LOGF(LogCameraSystem, Error,
				"Can't activate camera rig '%ls' on layer '%ls' because no camera system was found! "
					 "Neither the player controller ('%ls') or the current view target ('%ls') have one.",
				*GetNameSafe(CameraRig),
				*UEnum::GetValueAsString(EvaluationLayer),
				*GetNameSafe(PlayerController),
				*GetNameSafe(PlayerController->GetViewTarget()));
	}
}

