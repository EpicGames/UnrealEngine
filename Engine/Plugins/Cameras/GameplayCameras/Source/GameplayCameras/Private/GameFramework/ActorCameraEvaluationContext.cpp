// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ActorCameraEvaluationContext.h"

#include "Camera/CameraComponent.h"
#include "Core/CameraAsset.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigAsset.h"
#include "Core/PostProcessSettingsCollection.h"
#include "Directors/SingleCameraDirector.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorCameraEvaluationContext)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_EVALUATION_CONTEXT(FActorCameraEvaluationContext)

FActorCameraEvaluationContext::FActorCameraEvaluationContext()
{
}

FActorCameraEvaluationContext::FActorCameraEvaluationContext(UCameraComponent* InCameraComponent)
{
	FCameraEvaluationContextInitializeParams Params;
	Params.Owner = InCameraComponent;
	Params.CameraAsset = MakeCameraComponentCameraAsset(InCameraComponent);
	Initialize(Params);
}

FActorCameraEvaluationContext::FActorCameraEvaluationContext(AActor* InActor)
{
	FCameraEvaluationContextInitializeParams Params;
	Params.Owner = InActor;
	Params.CameraAsset = MakeCalcCameraActorCameraAsset(InActor);
	Initialize(Params);
}

UCameraAsset* FActorCameraEvaluationContext::MakeCameraComponentCameraAsset(UObject* OuterObject)
{
	UCameraComponentCameraNode* WrapperCameraNode = NewObject<UCameraComponentCameraNode>(OuterObject, NAME_None, RF_Transient);
	return MakeSimpleCameraAsset(OuterObject, WrapperCameraNode);
}

UCameraAsset* FActorCameraEvaluationContext::MakeCalcCameraActorCameraAsset(UObject* OuterObject)
{
	UCalcCameraActorCameraNode* WrapperCameraNode = NewObject<UCalcCameraActorCameraNode>(OuterObject, NAME_None, RF_Transient);
	return MakeSimpleCameraAsset(OuterObject, WrapperCameraNode);
}

UCameraAsset* FActorCameraEvaluationContext::MakeSimpleCameraAsset(UObject* OuterObject, UCameraNode* RootNode)
{
	UCameraRigAsset* CameraRig = NewObject<UCameraRigAsset>(OuterObject, NAME_None, RF_Transient);
	CameraRig->RootNode = RootNode;

	USingleCameraDirector* SingleDirector = NewObject<USingleCameraDirector>(OuterObject, NAME_None, RF_Transient);
	SingleDirector->CameraRig = CameraRig;

	UCameraAsset* CameraAsset = NewObject<UCameraAsset>(OuterObject, NAME_None, RF_Transient);
	CameraAsset->SetCameraDirector(SingleDirector);

	return CameraAsset;
}

void FActorCameraEvaluationContext::ApplyMinimalViewInfo(const FMinimalViewInfo& ViewInfo, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& CameraPose = OutResult.CameraPose;
	CameraPose.SetLocation(ViewInfo.Location);
	CameraPose.SetRotation(ViewInfo.Rotation);
	CameraPose.SetFieldOfView(ViewInfo.FOV);
	CameraPose.SetFocalLength(-1.f);
	CameraPose.SetNearClippingPlane(ViewInfo.PerspectiveNearClipPlane);
	CameraPose.SetSensorWidth(CameraPose.GetSensorHeight() * ViewInfo.AspectRatio);
	CameraPose.SetAspectRatioAxisConstraint(ViewInfo.AspectRatioAxisConstraint.Get(CameraPose.GetAspectRatioAxisConstraint()));
	CameraPose.SetConstrainAspectRatio(ViewInfo.bConstrainAspectRatio);

	if (ViewInfo.PostProcessBlendWeight > 0.f)
	{
		OutResult.PostProcessSettings.LerpAll(ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight);
	}
}

class FCameraComponentCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCameraComponentCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraComponentCameraNodeEvaluator)

void FCameraComponentCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	UCameraComponent* CameraComponent = Cast<UCameraComponent>(Params.EvaluationContext->GetOwner());

	if (CameraComponent)
	{
		FMinimalViewInfo CameraView;
		CameraComponent->GetCameraView(Params.DeltaTime, CameraView);
		FActorCameraEvaluationContext::ApplyMinimalViewInfo(CameraView, OutResult);
	}
}

class FCalcCameraActorCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCalcCameraActorCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCalcCameraActorCameraNodeEvaluator)

void FCalcCameraActorCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	AActor* Actor = Cast<AActor>(Params.EvaluationContext->GetOwner());

	if (Actor)
	{
		FMinimalViewInfo CameraView;
		Actor->CalcCamera(Params.DeltaTime, CameraView);
		FActorCameraEvaluationContext::ApplyMinimalViewInfo(CameraView, OutResult);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UCameraComponentCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCameraComponentCameraNodeEvaluator>();
}

FCameraNodeEvaluatorPtr UCalcCameraActorCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCalcCameraActorCameraNodeEvaluator>();
}

