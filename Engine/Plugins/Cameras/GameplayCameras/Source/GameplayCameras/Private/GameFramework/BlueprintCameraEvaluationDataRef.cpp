// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/BlueprintCameraEvaluationDataRef.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableTable.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintCameraEvaluationDataRef)

FBlueprintCameraEvaluationDataRef FBlueprintCameraEvaluationDataRef::MakeExternalRef(FCameraNodeEvaluationResult* InResult)
{
	FBlueprintCameraEvaluationDataRef Ref;
	Ref.Result = InResult;
	return Ref;
}

FBlueprintCameraEvaluationDataRef FBlueprintCameraEvaluationDataRef::MakeOwningRef()
{
	FBlueprintCameraEvaluationDataRef Ref;
	Ref.SharedResult = MakeShared<FCameraNodeEvaluationResult>();
	Ref.Result = Ref.SharedResult.Get();
	return Ref;
}

FBlueprintCameraEvaluationDataRef::FBlueprintCameraEvaluationDataRef()
{
}

FBlueprintCameraEvaluationDataRef UBlueprintCameraEvaluationDataFunctionLibrary::MakeCameraEvaluationData()
{
	return FBlueprintCameraEvaluationDataRef::MakeOwningRef();
}

FBlueprintCameraPose UBlueprintCameraEvaluationDataFunctionLibrary::GetCameraPose(const FBlueprintCameraEvaluationDataRef& CameraData)
{
	using namespace UE::Cameras;

	if (const FCameraNodeEvaluationResult* Result = CameraData.GetResult())
	{
		return FBlueprintCameraPose::FromCameraPose(Result->CameraPose);
	}

	return FBlueprintCameraPose();
}

void UBlueprintCameraEvaluationDataFunctionLibrary::SetCameraPose(const FBlueprintCameraEvaluationDataRef& CameraData, const FBlueprintCameraPose& CameraPose)
{
	using namespace UE::Cameras;

	if (FCameraNodeEvaluationResult* Result = CameraData.GetResult())
	{
		CameraPose.ApplyTo(Result->CameraPose);
	}
}

void UBlueprintCameraEvaluationDataFunctionLibrary::BlendCameraEvaluationData(const FBlueprintCameraEvaluationDataRef& FromCameraData, const FBlueprintCameraEvaluationDataRef& ToCameraData, float Factor)
{
	using namespace UE::Cameras;

	FCameraNodeEvaluationResult* FromResult = FromCameraData.GetResult();
	const FCameraNodeEvaluationResult* ToResult = ToCameraData.GetResult();
	if (FromResult && ToResult)
	{
		FromResult->LerpAll(*ToResult, Factor, true);
	}
}

void UBlueprintCameraEvaluationDataFunctionLibrary::SetDefaultCameraRigParameters(const FBlueprintCameraEvaluationDataRef& CameraData, const UCameraRigAsset* CameraRig)
{
	using namespace UE::Cameras;

	if (FCameraNodeEvaluationResult* Result = CameraData.GetResult())
	{
		FCameraObjectInterfaceParameterOverrideHelper Helper(*Result);
		Helper.ApplyParameterDefaults(CameraRig);
	}
}

#define UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE(ErrorResult)\
	if (!CameraData.IsValid())\
	{\
		FFrame::KismetExecutionMessage(TEXT("No camera variable table has been set"), ELogVerbosity::Error);\
		return ErrorResult;\
	}

#define UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE_PARAM(ErrorResult)\
	if (!Variable)\
	{\
		FFrame::KismetExecutionMessage(TEXT("No camera variable asset was given"), ELogVerbosity::Error);\
		return ErrorResult;\
	}\

#define UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(VariableType)\
	static VariableType ErrorResult {};\
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE(ErrorResult)\
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE_PARAM(ErrorResult)\
	return CameraData.GetResult()->VariableTable.GetValue<VariableType>(Variable->GetVariableID(), Variable->GetDefaultValue());

#define UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(VariableType)\
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE()\
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE_PARAM()\
	CameraData.GetResult()->VariableTable.SetValue(Variable, Value, true);

bool UBlueprintCameraVariableTableFunctionLibrary::GetBooleanCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UBooleanCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(bool);
}

int32 UBlueprintCameraVariableTableFunctionLibrary::GetInteger32CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UInteger32CameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(int32);
}

float UBlueprintCameraVariableTableFunctionLibrary::GetFloatCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UFloatCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(float);
}

double UBlueprintCameraVariableTableFunctionLibrary::GetDoubleCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UDoubleCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(double);
}

FVector2D UBlueprintCameraVariableTableFunctionLibrary::GetVector2CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector2dCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(FVector2d);
}

FVector UBlueprintCameraVariableTableFunctionLibrary::GetVector3CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector3dCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(FVector3d);
}

FVector4 UBlueprintCameraVariableTableFunctionLibrary::GetVector4CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector4dCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(FVector4d);
}

FRotator UBlueprintCameraVariableTableFunctionLibrary::GetRotatorCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, URotator3dCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(FRotator3d);
}

FTransform UBlueprintCameraVariableTableFunctionLibrary::GetTransformCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UTransform3dCameraVariable* Variable)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE(FTransform3d);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetBooleanCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UBooleanCameraVariable* Variable, bool Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(bool);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetInteger32CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UInteger32CameraVariable* Variable, int32 Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(int32);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetFloatCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UFloatCameraVariable* Variable, float Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(float);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetDoubleCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UDoubleCameraVariable* Variable, double Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(double);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetVector2CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector2dCameraVariable* Variable, const FVector2D& Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(FVector2d);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetVector3CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector3dCameraVariable* Variable, const FVector& Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(FVector3d);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetVector4CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector4dCameraVariable* Variable, const FVector4& Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(FVector4d);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetRotatorCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, URotator3dCameraVariable* Variable, const FRotator& Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(FRotator3d);
}

void UBlueprintCameraVariableTableFunctionLibrary::SetTransformCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UTransform3dCameraVariable* Variable, const FTransform& Value)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE(FTransform3d);
}

#undef UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE
#undef UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_VALIDATE_PARAM
#undef UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_GET_VARIABLE
#undef UE_PRIVATE_BLUEPRINT_CAMERA_VARIABLE_TABLE_SET_VARIABLE

#define UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(ErrorResult)\
	using namespace UE::Cameras;\
	if (!DataID.IsValid())\
	{\
		FFrame::KismetExecutionMessage(TEXT("Invalid camera context data ID"), ELogVerbosity::Error);\
		return ErrorResult;\
	}\
	if (!CameraData.IsValid())\
	{\
		FFrame::KismetExecutionMessage(TEXT("No camera context data table has been set"), ELogVerbosity::Error);\
		return ErrorResult;\
	}

FName UBlueprintCameraContextDataTableFunctionLibrary::GetNameData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(NAME_None);
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetNameData(DataID);
}

FString UBlueprintCameraContextDataTableFunctionLibrary::GetStringData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(FString());
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetStringData(DataID);
}

uint8 UBlueprintCameraContextDataTableFunctionLibrary::GetEnumData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UEnum* EnumType)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(0);
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetEnumData(DataID, EnumType);
}

FInstancedStruct UBlueprintCameraContextDataTableFunctionLibrary::GetStructData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UScriptStruct* DataStructType)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(FInstancedStruct());
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetInstancedStructData(DataID, DataStructType);
}

UObject* UBlueprintCameraContextDataTableFunctionLibrary::GetObjectData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(nullptr);
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetObjectData(DataID);
}

UClass* UBlueprintCameraContextDataTableFunctionLibrary::GetClassData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(nullptr);
	const FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	return ActualTable.GetClassData(DataID);
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetNameData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FName& Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetNameData(DataID, Data);
	return true;
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetStringData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FString& Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetStringData(DataID, Data);
	return true;
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetEnumData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UEnum* EnumType, uint8 Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetEnumData(DataID, EnumType, Data);
	return true;
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetStructData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FInstancedStruct& Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetInstancedStructData(DataID, Data);
	return true;
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetObjectData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, UObject* Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetObjectData(DataID, Data);
	return true;
}

bool UBlueprintCameraContextDataTableFunctionLibrary::SetClassData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, UClass* Data)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE(false);
	FCameraContextDataTable& ActualTable = CameraData.GetResult()->ContextDataTable;
	ActualTable.SetClassData(DataID, Data);
	return true;
}

#undef UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_DATA_TABLE_VALIDATE

#define UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE()\
	using namespace UE::Cameras;\
	if (!CameraData.IsValid())\
	{\
		FFrame::KismetExecutionMessage(TEXT("No valid camera data was provided"), ELogVerbosity::Error);\
		return;\
	}\

void UBlueprintPostProcessSettingsCollectionFunctionLibrary::GetPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, FPostProcessSettings& OutPostProcessSettings)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE()
	OutPostProcessSettings = CameraData.GetResult()->PostProcessSettings.Get();
}

void UBlueprintPostProcessSettingsCollectionFunctionLibrary::SetPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE()
	CameraData.GetResult()->PostProcessSettings.Get() = PostProcessSettings;
}

void UBlueprintPostProcessSettingsCollectionFunctionLibrary::OverrideChangedPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE()
	CameraData.GetResult()->PostProcessSettings.OverrideChanged(PostProcessSettings);
}

void UBlueprintPostProcessSettingsCollectionFunctionLibrary::LerpPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings, float BlendFactor)
{
	UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE()
	CameraData.GetResult()->PostProcessSettings.LerpAll(PostProcessSettings, BlendFactor);
}

#undef UE_PRIVATE_BLUEPRINT_CAMERA_CONTEXT_POST_PROCESS_VALIDATE

