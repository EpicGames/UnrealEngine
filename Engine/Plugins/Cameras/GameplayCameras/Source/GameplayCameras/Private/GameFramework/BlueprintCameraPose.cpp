// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/BlueprintCameraPose.h"

#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintCameraPose)

FBlueprintCameraPose FBlueprintCameraPose::FromCameraPose(const FCameraPose& InCameraPose)
{
	FBlueprintCameraPose Result;
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	Result.PropName = InCameraPose.Get##PropName();
UE_CAMERA_POSE_FOR_ALL_PROPERTIES()
#undef UE_CAMERA_POSE_FOR_PROPERTY
	return Result;
}

void FBlueprintCameraPose::ApplyTo(FCameraPose& OutCameraPose) const
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	OutCameraPose.Set##PropName(PropName, false);
UE_CAMERA_POSE_FOR_ALL_PROPERTIES()
#undef UE_CAMERA_POSE_FOR_PROPERTY
}

FTransform UBlueprintCameraPoseFunctionLibrary::GetTransform(const FBlueprintCameraPose& CameraPose)
{
	FTransform Transform;
	Transform.SetLocation(CameraPose.Location);
	Transform.SetRotation(CameraPose.Rotation.Quaternion());
	return Transform;
}

double UBlueprintCameraPoseFunctionLibrary::GetEffectiveFieldOfView(const FBlueprintCameraPose& CameraPose)
{
	return FCameraPose::GetEffectiveFieldOfView(
			CameraPose.FocalLength, 
			CameraPose.FieldOfView, 
			CameraPose.SensorWidth, 
			CameraPose.SensorHeight, 
			CameraPose.SqueezeFactor);
}

double UBlueprintCameraPoseFunctionLibrary::GetSensorAspectRatio(const FBlueprintCameraPose& CameraPose)
{
	return FCameraPose::GetSensorAspectRatio(
			CameraPose.SensorWidth,
			CameraPose.SensorHeight);
}

FRay UBlueprintCameraPoseFunctionLibrary::GetAimRay(const FBlueprintCameraPose& CameraPose)
{
	const bool bDirectionIsNormalized = true;
	const FVector TargetDir{ 1, 0, 0 };
	return FRay(CameraPose.Location, CameraPose.Rotation.RotateVector(TargetDir), bDirectionIsNormalized);
}

FVector UBlueprintCameraPoseFunctionLibrary::GetAimDir(const FBlueprintCameraPose& CameraPose)
{
	return CameraPose.Rotation.RotateVector(FVector{ 1, 0, 0 });
}

FVector UBlueprintCameraPoseFunctionLibrary::GetTarget(const FBlueprintCameraPose& CameraPose)
{
	return CameraPose.Location + CameraPose.TargetDistance * GetAimDir(CameraPose);
}

FVector UBlueprintCameraPoseFunctionLibrary::GetTargetAtDistance(const FBlueprintCameraPose& CameraPose, double TargetDistance)
{
	return CameraPose.Location + TargetDistance * GetAimDir(CameraPose);
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetTransform(const FBlueprintCameraPose& CameraPose, const FTransform& Transform)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.Location = Transform.GetLocation();
	Result.Rotation = Transform.GetRotation().Rotator();
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::MakeCameraPoseFromCameraComponent(const UCameraComponent* CameraComponent)
{
	FBlueprintCameraPose Result;
	if (CameraComponent)
	{
		const FTransform& CameraComponentTransform = CameraComponent->GetComponentTransform();

		FTransform AdditiveTransform;
		float AdditiveFOV;
		CameraComponent->GetAdditiveOffset(AdditiveTransform, AdditiveFOV);

		Result.Location = CameraComponentTransform.GetLocation() + AdditiveTransform.GetLocation();
		Result.Rotation = (AdditiveTransform.GetRotation() * CameraComponentTransform.GetRotation()).Rotator();
		Result.FieldOfView = CameraComponent->FieldOfView + AdditiveFOV;
		Result.ConstrainAspectRatio = CameraComponent->bConstrainAspectRatio;
		Result.OverrideAspectRatioAxisConstraint = CameraComponent->bOverrideAspectRatioAxisConstraint;
		Result.AspectRatioAxisConstraint = CameraComponent->AspectRatioAxisConstraint;
		Result.SensorWidth = CameraComponent->AspectRatio * Result.SensorHeight;
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid camera component was given"), ELogVerbosity::Error);
	}
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::MakeCameraPoseFromCineCameraComponent(const UCineCameraComponent* CameraComponent)
{
	FBlueprintCameraPose Result = MakeCameraPoseFromCameraComponent(CameraComponent);
	if (CameraComponent)
	{
		Result.TargetDistance = CameraComponent->CurrentFocusDistance;
		Result.FieldOfView = -1.f;
		Result.FocalLength = CameraComponent->CurrentFocalLength;
		Result.Aperture = CameraComponent->CurrentAperture;
		Result.SensorWidth = CameraComponent->Filmback.SensorWidth;
		Result.SensorHeight = CameraComponent->Filmback.SensorHeight;
		Result.SqueezeFactor = CameraComponent->LensSettings.SqueezeFactor;
	}
	// Error message already emitted in MakeCameraPoseFromCameraComponent.
	return Result;
}

// Deprecated methods.

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetLocation(const FBlueprintCameraPose& CameraPose, const FVector& Location)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.Location = Location;
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetRotation(const FBlueprintCameraPose& CameraPose, const FRotator& Rotation)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.Rotation = Rotation;
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetTargetDistance(const FBlueprintCameraPose& CameraPose, double TargetDistance)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.TargetDistance = TargetDistance;
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetFieldOfView(const FBlueprintCameraPose& CameraPose, float FieldOfView)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.FieldOfView = FieldOfView;
	return Result;
}

FBlueprintCameraPose UBlueprintCameraPoseFunctionLibrary::SetFocalLength(const FBlueprintCameraPose& CameraPose, float FocalLength)
{
	FBlueprintCameraPose Result(CameraPose);
	Result.FocalLength = FocalLength;
	return Result;
}

