// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/CameraRigParameterInterop.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraVariableTable.h"
#include "GameFramework/BlueprintCameraEvaluationDataRef.h"

#define LOCTEXT_NAMESPACE "CameraRigParameterInterop"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigParameterInterop)

UCameraRigParameterInterop::UCameraRigParameterInterop(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UCameraRigParameterInterop::GetCameraParameter(const FBlueprintCameraEvaluationDataRef& CameraData, UCameraRigAsset* CameraRig, FName ParameterName, int32& ReturnValue)
{
	checkNoEntry();
}

void UCameraRigParameterInterop::SetCameraParameter(const FBlueprintCameraEvaluationDataRef& CameraData, UCameraRigAsset* CameraRig, FName ParameterName, const int32& NewValue)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UCameraRigParameterInterop::execGetCameraParameter)
{
	P_GET_STRUCT_REF(FBlueprintCameraEvaluationDataRef, CameraData);
	P_GET_OBJECT(UCameraRigAsset, CameraRig);
	P_GET_PROPERTY(FNameProperty, ParameterName);

	// Read wildcard value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	const FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetPtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (TargetProperty == nullptr || TargetPtr == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("InvalidGetCameraParameterReturnValue", "Failed to resolve ReturnValue for GetCameraParameter")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN

		using namespace UE::Cameras;

		const FCameraObjectInterfaceParameterDefinition* ParameterDefinition = CameraRig->GetParameterDefinitions()
			.FindByPredicate(
				[ParameterName](const FCameraObjectInterfaceParameterDefinition& Item)
				{
					return Item.ParameterName == ParameterName;
				});
		if (!ParameterDefinition)
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(LOCTEXT("ParameterDefinitionNotFound", "No such camera parameter: {0}"), FText::FromName(ParameterName))
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else if (!CameraData.IsValid())
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				LOCTEXT("InvalidCameraData", "CameraData is an invalid reference")
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else
		{
			bool bGotValue = false;

			if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Blendable)
			{
				const FCameraVariableTable& VariableTable = CameraData.GetResult()->VariableTable;
				const uint8* RawValue = VariableTable.TryGetValue(ParameterDefinition->VariableID, ParameterDefinition->VariableType, ParameterDefinition->BlendableStructType);
				if (RawValue)
				{
					TargetProperty->CopyCompleteValue(TargetPtr, RawValue);
					bGotValue = true;
				}
			}
			else if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Data)
			{
				const FCameraContextDataTable& ContextDataTable = CameraData.GetResult()->ContextDataTable;
				const uint8* RawValue = ContextDataTable.TryGetRawDataPtr(ParameterDefinition->DataID, ParameterDefinition->DataType, ParameterDefinition->DataTypeObject);
				if (RawValue)
				{
					TargetProperty->CopyCompleteValue(TargetPtr, RawValue);
					bGotValue = true;
				}
			}

			if (!bGotValue)
			{
				const FInstancedPropertyBag& DefaultParameters = CameraRig->GetDefaultParameters();
				const UPropertyBag* DefaultParametersStruct = DefaultParameters.GetPropertyBagStruct();
				const FPropertyBagPropertyDesc* PropertyDesc = DefaultParametersStruct->FindPropertyDescByID(ParameterDefinition->ParameterGuid);
				if (PropertyDesc && PropertyDesc->CachedProperty)
				{
					const void* RawDefaultValue = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>((const void*)DefaultParameters.GetValue().GetMemory());
					TargetProperty->CopyCompleteValue(TargetPtr, RawDefaultValue);
					bGotValue = true;
				}
			}

			if (!bGotValue)
			{
				FBlueprintExceptionInfo ExceptionInfo(
						EBlueprintExceptionType::NonFatalError,
						FText::Format(LOCTEXT("NoDefaultValueFound", "Parameter {0} does not exist on camera rig {1}"), FText::FromName(ParameterName), FText::FromName(CameraRig->GetFName()))
						);
				FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			}
		}

		P_NATIVE_END
	}
}

DEFINE_FUNCTION(UCameraRigParameterInterop::execSetCameraParameter)
{
	P_GET_STRUCT_REF(FBlueprintCameraEvaluationDataRef, CameraData);
	P_GET_OBJECT(UCameraRigAsset, CameraRig);
	P_GET_PROPERTY(FNameProperty, ParameterName);

	// Read wildcard value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	const FProperty* SourceProperty = Stack.MostRecentProperty;
	const uint8* SourcePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (SourceProperty == nullptr || SourcePtr == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("InvalidSetCameraParameterNewValue", "Failed to resolve NewValue for SetCameraParameter")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN

		using namespace UE::Cameras;

		const FCameraObjectInterfaceParameterDefinition* ParameterDefinition = CameraRig->GetParameterDefinitions()
			.FindByPredicate(
				[ParameterName](const FCameraObjectInterfaceParameterDefinition& Item)
				{
					return Item.ParameterName == ParameterName;
				});
		if (!ParameterDefinition)
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(LOCTEXT("ParameterDefinitionNotFound", "No such camera parameter: {0}"), FText::FromName(ParameterName))
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else if (!CameraData.IsValid())
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				LOCTEXT("InvalidCameraData", "CameraData is an invalid reference")
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else
		{
			if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Blendable)
			{
				FCameraVariableTable& VariableTable = CameraData.GetResult()->VariableTable;
				const bool bDidSet = VariableTable.TrySetValue(ParameterDefinition->VariableID, ParameterDefinition->VariableType, ParameterDefinition->BlendableStructType, SourcePtr);
				if (!bDidSet)
				{
					UCameraObjectInterfaceBlendableParameter* BlendableParameter = CameraRig->Interface.FindBlendableParameterByGuid(ParameterDefinition->ParameterGuid);
					if (ensureMsgf(
								BlendableParameter, 
								TEXT("Can't find parameter '%s' on camera rig '%s', was it changed since it was last built?"), 
								*ParameterDefinition->ParameterName.ToString(), *GetNameSafe(CameraRig)))
					{
						FCameraVariableDefinition VariableDefinition = BlendableParameter->GetVariableDefinition();
						VariableDefinition.bIsPrivate = false;  // If the parameter isn't in the table, it's probably meant for another rig, so let's make sure it propagates.
						VariableTable.AddVariable(VariableDefinition);
						VariableTable.SetValue(ParameterDefinition->VariableID, ParameterDefinition->VariableType, ParameterDefinition->BlendableStructType, SourcePtr);
					}
				}
			}
			else if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Data)
			{
				FCameraContextDataTable& ContextDataTable = CameraData.GetResult()->ContextDataTable;
				uint8* RawValue = ContextDataTable.TryGetMutableRawDataPtr(ParameterDefinition->DataID, ParameterDefinition->DataType, ParameterDefinition->DataTypeObject);
				if (!RawValue)
				{
					UCameraObjectInterfaceDataParameter* DataParameter = CameraRig->Interface.FindDataParameterByGuid(ParameterDefinition->ParameterGuid);
					if (ensureMsgf(
								DataParameter, 
								TEXT("Can't find parameter '%s' on camera rig '%s', was it changed since it was last built?"), 
								*ParameterDefinition->ParameterName.ToString(), *GetNameSafe(CameraRig)))
					{
						FCameraContextDataDefinition DataDefinition = DataParameter->GetDataDefinition();
						ContextDataTable.AddData(DataDefinition);
						RawValue = ContextDataTable.TryGetMutableRawDataPtr(ParameterDefinition->DataID, ParameterDefinition->DataType, ParameterDefinition->DataTypeObject);
					}
				}
				if (RawValue)
				{
					SourceProperty->CopyCompleteValue(RawValue, SourcePtr);
				}
			}
		}

		P_NATIVE_END
	}
}

FVector UCameraRigParameterInteropLibrary::MakeLiteralVector(FVector Value)
{
	return Value;
}

FVector3f UCameraRigParameterInteropLibrary::MakeLiteralVector3f(FVector3f Value)
{
	return Value;
}

FVector2D UCameraRigParameterInteropLibrary::MakeLiteralVector2D(FVector2D Value)
{
	return Value;
}

FRotator UCameraRigParameterInteropLibrary::MakeLiteralRotator(FRotator Value)
{
	return Value;
}

FLinearColor UCameraRigParameterInteropLibrary::MakeLiteralLinearColor(FLinearColor Value)
{
	return Value;
}

#undef LOCTEXT_NAMESPACE

