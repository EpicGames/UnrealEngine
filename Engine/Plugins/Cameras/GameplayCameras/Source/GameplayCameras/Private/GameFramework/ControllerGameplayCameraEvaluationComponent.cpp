// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ControllerGameplayCameraEvaluationComponent.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "GameFramework/IGameplayCameraSystemHost.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControllerGameplayCameraEvaluationComponent)

UControllerGameplayCameraEvaluationComponent::UControllerGameplayCameraEvaluationComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
}

void UControllerGameplayCameraEvaluationComponent::Initialize(TScriptInterface<IGameplayCameraSystemHost> InCameraSystemHost, APlayerController* InPlayerController)
{
	using namespace UE::Cameras;

	if (!ensureMsgf(
				CameraSystemHost == nullptr,
				TEXT("This component has already been initialized!")))
	{
		return;
	}

	ensure(InCameraSystemHost);
	CameraSystemHost = InCameraSystemHost;

	APlayerController* PlayerController = InPlayerController;
	if (!PlayerController)
	{
		PlayerController = GetOwner<APlayerController>();
	}

	FCameraEvaluationContextInitializeParams InitParams;
	InitParams.Owner = this;
	InitParams.PlayerController = PlayerController;
	EvaluationContext = MakeShared<FCameraEvaluationContext>(InitParams);
	EvaluationContext->GetInitialResult().bIsValid = true;	
	
	RegisterComponent();
}

void UControllerGameplayCameraEvaluationComponent::ActivateCameraRig(UCameraRigAsset* CameraRig, ECameraRigLayer EvaluationLayer)
{
	FCameraRigInfo NewCameraRigInfo;
	NewCameraRigInfo.CameraRig = CameraRig;
	NewCameraRigInfo.EvaluationLayer = EvaluationLayer;
	NewCameraRigInfo.bActivated = false;
	CameraRigInfos.Add(NewCameraRigInfo);

	if (IsActive())
	{
		ActivateCameraRigs();
	}
}

void UControllerGameplayCameraEvaluationComponent::BeginPlay()
{
	Super::BeginPlay();

	ActivateCameraRigs();
}

void UControllerGameplayCameraEvaluationComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	CameraRigInfos.Reset();
	EvaluationContext.Reset();

	Super::EndPlay(EndPlayReason);
}

void UControllerGameplayCameraEvaluationComponent::ActivateCameraRigs()
{
	using namespace UE::Cameras;

	if (!ensureMsgf(
				CameraSystemHost && EvaluationContext,
				TEXT("This component hasn't been initialized!")))
	{
		return;
	}

	TSharedPtr<FCameraSystemEvaluator> SystemEvaluator = CameraSystemHost->GetCameraSystemEvaluator();
	FRootCameraNodeEvaluator* RootNodeEvaluator = SystemEvaluator->GetRootNodeEvaluator();

	for (FCameraRigInfo& CameraRigInfo : CameraRigInfos)
	{
		if (!CameraRigInfo.bActivated)
		{
			FActivateCameraRigParams Params;
			Params.CameraRig = CameraRigInfo.CameraRig;
			Params.EvaluationContext = EvaluationContext;
			Params.Layer = CameraRigInfo.EvaluationLayer;
			RootNodeEvaluator->ActivateCameraRig(Params);

			CameraRigInfo.bActivated = true;
		}
	}
}

UControllerGameplayCameraEvaluationComponent* UControllerGameplayCameraEvaluationComponent::FindComponent(AActor* OwnerActor)
{
	return OwnerActor->FindComponentByClass<UControllerGameplayCameraEvaluationComponent>();
}

UControllerGameplayCameraEvaluationComponent* UControllerGameplayCameraEvaluationComponent::FindOrAddComponent(AActor* OwnerActor, bool* bOutCreated)
{
	UControllerGameplayCameraEvaluationComponent* ControllerComponent = FindComponent(OwnerActor);
	if (!ControllerComponent)
	{
		ControllerComponent = NewObject<UControllerGameplayCameraEvaluationComponent>(
				OwnerActor, TEXT("ControllerGameplayCameraEvaluationComponent"), RF_Transient);
		if (bOutCreated)
		{
			*bOutCreated = true;
		}
	}
	return ControllerComponent;
}

