// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraActor.h"

#include "CineCameraComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/GameplayCameraComponent.h"
#include "Kismet/GameplayStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraActor)

#define LOCTEXT_NAMESPACE "GameplayCameraActor"

AGameplayCameraActor::AGameplayCameraActor(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CameraComponent = CreateDefaultSubobject<UGameplayCameraComponent>(TEXT("CameraComponent"));
	RootComponent = CameraComponent;

	OutputCameraComponent = CreateDefaultSubobject<UCineCameraComponent>(TEXT("OutputCameraComponent"), true);
	OutputCameraComponent->SetupAttachment(CameraComponent);
}

USceneComponent* AGameplayCameraActor::GetDefaultAttachComponent() const
{
	return CameraComponent;
}

UGameplayCameraComponentBase* AGameplayCameraActor::GetCameraComponentBase() const
{
	return CameraComponent;
}

#undef LOCTEXT_NAMESPACE

