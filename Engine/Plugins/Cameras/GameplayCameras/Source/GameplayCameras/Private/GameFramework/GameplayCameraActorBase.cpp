// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraActorBase.h"

#include "Core/CameraSystemEvaluator.h"
#include "GameFramework/GameplayCameraComponentBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraActorBase)

#define LOCTEXT_NAMESPACE "GameplayCameraActorBase"

AGameplayCameraActorBase::AGameplayCameraActorBase(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void AGameplayCameraActorBase::CalcCamera(float DeltaTime, FMinimalViewInfo& OutResult)
{
	Super::CalcCamera(DeltaTime, OutResult);
}

#undef LOCTEXT_NAMESPACE

