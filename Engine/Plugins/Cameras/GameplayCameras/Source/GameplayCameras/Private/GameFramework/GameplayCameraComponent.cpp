// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraComponent.h"

#include "Core/CameraAsset.h"
#include "GameplayCamerasDelegates.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/EngineVersionComparison.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraComponent)

#define LOCTEXT_NAMESPACE "GameplayCameraComponent"

UGameplayCameraComponent::UGameplayCameraComponent(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UGameplayCameraComponent::PostLoad()
{
	Super::PostLoad();

	if (Camera_DEPRECATED)
	{
		CameraReference.SetCameraAsset(Camera_DEPRECATED);
		Camera_DEPRECATED = nullptr;
	}
}

void UGameplayCameraComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		// Make sure parameters are up to date when cooking.
		const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
		if (TargetPlatform && TargetPlatform->RequiresCookedData())
		{
			CameraReference.RebuildParametersIfNeeded();
		}
	}
#endif  // WITH_EDITOR

	Super::PreSave(ObjectSaveContext);
}

void UGameplayCameraComponent::OnRegister()
{
	using namespace UE::Cameras;

	Super::OnRegister();

#if WITH_EDITOR
	FGameplayCamerasDelegates::OnCameraAssetBuilt().AddUObject(this, &UGameplayCameraComponent::OnCameraAssetBuilt);

	Super::CreateCameraSpriteComponent(TEXT("/GameplayCameras/Textures/S_GameplayCamera.S_GameplayCamera"));
#endif  // WITH_EDITOR
}

void UGameplayCameraComponent::OnUnregister()
{
	using namespace UE::Cameras;

#if WITH_EDITOR
	FGameplayCamerasDelegates::OnCameraAssetBuilt().RemoveAll(this);
#endif  // WITH_EDITOR

	Super::OnUnregister();
}

UCameraAsset* UGameplayCameraComponent::OnCreateEvaluationContext()
{
#if WITH_EDITOR
	CameraReference.RebuildParametersIfNeeded();
#endif  // WITH_EDITOR

	return CameraReference.GetCameraAsset();
}

void UGameplayCameraComponent::OnUpdateEvaluationContext(bool bForceApplyParameterOverrides)
{
	using namespace UE::Cameras;

	FCameraNodeEvaluationResult& InitialResult = GetEvaluationContext()->GetInitialResult();

	if (bForceApplyParameterOverrides)
	{
		CameraReference.ApplyParameterOverrides(InitialResult, false);
	}
#if UE_VERSION_OLDER_THAN(5,7,0)
	// Before 5.7.0, we don't have a notify callback from Sequencer to know that it's animating parameters,
	// so we need to always re-apply values.
	else
	{
		ApplyChangedParameterOverrides();
	}
#endif  // pre-5.7.0
}

void UGameplayCameraComponent::NotifyChangeCameraReference()
{
	// Sequencer animated some of our parameters... look for those whose value changed, compared to our
	// cached parameter bag, and re-apply them to the evaluation context.
	// TODO: This isn't a very efficient process, but it's unclear how to reconcile the reference's parameters struct
	//		 with the evaluation context's variable/context-data tables.
	ApplyChangedParameterOverrides();
}

void UGameplayCameraComponent::ApplyChangedParameterOverrides()
{
	using namespace UE::Cameras;

	const UCameraAsset* CameraAsset = CameraReference.GetCameraAsset();
	if (CameraAsset && HasEvaluationContext())
	{
		FCameraNodeEvaluationResult& InitialResult = GetEvaluationContext()->GetInitialResult();

		TArray<FGuid> ChangedParameterGuids;
		FCameraObjectInterfaceParameterOverrideHelper Helper(InitialResult);
		GetChangedParameterOverrides(CameraReference.GetParameters(), CachedParameterOverrides, ChangedParameterGuids);
		Helper.ApplyFilteredParameters(
				CameraAsset,
				CameraAsset->GetParameterDefinitions(),
				CameraReference.GetParameters(),
				[ChangedParameterGuids](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
				{
					return ChangedParameterGuids.Contains(Definition.ParameterGuid);
				});

		CachedParameterOverrides = CameraReference.GetParameters();
	}
}

#if WITH_EDITOR

void UGameplayCameraComponent::OnCameraAssetBuilt(const UCameraAsset* InCameraAsset)
{
	using namespace UE::Cameras;

	if (InCameraAsset != CameraReference.GetCameraAsset())
	{
		return;
	}

	// If our camera asset was just built, it may have some new parameters. We need to rebuild
	// our variable table and context data table, and re-apply overrides.
	if (CameraReference.NeedsRebuildParameters())
	{
		Modify();
		CameraReference.RebuildParameters();
		if (HasEvaluationContext())
		{
			const FCameraAssetAllocationInfo& AllocationInfo = InCameraAsset->GetAllocationInfo();
			ReinitializeEvaluationContext(AllocationInfo.VariableTableInfo, AllocationInfo.ContextDataTableInfo);
			UpdateEvaluationContext(true);
		}
	}
}

void UGameplayCameraComponent::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::Cameras;

	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UGameplayCameraComponent, CameraReference))
	{
		if (HasEvaluationContext())
		{
			if (PropertyChangedEvent.GetPropertyName() == TEXT("CameraAsset"))
			{
				// The camera asset has changed! Recreate the context.
				RecreateEditorWorldEvaluationContext();
			}
			else
			{
				// Otherwise, maybe one of the parameter overrides has changed. Re-apply them.
				UpdateEvaluationContext(true);
			}
		}
	}
}

#endif  // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

