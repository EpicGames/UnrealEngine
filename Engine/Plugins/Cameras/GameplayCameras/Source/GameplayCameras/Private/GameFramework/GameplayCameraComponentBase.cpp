// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraComponentBase.h"

#include "CineCameraComponent.h"
#include "Components/BillboardComponent.h"
#include "Core/CameraAsset.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Debug/CameraDebugRenderer.h"
#include "Directors/SingleCameraDirector.h"
#include "Engine/Canvas.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasModule.h"
#include "Kismet/GameplayStatics.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EngineVersionComparison.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "UObject/ICookInfo.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
#include "PrimitiveDrawInterface.h"
#else
#include "SceneManagement.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditorViewport.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraComponentBase)

#define LOCTEXT_NAMESPACE "GameplayCameraComponentBase"

UGameplayCameraComponentBase::UGameplayCameraComponentBase(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	bAutoActivate = true;
	bTickInEditor = true;
	bWantsOnUpdateTransform = true;

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
}

void UGameplayCameraComponentBase::BeginDestroy()
{
	DestroyCameraSystemHost();
	DestroyEvaluationContext();

	Super::BeginDestroy();
}

void UGameplayCameraComponentBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UGameplayCameraComponentBase* This = CastChecked<UGameplayCameraComponentBase>(InThis);
	This->OnAddReferencedObjects(Collector);
	if (This->EvaluationContext.IsValid())
	{
		This->EvaluationContext->AddReferencedObjects(Collector);
	}
}

TSharedPtr<const UE::Cameras::FCameraEvaluationContext> UGameplayCameraComponentBase::GetEvaluationContext() const
{
	return EvaluationContext;
}

TSharedPtr<UE::Cameras::FCameraEvaluationContext> UGameplayCameraComponentBase::GetEvaluationContext()
{
	return EvaluationContext;
}

void UGameplayCameraComponentBase::ActivateCameraForPlayerIndex(int32 PlayerIndex, bool bSetAsViewTarget)
{
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(this, PlayerIndex);
	ActivateCameraForPlayerController(PlayerController, bSetAsViewTarget);
}

void UGameplayCameraComponentBase::ActivateCameraForPlayerController(APlayerController* PlayerController, bool bSetAsViewTarget)
{
	using namespace UE::Cameras;

	// Make sure we are activated, since we need to tick and udpate our evaluation context and, possibly,
	// our private camera system.
	Super::Activate(false);

	if (!PlayerController && DefaultPlayer != EAutoReceiveInput::Disabled)
	{
		const int32 PlayerIndex = DefaultPlayer.GetIntValue() - 1;
		PlayerController = UGameplayStatics::GetPlayerController(this, PlayerIndex);
	}

	EnsureEvaluationContext(PlayerController);
	EnsureCameraSystemHostIfNeeded();

	if (bSetAsViewTarget)
	{
		if (PlayerController)
		{
			AActor* OwnerActor = GetOwner();
			PlayerController->SetViewTarget(OwnerActor);
		}
		else
		{
			FFrame::KismetExecutionMessage(
				*FString::Format(
					TEXT("Gameplay Camera component '{0}.{1}' cannot set itself as view target because no player controller was provided."),
					{ *GetNameSafe(GetOwner()), *GetNameSafe(this) }),
				ELogVerbosity::Error);
		}
	}
}

bool UGameplayCameraComponentBase::CanRunCameraSystem() const
{
	using namespace UE::Cameras;

#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		const bool bCanRunInEditor = bRunInEditor && GameplayCamerasModule.GetLiveEditManager()->CanRunInEditor();
		return (!bIsEditorWorld || bCanRunInEditor);
	}
	return !bIsEditorWorld;
#else
	return true;
#endif  // WITH_EDITOR
}

bool UGameplayCameraComponentBase::EnsureCameraSystemHostIfNeeded()
{
	using namespace UE::Cameras;

	if (!bRunStandaloneCameraSystem)
	{
		return false;
	}

	if (!CanRunCameraSystem())
	{
		return false;
	}

	// Create a hosted camera system, and then create and active the evalution context.
	const bool bHadCameraSystem = HasCameraSystem();
	if (!bHadCameraSystem)
	{
		AActor* OwnerActor = GetOwner();
		UE_LOGF(LogCameraSystem, Log, 
				"Creating camera system host for gameplay camera '%ls'.",
				*GetNameSafe(OwnerActor));

		FCameraSystemEvaluatorCreateParams Params;
		Params.Owner = this;
#if WITH_EDITOR
		if (bIsEditorWorld)
		{
			Params.Role = ECameraSystemEvaluatorRole::EditorPreview;
		}
#endif  // WITH_EDITOR
		InitializeCameraSystem(Params);
	}

	// We need our evaluation context to run anything.
	if (!ensureMsgf(
				EvaluationContext.IsValid(),
				TEXT("Can't activate Gameplay Camera component '%s.%s': failed to create evaluation context!"),
				*GetNameSafe(GetOwner()), *GetNameSafe(this)))
	{
		return false;
	}

	// If we already had a camera system running, we probably already activated our evaluation context and have 
	// nothing to do. However, if we just created the camera system, then our evaluation context should not be active
	// yet, and we'll activate it now.
	if (!ensureMsgf(
				bHadCameraSystem || !EvaluationContext->IsActive(),
				TEXT("Can't activate Gameplay Camera component '%s.%s': it is already active!"),
				*GetNameSafe(GetOwner()), *GetNameSafe(this)))
	{
		return false;
	}

	TSharedPtr<FCameraSystemEvaluator> HostedEvaluator = GetCameraSystemEvaluator();
	if (ensure(HostedEvaluator.IsValid()))
	{
		if (!EvaluationContext->IsActive())
		{
			UE_LOGF(LogCameraSystem, Log, 
					"Activating gameplay camera '%ls.%ls' with its hosted camera system.",
					*GetNameSafe(GetOwner()), *GetNameSafe(this));

			FCameraEvaluationContextStack& ContextStack = HostedEvaluator->GetEvaluationContextStack();
			ContextStack.PushContext(EvaluationContext.ToSharedRef());
		}

		ensureMsgf(
				EvaluationContext->GetCameraSystemEvaluator() == HostedEvaluator,
				TEXT("Gameplay Camera Component '%s.%s' has a mismatch between evaluation context and camera system!"),
				*GetNameSafe(GetOwner()), *GetNameSafe(this));
	}

	return true;
}

void UGameplayCameraComponentBase::DestroyCameraSystemHost()
{
	// Reset the output camera component and destroy the hosted camera system.
	if (OutputCameraComponent)
	{
		OutputCameraComponent->SetRelativeTransform(FTransform());
	}

	DestroyCameraSystem();
}

void UGameplayCameraComponentBase::EnsureEvaluationContext(APlayerController* PlayerController)
{
	using namespace UE::Cameras;

	if (EvaluationContext.IsValid() && EvaluationContext->GetPlayerController() != PlayerController)
	{
		FFrame::KismetExecutionMessage(
				*FString::Format(
					TEXT("Trying to create an evaluation context for Player Controller '{0}' on Gameplay Camera component '{1}.{2}' "
						 "but it was already created for Player Controller '{3}'. The evaluation context will be re-created."),
					{ 
						*GetNameSafe(PlayerController), 
						*GetNameSafe(GetOwner()),
						*GetNameSafe(this),
						*GetNameSafe(EvaluationContext->GetPlayerController())
					}),
				ELogVerbosity::Warning);

		DestroyEvaluationContext();
	}

	if (!EvaluationContext.IsValid())
	{
		UCameraAsset* CameraAsset = OnCreateEvaluationContext();

		// If we have no camera asset specified, make a placeholder one and log a warning.
		if (!CameraAsset)
		{
			UE_LOGF(LogCameraSystem, Warning, 
					"No camera asset specified on Gameplay Camera component '%ls.%ls', using a placeholder one.",
					*GetNameSafe(this), *GetNameSafe(GetOwner()));

			UCameraRigAsset* PlaceholderCameraRig = NewObject<UCameraRigAsset>();

			USingleCameraDirector* PlaceholderCameraDirector = NewObject<USingleCameraDirector>();
			PlaceholderCameraDirector->CameraRig = PlaceholderCameraRig;

			UCameraAsset* PlaceholderCameraAsset = NewObject<UCameraAsset>();
			PlaceholderCameraAsset->SetCameraDirector(PlaceholderCameraDirector);

			CameraAsset = PlaceholderCameraAsset;
		}

		EvaluationContext = MakeShared<FGameplayCameraComponentEvaluationContext>();

		FCameraEvaluationContextInitializeParams InitParams;
		InitParams.Owner = this;
		InitParams.CameraAsset = CameraAsset;
		InitParams.PlayerController = PlayerController;
		EvaluationContext->Initialize(InitParams);

		const bool bForceApplyParameterOverrides = true;
		UpdateEvaluationContext(bForceApplyParameterOverrides);
	}

	ensure(EvaluationContext.IsValid());
}

void UGameplayCameraComponentBase::DestroyEvaluationContext()
{
	EvaluationContext = nullptr;
}

#define UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE(ErrorMsg)\
	FFrame::KismetExecutionMessage(\
			*FString::Format(\
				TEXT(#ErrorMsg " on Gameplay Camera component '{0}.{1}': it isn't active."),\
				{ *GetNameSafe(GetOwner()), *GetNameSafe(this) }),\
			ELogVerbosity::Error);\

FBlueprintCameraEvaluationDataRef UGameplayCameraComponentBase::GetInitialResult() const
{
	if (EvaluationContext)
	{
		return FBlueprintCameraEvaluationDataRef::MakeExternalRef(&EvaluationContext->GetInitialResult());
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't get shared camera data");
		return FBlueprintCameraEvaluationDataRef();
	}
}

FBlueprintCameraEvaluationDataRef UGameplayCameraComponentBase::GetConditionalResult(ECameraEvaluationDataCondition Condition) const
{
	if (EvaluationContext)
	{
		return FBlueprintCameraEvaluationDataRef::MakeExternalRef(&EvaluationContext->GetOrAddConditionalResult(Condition));
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't get conditional camera data");
		return  FBlueprintCameraEvaluationDataRef();
	}
}

FCameraRigInstanceID UGameplayCameraComponentBase::ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRig)
{
	if (HasCameraSystem() && ensure(EvaluationContext))
	{
		return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, EvaluationContext, ECameraRigLayer::Base);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't activate base camera rig");
		return FCameraRigInstanceID();
	}
}

FCameraRigInstanceID UGameplayCameraComponentBase::ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRig)
{
	if (HasCameraSystem() && ensure(EvaluationContext))
	{
		return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, EvaluationContext, ECameraRigLayer::Global);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't activate global camera rig");
		return FCameraRigInstanceID();
	}
}

FCameraRigInstanceID UGameplayCameraComponentBase::ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRig)
{
	if (HasCameraSystem() && ensure(EvaluationContext))
	{
		return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, EvaluationContext, ECameraRigLayer::Visual);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't activate visual camera rig");
		return FCameraRigInstanceID();
	}
}

void UGameplayCameraComponentBase::DeactivateCameraRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	if (HasCameraSystem())
	{
		IGameplayCameraSystemHost::DeactivateCameraRig(InstanceID, bImmediately);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't deactivate camera rig");
	}
}

FCameraRigInstanceID UGameplayCameraComponentBase::StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	if (HasCameraSystem() && ensure(EvaluationContext))
	{
		return IGameplayCameraSystemHost::StartCameraModifierRig(CameraRig, EvaluationContext, ECameraRigLayer::Global, OrderKey);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't start global camera modifier rig");
		return FCameraRigInstanceID();
	}
}

FCameraRigInstanceID UGameplayCameraComponentBase::StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	if (HasCameraSystem() && ensure(EvaluationContext))
	{
		return IGameplayCameraSystemHost::StartCameraModifierRig(CameraRig, EvaluationContext, ECameraRigLayer::Visual, OrderKey);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't start visual camera modifier rig");
		return FCameraRigInstanceID();
	}
}

void UGameplayCameraComponentBase::StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	if (HasCameraSystem())
	{
		IGameplayCameraSystemHost::StopCameraModifierRig(InstanceID, bImmediately);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't stop camera modifier rig");
	}
}

FCameraShakeInstanceID UGameplayCameraComponentBase::StartCameraShakeAsset(const UCameraShakeAsset* CameraShake, float ShakeScale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRotation)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::StartCameraShake(CameraShake, ShakeScale, PlaySpace, UserPlaySpaceRotation);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't start camera shake");
		return FCameraShakeInstanceID();
	}
}

bool UGameplayCameraComponentBase::IsCameraShakeAssetPlaying(FCameraShakeInstanceID InInstanceID) const
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::IsCameraShakePlaying(InInstanceID);
	}
	return false;
}

bool UGameplayCameraComponentBase::StopCameraShakeAsset(FCameraShakeInstanceID InInstanceID, bool bImmediately)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::StopCameraShake(InInstanceID, bImmediately);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't stop camera shake");
		return false;
	}
}

FCameraActionInstanceID UGameplayCameraComponentBase::StartAction(const UCameraAction* CameraAction)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::StartAction(CameraAction);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't start action");
		return FCameraActionInstanceID();
	}
}

bool UGameplayCameraComponentBase::IsActionRunning(FCameraActionInstanceID InInstanceID)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::IsActionRunning(InInstanceID);
	}
	return false;
}

bool UGameplayCameraComponentBase::StopAction(FCameraActionInstanceID InInstanceID)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::StopAction(InInstanceID);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't stop action");
		return false;
	}
}

bool UGameplayCameraComponentBase::StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass)
{
	if (HasCameraSystem())
	{
		return IGameplayCameraSystemHost::StopAllActionsOfClass(InActionClass);
	}
	else
	{
		UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_ERROR_MESSAGE("Can't stop actions");
		return false;
	}
}

#undef UE_PRIVATE_GAMEPLAY_CAMERA_COMPONENT_VALIDATE_EVALUATION_CONTEXT

FRotator UGameplayCameraComponentBase::GetEvaluatedCameraRotation() const
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator.IsValid())
	{
		const FCameraSystemEvaluationResult& Result = CameraSystemEvaluator->GetEvaluatedResult();
		return Result.CameraPose.GetRotation();
	}
	return FRotator();
}

void UGameplayCameraComponentBase::OnRegister()
{
	using namespace UE::Cameras;

	Super::OnRegister();

#if WITH_EDITOR

	UWorld* World = GetWorld();
	bIsEditorWorld = (World && (World->WorldType == EWorldType::Editor || World->WorldType == EWorldType::EditorPreview));

	const TCHAR* ShowFlagName = TEXT("GameplayCameras");
	CustomShowFlag = FEngineShowFlags::FindIndexByName(ShowFlagName);

#endif  // WITH_EDITOR
}

#if WITH_EDITOR

void UGameplayCameraComponentBase::CreateCameraSpriteComponent(const FString& SpriteTexturePath)
{
	UTexture2D* EditorSpriteTexture = nullptr;
	{
		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		EditorSpriteTexture = LoadObject<UTexture2D>(nullptr, SpriteTexturePath);
	}

	if (EditorSpriteTexture)
	{
		bVisualizeComponent = true;
		CreateSpriteComponent(EditorSpriteTexture);
	}

	if (SpriteComponent)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Cameras");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Cameras", "Cameras");
	}
}

#endif  // WITH_EDITOR

void UGameplayCameraComponentBase::BeginPlay()
{
	using namespace UE::Cameras;

	Super::BeginPlay();

	// If we have been activated in OnRegister() (which happens when bAutoActivate is true), our code 
	// inside Activate() has postponed setting up the camera system evaluation until now, so let's
	// do it.
	// However, it can happen that some BP construction script already called ActivateCameraForXyz()
	// before we got to start play (e.g. from a parent actor) and so in this case, let's skip
	// re-activating for nothing.
	if (IsActive() && !EvaluationContext && GetNetMode() != NM_DedicatedServer)
	{
		ActivateCameraForPlayerController(nullptr, false);
	}
}

void UGameplayCameraComponentBase::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	DestroyCameraSystemHost();
	DestroyEvaluationContext();

	Super::EndPlay(EndPlayReason);
}

void UGameplayCameraComponentBase::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (OutputCameraComponent && bIsOwnedOutputCameraComponent)
	{
		OutputCameraComponent->DestroyComponent();
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UGameplayCameraComponentBase::Activate(bool bReset)
{
	// When auto-activing, this method gets called during OnRegister, before we have started playing.
	// In this case, we don't activate the camera right away, we wait until BeginPlay.
	const bool bDoActivate = (bReset || ShouldActivate()) && HasBegunPlay();

	Super::Activate(bReset);

	if (bDoActivate)
	{
		EnsureEvaluationContext(nullptr);
		EnsureCameraSystemHostIfNeeded();

		if (OutputCameraComponent && bIsOwnedOutputCameraComponent)
		{
			OutputCameraComponent->Activate(bReset);
		}
	}
}

void UGameplayCameraComponentBase::Deactivate()
{
	DestroyCameraSystemHost();
	DestroyEvaluationContext();

	if (OutputCameraComponent && bIsOwnedOutputCameraComponent)
	{
		OutputCameraComponent->Deactivate();
	}

	Super::Deactivate();
}

void UGameplayCameraComponentBase::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	using namespace UE::Cameras;

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

#if WITH_EDITOR
	// Make sure things are setup (or not) if we want to run the camera logic in editor (or not).
	AutoManageEditorPreviewEvaluator();
#endif  // WITH_EDITOR

	UpdateEvaluationContext(false);

#if WITH_EDITOR
	if (bIsEditorWorld)
	{
		UpdateCameraSystemForEditorPreview(DeltaTime);
	}
	else if (bRunStandaloneCameraSystem)
	{
		UpdateCameraSystem(DeltaTime);
		UpdateControlRotationIfNeeded();
	}
#else
	if (bRunStandaloneCameraSystem)
	{
		UpdateCameraSystem(DeltaTime);
		UpdateControlRotationIfNeeded();
	}
#endif  // WITH_EDITOR

	UpdateOutputCameraComponent();

#if WITH_EDITOR
	UpdateCameraSpriteComponent();
#endif  // WITH_EDITOR
}

void UGameplayCameraComponentBase::UpdateEvaluationContext(bool bForceApplyParameterOverrides)
{
	using namespace UE::Cameras;

	if (EvaluationContext)
	{
		FCameraNodeEvaluationResult& InitialResult = EvaluationContext->GetInitialResult();

		const FTransform& OwnerTransform = GetComponentTransform();
		InitialResult.CameraPose.SetTransform(OwnerTransform, true);
		InitialResult.bIsCameraCut = false;
		InitialResult.bIsValid = true;

		if (bIsCameraCutNextFrame)
		{
			InitialResult.bIsCameraCut = true;
			bIsCameraCutNextFrame = false;
		}

		OnUpdateEvaluationContext(bForceApplyParameterOverrides);

#if WITH_EDITOR
		EvaluationContext->UpdateForEditorPreview();
#endif  // WITH_EDITOR
	}
}

void UGameplayCameraComponentBase::GetChangedParameterOverrides(
		const FInstancedPropertyBag& InParameterOverrides,
		FInstancedPropertyBag& InOutCachedParameterOverrides,
		TArray<FGuid>& OutChangedParameterGuids)
{
	const UPropertyBag* ParameterStruct = InParameterOverrides.GetPropertyBagStruct();
	const UPropertyBag* CachedParameterStruct = InOutCachedParameterOverrides.GetPropertyBagStruct();
	const uint8* ParametersContainer = InParameterOverrides.GetValue().GetMemory();
	const uint8* CachedParameterContainer = InOutCachedParameterOverrides.GetValue().GetMemory();
	if (!ParameterStruct || !ParametersContainer)
	{
		InOutCachedParameterOverrides.Reset();
		return;
	}
	if (!CachedParameterStruct || CachedParameterStruct != ParameterStruct || !CachedParameterContainer)
	{
		for (const FPropertyBagPropertyDesc& PropertyDesc : ParameterStruct->GetPropertyDescs())
		{
			OutChangedParameterGuids.Add(PropertyDesc.ID);
		}
		InOutCachedParameterOverrides = InParameterOverrides;
		return;
	}

	for (const FPropertyBagPropertyDesc& PropertyDesc : ParameterStruct->GetPropertyDescs())
	{
		if (!PropertyDesc.CachedProperty)
		{
			continue;
		}

		const void* ValuePtr = PropertyDesc.CachedProperty->ContainerPtrToValuePtr<void>(ParametersContainer);
		const void* CachedValuePtr = PropertyDesc.CachedProperty->ContainerPtrToValuePtr<void>(CachedParameterContainer);
		if (!ValuePtr || !CachedValuePtr)
		{
			continue;
		}

		if (!PropertyDesc.CachedProperty->Identical(ValuePtr, CachedValuePtr))
		{
			OutChangedParameterGuids.Add(PropertyDesc.ID);
		}
	}
}

void UGameplayCameraComponentBase::UpdateControlRotationIfNeeded()
{
	using namespace UE::Cameras;

	if (!HasCameraSystem() || !HasEvaluationContext() || !bSetControlRotationWhenViewTarget)
	{
		return;
	}

	APlayerController* PlayerController = EvaluationContext->GetPlayerController();
	if (!PlayerController)
	{
		return;
	}

	// If the player camera manager is hosting a camera system, it probably already handles control
	// rotation in its own way.
	if (IGameplayCameraSystemHost* CameraManagerHost = Cast<IGameplayCameraSystemHost>(PlayerController->PlayerCameraManager))
	{
		return;
	}

	// Set control rotation if we are the view target.
	AActor* OwnerActor = GetOwner();
	if (CameraSystemEvaluator && OwnerActor && PlayerController->GetViewTarget() == OwnerActor)
	{
		const FCameraSystemEvaluationResult& Result = CameraSystemEvaluator->GetPreVisualLayerEvaluatedResult();
		const FRotator3d& ControlRotation = Result.CameraPose.GetRotation();
		PlayerController->SetControlRotation(ControlRotation);
	}
}

bool UGameplayCameraComponentBase::IsEditorWorld() const
{
#if WITH_EDITOR
	return bIsEditorWorld;
#else
	return false;
#endif  // WITH_EDITOR
}

#if WITH_EDITOR

void UGameplayCameraComponentBase::ReinitializeEvaluationContext(
			const FCameraVariableTableAllocationInfo& VariableTableAllocationInfo,
			const FCameraContextDataTableAllocationInfo& ContextDataTableAllocationInfo)
{
	using namespace UE::Cameras;

	if (EvaluationContext)
	{
		FCameraNodeEvaluationResult& InitialResult = EvaluationContext->GetInitialResult();
		InitialResult.VariableTable.Initialize(VariableTableAllocationInfo);
		InitialResult.ContextDataTable.Initialize(ContextDataTableAllocationInfo);

		// Also freeze/remove any of our currently running camera rigs, because they might continue
		// accessing variables and data that don't exist anymore.
		if (TSharedPtr<FCameraSystemEvaluator> HostEvaluator = EvaluationContext->GetCameraSystemEvaluator())
		{
			FRootCameraNodeEvaluator* RootEvaluator = HostEvaluator->GetRootNodeEvaluator();
			RootEvaluator->DeactivateAllCameraRigs(EvaluationContext, true);
		}
	}
}

void UGameplayCameraComponentBase::RecreateEditorWorldEvaluationContext()
{
	using namespace UE::Cameras;

	if (!bIsEditorWorld)
	{
		return;
	}

	// We should only be calling this method to recreate the editor preview evaluator, so check that
	// this is indeed the case.
	if (EvaluationContext && CameraSystemEvaluator)
	{
		ensure(CameraSystemEvaluator->GetRole() == ECameraSystemEvaluatorRole::EditorPreview);

		FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
		TArray<TSharedPtr<FCameraEvaluationContext>> AllContexts;
		ContextStack.GetAllContexts(AllContexts);
		ensure(AllContexts.Num() == 1 && AllContexts[0] == EvaluationContext);
	}

	// Teardown and rebuild the main evaluation context.
	if (EvaluationContext)
	{
		if (CameraSystemEvaluator)
		{
			FRootCameraNodeEvaluator* RootEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
			RootEvaluator->DeactivateAllCameraRigs(EvaluationContext.ToSharedRef(), true);
			CameraSystemEvaluator->RemoveEvaluationContext(EvaluationContext.ToSharedRef());
		}
		EvaluationContext = nullptr;

		EnsureEvaluationContext(nullptr);
		if (CameraSystemEvaluator)
		{
			CameraSystemEvaluator->PushEvaluationContext(EvaluationContext.ToSharedRef());
		}
	}
}

#endif  // WITH_EDITOR

void UGameplayCameraComponentBase::EnsureOutputCameraComponent()
{
	if (OutputCameraComponent)
	{
		return;
	}

	// If our parent actor has put a camera component under us, we can use it. This is useful for making that camera
	// component "visible" to the user in the editor.
	for (USceneComponent* AttachChild : GetAttachChildren())
	{
		if (UCineCameraComponent* ChildCamera = Cast<UCineCameraComponent>(AttachChild))
		{
			OutputCameraComponent = ChildCamera;
			bIsOwnedOutputCameraComponent = false;
			break;
		}
	}

	// If we didn't find any child camera component to use, we need to create one ourselves. A component cannot create 
	// an editor-friendly child component that is visible, so we leave that one hidden (i.e. held in a non-visible 
	// property)
	if (OutputCameraComponent == nullptr)
	{
		OutputCameraComponent = NewObject<UCineCameraComponent>(this, TEXT("OutputCameraComponent"), RF_Transient);
		OutputCameraComponent->SetAutoActivate(false);
		OutputCameraComponent->SetupAttachment(this);
		OutputCameraComponent->CreationMethod = CreationMethod;
		OutputCameraComponent->RegisterComponentWithWorld(GetWorld());

		bIsOwnedOutputCameraComponent = true;
	}

	// Make sure the component is active, otherwise it won't be picked up as a valid view by the engine.
	if (ensure(OutputCameraComponent))
	{
		OutputCameraComponent->Activate();
	}
}

void UGameplayCameraComponentBase::UpdateOutputCameraComponent()
{
	using namespace UE::Cameras;

	// Make sure we have a camera component to write to.
	//
	// Ideally this would be done once in OnRegister() but we don't know at that time about our attached children.
	// The next best place would be in BeginPlay(), but that's not called in editor contexts.
	// So we lazily do it here instead.
	EnsureOutputCameraComponent();

	if (!OutputCameraComponent || bPlaybackMode)
	{
		return;
	}

	bool bGotValidTransform = false;
	if (CameraSystemEvaluator)
	{
		FRootCameraNodeEvaluator* RootNodeEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
		if (RootNodeEvaluator && RootNodeEvaluator->HasAnyActiveCameraRig())
		{
			const FCameraSystemEvaluationResult& Result = CameraSystemEvaluator->GetEvaluatedResult();

			OutputCameraComponent->SetWorldTransform(Result.CameraPose.GetTransform());
			OutputCameraComponent->SetFieldOfView(Result.CameraPose.GetEffectiveFieldOfView());
			OutputCameraComponent->CurrentAperture = Result.CameraPose.GetAperture();

			OutputCameraComponent->Filmback.SensorWidth = Result.CameraPose.GetSensorWidth();
			OutputCameraComponent->Filmback.SensorHeight = Result.CameraPose.GetSensorHeight();
			OutputCameraComponent->Filmback.SensorHorizontalOffset = Result.CameraPose.GetSensorHorizontalOffset();
			OutputCameraComponent->Filmback.SensorVerticalOffset = Result.CameraPose.GetSensorVerticalOffset();

			OutputCameraComponent->Overscan = Result.CameraPose.GetOverscan();
			OutputCameraComponent->bConstrainAspectRatio = Result.CameraPose.GetConstrainAspectRatio();
			OutputCameraComponent->bOverrideAspectRatioAxisConstraint = Result.CameraPose.GetOverrideAspectRatioAxisConstraint();
			OutputCameraComponent->AspectRatioAxisConstraint = Result.CameraPose.GetAspectRatioAxisConstraint();

			OutputCameraComponent->FocusSettings.ManualFocusDistance = Result.CameraPose.GetFocusDistance();
			OutputCameraComponent->FocusSettings.FocusMethod = (
					(Result.CameraPose.GetEnablePhysicalCamera() && Result.CameraPose.GetFocusDistance() > 0.f) ? 
					ECameraFocusMethod::Manual : 
					ECameraFocusMethod::DoNotOverride);

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
			OutputCameraComponent->ExposureMethod = (Result.CameraPose.GetEnablePhysicalCamera() ? ECameraExposureMethod::Enabled : ECameraExposureMethod::DoNotOverride);
#endif

			OutputCameraComponent->ProjectionMode = Result.CameraPose.GetProjectionMode();
			OutputCameraComponent->OrthoWidth = Result.CameraPose.GetOrthographicWidth();

			OutputCameraComponent->PostProcessSettings = Result.PostProcessSettings.Get();
			OutputCameraComponent->PostProcessBlendWeight = 1.f;

			if (Result.CameraPose.GetEnableFirstPerson())
			{
				const float FirstPersonFieldOfView = Result.CameraPose.GetFirstPersonFieldOfView();
				const float FirstPersonScale = Result.CameraPose.GetFirstPersonScale();

				OutputCameraComponent->bEnableFirstPersonFieldOfView = (FirstPersonFieldOfView > 0.f);
				OutputCameraComponent->bEnableFirstPersonScale = (FirstPersonScale > 0.f);
				OutputCameraComponent->FirstPersonFieldOfView = (
						FirstPersonFieldOfView > 0.f ? FirstPersonFieldOfView : OutputCameraComponent->FieldOfView);
				OutputCameraComponent->FirstPersonScale = (
						FirstPersonScale > 0.f ? FirstPersonScale : 1.f);
			}
			else
			{
				OutputCameraComponent->bEnableFirstPersonFieldOfView = false;
				OutputCameraComponent->bEnableFirstPersonScale = false;
				OutputCameraComponent->FirstPersonFieldOfView = OutputCameraComponent->FieldOfView;
				OutputCameraComponent->FirstPersonScale = 1.f;
			}
			
			bGotValidTransform = true;
		}
	}
	
	if (!bGotValidTransform)
	{
		OutputCameraComponent->SetRelativeTransform(FTransform());
	}
}

#if WITH_EDITOR

void UGameplayCameraComponentBase::UpdateCameraSpriteComponent()
{
	if (SpriteComponent)
	{
		if (OutputCameraComponent)
		{
			const FVector3d RootLocation = GetComponentLocation();
			const FVector3d OutputLocation = OutputCameraComponent->GetComponentLocation();
			const double RootToOutputDistance = FVector3d::Distance(RootLocation, OutputLocation);

			const bool bShouldBeVisible = (RootToOutputDistance > EditorSpriteHiddenWhenOutputCameraWithinDistance);
			SpriteComponent->SetIsTemporarilyHiddenInEditor(!bShouldBeVisible);
		}
		else
		{
			SpriteComponent->SetIsTemporarilyHiddenInEditor(false);
		}
	}
}

#endif  // WITH_EDITOR

void UGameplayCameraComponentBase::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	if (EvaluationContext && Teleport != ETeleportType::None)
	{
		bIsCameraCutNextFrame = true;
	}

#if WITH_EDITOR

	if (bIsEditorWorld && EvaluationContext)
	{
		UpdateEvaluationContext(false);
	}

#endif  // WITH_EDITOR
}

#if WITH_EDITOR

void UGameplayCameraComponentBase::AutoManageEditorPreviewEvaluator()
{
	using namespace UE::Cameras;

	if (!bIsEditorWorld)
	{
		return;
	}
	
	const bool bCanRun = CanRunCameraSystem();
	if (bCanRun && !(CameraSystemEvaluator && EvaluationContext))
	{
		// We want to run the camera logic in the editor but we haven't set things up for that.
		// Let's create the preview evaluator and the evaluation context.
		EnsureEvaluationContext(nullptr);
		EnsureCameraSystemHostIfNeeded();

		if (ensure(EvaluationContext))
		{
			EvaluationContext->SetEditorPreviewCameraRigIndex(EditorPreviewCameraRigIndex);
		}

		// OutputCameraComponent will be updated on the next tick.
	}
	else if (!bCanRun && (CameraSystemEvaluator || EvaluationContext))
	{
		// We don't want to run the camera logic in the editor anymore. Let's tear things down.
		DestroyCameraSystemHost();
		DestroyEvaluationContext();
	}
}

void UGameplayCameraComponentBase::OnEditorPreviewCameraRigIndexChanged()
{
	if (bIsEditorWorld)
	{
		const bool bCanRun = CanRunCameraSystem();
		if (bCanRun && CameraSystemEvaluator && EvaluationContext)
		{
			EvaluationContext->SetEditorPreviewCameraRigIndex(EditorPreviewCameraRigIndex);
		}
	}
}

bool UGameplayCameraComponentBase::GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut)
{
	if (OutputCameraComponent)
	{
		OutputCameraComponent->GetEditorPreviewInfo(DeltaTime, ViewOut);
		return true;
	}
	return false;
}

void UGameplayCameraComponentBase::OnDrawVisualizationHUD(const FViewport* Viewport, const FSceneView* SceneView, FCanvas* Canvas) const
{
	using namespace UE::Cameras;

	const bool bCanRun = CanRunCameraSystem();
	const bool bHasShowFlag = SceneView->Family->EngineShowFlags.GetSingleFlag(CustomShowFlag);
	if (bCanRun && bHasShowFlag && CameraSystemEvaluator && EvaluationContext)
	{
		const AActor* OwnerActor = GetOwner();

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		const AActor* ViewActor = SceneView->ViewActor.Get();
#else
		const AActor* ViewActor = SceneView->ViewActor;
#endif
		const bool bIsLockedToCamera = (ViewActor == OwnerActor);

		FCameraSystemEditorPreviewParams Params;
		Params.Canvas = Canvas;
		Params.SceneView = SceneView;
		Params.bIsLockedToCamera = bIsLockedToCamera;
		Params.bDrawWorldDebug = false;

		CameraSystemEvaluator->DrawEditorPreview(Params);
	}
}

void UGameplayCameraComponentBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::Cameras;

	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UGameplayCameraComponentBase, bRunInEditor))
	{
		AutoManageEditorPreviewEvaluator();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UGameplayCameraComponentBase, EditorPreviewCameraRigIndex))
	{
		OnEditorPreviewCameraRigIndexChanged();
	}
}

#endif  // WITH_EDITOR

namespace UE::Cameras
{

UE_DEFINE_CAMERA_EVALUATION_CONTEXT(FGameplayCameraComponentEvaluationContext)

#if WITH_EDITOR

void FGameplayCameraComponentEvaluationContext::UpdateForEditorPreview()
{
	FCameraSystemEvaluator* ActiveEvaluator = GetPrivateCameraSystemEvaluator();
	if (ActiveEvaluator && ActiveEvaluator->GetRole() == ECameraSystemEvaluatorRole::EditorPreview)
	{
		if (GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->Viewport)
		{
			FIntPoint ViewportSize = GCurrentLevelEditingViewportClient->Viewport->GetSizeXY();
			OverrideViewportSize = ViewportSize;
		}
		else
		{
			OverrideViewportSize.Reset();
		}
	}
}

#endif  // WITH_EDITOR

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

