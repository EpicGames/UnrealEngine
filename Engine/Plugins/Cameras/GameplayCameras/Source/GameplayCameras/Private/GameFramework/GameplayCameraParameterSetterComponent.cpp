// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraParameterSetterComponent.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraVariableSetter.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameFramework/IGameplayCameraSystemHost.h"
#include "Kismet/GameplayStatics.h"
#include "Services/CameraParameterSetterService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraParameterSetterComponent)

UGameplayCameraParameterSetterComponent::UGameplayCameraParameterSetterComponent(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UGameplayCameraParameterSetterComponent::BeginPlay()
{
	Super::BeginPlay();

	if (AActor* OwnerActor = GetOwner())
	{
		OwnerActor->OnActorBeginOverlap.AddDynamic(this, &UGameplayCameraParameterSetterComponent::OnActorBeginOverlap);
		OwnerActor->OnActorEndOverlap.AddDynamic(this, &UGameplayCameraParameterSetterComponent::OnActorEndOverlap);
	}
}

void UGameplayCameraParameterSetterComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (AActor* OwnerActor = GetOwner())
	{
		OwnerActor->OnActorBeginOverlap.RemoveAll(this);
		OwnerActor->OnActorEndOverlap.RemoveAll(this);
	}

	Super::EndPlay(EndPlayReason);
}

void UGameplayCameraParameterSetterComponent::OnActorBeginOverlap(AActor* OverlappedActor, AActor* OtherActor)
{
	StartParameterSetters();
}

void UGameplayCameraParameterSetterComponent::OnActorEndOverlap(AActor* OverlappedActor, AActor* OtherActor)
{
	StopParameterSetters(false);
}

TSharedPtr<UE::Cameras::FCameraParameterSetterService> UGameplayCameraParameterSetterComponent::GetParameterSetterService()
{
	using namespace UE::Cameras;

	// For now we only support one local player.
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(this, 0);
	if (!PlayerController)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't set camera parameters: no player controller found!");
		return nullptr;
	}

	IGameplayCameraSystemHost* CameraSystemHost = IGameplayCameraSystemHost::FindActiveHost(PlayerController);
	if (!CameraSystemHost)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't set camera parameters: no camera system found under the player controller!");
		return nullptr;
	}

	TSharedPtr<FCameraSystemEvaluator> SystemEvaluator = CameraSystemHost->GetCameraSystemEvaluator();
	if (!SystemEvaluator)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't set camera parameters: no camera system is active!");
		return nullptr;
	}

	return SystemEvaluator->FindEvaluationService<FCameraParameterSetterService>();
}

void UGameplayCameraParameterSetterComponent::StartParameterSetters()
{
	using namespace UE::Cameras;

	if (!CameraRigReference.GetCameraRig())
	{
		return;
	}

	TSharedPtr<FCameraParameterSetterService> ParameterSetterService = GetParameterSetterService();
	if (!ParameterSetterService)
	{
		return;
	}

	const UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig();
	const FInstancedOverridablePropertyBag& ParameterValues = CameraRigReference.GetParameters();
	const uint8* ParameterValuesPtr = ParameterValues.GetValue().GetMemory();

	for (const FCameraObjectInterfaceParameterDefinition& ParameterDefinition : CameraRig->GetParameterDefinitions())
	{
		if (ParameterDefinition.ParameterType != ECameraObjectInterfaceParameterType::Blendable)
		{
			continue;
		}

		if (!ParameterValues.IsPropertyOverriden(ParameterDefinition.ParameterGuid))
		{
			continue;
		}

		const FPropertyBagPropertyDesc* PropertyDesc = ParameterValues.FindPropertyDescByID(ParameterDefinition.ParameterGuid);
		if (!PropertyDesc || !PropertyDesc->CachedProperty)
		{
			continue;
		}

		FCameraVariableSetterHandle SetterHandle;
		const void* RawValue = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(ParameterValuesPtr);

		switch (ParameterDefinition.VariableType)
		{
#define UE_CAMERA_VARIABLE_FOR_TYPE(VariableType, VariableName)\
			case ECameraVariableType::VariableName:\
				{\
					TCameraVariableSetter<VariableType> Setter(\
							ParameterDefinition.VariableID, *reinterpret_cast<const VariableType*>(RawValue));\
					InitializeParameterSetter(Setter);\
					SetterHandle = ParameterSetterService->AddCameraVariableSetter(Setter);\
				}\
				break;
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
			default:
				break;
		}

		if (SetterHandle.IsValid())
		{
			SetterHandles.Add(SetterHandle);
		}
	}
}

void UGameplayCameraParameterSetterComponent::InitializeParameterSetter(UE::Cameras::FCameraVariableSetter& VariableSetter)
{
	VariableSetter.BlendInTime = BlendInTime;
	VariableSetter.BlendOutTime = BlendOutTime;
	VariableSetter.BlendType = BlendType;
}

void UGameplayCameraParameterSetterComponent::StopParameterSetters(bool bImmediately)
{
	using namespace UE::Cameras;

	TSharedPtr<FCameraParameterSetterService> ParameterSetterService = GetParameterSetterService();
	if (!ParameterSetterService)
	{
		return;
	}

	for (FCameraVariableSetterHandle Handle : SetterHandles)
	{
		ParameterSetterService->StopCameraVariableSetter(Handle, bImmediately);
	}
}

