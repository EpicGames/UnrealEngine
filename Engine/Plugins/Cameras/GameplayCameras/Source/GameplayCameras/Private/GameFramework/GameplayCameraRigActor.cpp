// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraRigActor.h"

#include "CineCameraComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/GameplayCameraRigComponent.h"
#include "Kismet/GameplayStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraRigActor)

#define LOCTEXT_NAMESPACE "GameplayCameraRigActor"

AGameplayCameraRigActor::AGameplayCameraRigActor(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CameraRigComponent = CreateDefaultSubobject<UGameplayCameraRigComponent>(TEXT("CameraRigComponent"));
	RootComponent = CameraRigComponent;

	OutputCameraComponent = CreateDefaultSubobject<UCineCameraComponent>(TEXT("OutputCameraComponent"), true);
	OutputCameraComponent->SetupAttachment(CameraRigComponent);
}

USceneComponent* AGameplayCameraRigActor::GetDefaultAttachComponent() const
{
	return CameraRigComponent;
}

UGameplayCameraComponentBase* AGameplayCameraRigActor::GetCameraComponentBase() const
{
	return CameraRigComponent;
}

#undef LOCTEXT_NAMESPACE

