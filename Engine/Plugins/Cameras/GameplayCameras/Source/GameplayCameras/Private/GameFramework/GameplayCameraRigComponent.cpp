// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraRigComponent.h"

#include "Build/CameraAssetBuilder.h"
#include "Build/CameraBuildContext.h"
#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Directors/SingleCameraDirector.h"
#include "GameFramework/Actor.h"  // IWYU pragma: keep
#include "GameplayCamerasDelegates.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/EngineVersionComparison.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraRigComponent)

UGameplayCameraRigComponent::UGameplayCameraRigComponent(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UGameplayCameraRigComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		// Make sure parameters are up to date when cooking.
		const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
		if (TargetPlatform && TargetPlatform->RequiresCookedData())
		{
			CameraRigReference.RebuildParametersIfNeeded();
		}
	}
#endif  // WITH_EDITOR

	Super::PreSave(ObjectSaveContext);
}

void UGameplayCameraRigComponent::OnRegister()
{
	using namespace UE::Cameras;

	Super::OnRegister();

#if WITH_EDITOR
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().AddUObject(this, &UGameplayCameraRigComponent::OnCameraRigAssetBuilt);

	Super::CreateCameraSpriteComponent(TEXT("/GameplayCameras/Textures/S_GameplayCameraRig.S_GameplayCameraRig"));
#endif  // WITH_EDITOR
}

void UGameplayCameraRigComponent::OnUnregister()
{
	using namespace UE::Cameras;

#if WITH_EDITOR
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().RemoveAll(this);
#endif  // WITH_EDITOR

	Super::OnUnregister();
}

UCameraAsset* UGameplayCameraRigComponent::OnCreateEvaluationContext()
{
#if WITH_EDITOR
	CameraRigReference.RebuildParametersIfNeeded();
#endif  // WITH_EDITOR

	if (!GeneratedCameraAsset && CameraRigReference.GetCameraRig())
	{
		USingleCameraDirector* SingleDirector = NewObject<USingleCameraDirector>(this, TEXT("GeneratedCameraDirector"), RF_Transient);
		SingleDirector->CameraRig = CameraRigReference.GetCameraRig();

		GeneratedCameraAsset = NewObject<UCameraAsset>(this, TEXT("GeneratedCameraAsset"), RF_Transient);
		GeneratedCameraAsset->SetCameraDirector(SingleDirector);

		ensure(!bIsBuildingGeneratedCameraAsset);
		BuildGeneratedCamera();
		OnCameraRigAssetBuiltImpl();
	}

	return GeneratedCameraAsset;
}

void UGameplayCameraRigComponent::BuildGeneratedCamera()
{
	using namespace UE::Cameras;

	if (GeneratedCameraAsset)
	{
		TGuardValue<bool> ReentrancyGuard(bIsBuildingGeneratedCameraAsset, true);

		// Build only the camera asset, not the referenced camera rigs... there's only one camera rig
		// in this camera (ours) and it was just built, that's why we're in this callback in the
		// first place.
		const bool bBuildReferencedAssets = false;

		FCameraBuildLog BuildLog;
		BuildLog.SetForwardMessagesToLogging(true);

		// Build the asset as if we were cooking so we get the referenced camera rig's allocation info
		// accumulated into ours. It's more efficient and more importantly it lets us initialize the
		// evaluation context's variable and context data tables with all the entries we need for later
		// calling CameraRigReference.EnsureAllocationInfo().
		FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::Cooking);
		FCameraAssetBuilder Builder(BuildContext);
		Builder.BuildCamera(GeneratedCameraAsset, bBuildReferencedAssets);
	}
}

void UGameplayCameraRigComponent::OnCameraRigAssetBuiltImpl()
{
	using namespace UE::Cameras;

	CameraRigReference.RebuildParametersIfNeeded();

	if (HasEvaluationContext())
	{
#if WITH_EDITOR
		const UCameraRigAsset* CameraRigAsset = CameraRigReference.GetCameraRig();
		const FCameraObjectAllocationInfo& AllocationInfo = CameraRigAsset->AllocationInfo;
		ReinitializeEvaluationContext(AllocationInfo.VariableTableInfo, AllocationInfo.ContextDataTableInfo);
#endif  // WITH_EDITOR

		UpdateEvaluationContext(true);
	}
}

void UGameplayCameraRigComponent::OnUpdateEvaluationContext(bool bForceApplyParameterOverrides)
{
	using namespace UE::Cameras;

	FCameraNodeEvaluationResult& InitialResult = GetEvaluationContext()->GetInitialResult();

	if (bForceApplyParameterOverrides)
	{
		CameraRigReference.ApplyParameterOverrides(InitialResult, false);
	}
#if UE_VERSION_OLDER_THAN(5,7,0)
	// Before 5.7.0, we don't have a notify callback from Sequencer to know that it's animating parameters,
	// so we need to always re-apply values.
	else
	{
		ApplyChangedParameterOverrides();
	}
#endif  // pre-5.7.0
}

void UGameplayCameraRigComponent::NotifyChangeCameraRigReference()
{
	// Sequencer animated some of our parameters... look for those whose value changed, compared to our
	// cached parameter bag, and re-apply them to the evaluation context.
	// TODO: This isn't a very efficient process, but it's unclear how to reconcile the reference's parameters struct
	//		 with the evaluation context's variable/context-data tables.
	ApplyChangedParameterOverrides();
}

void UGameplayCameraRigComponent::ApplyChangedParameterOverrides()
{
	using namespace UE::Cameras;

	const UCameraRigAsset* CameraRigAsset = CameraRigReference.GetCameraRig();
	if (CameraRigAsset && HasEvaluationContext())
	{
		FCameraNodeEvaluationResult& InitialResult = GetEvaluationContext()->GetInitialResult();

		TArray<FGuid> ChangedParameterGuids;
		FCameraObjectInterfaceParameterOverrideHelper Helper(InitialResult);
		GetChangedParameterOverrides(CameraRigReference.GetParameters(), CachedParameterOverrides, ChangedParameterGuids);
		Helper.ApplyFilteredParameters(
				CameraRigAsset,
				CameraRigAsset->GetParameterDefinitions(),
				CameraRigReference.GetParameters(),
				[ChangedParameterGuids](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
				{
					return ChangedParameterGuids.Contains(Definition.ParameterGuid);
				});

		CachedParameterOverrides = CameraRigReference.GetParameters();
	}
}

#if WITH_EDITOR

void UGameplayCameraRigComponent::OnCameraRigAssetBuilt(const UCameraRigAsset* InCameraRigAsset)
{
	using namespace UE::Cameras;

	if (InCameraRigAsset != CameraRigReference.GetCameraRig() || bIsBuildingGeneratedCameraAsset)
	{
		return;
	}

	// If our camera rig asset was just built, it may have some new parameters. We need to rebuild
	// our variable table and context data table, and re-apply overrides.
	BuildGeneratedCamera();
	
	OnCameraRigAssetBuiltImpl();
}

void UGameplayCameraRigComponent::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::Cameras;

	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UGameplayCameraRigComponent, CameraRigReference))
	{
		if (HasEvaluationContext())
		{
			if (PropertyChangedEvent.GetPropertyName() == TEXT("CameraRig"))
			{
				// The camera rig asset has changed! Recreate the context.
				GeneratedCameraAsset = nullptr;
				RecreateEditorWorldEvaluationContext();
			}
			else
			{
				// Otherwise, maybe one of the parameter overrides has changed. Re-apply them.
				UpdateEvaluationContext(true);
			}
		}
	}
}

#endif  // WITH_EDITOR

