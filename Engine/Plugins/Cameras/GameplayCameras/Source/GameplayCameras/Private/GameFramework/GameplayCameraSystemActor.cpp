// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraSystemActor.h"

#include "GameFramework/GameplayCameraSystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraSystemActor)

#define LOCTEXT_NAMESPACE "GameplayCameraSystemActor"

AGameplayCameraSystemActor::AGameplayCameraSystemActor(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CameraSystemComponent = CreateDefaultSubobject<UGameplayCameraSystemComponent>(TEXT("CameraSystemComponent"));
	RootComponent = CameraSystemComponent;
}

void AGameplayCameraSystemActor::CalcCamera(float DeltaTime, struct FMinimalViewInfo& OutResult)
{
	CameraSystemComponent->GetCameraView(DeltaTime, OutResult);
}

#undef LOCTEXT_NAMESPACE

