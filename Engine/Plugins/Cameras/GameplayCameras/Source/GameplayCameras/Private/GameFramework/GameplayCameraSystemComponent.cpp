// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCameraSystemComponent.h"

#include "Components/BillboardComponent.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/PersistentBlendStackCameraNode.h"
#include "Core/RootCameraNode.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"
#include "Kismet/GameplayStatics.h"
#include "Services/CameraModifierService.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/ICookInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraSystemComponent)

#define LOCTEXT_NAMESPACE "GameplayCameraSystemComponent"

UGameplayCameraSystemComponent::UGameplayCameraSystemComponent(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UGameplayCameraSystemComponent::GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator.IsValid())
	{
		FCameraSystemEvaluationParams UpdateParams;
		UpdateParams.DeltaTime = DeltaTime;
		CameraSystemEvaluator->Update(UpdateParams);

		CameraSystemEvaluator->GetEvaluatedCameraView(DesiredView);

		if (bSetPlayerControllerRotation)
		{
			if (APlayerController* PlayerController = WeakPlayerController.Get())
			{
				PlayerController->SetControlRotation(CameraSystemEvaluator->GetEvaluatedResult().CameraPose.GetRotation());
			}
		}
	}
}

void UGameplayCameraSystemComponent::OnRegister()
{
	using namespace UE::Cameras;

	Super::OnRegister();

#if WITH_EDITOR
	CreateCameraSystemSpriteComponent();
#endif  // WITH_EDITOR
}

#if WITH_EDITOR

void UGameplayCameraSystemComponent::CreateCameraSystemSpriteComponent()
{
	UTexture2D* EditorSpriteTexture = nullptr;
	{
		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		EditorSpriteTexture = LoadObject<UTexture2D>(
				nullptr,
				TEXT("/GameplayCameras/Textures/S_GameplayCameraSystem.S_GameplayCameraSystem"));
	}

	if (EditorSpriteTexture)
	{
		bVisualizeComponent = true;
		CreateSpriteComponent(EditorSpriteTexture);
	}

	if (SpriteComponent)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Cameras");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Cameras", "Cameras");
		SpriteComponent->SetRelativeScale3D(FVector3d(EditorSpriteTextureScale));
	}
}

#endif  // WITH_EDITOR


void UGameplayCameraSystemComponent::ActivateCameraSystemForPlayerIndex(int32 PlayerIndex)
{
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(this, PlayerIndex);
	if (!PlayerController)
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't activate gameplay camera system: no player controller found!"),
				ELogVerbosity::Error);
		return;
	}

	ActivateCameraSystemForPlayerController(PlayerController);
}

void UGameplayCameraSystemComponent::ActivateCameraSystemForPlayerController(APlayerController* PlayerController)
{
	using namespace UE::Cameras;

	if (!PlayerController)
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't activate gameplay camera system: invalid player controller given!"),
				ELogVerbosity::Error);
		return;
	}

	if (APlayerController* ActivePlayerController = WeakPlayerController.Get())
	{
		if (ActivePlayerController != PlayerController)
		{
			DeactivateCameraSystem();
		}
	}

	EnsureCameraSystemInitialized();

	AActor* OwningActor = GetOwner();
	if (!OwningActor)
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't activate gameplay camera system: no owning actor found!"),
				ELogVerbosity::Error);
		return;
	}

	PlayerController->SetViewTarget(OwningActor);
	WeakPlayerController = PlayerController;

	// Make sure the component is active.
	Activate();
}

bool UGameplayCameraSystemComponent::IsCameraSystemActiveForPlayController(APlayerController* PlayerController) const
{
	APlayerController* ActivatedPlayerController = WeakPlayerController.Get();
	if (!ActivatedPlayerController || ActivatedPlayerController  != PlayerController)
	{
		return false;
	}

	AActor* OwningActor = GetOwner();
	if (!OwningActor)
	{
		return false;
	}
	
	if (!HasCameraSystem())
	{
		return false;
	}

	if (!ActivatedPlayerController->PlayerCameraManager)
	{
		return false;
	}

	return ActivatedPlayerController->PlayerCameraManager->GetViewTarget() == OwningActor;
}

void UGameplayCameraSystemComponent::DeactivateCameraSystem(AActor* NextViewTarget)
{
	APlayerController* PlayerController = WeakPlayerController.Get();
	if (!PlayerController)
	{
		return;
	}

	PlayerController->SetViewTarget(NextViewTarget);
	WeakPlayerController.Reset();
}

void UGameplayCameraSystemComponent::BeginPlay()
{
	Super::BeginPlay();

	if (IsActive() && AutoActivateForPlayer != EAutoReceiveInput::Disabled && GetNetMode() != NM_DedicatedServer)
	{
		const int32 PlayerIndex = AutoActivateForPlayer.GetIntValue() - 1;
		ActivateCameraSystemForPlayerIndex(PlayerIndex);
	}
}

void UGameplayCameraSystemComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	DeactivateCameraSystem();

	Super::EndPlay(EndPlayReason);
}

FCameraRigInstanceID UGameplayCameraSystemComponent::StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator)
	{
		TSharedPtr<FCameraModifierService> CameraModifierService = CameraSystemEvaluator->FindEvaluationService<FCameraModifierService>();
		return CameraModifierService->StartCameraModifierRig(CameraRig, ECameraRigLayer::Global, OrderKey);
	}

	return FCameraRigInstanceID();
}

FCameraRigInstanceID UGameplayCameraSystemComponent::StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator)
	{
		TSharedPtr<FCameraModifierService> CameraModifierService = CameraSystemEvaluator->FindEvaluationService<FCameraModifierService>();
		return CameraModifierService->StartCameraModifierRig(CameraRig, ECameraRigLayer::Visual);
	}

	return FCameraRigInstanceID();
}

void UGameplayCameraSystemComponent::StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator)
	{
		TSharedPtr<FCameraModifierService> CameraModifierService = CameraSystemEvaluator->FindEvaluationService<FCameraModifierService>();
		CameraModifierService->StopCameraModifierRig(InstanceID, bImmediately);
	}
}

#undef LOCTEXT_NAMESPACE

