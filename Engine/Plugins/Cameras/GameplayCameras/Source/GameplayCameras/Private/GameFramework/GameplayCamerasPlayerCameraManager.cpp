// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayCamerasPlayerCameraManager.h"

#include "Camera/CameraComponent.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraRigTransition.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Core/RootCameraNodeCameraRigEvent.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "GameFramework/ActorCameraEvaluationContext.h"
#include "GameFramework/GameplayCameraComponentBase.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/ViewTargetTransitionParamsBlendNode.h"
#include "GameplayCamerasSettings.h"
#include "Services/CameraModifierService.h"
#include "Services/CameraShakeService.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCamerasPlayerCameraManager)

AGameplayCamerasPlayerCameraManager::AGameplayCamerasPlayerCameraManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void AGameplayCamerasPlayerCameraManager::BeginDestroy()
{
	DestroyCameraSystem();

	Super::BeginDestroy();
}

void AGameplayCamerasPlayerCameraManager::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	AGameplayCamerasPlayerCameraManager* This = CastChecked<AGameplayCamerasPlayerCameraManager>(InThis);
	This->IGameplayCameraSystemHost::OnAddReferencedObjects(Collector);
}

void AGameplayCamerasPlayerCameraManager::StealPlayerController(APlayerController* PlayerController)
{
	using namespace UE::Cameras;

	if (!ensure(PlayerController->PlayerCameraManager != this))
	{
		return;
	}

	OriginalCameraManager = PlayerController->PlayerCameraManager;
	AActor* OriginalViewTarget = OriginalCameraManager->GetViewTarget();

	PlayerController->PlayerCameraManager = this;
	InitializeFor(PlayerController);

	SetViewTarget(OriginalViewTarget);
}

void AGameplayCamerasPlayerCameraManager::ReleasePlayerController()
{
	if (!ensure(PCOwner && PCOwner->PlayerCameraManager == this))
	{
		return;
	}

	PCOwner->PlayerCameraManager = OriginalCameraManager;

	ViewTarget.Target = nullptr;

	OriginalCameraManager = nullptr;

	DestroyCameraSystem();

	PCOwner = nullptr;
}

void AGameplayCamerasPlayerCameraManager::ActivateGameplayCamera(UGameplayCameraComponentBase* GameplayCamera, EGameplayCameraComponentActivationMode ActivationMode)
{
	using namespace UE::Cameras;

	if (!GameplayCamera)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't activate a null Gameplay Camera!");
		return;
	}

	if (GameplayCamera->bRunStandaloneCameraSystem)
	{
		UE_LOGF(LogCameraSystem, Error, 
				"Can't activate Gameplay Camera '%ls.%ls': it is set to run in 'standalone' mode. Please disable 'Run Standalone Camera System' on it.",
				*GetNameSafe(GameplayCamera->GetOwner()), *GetNameSafe(GameplayCamera));
		return;
	}

	// See if there's a context for this component on the context stack.
	FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
	TSharedPtr<FCameraEvaluationContext> GameplayCameraContext = ContextStack.FindContextByOwner(GameplayCamera);
	const bool bFoundExistingContext = GameplayCameraContext.IsValid();
	if (!GameplayCameraContext)
	{
		// Make sure the component is active, so that it has an evaluation context.
		APlayerController* PlayerController = GetOwningPlayerController();
		GameplayCamera->ActivateCameraForPlayerController(PlayerController, false);
		GameplayCameraContext = GameplayCamera->GetEvaluationContext();
	}
	if (!GameplayCameraContext.IsValid())
	{
		UE_LOGF(LogCameraSystem, Error, 
				"Can't activate Gameplay Camera '%ls.%ls': can't create an evaluation context!",
				*GetNameSafe(GameplayCamera->GetOwner()), *GetNameSafe(GameplayCamera));
		return;
	}
	if (GameplayCameraContext->IsActive())
	{
		UE_LOGF(LogCameraSystem, Error, 
				"Can't activate Gameplay Camera '%ls.%ls': its evaluation context is already active!",
				*GetNameSafe(GameplayCamera->GetOwner()), *GetNameSafe(GameplayCamera));
		return;
	}

	switch (ActivationMode)
	{
		case EGameplayCameraComponentActivationMode::Push:
			{
				// If the context already existed in the stack, this will move it to the top without re-activating it.
				// Otherwise, it will activate it and push it on the top.
				ContextStack.PushContext(GameplayCameraContext.ToSharedRef());
			}
			break;
		case EGameplayCameraComponentActivationMode::InsertOrPush:
			{
				// If there's an active context, insert the new one inside it. If our context already existed,
				// we'll need to deactivate it first.
				// If there's no active context, simply push or move our context to the top.
				if (TSharedPtr<FCameraEvaluationContext> ActiveContext = ContextStack.GetActiveContext())
				{
					if (bFoundExistingContext)
					{
						ContextStack.RemoveContext(GameplayCameraContext.ToSharedRef());
					}

					const bool bSuccess = ActiveContext->AddChildContext(GameplayCameraContext.ToSharedRef());
					if (!bSuccess)
					{
						UObject* ActiveContextOwner = ActiveContext->GetOwner();
						UE_LOGF(LogCameraSystem, Error,
								"Couldn't insert camera director for '%ls.%ls' inside camera director for '%ls'. "
									 "Activation failed.",
								*GetNameSafe(GameplayCamera->GetOwner()), *GetNameSafe(GameplayCamera),
								*GetNameSafe(ActiveContextOwner));
					}
				}
				else
				{
					ContextStack.PushContext(GameplayCameraContext.ToSharedRef());
				}
			}
			break;
		default:
			ensureMsgf(false, TEXT("Unknown activation mode: %s"), *UEnum::GetValueAsString(ActivationMode));
			break;
	}
}

void AGameplayCamerasPlayerCameraManager::DeactivateGameplayCamera(UGameplayCameraComponentBase* GameplayCamera, bool bDeactivateAllCameraRigs)
{
	using namespace UE::Cameras;

	FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
	TSharedPtr<FCameraEvaluationContext> ExistingContext = ContextStack.FindContextByOwner(GameplayCamera);
	if (ExistingContext)
	{
		if (bDeactivateAllCameraRigs)
		{
			FRootCameraNodeEvaluator* RootEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
			RootEvaluator->DeactivateAllCameraRigs(ExistingContext, true);
		}

		ContextStack.RemoveContext(ExistingContext.ToSharedRef());
	}
}

FCameraRigInstanceID AGameplayCamerasPlayerCameraManager::ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRig)
{
	EnsureNullContext();
	return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, NullContext, ECameraRigLayer::Base);
}

FCameraRigInstanceID AGameplayCamerasPlayerCameraManager::ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRig)
{
	EnsureNullContext();
	return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, NullContext, ECameraRigLayer::Global);
}

FCameraRigInstanceID AGameplayCamerasPlayerCameraManager::ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRig)
{
	EnsureNullContext();
	return IGameplayCameraSystemHost::ActivateCameraRig(CameraRig, NullContext, ECameraRigLayer::Visual);
}

void AGameplayCamerasPlayerCameraManager::DeactivateCameraRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	EnsureNullContext();
	IGameplayCameraSystemHost::DeactivateCameraRig(InstanceID, bImmediately);
}

FCameraRigInstanceID AGameplayCamerasPlayerCameraManager::StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	EnsureNullContext();
	return IGameplayCameraSystemHost::StartCameraModifierRig(CameraRig, NullContext, ECameraRigLayer::Global, OrderKey);
}

FCameraRigInstanceID AGameplayCamerasPlayerCameraManager::StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey)
{
	EnsureNullContext();
	return IGameplayCameraSystemHost::StartCameraModifierRig(CameraRig, NullContext, ECameraRigLayer::Visual, OrderKey);
}

void AGameplayCamerasPlayerCameraManager::StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	IGameplayCameraSystemHost::StopCameraModifierRig(InstanceID, bImmediately);
}

FCameraShakeInstanceID AGameplayCamerasPlayerCameraManager::StartCameraShakeAsset(const UCameraShakeAsset* CameraShake, float ShakeScale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRotation)
{
	return IGameplayCameraSystemHost::StartCameraShake(CameraShake, ShakeScale, PlaySpace, UserPlaySpaceRotation);
}

bool AGameplayCamerasPlayerCameraManager::IsCameraShakeAssetPlaying(FCameraShakeInstanceID InInstanceID) const
{
	return IGameplayCameraSystemHost::IsCameraShakePlaying(InInstanceID);
}

bool AGameplayCamerasPlayerCameraManager::StopCameraShakeAsset(FCameraShakeInstanceID InInstanceID, bool bImmediately)
{
	return IGameplayCameraSystemHost::StopCameraShake(InInstanceID, bImmediately);
}

void AGameplayCamerasPlayerCameraManager::EnsureNullContext()
{
	using namespace UE::Cameras;

	if (!NullContext)
	{
		FCameraEvaluationContextInitializeParams InitParams;
		InitParams.Owner = this;
		InitParams.PlayerController = GetOwningPlayerController();
		NullContext = MakeShared<FCameraEvaluationContext>(InitParams);
	}
}

void AGameplayCamerasPlayerCameraManager::InitializeFor(APlayerController* PlayerController)
{
	using namespace UE::Cameras;

	if (!bOverrideViewRotationMode)
	{
		const UGameplayCamerasSettings* Settings = GetDefault<UGameplayCamerasSettings>();
		ViewRotationMode = Settings->DefaultViewRotationMode;
	}

	EnsureCameraSystemInitialized();
	if (ensure(CameraSystemEvaluator))
	{
		FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
		ContextStack.OnStackChanged().AddUObject(this, &AGameplayCamerasPlayerCameraManager::OnContextStackChanged);
	}

	Super::InitializeFor(PlayerController);
}

void AGameplayCamerasPlayerCameraManager::SetViewTarget(AActor* NewViewTarget, FViewTargetTransitionParams TransitionParams)
{
	using namespace UE::Cameras;

	// We want to keep our view target in sync with whatever is the active context owner in the camera system.
	// If that context owner isn't an actor, and isn't inside an actor (like a component), we use the player
	// controller as the view target.

	ensure(bIsSettingNewViewTarget == false);
	TGuardValue<bool> ReentrancyGuard(bIsSettingNewViewTarget, true);
	FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();

	// If the view target is null, this is sort of a shortcut for "we're done with the current view target",
	// so pop the context stack and reactivate the previous context.
	if (NewViewTarget == nullptr)
	{
		ContextStack.PopContext();

		if (TSharedPtr<FCameraEvaluationContext> NewActiveContext = ContextStack.GetActiveContext())
		{
			if (UObject* NewActiveContextOwner = NewActiveContext->GetOwner())
			{
				NewViewTarget = Cast<AActor>(NewActiveContextOwner);
				if (!NewViewTarget)
				{
					NewViewTarget = NewActiveContextOwner->GetTypedOuter<AActor>();
				}
			}
		}
	}

	// We pass empty transition params here because we never want to use PendingViewTarget, just ViewTarget.
	Super::SetViewTarget(NewViewTarget, FViewTargetTransitionParams());

	if (!NewViewTarget)
	{
		return;
	}

	// See if we can find the view target in the context stack. If so, reactivate it instead of potentially
	// making a new context for the same thing.
	TSharedPtr<FCameraEvaluationContext> ExistingContext = ContextStack.FindContextByPredicate(
			[NewViewTarget](TSharedRef<FCameraEvaluationContext> Item)
			{
				UObject* Owner = Item->GetOwner();
				return Owner && (Owner == NewViewTarget || Owner->GetTypedOuter<AActor>() == NewViewTarget);
			});
	if (ExistingContext)
	{
		// This will move the context to the top if it's already in the stack (which it is, we found it there).
		ContextStack.PushContext(ExistingContext.ToSharedRef());
	}
	else
	{
		// Create a new context for this view target.
		if (UCameraComponent* CameraComponent = NewViewTarget->FindComponentByClass<UCameraComponent>())
		{
			TSharedRef<FActorCameraEvaluationContext> NewContext = MakeShared<FActorCameraEvaluationContext>(CameraComponent);
			CameraSystemEvaluator->PushEvaluationContext(NewContext);
			ViewTargetContexts.Add(NewContext);
		}
		else
		{
			TSharedRef<FActorCameraEvaluationContext> NewContext = MakeShared<FActorCameraEvaluationContext>(NewViewTarget);
			CameraSystemEvaluator->PushEvaluationContext(NewContext);
			ViewTargetContexts.Add(NewContext);
		}
	}

	// If transition parameters were given, override the next activation for the new evaluation context.
	TSharedPtr<FCameraEvaluationContext> NextContext = CameraSystemEvaluator->GetEvaluationContextStack().GetActiveContext();
	if (NextContext && TransitionParams.BlendTime > 0.f)
	{
		UViewTargetTransitionParamsBlendCameraNode* BlendNode = NewObject<UViewTargetTransitionParamsBlendCameraNode>(GetTransientPackage());
		BlendNode->TransitionParams = TransitionParams;

		UCameraRigTransition* Transition = NewObject<UCameraRigTransition>(GetTransientPackage());
		Transition->Blend = BlendNode;

		FCameraDirectorEvaluator* DirectorEvaluator = NextContext->GetDirectorEvaluator();
		DirectorEvaluator->OverrideNextActivationTransition(Transition);
	}
}

void AGameplayCamerasPlayerCameraManager::ProcessViewRotation(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot)
{
	switch (ViewRotationMode)
	{
		case EGameplayCamerasViewRotationMode::PreviewUpdate:
			RunViewRotationPreviewUpdate(DeltaTime, OutViewRotation, OutDeltaRot);
			break;
	}

	Super::ProcessViewRotation(DeltaTime, OutViewRotation, OutDeltaRot);
}

void AGameplayCamerasPlayerCameraManager::RunViewRotationPreviewUpdate(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot)
{
	using namespace UE::Cameras;

	if (HasCameraSystem())
	{
		FCameraSystemEvaluationParams Params;
		Params.DeltaTime = DeltaTime;

		FCameraSystemViewRotationEvaluationResult Result;
		Result.ViewRotation = OutViewRotation;
		Result.DeltaRotation = OutDeltaRot;

		CameraSystemEvaluator->ViewRotationPreviewUpdate(Params, Result);

		OutViewRotation = Result.ViewRotation;
		OutDeltaRot = Result.DeltaRotation;
	}
}

void AGameplayCamerasPlayerCameraManager::DoUpdateCamera(float DeltaTime)
{
	using namespace UE::Cameras;

	// Don't up-call Super::DoUpdateCamera because:
	//
	// 1) it runs the view targets' CalcCamera or GetCameraView, which we already do inside our camera system,
	//    and we don't want them to double-update.
	// 2) it does a bunch of stuff regarding the pending view target that we don't care about.
	// 3) it applies the camera modifiers twice (once for the view target, once for the pending view target)
	//    which means that things like shakes can look like they're twice as "intense" during blends, and also
	//    means that modifiers get applied at the wrong level.
	//
	// So as a result, until we refactor the base camera manager class, we have to re-do some of the logic we
	// want to keep, such as color scale interpolation and screen/audio fading.
	//

	if (bEnableColorScaleInterp)
	{
		float BlendPct = FMath::Clamp((GetWorld()->TimeSeconds - ColorScaleInterpStartTime) / ColorScaleInterpDuration, 0.f, 1.0f);
		ColorScale = FMath::Lerp(OriginalColorScale, DesiredColorScale, BlendPct);
		if (BlendPct == 1.0f)
		{
			bEnableColorScaleInterp = false;
		}
	}

	if (bEnableFading)
	{
		if (bAutoAnimateFade)
		{
			FadeTimeRemaining = FMath::Max(FadeTimeRemaining - DeltaTime, 0.0f);
			if (FadeTime > 0.0f)
			{
				FadeAmount = FadeAlpha.X + ((1.f - FadeTimeRemaining / FadeTime) * (FadeAlpha.Y - FadeAlpha.X));
			}

			if ((bHoldFadeWhenFinished == false) && (FadeTimeRemaining <= 0.f))
			{
				// done
				StopCameraFade();
			}
		}

		if (bFadeAudio)
		{
			ApplyAudioFade();
		}
	}

	if (CameraSystemEvaluator.IsValid())
	{
		FCameraSystemEvaluationParams UpdateParams;
		UpdateParams.DeltaTime = DeltaTime;
		CameraSystemEvaluator->Update(UpdateParams);

		FMinimalViewInfo DesiredView;
		CameraSystemEvaluator->GetEvaluatedCameraView(DesiredView);

		ApplyCameraModifiers(DeltaTime, DesiredView);

		FillCameraCache(DesiredView);

		CleanUpViewTargetContexts();

		SetActorLocationAndRotation(DesiredView.Location, DesiredView.Rotation, false);
	}
}

void AGameplayCamerasPlayerCameraManager::OnContextStackChanged()
{
	using namespace UE::Cameras;

	// When the context stack changes, such as when a gameplay camera component activates directly
	// against our camera system host, we want to update the view target so that it's always in sync
	// with whichever owns the active evaluation context.
	//
	// This is as opposed to going through SetViewTarget or some other APlayerCameraManager method.

	if (ensure(CameraSystemEvaluator) && !bIsSettingNewViewTarget)
	{
		TGuardValue<bool> ReentrancyGuard(bIsSettingNewViewTarget, true);

		AActor* NewViewTarget = nullptr;
		FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
		if (TSharedPtr<FCameraEvaluationContext> ActiveContext = ContextStack.GetActiveContext())
		{
			UObject* ActiveContextOwner = ActiveContext->GetOwner();
			if (ActiveContextOwner)
			{
				NewViewTarget = ActiveContextOwner->GetTypedOuter<AActor>();
			}
		}
		ViewTarget.SetNewTarget(NewViewTarget);
		ViewTarget.CheckViewTarget(PCOwner);
		BlendParams = FViewTargetTransitionParams();
	}
}

void AGameplayCamerasPlayerCameraManager::CleanUpViewTargetContexts()
{
	using namespace UE::Cameras;

	FRootCameraNodeEvaluator* RootEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();

	for (auto It = ViewTargetContexts.CreateIterator(); It; ++It)
	{
		TSharedRef<FCameraEvaluationContext> Context(*It);
		if (!RootEvaluator->HasAnyRunningCameraRig(Context))
		{
			It.RemoveCurrent();
		}
	}
}

void AGameplayCamerasPlayerCameraManager::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	int Indentation = 1;
	int LineNumber = FMath::CeilToInt(YPos / YL);

	UFont* DrawFont = GEngine->GetSmallFont();
	Canvas->SetDrawColor(FColor::Yellow);
	Canvas->DrawText(
			DrawFont, 
			FString::Printf(TEXT("Please use the Camera Debugger panel to inspect '%s'."), *GetNameSafe(this)),
			Indentation * YL, (LineNumber++) * YL);

	YPos = LineNumber * YL;

	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);
}

