// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameplayControlRotationComponent.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"
#include "GameFramework/GameplayCameraComponent.h"
#include "GameFramework/IGameplayCameraSystemHost.h"
#include "GameFramework/Pawn.h"
#include "GameplayCameras.h"
#include "Kismet/GameplayStatics.h"
#include "Math/ColorList.h"
#include "Services/PlayerControlRotationService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayControlRotationComponent)

#define LOCTEXT_NAMESPACE "GameplayControlRotationComponent"

UGameplayControlRotationComponent::UGameplayControlRotationComponent(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
}

void UGameplayControlRotationComponent::BeginPlay()
{
	Super::BeginPlay();

	if (IsActive() && AutoActivateForPlayer != EAutoReceiveInput::Disabled && GetNetMode() != NM_DedicatedServer)
	{
		const int32 PlayerIndex = AutoActivateForPlayer.GetIntValue() - 1;
		ActivateControlRotationManagementForPlayerIndex(PlayerIndex);
	}
}

void UGameplayControlRotationComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	using namespace UE::Cameras;

	TeardownControlRotationService(true);

	Super::EndPlay(EndPlayReason);
}

void UGameplayControlRotationComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (PlayerController && ControlRotationService)
	{
		// This may be technically one frame late (i.e. we set the control rotation computed 
		// late last tick) unless the camera system is setup to process player input into camera
		// rotation early in the frame.
		PlayerController->SetControlRotation(ControlRotationService->GetCurrentControlRotation());
	}
}

void UGameplayControlRotationComponent::ActivateControlRotationManagementForPlayerIndex(int32 PlayerIndex)
{
	APlayerController* ForPlayerController = UGameplayStatics::GetPlayerController(this, PlayerIndex);
	ActivateControlRotationManagementForPlayerController(ForPlayerController);
}

void UGameplayControlRotationComponent::ActivateControlRotationManagementForPlayerController(APlayerController* InPlayerController)
{
	InitializeControlRotationService(InPlayerController);
}

void UGameplayControlRotationComponent::DeactivateControlRotationManagement()
{
	TeardownControlRotationService(false);
}

void UGameplayControlRotationComponent::InitializeControlRotationService(APlayerController* InPlayerController)
{
	using namespace UE::Cameras;

	if (ControlRotationService)
	{
		UE_LOGF(LogCameraSystem, Error,
				"GameplayControlRotationComponent '%ls' has already been activated",
				*GetNameSafe(this));
		return;
	}

	if (!InPlayerController)
	{
		UE_LOGF(LogCameraSystem, Error, 
				"GameplayControlRotationComponent '%ls' can't activate: no player controller given or found!",
				*GetNameSafe(this));
		return;
	}

	IGameplayCameraSystemHost* FoundHost = IGameplayCameraSystemHost::FindActiveHost(InPlayerController);
	if (!FoundHost)
	{
		UE_LOGF(LogCameraSystem, Error, 
				"Can't find camera system host on the player controller. "
				 	 "UGameplayControlRotationComponent requires using AGameplayCamerasPlayerCameraManager, or similar, as a camera manager.");
		return;
	}

	PlayerController = InPlayerController;
	CameraSystemHost = FoundHost->GetAsScriptInterface();

	// Create the evaluation service, with a copy of our parameters.
	FPlayerControlRotationParams ServiceParams;
	ServiceParams.AxisActionAngularSpeedThreshold = AxisActionAngularSpeedThreshold;
	ServiceParams.AxisActionMagnitudeThreshold = AxisActionMagnitudeThreshold;
	ServiceParams.AxisActions = AxisActions;
	// We will set the control rotation ourselves.
	ServiceParams.bApplyControlRotation = false;

	ControlRotationService = MakeShared<FPlayerControlRotationEvaluationService>(ServiceParams);

	TSharedPtr<FCameraSystemEvaluator> CameraSystem = CameraSystemHost->GetCameraSystemEvaluator();
	CameraSystem->RegisterEvaluationService(ControlRotationService.ToSharedRef());
}

void UGameplayControlRotationComponent::TeardownControlRotationService(bool bAllowUninitialized)
{
	using namespace UE::Cameras;

	if (!ControlRotationService || !CameraSystemHost)
	{
		if (!bAllowUninitialized)
		{
			UE_LOGF(LogCameraSystem, Error,
					"GameplayCameraComponent '%ls' isn't active",
					*GetNameSafe(this));
		}
		return;
	}

	TSharedPtr<FCameraSystemEvaluator> CameraSystem = CameraSystemHost->GetCameraSystemEvaluator();
	CameraSystem->UnregisterEvaluationService(ControlRotationService.ToSharedRef());

	ControlRotationService = nullptr;
	CameraSystemHost = nullptr;
	PlayerController = nullptr;
}

#undef LOCTEXT_NAMESPACE

