// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/IGameplayCameraSystemHost.h"

#include "Camera/PlayerCameraManager.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigInstanceID.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Debug/CameraSystemDebugRegistry.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "Engine/World.h"
#include "GameFramework/GameplayCamerasPlayerCameraManager.h"
#include "GameFramework/PlayerController.h"
#include "Services/CameraActionService.h"
#include "Services/CameraModifierService.h"
#include "Services/CameraShakeService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IGameplayCameraSystemHost)

namespace UE::Cameras
{

extern int32 GGameplayCamerasDebugSystemID;

}  // namespace UE::Cameras

void IGameplayCameraSystemHost::InitializeCameraSystem()
{
	using namespace UE::Cameras;

	FCameraSystemEvaluatorCreateParams Params;
	Params.Owner = GetAsObject();
	InitializeCameraSystem(Params);
}

void IGameplayCameraSystemHost::InitializeCameraSystem(const UE::Cameras::FCameraSystemEvaluatorCreateParams& Params)
{
	ensure(!CameraSystemEvaluator.IsValid());
	ensure(Params.Owner != nullptr && Params.Owner == GetAsObject());

	CameraSystemEvaluator = MakeShared<FCameraSystemEvaluator>();
	CameraSystemEvaluator->Initialize(Params);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	ensure(!DebugDrawDelegateHandle.IsValid());
	UWorld* World = Params.Owner->GetWorld();
	if (World && World->IsGameWorld())
	{
		DebugDrawDelegateHandle = UDebugDrawService::Register(
				TEXT("Game"), FDebugDrawDelegate::CreateRaw(this, &IGameplayCameraSystemHost::DebugDraw));
	}
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

TScriptInterface<IGameplayCameraSystemHost> IGameplayCameraSystemHost::GetAsScriptInterface()
{
	TScriptInterface<IGameplayCameraSystemHost> Result(GetAsObject());
	checkSlow(Result.GetInterface() != nullptr);
	return Result;
}

FCameraRigInstanceID IGameplayCameraSystemHost::ActivateCameraRig(
		const UCameraRigAsset* CameraRig, 
		TSharedPtr<FCameraEvaluationContext> EvaluationContext,
		ECameraRigLayer EvaluationLayer)
{
	using namespace UE::Cameras;

	if (!CameraRig)
	{
		FFrame::KismetExecutionMessage(
				*FString::Printf(TEXT("Can't activate camera rig on '%s': no camera rig given!"), *GetNameSafe(GetAsObject())),
				ELogVerbosity::Error);
		return FCameraRigInstanceID();
	}
	
	if (!EvaluationContext)
	{
		FFrame::KismetExecutionMessage(
				*FString::Printf(
					TEXT("Can't activate camera rig '%s' on '%s': invalid evaluation context given!"), 
					*GetNameSafe(CameraRig), *GetNameSafe(GetAsObject())),
				ELogVerbosity::Error);
		return FCameraRigInstanceID();
	}

	if (ensure(CameraSystemEvaluator))
	{
		FActivateCameraRigParams Params;
		Params.CameraRig = CameraRig;
		Params.EvaluationContext = EvaluationContext;
		Params.Layer = EvaluationLayer;

		FRootCameraNodeEvaluator* RootNodeEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
		return RootNodeEvaluator->ActivateCameraRig(Params);
	}

	return FCameraRigInstanceID();
}

void IGameplayCameraSystemHost::DeactivateCameraRig(FCameraRigInstanceID InInstanceID, bool bImmediately)
{
	using namespace UE::Cameras;

	if (InInstanceID.IsValid())
	{
		FDeactivateCameraRigParams Params;
		Params.InstanceID = InInstanceID;
		Params.bDeactiveImmediately = bImmediately;

		FRootCameraNodeEvaluator* RootNodeEvaluator = CameraSystemEvaluator->GetRootNodeEvaluator();
		RootNodeEvaluator->DeactivateCameraRig(Params);
	}
}

FCameraRigInstanceID IGameplayCameraSystemHost::StartCameraModifierRig(
		const UCameraRigAsset* CameraRig,
		TSharedPtr<FCameraEvaluationContext> EvaluationContext,
		ECameraRigLayer EvaluationLayer,
		int32 OrderKey)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraModifierService> CameraModifierService = CameraSystemEvaluator->FindEvaluationService<FCameraModifierService>();
		if (ensure(CameraModifierService))
		{
			return CameraModifierService->StartCameraModifierRig(CameraRig, EvaluationContext.ToSharedRef(), EvaluationLayer, OrderKey);
		}
	}

	return FCameraRigInstanceID();
}

void IGameplayCameraSystemHost::StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraModifierService> CameraModifierService = CameraSystemEvaluator->FindEvaluationService<FCameraModifierService>();
		CameraModifierService->StopCameraModifierRig(InstanceID, bImmediately);
	}
}

FCameraShakeInstanceID IGameplayCameraSystemHost::StartCameraShake(const UCameraShakeAsset* CameraShake, float ShakeScale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRotation)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		FStartCameraShakeParams Params;
		Params.CameraShake = CameraShake;
		Params.ShakeScale = ShakeScale;
		Params.PlaySpace = PlaySpace;
		Params.UserPlaySpaceRotation = UserPlaySpaceRotation;

		TSharedPtr<FCameraShakeService> CameraShakeService = CameraSystemEvaluator->FindEvaluationService<FCameraShakeService>();
		return CameraShakeService->StartCameraShake(Params);
	}

	return FCameraShakeInstanceID();
}

bool IGameplayCameraSystemHost::IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraShakeService> CameraShakeService = CameraSystemEvaluator->FindEvaluationService<FCameraShakeService>();
		return CameraShakeService->IsCameraShakePlaying(InInstanceID);
	}
	return false;
}

bool IGameplayCameraSystemHost::StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraShakeService> CameraShakeService = CameraSystemEvaluator->FindEvaluationService<FCameraShakeService>();
		return CameraShakeService->StopCameraShake(InInstanceID, bImmediately);
	}
	return false;
}

FCameraActionInstanceID IGameplayCameraSystemHost::StartAction(const UCameraAction* CameraAction)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraActionService> CameraActionService = CameraSystemEvaluator->FindEvaluationService<FCameraActionService>();
		return CameraActionService->StartAction(CameraAction);
	}
	return FCameraActionInstanceID();
}

bool IGameplayCameraSystemHost::IsActionRunning(const FCameraActionInstanceID InInstanceID) const
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraActionService> CameraActionService = CameraSystemEvaluator->FindEvaluationService<FCameraActionService>();
		return CameraActionService->IsActionRunning(InInstanceID);
	}
	return false;
}

bool IGameplayCameraSystemHost::StopAction(const FCameraActionInstanceID InInstanceID)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraActionService> CameraActionService = CameraSystemEvaluator->FindEvaluationService<FCameraActionService>();
		return CameraActionService->StopAction(InInstanceID);
	}
	return false;
}

bool IGameplayCameraSystemHost::StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass)
{
	using namespace UE::Cameras;

	if (ensure(CameraSystemEvaluator))
	{
		TSharedPtr<FCameraActionService> CameraActionService = CameraSystemEvaluator->FindEvaluationService<FCameraActionService>();
		return CameraActionService->StopAllActionsOfClass(InActionClass);
	}
	return false;
}

IGameplayCameraSystemHost* IGameplayCameraSystemHost::FindActiveHost(APlayerController* PlayerController)
{
	if (PlayerController)
	{
		if (PlayerController->PlayerCameraManager)
		{
			if (IGameplayCameraSystemHost* CameraManagerHost = Cast<IGameplayCameraSystemHost>(PlayerController->PlayerCameraManager))
			{
				return CameraManagerHost;
			}
		}
		if (AActor* ViewTarget = PlayerController->GetViewTarget())
		{
			if (IGameplayCameraSystemHost* ViewTargetHost = ViewTarget->FindComponentByInterface<IGameplayCameraSystemHost>())
			{
				return ViewTargetHost;
			}
		}
	}
	return nullptr;
}

void IGameplayCameraSystemHost::EnsureCameraSystemInitialized()
{
	if (!CameraSystemEvaluator.IsValid())
	{
		InitializeCameraSystem();
	}
}

void IGameplayCameraSystemHost::DestroyCameraSystem()
{
#if UE_GAMEPLAY_CAMERAS_DEBUG
	if (DebugDrawDelegateHandle.IsValid())
	{
		UDebugDrawService::Unregister(DebugDrawDelegateHandle);
		DebugDrawDelegateHandle.Reset();
	}
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	CameraSystemEvaluator.Reset();
}

void IGameplayCameraSystemHost::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	if (CameraSystemEvaluator.IsValid())
	{
		CameraSystemEvaluator->AddReferencedObjects(Collector);
	}
}

void IGameplayCameraSystemHost::UpdateCameraSystem(float DeltaTime)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator.IsValid())
	{
		FCameraSystemEvaluationParams Params;
		Params.DeltaTime = DeltaTime;
		CameraSystemEvaluator->Update(Params);
	}
}

#if WITH_EDITOR

void IGameplayCameraSystemHost::UpdateCameraSystemForEditorPreview(float DeltaTime)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator.IsValid())
	{
		FCameraSystemEvaluationParams Params;
		Params.DeltaTime = DeltaTime;
		CameraSystemEvaluator->EditorPreviewUpdate(Params);
	}
}

#endif  // WITH_EDITOR

TSharedPtr<UE::Cameras::FCameraSystemEvaluator> IGameplayCameraSystemHost::GetCameraSystemEvaluator()
{
	return CameraSystemEvaluator;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void IGameplayCameraSystemHost::DebugDraw(UCanvas* Canvas, APlayerController* PlayerController)
{
	using namespace UE::Cameras;

	if (CameraSystemEvaluator.IsValid())
	{
		UWorld* OwnerWorld = nullptr;
		if (UObject* CameraSystemOwner = CameraSystemEvaluator->GetOwner())
		{
			OwnerWorld = CameraSystemOwner->GetWorld();
		}

		// Find the actual player controller as best we can.
		APlayerController* ActualPlayerController = PlayerController;
		if (!ActualPlayerController)
		{
			const FCameraEvaluationContextStack& ContextStack = CameraSystemEvaluator->GetEvaluationContextStack();
			if (TSharedPtr<FCameraEvaluationContext> ActiveContext = ContextStack.GetActiveContext())
			{
				ActualPlayerController = ActiveContext->GetPlayerController();
			}
		}

		if (!ActualPlayerController && OwnerWorld)
		{
			ActualPlayerController = OwnerWorld->GetFirstPlayerController();
		}

		const UObject* ThisAsObject = GetAsObject();
		const AActor* ViewTarget = ActualPlayerController ? ActualPlayerController->GetViewTarget() : nullptr;
		const bool bThisIsCameraManager = (ActualPlayerController && ThisAsObject == ActualPlayerController->PlayerCameraManager);
		const bool bThisIsViewTarget = (ThisAsObject && ViewTarget && ThisAsObject->GetTypedOuter<AActor>() == ViewTarget);
		const bool bHasCameraManager = (ActualPlayerController && ActualPlayerController->PlayerCameraManager->IsA<AGameplayCamerasPlayerCameraManager>());

		// We're looking from the outside if we are not the view target, or if we don't have a player
		// anymore (which happens in spectator mode like with the debug camera).
		const bool bIsDebugCameraEnabled = (
				(!bThisIsCameraManager && !bThisIsViewTarget) ||
				!ActualPlayerController || !ActualPlayerController->Player);

		// The default camera system is the one running the camera manager or, if there none there, the 
		// one running the current view target.
		const bool bIsDefaultCameraSystem = (bThisIsCameraManager || (bThisIsViewTarget && !bHasCameraManager));

		FCameraSystemDebugUpdateParams DebugUpdateParams;
		DebugUpdateParams.CanvasObject = Canvas;
		DebugUpdateParams.bIsDebugCameraEnabled = bIsDebugCameraEnabled;
		DebugUpdateParams.bIsDefaultCameraSystem = bIsDefaultCameraSystem;
		CameraSystemEvaluator->DebugUpdate(DebugUpdateParams);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

