// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ViewTargetTransitionParamsBlendNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewTargetTransitionParamsBlendNode)

namespace UE::Cameras
{

/**
 * A blend node evaluator for UViewTargetTransitionParamsBlendCameraNode, which emulates the basic engine
 * blend curves for view targets.
 */
class FViewTargetTransitionParamsBlendCameraNodeEvaluator : public FSimpleBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FViewTargetTransitionParamsBlendCameraNodeEvaluator, FSimpleBlendCameraNodeEvaluator)

protected:

	// FSimpleBlendCameraNodeEvaluator interface.
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

private:

	float CurrentTime = 0.f;
};

void FViewTargetTransitionParamsBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UViewTargetTransitionParamsBlendCameraNode* TransitionParamsNode = GetCameraNodeAs<UViewTargetTransitionParamsBlendCameraNode>();
	CurrentTime += Params.DeltaTime;
	if (CurrentTime >= TransitionParamsNode->TransitionParams.BlendTime)
	{
		CurrentTime = TransitionParamsNode->TransitionParams.BlendTime;
		SetBlendFinished();
	}

	FSimpleBlendCameraNodeEvaluator::OnRun(Params, OutResult);
}

void FViewTargetTransitionParamsBlendCameraNodeEvaluator::OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult)
{
	const UViewTargetTransitionParamsBlendCameraNode* TransitionParamsNode = GetCameraNodeAs<UViewTargetTransitionParamsBlendCameraNode>();
	float TimeFactor = 1.f;
	if (TransitionParamsNode->TransitionParams.BlendTime > 0.f)
	{
		TimeFactor = CurrentTime / TransitionParamsNode->TransitionParams.BlendTime;
	}
	OutResult.BlendFactor = TransitionParamsNode->TransitionParams.GetBlendAlpha(TimeFactor);
}

void FViewTargetTransitionParamsBlendCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << CurrentTime;
}

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FViewTargetTransitionParamsBlendCameraNodeEvaluator)

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UViewTargetTransitionParamsBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FViewTargetTransitionParamsBlendCameraNodeEvaluator>();
}

