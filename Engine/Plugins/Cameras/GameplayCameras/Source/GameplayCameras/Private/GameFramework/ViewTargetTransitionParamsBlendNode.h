// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/PlayerCameraManager.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "ViewTargetTransitionParamsBlendNode.generated.h"

/**
 * A blend node that implements the blend algorithms of the FViewTargetTransitionParams.
 */
UCLASS(MinimalAPI, Hidden)
class UViewTargetTransitionParamsBlendCameraNode : public USimpleBlendCameraNode
{
	GENERATED_BODY()

public:

	/** The transition params to use. */
	UPROPERTY()
	FViewTargetTransitionParams TransitionParams;

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

