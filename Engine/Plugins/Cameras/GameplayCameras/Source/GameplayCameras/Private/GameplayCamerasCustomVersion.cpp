// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasCustomVersion.h"

#include "Serialization/CustomVersion.h"

FCustomVersionRegistration GGameplayCamerasCustomVersion(
		FGameplayCamerasCustomVersion::GUID,
		FGameplayCamerasCustomVersion::LatestVersion,
		TEXT("GameplayCamerasCustomVersion"));

