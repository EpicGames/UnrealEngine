// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasDelegates.h"

namespace UE::Cameras
{

FOnCameraAssetBuilt FGameplayCamerasDelegates::OnCameraAssetBuiltDelegate;
FOnCameraRigAssetBuilt FGameplayCamerasDelegates::OnCameraRigAssetBuiltDelegate;
FOnCameraShakeAssetBuilt FGameplayCamerasDelegates::OnCameraShakeAssetBuiltDelegate;
FOnCameraNodeChanged FGameplayCamerasDelegates::OnCustomCameraNodeParametersChangedDelegate;

}  // namespace UE::Cameras

