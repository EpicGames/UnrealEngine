// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGameplayCamerasModule.h"

#include "Debug/CameraDebugColors.h"
#include "GameplayCameras.h"
#include "Modules/ModuleManager.h"
#include "Nodes/Framing/CameraFramingZone.h"
#include "ShowFlags.h"
#include "UObject/UObjectBase.h"

#if UE_GAMEPLAY_CAMERAS_TRACE
#include "Debug/CameraSystemTrace.h"
#include "Features/IModularFeatures.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#endif // UE_GAMEPLAY_CAMERAS_TRACE

#define LOCTEXT_NAMESPACE "GameplayCamerasModule"

DEFINE_LOG_CATEGORY(LogCameraSystem);

namespace UE::Cameras
{

TCustomShowFlag<> GameplayCamerasShowFlag(TEXT("GameplayCameras"), true, SFG_Developer, LOCTEXT("ShowFlagDisplayName", "Gameplay Cameras"));

#if UE_GAMEPLAY_CAMERAS_TRACE
/**
 * Rewind debugger runtime extension to control camera system traces.
 */
class FCameraSystemRewindDebuggerRuntimeExtension : public IRewindDebuggerRuntimeExtension
{
public:
	virtual void RecordingStarted() override
	{
		Trace::ToggleChannel(*FCameraSystemTrace::ChannelName, true);
	}

	virtual void RecordingStopped() override
	{
		Trace::ToggleChannel(*FCameraSystemTrace::ChannelName, false);
	}
};
#endif // UE_GAMEPLAY_CAMERAS_TRACE

} // namespace UE::Cameras

IGameplayCamerasModule& IGameplayCamerasModule::Get()
{
	return FModuleManager::LoadModuleChecked<IGameplayCamerasModule>("GameplayCameras");
}

class FGameplayCamerasModule : public IGameplayCamerasModule
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override
	{
		RegisterBuiltInBlendableStructs();
		
#if UE_GAMEPLAY_CAMERAS_DEBUG
		UE::Cameras::FCameraDebugColors::RegisterBuiltinColorSchemes();
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if UE_GAMEPLAY_CAMERAS_TRACE
		using namespace UE::Cameras;

		RewindDebuggerRuntimeExtension = MakeShared<FCameraSystemRewindDebuggerRuntimeExtension>();

		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerRuntimeExtension.Get());
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
	}

	virtual void ShutdownModule() override
	{
#if UE_GAMEPLAY_CAMERAS_TRACE
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerRuntimeExtension.Get());
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
		UnregisterBuiltInBlendableStructs();
	}

public:

	// IGameplayCamerasModule interface
	virtual void RegisterBlendableStruct(const UScriptStruct* StructType, UE::Cameras::FBlendableStructTypeErasedInterpolator Interpolator) override
	{
		using namespace UE::Cameras;

		if (!ensure(EnumHasAllFlags(StructType->StructFlags, STRUCT_IsPlainOldData)))
		{
			return;
		}

		const bool bAlreadyRegistered = BlendableStructs.ContainsByPredicate([StructType](const FBlendableStructInfo& Item)
				{
					return Item.StructType == StructType;
				});
		if (ensure(!bAlreadyRegistered))
		{
			FBlendableStructInfo& NewInfo = BlendableStructs.Emplace_GetRef();
			NewInfo.StructType = StructType;
			NewInfo.Interpolator = Interpolator;
		}
	}

	virtual TConstArrayView<UE::Cameras::FBlendableStructInfo> GetBlendableStructs() const override
	{
		return BlendableStructs;
	}

	virtual void UnregisterBlendableStruct(const UScriptStruct* StructType) override
	{
		using namespace UE::Cameras;

		BlendableStructs.RemoveAll([StructType](const FBlendableStructInfo& Item)
				{
					return Item.StructType == StructType;
				});
	}

#if WITH_EDITOR
	virtual TSharedPtr<IGameplayCamerasLiveEditManager> GetLiveEditManager() const override
	{
		return LiveEditManager;
	}

	virtual void SetLiveEditManager(TSharedPtr<IGameplayCamerasLiveEditManager> InLiveEditManager) override
	{
		LiveEditManager = InLiveEditManager;
	}
#endif

private:

	void RegisterBuiltInBlendableStructs()
	{
		using namespace UE::Cameras;

		RegisterBlendableStruct(FCameraFramingZone::StaticStruct(), &FCameraFramingZone::TypeErasedInterpolate);
	}

	void UnregisterBuiltInBlendableStructs()
	{
		using namespace UE::Cameras;

		if (UObjectInitialized())
		{
			UnregisterBlendableStruct(FCameraFramingZone::StaticStruct());
		}
	}

private:

	TArray<UE::Cameras::FBlendableStructInfo> BlendableStructs;

#if WITH_EDITORONLY_DATA
	FDelegateHandle OnMovieSceneSectionAddedToTrackHandle;
#endif

#if WITH_EDITOR
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager;
#endif

#if UE_GAMEPLAY_CAMERAS_TRACE
	TSharedPtr<UE::Cameras::FCameraSystemRewindDebuggerRuntimeExtension> RewindDebuggerRuntimeExtension;
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
};

IMPLEMENT_MODULE(FGameplayCamerasModule, GameplayCameras);

#undef LOCTEXT_NAMESPACE

