// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCamerasSettings)

UGameplayCamerasSettings::UGameplayCamerasSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CategoryName = TEXT("Plugins");
}

