// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraContextDataTable.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableTable.h"
#include "Misc/EngineVersionComparison.h"
#include "StructUtils/OverridablePropertyBag.h"
#include "Templates/Function.h"

namespace UE::Cameras
{

namespace Internal
{

template<typename ParameterType>
void ApplyBlendableParameterOverride(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const ParameterType& ParameterValue,
		FCameraVariableTable& VariableTable,
		bool bDrivenOnly)
{
	using ValueType = typename ParameterType::ValueType;

	const FCameraVariableID ParameterVariableID(ParameterDefinition.VariableID);
	if (ParameterValue.VariableID.IsValid())
	{
		// The override is driven by a variable... read its value and set it as the value for the
		// prefab's variable. Basically, we forward the value from one variable to the next.
		ValueType OverrideValue;
		if (VariableTable.TryGetValue<ValueType>(ParameterValue.VariableID, OverrideValue))
		{
			VariableTable.SetValue<ValueType>(ParameterVariableID, OverrideValue);
		}
		else if (ParameterValue.Variable)
		{
			VariableTable.SetValue<ValueType>(ParameterVariableID, ParameterValue.Variable->GetDefaultValue());
		}
		else
		{
			VariableTable.SetValue<ValueType>(ParameterVariableID, ParameterValue.Value);
		}
	}
	else if (!bDrivenOnly)
	{
		// The override is a fixed value. Just set that on the prefab's variable.
		VariableTable.SetValue<ValueType>(ParameterVariableID, ParameterValue.Value);
	}
}

void ApplyBlendableParameterOverride(
		const UObject* CameraObject,
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FInstancedPropertyBag& PropertyBag,
		const FPropertyBagPropertyDesc& PropertyBagPropertyDesc,
		FCameraVariableTable& VariableTable,
		bool bDrivenOnly)
{
	ensure(ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable);

	if (!ParameterDefinition.VariableID)
	{
		// Ignore un-built parameter overrides in the editor since the user could have just added
		// an override while PIE is running. They need to hit the Build button for the override
		// to apply.
		// Outside of the editor, report this as an error.
#if !WITH_EDITOR
		UE_LOGF(LogCameraSystem, Error,
				"Invalid blendable parameter override '%ls' in camera rig '%ls'. Was it built/cooked?",
				*ParameterDefinition.ParameterName.ToString(),
				*GetPathNameSafe(CameraObject));
#endif
		return;
	}

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	TValueOrError<FStructView, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueStruct(PropertyBagPropertyDesc);
#else
	TValueOrError<FStructView, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueStruct(PropertyBagPropertyDesc.Name);
#endif
	if (!ensureMsgf(
				ParameterValueOrError.HasValue() && !ParameterValueOrError.HasError(),
				TEXT("Camera parameter has no valid value! Error: %s"),
				*UEnum::GetValueAsString(ParameterValueOrError.GetError())))
	{
		return;
	}

	const FStructView& ParameterValue = ParameterValueOrError.GetValue();
	const UScriptStruct* ParameterType = ParameterValue.GetScriptStruct();

	switch (ParameterDefinition.VariableType)
	{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		case ECameraVariableType::ValueName:\
			{\
				check(ParameterType == F##ValueName##CameraParameter::StaticStruct());\
				const F##ValueName##CameraParameter& TypedParameterValue = ParameterValue.Get<F##ValueName##CameraParameter>();\
				ApplyBlendableParameterOverride(ParameterDefinition, TypedParameterValue, VariableTable, bDrivenOnly);\
			}\
			break;
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
		case ECameraVariableType::BlendableStruct:
			{
				const uint8* RawValuePtr = ParameterValue.GetMemory();
				VariableTable.SetValue(ParameterDefinition.VariableID, ParameterDefinition.VariableType, ParameterDefinition.BlendableStructType, RawValuePtr);
			}
			break;
		default:
			ensure(false);
			break;
	}
}

template<typename ParameterType>
void OverrideContextDataTableEntry(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const ParameterType& ParameterValue,
		FCameraContextDataTable& ContextDataTable)
{
	const uint8* RawParameterValue = reinterpret_cast<const uint8*>(&ParameterValue);
	ContextDataTable.TrySetData(ParameterDefinition.DataID, ParameterDefinition.DataType, ParameterDefinition.DataTypeObject, RawParameterValue);
}

template<>
void OverrideContextDataTableEntry<FStructView>(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FStructView& ParameterValue,
		FCameraContextDataTable& ContextDataTable)
{
	const uint8* RawParameterValue = ParameterValue.GetMemory();
	ContextDataTable.TrySetData(ParameterDefinition.DataID, ParameterDefinition.DataType, ParameterDefinition.DataTypeObject, RawParameterValue);
}

template<typename ParameterType>
void OverrideContextDataTableEntryElement(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		int32 Index,
		const ParameterType& ParameterValue,
		FCameraContextDataTable& ContextDataTable)
{
	const uint8* RawParameterValue = reinterpret_cast<const uint8*>(&ParameterValue);
	ContextDataTable.TrySetArrayData(ParameterDefinition.DataID, ParameterDefinition.DataType, ParameterDefinition.DataTypeObject, Index, RawParameterValue);
}

template<>
void OverrideContextDataTableEntryElement<FStructView>(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		int32 Index,
		const FStructView& ParameterValue,
		FCameraContextDataTable& ContextDataTable)
{
	const uint8* RawParameterValue = ParameterValue.GetMemory();
	ContextDataTable.TrySetArrayData(ParameterDefinition.DataID, ParameterDefinition.DataType, ParameterDefinition.DataTypeObject, Index, RawParameterValue);
}

template<typename ParameterType>
void ApplyDataParameterOverride(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const TValueOrError<ParameterType, EPropertyBagResult>& ParameterValueOrError,
		FCameraContextDataTable& ContextDataTable)
{
	if (!ensureMsgf(
				ParameterValueOrError.HasValue() && !ParameterValueOrError.HasError(),
				TEXT("Camera parameter has no valid value! Error: %s"),
				*UEnum::GetValueAsString(ParameterValueOrError.GetError())))
	{
		return;
	}

	// Write the override value into the context data table.
	const ParameterType& ParameterValue = ParameterValueOrError.GetValue();
	OverrideContextDataTableEntry<ParameterType>(ParameterDefinition, ParameterValue, ContextDataTable);
}

template<typename ParameterType>
void ApplyDataParameterElementOverride(
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const TValueOrError<ParameterType, EPropertyBagResult>& ElementValueOrError,
		int32 Index,
		FCameraContextDataTable& ContextDataTable)
{
	if (!ensureMsgf(
				ElementValueOrError.HasValue() && !ElementValueOrError.HasError(),
				TEXT("Camera parameter has no valid value! Error: %s"),
				*UEnum::GetValueAsString(ElementValueOrError.GetError())))
	{
		return;
	}

	// Write the override value into the context data table's entry array.
	const ParameterType& ParameterValue = ElementValueOrError.GetValue();
	OverrideContextDataTableEntryElement<ParameterType>(ParameterDefinition, Index, ParameterValue, ContextDataTable);
}

void ApplyDataParameterSingleOverride(
		const UObject* CameraObject,
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FInstancedPropertyBag& PropertyBag,
		const FPropertyBagPropertyDesc& PropertyBagPropertyDesc,
		FCameraContextDataTable& ContextDataTable)
{
	switch (ParameterDefinition.DataType)
	{
		case ECameraContextDataType::Name:
			{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
				TValueOrError<FName, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueName(PropertyBagPropertyDesc);
#else
				TValueOrError<FName, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueName(PropertyBagPropertyDesc.Name);
#endif
				ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
			}
			break;
		case ECameraContextDataType::String:
			{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
				TValueOrError<FString, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueString(PropertyBagPropertyDesc);
#else
				TValueOrError<FString, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueString(PropertyBagPropertyDesc.Name);
#endif
				ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
			}
			break;
		case ECameraContextDataType::Enum:
			{
				const UEnum* EnumType = Cast<const UEnum>(ParameterDefinition.DataTypeObject);
				if (ensure(EnumType))
				{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
					TValueOrError<uint8, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueEnum(PropertyBagPropertyDesc, EnumType);
#else
					TValueOrError<uint8, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueEnum(PropertyBagPropertyDesc.Name, EnumType);
#endif
					ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
				}
			}
			break;
		case ECameraContextDataType::Struct:
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(ParameterDefinition.DataTypeObject);
				if (ensure(StructType))
				{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
					TValueOrError<FStructView, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueStruct(PropertyBagPropertyDesc, StructType);
#else
					TValueOrError<FStructView, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueStruct(PropertyBagPropertyDesc.Name, StructType);
#endif
					ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
				}
			}
			break;
		case ECameraContextDataType::Object:
			{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
				TValueOrError<UObject*, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueObject(PropertyBagPropertyDesc);
#else
				TValueOrError<UObject*, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueObject(PropertyBagPropertyDesc.Name);
#endif
				ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
			}
			break;
		case ECameraContextDataType::Class:
			{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
				TValueOrError<UClass*, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueClass(PropertyBagPropertyDesc);
#else
				TValueOrError<UClass*, EPropertyBagResult> ParameterValueOrError = PropertyBag.GetValueClass(PropertyBagPropertyDesc.Name);
#endif
				ApplyDataParameterOverride(ParameterDefinition, ParameterValueOrError, ContextDataTable);
			}
			break;
		default:
			ensure(false);
			break;
	}
}

void ApplyDataParameterArrayOverride(
		const UObject* CameraObject,
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FInstancedPropertyBag& PropertyBag,
		const FPropertyBagPropertyDesc& PropertyBagPropertyDesc,
		FCameraContextDataTable& ContextDataTable)
{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> ArrayOrError = PropertyBag.GetArrayRef(PropertyBagPropertyDesc);
#else
	TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> ArrayOrError = PropertyBag.GetArrayRef(PropertyBagPropertyDesc.Name);
#endif
	if (!ensureMsgf(
				ArrayOrError.HasValue() && !ArrayOrError.HasError(),
				TEXT("Camera parameter has no valid value! Error: %s"),
				*UEnum::GetValueAsString(ArrayOrError.GetError())))
	{
		return;
	}

	const FPropertyBagArrayRef& ArrayRef = ArrayOrError.GetValue();
	const int32 ArrayNum = ArrayRef.Num();

	const bool bSetNumSuccess = ContextDataTable.TrySetArrayDataNum(ParameterDefinition.DataID, ArrayNum);
	if (!ensureMsgf(bSetNumSuccess, TEXT("Camera parameter array '%s' can't be resized!"), *ParameterDefinition.ParameterName.ToString()))
	{
		return;
	}

	switch (ParameterDefinition.DataType)
	{
		case ECameraContextDataType::Name:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				TValueOrError<FName, EPropertyBagResult> ElementValueOrError = ArrayRef.GetValueName(Index);
				ApplyDataParameterElementOverride(ParameterDefinition, ElementValueOrError, Index, ContextDataTable);
			}
			break;
		case ECameraContextDataType::String:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				TValueOrError<FString, EPropertyBagResult> ParameterValueOrError = ArrayRef.GetValueString(Index);
				ApplyDataParameterElementOverride(ParameterDefinition, ParameterValueOrError, Index, ContextDataTable);
			}
			break;
		case ECameraContextDataType::Enum:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				const UEnum* EnumType = Cast<const UEnum>(ParameterDefinition.DataTypeObject);
				if (ensure(EnumType))
				{
					TValueOrError<uint8, EPropertyBagResult> ParameterValueOrError = ArrayRef.GetValueEnum(Index, EnumType);
					ApplyDataParameterElementOverride(ParameterDefinition, ParameterValueOrError, Index, ContextDataTable);
				}
			}
			break;
		case ECameraContextDataType::Struct:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				const UScriptStruct* StructType = Cast<const UScriptStruct>(ParameterDefinition.DataTypeObject);
				if (ensure(StructType))
				{
					TValueOrError<FStructView, EPropertyBagResult> ParameterValueOrError = ArrayRef.GetValueStruct(Index, StructType);
					ApplyDataParameterElementOverride(ParameterDefinition, ParameterValueOrError, Index, ContextDataTable);
				}
			}
			break;
		case ECameraContextDataType::Object:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				TValueOrError<UObject*, EPropertyBagResult> ParameterValueOrError = ArrayRef.GetValueObject(Index);
				ApplyDataParameterElementOverride(ParameterDefinition, ParameterValueOrError, Index, ContextDataTable);
			}
			break;
		case ECameraContextDataType::Class:
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				TValueOrError<UClass*, EPropertyBagResult> ParameterValueOrError = ArrayRef.GetValueClass(Index);
				ApplyDataParameterElementOverride(ParameterDefinition, ParameterValueOrError, Index, ContextDataTable);
			}
			break;
		default:
			ensure(false);
			break;
	}
}

void ApplyDataParameterOverride(
		const UObject* CameraObject,
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FInstancedPropertyBag& PropertyBag,
		const FPropertyBagPropertyDesc& PropertyBagPropertyDesc,
		FCameraContextDataTable& ContextDataTable)
{
	ensure(ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Data);

	if (!ParameterDefinition.DataID)
	{
#if !WITH_EDITOR
		UE_LOGF(LogCameraSystem, Error,
				"Invalid data parameter override '%ls' in camera rig '%ls'. Was it built/cooked?",
				*ParameterDefinition.ParameterName.ToString(),
				*GetPathNameSafe(CameraObject));
		return;
#endif
	}

	if (ParameterDefinition.DataContainerType == ECameraContextDataContainerType::None)
	{
		ApplyDataParameterSingleOverride(CameraObject, ParameterDefinition, PropertyBag, PropertyBagPropertyDesc, ContextDataTable);
	}
	else if (ParameterDefinition.DataContainerType == ECameraContextDataContainerType::Array)
	{
		ApplyDataParameterArrayOverride(CameraObject, ParameterDefinition, PropertyBag, PropertyBagPropertyDesc, ContextDataTable);
	}

}

}  // namespace Internal

FCameraObjectInterfaceParameterOverrideHelper::FCameraObjectInterfaceParameterOverrideHelper(FCameraVariableTable* OutVariableTable, FCameraContextDataTable* OutContextDataTable)
	: VariableTable(OutVariableTable)
	, ContextDataTable(OutContextDataTable)
{
}

FCameraObjectInterfaceParameterOverrideHelper::FCameraObjectInterfaceParameterOverrideHelper(FCameraNodeEvaluationResult& OutResult)
	: VariableTable(&OutResult.VariableTable)
	, ContextDataTable(&OutResult.ContextDataTable)
{
}

void FCameraObjectInterfaceParameterOverrideHelper::ApplyFilteredParameters(
	const UObject* CameraObject,
	TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions,
	const FInstancedPropertyBag& ParameterOverrides,
	TFunctionRef<bool(const FCameraObjectInterfaceParameterDefinition&)> ParameterFilter)
{
	using namespace Internal;

	const UPropertyBag* ParameterOverridesStruct = ParameterOverrides.GetPropertyBagStruct();
	const uint8* ParameterOverridesMemory = ParameterOverrides.GetValue().GetMemory();
	if (!CameraObject || !ParameterOverridesStruct || !ParameterOverridesMemory)
	{
		return;
	}

	for (const FCameraObjectInterfaceParameterDefinition& Definition : ParameterDefinitions)
	{
		if ((!VariableTable && Definition.ParameterType == ECameraObjectInterfaceParameterType::Blendable) ||
				(!ContextDataTable && Definition.ParameterType == ECameraObjectInterfaceParameterType::Data))
		{
			continue;
		}

		if (!ParameterFilter(Definition))
		{
			continue;
		}

		const FPropertyBagPropertyDesc* PropertyDesc = ParameterOverridesStruct->FindPropertyDescByID(Definition.ParameterGuid);
		if (!ensure(PropertyDesc))
		{
			continue;
		}
		
		ApplyParameterValue(CameraObject, Definition, ParameterOverrides, *PropertyDesc);
	}
}

void FCameraObjectInterfaceParameterOverrideHelper::ApplyParameters(
		const UBaseCameraObject* CameraObject,
		const FInstancedPropertyBag& Parameters)
{
	// Apply all parameters in the property bag.
	ApplyFilteredParameters(
			CameraObject,
			CameraObject->GetParameterDefinitions(),
			Parameters,
			[](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
			{
				return true;
			});
}

void FCameraObjectInterfaceParameterOverrideHelper::ApplyParameterOverrides(
		const UBaseCameraObject* CameraObject,
		const FInstancedOverridablePropertyBag& ParameterOverrides)
{
	// Apply parameters that have been overriden in the property bag.
	ApplyFilteredParameters(
			CameraObject,
			CameraObject->GetParameterDefinitions(),
			ParameterOverrides,
			[ParameterOverrides](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
			{
				return ParameterOverrides.IsPropertyOverriden(Definition.ParameterGuid);
			});
}

void FCameraObjectInterfaceParameterOverrideHelper::ApplyParameterDefaults(const UBaseCameraObject* CameraObject, bool bUnwrittenOnly)
{
	// Apply all default parameters, but only if they haven't been set in the variable/context data tables.
	ApplyFilteredParameters(
			CameraObject,
			CameraObject->GetParameterDefinitions(),
			CameraObject->GetDefaultParameters(),
			[this, bUnwrittenOnly](const FCameraObjectInterfaceParameterDefinition& Definition) -> bool
			{
				if (bUnwrittenOnly)
				{
					if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
					{
						return Definition.VariableID.IsValid() && !VariableTable->IsValueWritten(Definition.VariableID);
					}
					else if (Definition.ParameterType == ECameraObjectInterfaceParameterType::Data)
					{
						return Definition.DataID.IsValid() && !ContextDataTable->IsValueWritten(Definition.DataID);
					}
					return false;
				}
				return true;
			});
}

void FCameraObjectInterfaceParameterOverrideHelper::ApplyParameterValue(
		const UObject* CameraObject,
		const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
		const FInstancedPropertyBag& PropertyBag,
		const FPropertyBagPropertyDesc& PropertyBagPropertyDesc)
{
	using namespace Internal;

	ensure(ParameterDefinition.ParameterGuid == PropertyBagPropertyDesc.ID);

	switch (ParameterDefinition.ParameterType)
	{
		case ECameraObjectInterfaceParameterType::Blendable:
			{
				if (ensure(VariableTable))
				{
					ApplyBlendableParameterOverride(
							CameraObject, 
							ParameterDefinition,
							PropertyBag, PropertyBagPropertyDesc, 
							*VariableTable,
							bDrivenOnly);
				}
			}
			break;
		case ECameraObjectInterfaceParameterType::Data:
			{
				if (ensure(ContextDataTable) && !bDrivenOnly)
				{
					ApplyDataParameterOverride(
							CameraObject, 
							ParameterDefinition,
							PropertyBag, PropertyBagPropertyDesc, 
							*ContextDataTable);
				}
			}
			break;
	}
}

}  // namespace UE::Cameras

