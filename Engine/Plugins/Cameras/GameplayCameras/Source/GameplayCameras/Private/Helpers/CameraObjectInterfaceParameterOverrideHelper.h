// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Misc/Guid.h"
#include "Templates/FunctionFwd.h"

class UBaseCameraObject;
class UCameraAsset;
class UObject;
struct FCameraObjectInterfaceParameterDefinition;
struct FInstancedOverridablePropertyBag;
struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;

namespace UE::Cameras
{

class FCameraContextDataTable;
class FCameraVariableTable;
struct FCameraNodeEvaluationResult;

/**
 * A helper class for applying camera object interface parameter overrides from a property bag, 
 * such as with camera asset references, camera rig asset references, and camera shake asset
 * references.
 */
struct FCameraObjectInterfaceParameterOverrideHelper
{
public:

	/** 
	 * Creates a new helper instance.
	 *
	 * The given variable or context data tables can be null, in which case blendable or data
	 * interface parameters will be skipped.
	 */
	FCameraObjectInterfaceParameterOverrideHelper(FCameraVariableTable* OutVariableTable, FCameraContextDataTable* OutContextDataTable);

	/** Creates a new helper instance. */
	FCameraObjectInterfaceParameterOverrideHelper(FCameraNodeEvaluationResult& OutResult);

public:

	/** Sets all values of interface parameters in the given variable and context data tables. */
	void ApplyParameters(
			const UBaseCameraObject* CameraObject,
			const FInstancedPropertyBag& Parameters);

	/** Sets overriden values of interface parameters in the given variable and context data tables. */
	void ApplyParameterOverrides(
			const UBaseCameraObject* CameraObject,
			const FInstancedOverridablePropertyBag& ParameterOverrides);

	/** Sets all of the given object's default parameter values in the given variable and context data tables. */
	void ApplyParameterDefaults(
			const UBaseCameraObject* CameraObject,
			bool bUnwrittenOnly = true);

public:

	/** Sets the parameter values for those that pass the given filter. */
	void ApplyFilteredParameters(
		const UObject* CameraObject,
		TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions,
		const FInstancedPropertyBag& ParameterOverrides,
		TFunctionRef<bool(const FCameraObjectInterfaceParameterDefinition&)> ParameterFilter);

private:

	void ApplyParameterValue(
			const UObject* CameraObject,
			const FCameraObjectInterfaceParameterDefinition& ParameterDefinition,
			const FInstancedPropertyBag& PropertyBag,
			const FPropertyBagPropertyDesc& PropertyBagPropertyDesc);

public:

	/** Only parameters that are dynamicaly driven should be applied. */
	bool bDrivenOnly = false;

private:

	FCameraVariableTable* VariableTable;
	FCameraContextDataTable* ContextDataTable;
};

}  // namespace UE::Cameras

