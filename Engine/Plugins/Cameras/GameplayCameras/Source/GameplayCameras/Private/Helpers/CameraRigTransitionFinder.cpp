// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CameraRigTransitionFinder.h"

#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigTransition.h"

namespace UE::Cameras
{

	const UCameraRigTransition* FCameraRigTransitionFinder::FindTransition(
		TArrayView<const TObjectPtr<UCameraRigTransition>> Transitions, 
		TSharedPtr<const FCameraEvaluationContext> FromEvaluationContext,
		const UCameraRigAsset* FromCameraRig, const UCameraAsset* FromCameraAsset, bool bFromFrozen,
		TSharedPtr<const FCameraEvaluationContext> ToEvaluationContext,
		const UCameraRigAsset* ToCameraRig, const UCameraAsset* ToCameraAsset)
{
	FCameraRigTransitionConditionMatchParams MatchParams;
	MatchParams.FromEvaluationContext = FromEvaluationContext;
	MatchParams.FromCameraRig = FromCameraRig;
	MatchParams.FromCameraAsset = FromCameraAsset;
	MatchParams.ToEvaluationContext = ToEvaluationContext;
	MatchParams.ToCameraRig = ToCameraRig;
	MatchParams.ToCameraAsset = ToCameraAsset;

	// The transition should be used if all its conditions pass.
	for (TObjectPtr<const UCameraRigTransition> Transition : Transitions)
	{
		if (Transition)
		{
			const bool bConditionsPass = Transition->AllConditionsMatch(MatchParams);
			if (bConditionsPass)
			{
				return Transition;
			}
		}
	}

	return nullptr;
}

}  // namespace UE::Cameras


