// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Templates/SharedPointerFwd.h"

class UCameraAsset;
class UCameraRigAsset;
class UCameraRigTransition;

namespace UE::Cameras
{

class FCameraEvaluationContext;

/**
 * Helper class for finding a camera rig transition that matches a given situation.
 */
class FCameraRigTransitionFinder
{
public:
	static const UCameraRigTransition* FindTransition(
			TArrayView<const TObjectPtr<UCameraRigTransition>> Transitions, 
			TSharedPtr<const FCameraEvaluationContext> FromEvaluationContext,
			const UCameraRigAsset* FromCameraRig, const UCameraAsset* FromCameraAsset, bool bFromFrozen,
			TSharedPtr<const FCameraEvaluationContext> ToEvaluationContext,
			const UCameraRigAsset* ToCameraRig, const UCameraAsset* ToCameraAsset);
};

}  // namespace UE::Cameras

