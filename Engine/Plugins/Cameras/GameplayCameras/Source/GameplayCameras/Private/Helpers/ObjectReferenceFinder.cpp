// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/ObjectReferenceFinder.h"

#include "UObject/Object.h"
#include "UObject/Package.h"

namespace UE::Cameras
{

FObjectReferenceFinder::FObjectReferenceFinder(UObject* InRootObject, TArrayView<UObject*> InReferencedObjects)
{
	Initialize(InRootObject);

	TargetObjects = TSet<UObject*>(InReferencedObjects);
}

void FObjectReferenceFinder::Initialize(UObject* InRootObject)
{
	RootObject = InRootObject;
	PackageScope = InRootObject->GetOutermost();

	SetIsPersistent(true);
	SetIsSaving(true);
	SetFilterEditorOnly(false);

	ArIsObjectReferenceCollector = true;
	ArShouldSkipBulkData = true;
}

void FObjectReferenceFinder::CollectReferences()
{
	ObjectsToVisit.Reset();
	VisitedObjects.Reset();

	ObjectsToVisit.Add(RootObject);
	while (ObjectsToVisit.Num() > 0)
	{
		UObject* CurObj = ObjectsToVisit.Pop(EAllowShrinking::No);
		if (!VisitedObjects.Contains(CurObj))
		{
			VisitedObjects.Add(CurObj);
			CurObj->Serialize(*this);
		}
	}
}

bool FObjectReferenceFinder::HasAnyObjectReference() const
{
	for (TPair<UObject*, int32> Pair : ObjectReferenceCounts)
	{
		if (Pair.Value > 0)
		{
			return true;
		}
	}
	return false;
}

int32 FObjectReferenceFinder::GetObjectReferenceCount(UObject* InObject) const
{
	if (const int32* RefCount = ObjectReferenceCounts.Find(InObject))
	{
		return *RefCount;
	}
	return 0;
}

FArchive& FObjectReferenceFinder::operator<<(UObject*& ObjRef)
{
	if (ObjRef != nullptr)
	{
		if (TargetObjects.Contains(ObjRef))
		{
			int32& RefCount = ObjectReferenceCounts.FindOrAdd(ObjRef, 0);
			++RefCount;
		}

		if (ObjRef->IsIn(PackageScope) && !VisitedObjects.Contains(ObjRef))
		{
			ObjectsToVisit.Add(ObjRef);
		}
	}
	return *this;
}

}  // namespace UE::Cameras

