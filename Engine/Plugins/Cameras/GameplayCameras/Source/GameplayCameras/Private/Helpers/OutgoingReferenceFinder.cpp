// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/OutgoingReferenceFinder.h"

#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

namespace UE::Cameras
{

FOutgoingReferenceFinder::FOutgoingReferenceFinder(UObject* InRootObject, UClass* InReferencedObjectClass)
{
	Initialize(InRootObject);

	TargetObjectClasses.Add(InReferencedObjectClass);
}

FOutgoingReferenceFinder::FOutgoingReferenceFinder(UObject* InRootObject, TArrayView<UClass*> InReferencedObjectClasses)
{
	Initialize(InRootObject);

	TargetObjectClasses.Append(InReferencedObjectClasses);
}

void FOutgoingReferenceFinder::Initialize(UObject* InRootObject)
{
	// Skip packages that correspond to C++ modules and other compiled things.
	ExcludePackageFlags |= PKG_CompiledIn;

	RootObject = InRootObject;
	PackageScope = InRootObject->GetOutermost();

	SetIsPersistent(true);
	SetIsSaving(true);
	SetFilterEditorOnly(false);

	ArIsObjectReferenceCollector = true;
	ArShouldSkipBulkData = true;
}

void FOutgoingReferenceFinder::CollectReferences()
{
	ValidMaxDistance = FMath::Clamp(MaxDistance, 0, 5);

	ObjectsToVisit.Reset();
	VisitedObjects.Reset();

	ObjectsToVisit.Add({ RootObject, PackageScope, 0 });
	while (ObjectsToVisit.Num() > 0)
	{
		FObjectToVisit CurObj = ObjectsToVisit.Pop(EAllowShrinking::No);
		if (!VisitedObjects.Contains(CurObj.Obj))
		{
			VisitedObjects.Add(CurObj.Obj);

			SerializeState = { CurObj.Obj, CurObj.Package, CurObj.Distance };
			CurObj.Obj->Serialize(*this);
			SerializeState = FSerializeState();
		}
	}
}

bool FOutgoingReferenceFinder::GetAllReferences(TArray<UObject*>& OutReferencedObjects) const
{
	bool bHasAnyReference = false;
	for (TPair<UClass*, TSet<UObject*>> Pair : ReferencedObjects)
	{
		for (UObject* Obj : Pair.Value)
		{
			OutReferencedObjects.Add(Obj);
		}
		bHasAnyReference |= (!Pair.Value.IsEmpty());
	}
	return bHasAnyReference;
}

FArchive& FOutgoingReferenceFinder::operator<<(UObject*& ObjRef)
{
	if (ObjRef != nullptr)
	{
		UClass* ObjClass = ObjRef->GetClass();
		if (MatchesAnyTargetClass(ObjClass))
		{
			TSet<UObject*>& ReferencesOfClass = ReferencedObjects.FindOrAdd(ObjClass);
			ReferencesOfClass.Add(ObjRef);
		}

		if (!VisitedObjects.Contains(ObjRef))
		{
			UPackage* ObjRefPackage = ObjRef->GetOutermost();
			// Note that this distance is maybe not the "correct" one. That is: there may be another path to ObjRef that 
			// has a lower distance. But that's OK. Either we queue a visit of ObjRef right now, and we'll ignore the
			// duplicate with the shorter distance, or the current distance is over the threshold and we skip visiting
			// it now, but we'll visit it when we encounter it again with a distance under the threshold.
			const int32 AddOneDistance = (ObjRefPackage == SerializeState.CurrentPackage ? 0 : 1);
			const int32 ObjRefDistance = SerializeState.CurrentDistance + AddOneDistance;
			const bool bExcludePackage = ((ObjRefPackage->GetPackageFlags() & ExcludePackageFlags) != 0);
			const bool bExcludeObject = IsIgnoredClass(ObjClass);
			if (ObjRefDistance <= ValidMaxDistance && !bExcludePackage && !bExcludeObject)
			{
				ObjectsToVisit.Add({ ObjRef, ObjRefPackage, ObjRefDistance });
			}
		}
	}
	return *this;
}

bool FOutgoingReferenceFinder::MatchesAnyTargetClass(UClass* InObjClass) const
{
	for (UClass* TargetObjectClass : TargetObjectClasses)
	{
		if (InObjClass->IsChildOf(TargetObjectClass))
		{
			return true;
		}
	}

	return false;
}

bool FOutgoingReferenceFinder::IsIgnoredClass(UClass* InObjClass) const
{
	return ExcludeClasses.ContainsByPredicate(
			[InObjClass](UClass* ExcludeClass)
			{
				return ExcludeClass ? InObjClass->IsChildOf(ExcludeClass) : false;
			});
}

}  // namespace UE::Cameras

