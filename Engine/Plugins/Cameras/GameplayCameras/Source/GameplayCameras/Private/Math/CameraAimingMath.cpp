// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/CameraAimingMath.h"

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigJoints.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Cameras
{

ECameraAimingResult FCameraAimingMath::ComputeCorrection(const FCameraNodeEvaluationResult& Result, const FVector3d& DesiredTarget, FRotator3d& OutCorrection)
{
	if (const FCameraRigJoint* FoundItem = FindPivotJoint(Result))
	{
		const FCameraRigJoint& PivotJoint(*FoundItem);
		const bool bGotCorrection = ComputeTwoBonesCorrection(Result.CameraPose, PivotJoint.Transform.GetLocation(), DesiredTarget, OutCorrection);
		return bGotCorrection ? ECameraAimingResult::Success : ECameraAimingResult::NoSolution;
	}
	return ECameraAimingResult::NoSolver;
}

bool FCameraAimingMath::ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection)
{
	// This is roughly the situation we are looking at, as seen from above:
	//
	//                         T
	//                         |
	//                         |    	
	//             , - ~ D ~ - X
	//         , '       |    /| ' ,
	//       ,           |   / |     ,
	//      ,            |  /  |      ,
	//     ,             | /   |       ,
	//     ,             |/    |       ,
	//     ,             P     |       ,
	//     ,              .    |       ,
	//      ,               .  |      ,
	//       ,                .C     ,
	//         ,                  , '
	//           ' - , _ _ _ ,  '
	//    
	// ...where:
	//
	//    P : the pivot
	//    C : the current camera position
	//    T : the current camera target
	//    D : the desired camera target
	//
	// The sphere is centered on P, with its radius determined by D.
	// The intersection of the camera's current line of sight with this circle is X.
	// What we want is to turn the camera by A, the angle between PD and PX.
	//
	// We first compute the sphere's properties:
	const FVector3d PivotToDesiredTarget = (DesiredTarget - PivotLocation);
	const double SphereRadius = PivotToDesiredTarget.Length();

	// Next we compute the intersection between the camera's line of sight and that sphere.
	const FVector3d CameraLocation = CurrentPose.GetLocation();
	const FVector3d CameraAim = CurrentPose.GetAimDir();

	double DistanceToX = 0.0;
	const bool bGotX = RaySphereIntersectExit(CameraLocation, CameraAim, PivotLocation, SphereRadius, DistanceToX);
	if (!bGotX)
	{
		return false;
	}

	// Finally we compute the angle between PX and PD.
	const FVector3d X = CameraLocation + CameraAim * DistanceToX;
	const FVector3d PX = (X - PivotLocation);
	const FVector3d PD = (DesiredTarget - PivotLocation);
	// IMPORTANT NOTE: This assumes a vertical pivot axis!
	const FRotator3d RotPX = PX.ToOrientationRotator();
	const FRotator3d RotPD = PD.ToOrientationRotator();
	OutCorrection = RotPD - RotPX;
	return true;
}

bool FCameraAimingMath::ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FRotator3d& AimOffsetAngles, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection)
{
	// See above for the math. The only difference is that we offset CameraAim by AimOffsetAngles.
	const FVector3d PivotToDesiredTarget = (DesiredTarget - PivotLocation);
	const double SphereRadius = PivotToDesiredTarget.Length();

	const FVector3d CameraLocation = CurrentPose.GetLocation();
	const FRotator3d CameraRotation = CurrentPose.GetRotation();
	const FVector3d OffsetCameraAim = CameraRotation.RotateVector(AimOffsetAngles.RotateVector(FVector3d::ForwardVector));

	double DistanceToX = 0.0;
	const bool bGotX = RaySphereIntersectExit(CameraLocation, OffsetCameraAim, PivotLocation, SphereRadius, DistanceToX);
	if (!bGotX)
	{
		return false;
	}

	const FVector3d X = CameraLocation + OffsetCameraAim * DistanceToX;
	const FVector3d PX = (X - PivotLocation);
	const FVector3d PD = (DesiredTarget - PivotLocation);
	const FRotator3d RotPX = PX.ToOrientationRotator();
	const FRotator3d RotPD = PD.ToOrientationRotator();
	OutCorrection = RotPD - RotPX;
	return true;
}

bool FCameraAimingMath::ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FVector3d& CustomAim, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection)
{
	// See above for the math. The only difference is that we use a custom camera aim vector.
	const FVector3d PivotToDesiredTarget = (DesiredTarget - PivotLocation);
	const double SphereRadius = PivotToDesiredTarget.Length();

	const FVector3d CameraLocation = CurrentPose.GetLocation();
	const FVector3d NormalizedCustomAim = CustomAim.GetUnsafeNormal();

	double DistanceToX = 0.0;
	const bool bGotX = RaySphereIntersectExit(CameraLocation, NormalizedCustomAim, PivotLocation, SphereRadius, DistanceToX);
	if (!bGotX)
	{
		return false;
	}

	const FVector3d X = CameraLocation + NormalizedCustomAim * DistanceToX;
	const FVector3d PX = (X - PivotLocation);
	const FVector3d PD = (DesiredTarget - PivotLocation);
	const FRotator3d RotPX = PX.ToOrientationRotator();
	const FRotator3d RotPD = PD.ToOrientationRotator();
	OutCorrection = RotPD - RotPX;
	return true;
}

const FCameraRigJoint* FCameraAimingMath::FindPivotJoint(const FCameraNodeEvaluationResult& Result)
{
	return FindPivotJoint(Result.CameraRigJoints);
}

const FCameraRigJoint* FCameraAimingMath::FindPivotJoint(const FCameraRigJoints& CameraRigJoints)
{
	const FCameraVariableDefinition& YawPitchDefinition = FBuiltInCameraVariables::Get().YawPitchDefinition;
	const FCameraRigJoint* FoundItem = CameraRigJoints.GetJoints().FindByPredicate(
			[&YawPitchDefinition](const FCameraRigJoint& Item)
			{
				return Item.VariableID == YawPitchDefinition.VariableID;
			});
	return FoundItem;
}

bool FCameraAimingMath::RaySphereIntersectExit(const FRay3d& Ray, const FVector3d& SphereOrigin, double SphereRadius, double& OutRayIntersectDistance)
{
	return RaySphereIntersectExit(Ray.Origin, Ray.Direction, SphereOrigin, SphereRadius, OutRayIntersectDistance);
}

bool FCameraAimingMath::RaySphereIntersectExit(const FVector3d& RayStart, const FVector3d& RayDir, const FVector3d& SphereOrigin, double SphereRadius, double& OutRayIntersectDistance)
{
	// The equation for a point on the sphere is:
	//
	//    (x - Ox)^2 + (y - Oy)^2 + (z - Oz)^2 = R^2
	//
	// ...where:
	//    x,y,z are the coordinates of the point
	//    Ox,Oy,Oz are the coordinates of the sphere's origin
	//    R is the radius of the sphere
	//
	// The equation for a point on the ray is:
	//
	//    x = rx + L*dx
	//    y = ry + L*dy
	//    z = rz + L*dz
	//
	// ...where:
	//    rx,ry,rz are the coordinates of the ray's start
	//    dx,dy,dz are the coordinates of the ray's direction vector
	//    L is the linear coordinate of the point along the ray
	//
	// So we have:
	//
	//    (rx + L*dx - Ox)^2 + (ry + L*dy - Oy)^2 + (rz + L*dz - Oz)^2 = R^2
	//
	// Decomposing each term, we have:
	//
	//    dx*dx*L^2 + (rx - Ox)^2 + 2*L*dx*(rx - Ox)
	//    dy*dy*L^2 + (ry - Oy)^2 + 2*L*dy*(ry - Oy)
	//    dz*dz*L^2 + (rz - Oz)^2 + 2*L*dz*(rz - Oz)
	//
	// We can group then around L, which is the variable we're solving for:
	//
	//    (dx*dx + dy*dy + dz*dz)*L^2 + 2*(dx*Fx + dy*Fy + dz*Fz)*L + (Fx*Fx + Fy*Fy + Fz*Fz) = R^2
	//
	// ...where Fx,Fy,Fz are the coordinates of the vector from the sphere origin to 
	// the ray's start.
	//
	// This is an equation of the form:  
	//
	//    a*L^2 + b*L + c = 0
	//
	// ...where:
	//
	//    a = (dx*dx + dy*dy + dz*dz) = d.d
	//    b = 2*(dx*Fx + dy*Fy + dz*Fz) = 2*d.F
	//    c = (Fx*Fx + Fy*Fy + Fz*Fz) - R^2 = F.F - R^2
	//
	// The discriminant for an equation of this form is:
	//
	//    D = b^2 - 4*a*c
	//
	// If the discriminant is positive, the solutions for the equations are:
	//
	//    L = (-b +- sqrt(D)) / (2*a)
	//
	// If the discriminant is zero, there's only one solution (such as when the ray barely "touches"
	// the sphere). If the discriminant is negative, there's no real solution.
	//
	// In practice, we only want the solution that's "in front" of the ray, and is the "farthest",
	// because we only want the exit point.
	//
	const FVector3d F = (RayStart - SphereOrigin);
	const double dxd = RayDir.SizeSquared();
	const double FxF = F.SizeSquared();
	const double dxF = FVector3d::DotProduct(RayDir, F);

	const double a = dxd;
	const double b = 2.0 * dxF;
	const double c = FxF - SphereRadius * SphereRadius;

	const double D = b * b - 4.0 * a * c;

	if (D > 0.0)
	{
		const double SqrtD = FMath::Sqrt(D);
		const double two_a = 2.0 * a;

		const double L1 = (-b + SqrtD) / two_a; 
		const double L2 = (-b - SqrtD) / two_a; 

		OutRayIntersectDistance = FMath::Max(L1, L2);
		return true;
	}

	if (D == 0.0)
	{
		const double SqrtD = FMath::Sqrt(D);
		const double two_a = 2.0 * a;

		const double L = (-b / two_a);

		OutRayIntersectDistance = L;
		return L >= 0.0;
	}

	// D < 0.0
	return false;
}

double FCameraAimingMath::GetAngleBetween(const FVector3d& AimA, const FVector3d& AimB)
{
	const FQuat4d Quat = FQuat4d::FindBetweenVectors(AimA, AimB);
	return Quat.GetAngle();
}

FRotator3d FCameraAimingMath::GetRotationBetween(const FVector3d& AimA, const FVector3d& AimB)
{
	const FQuat4d Quat = FQuat4d::FindBetweenVectors(AimA, AimB);
	return Quat.Rotator();
}

FRotator3d FCameraAimingMath::GetNoRollRotationBetween(const FVector3d& AimA, const FVector3d& AimB)
{
	const double SqLenA = AimA.SquaredLength();
	const double SqLenB = AimB.SquaredLength();
	if (SqLenA > UE_DOUBLE_SMALL_NUMBER && SqLenB > UE_DOUBLE_SMALL_NUMBER)
	{
		const FRotator3d RotationA = AimA.ToOrientationRotator();
		const FRotator3d RotationB = AimB.ToOrientationRotator();
		ensure(RotationA.Roll == 0.0 && RotationB.Roll == 0.0);
		return (RotationB - RotationA);
	}
	else
	{
		return FRotator3d::ZeroRotator;
	}
}

}  // namespace UE::Cameras

