// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/CameraFramingZoneMath.h"

#include "Algo/Sort.h"
#include "Nodes/Framing/CameraFramingZone.h"

namespace UE::Cameras
{

FFramingZone FFramingZone::FromScreenMargins(const FCameraFramingZone& Margins)
{
	FFramingZone NewZone;

	NewZone.LeftBound = Margins.Left;
	NewZone.TopBound = Margins.Top;
	NewZone.RightBound = 1.0 - Margins.Right;
	NewZone.BottomBound = 1.0 - Margins.Bottom;

	NewZone.ClampBounds();

	return NewZone;
}

FFramingZone FFramingZone::FromRelativeMargins(const FVector2d& ScreenLocation, const FCameraFramingZone& Margins)
{
	FFramingZone NewZone;

	NewZone.LeftBound = ScreenLocation.X - Margins.Left;
	NewZone.TopBound = ScreenLocation.Y - Margins.Top;
	NewZone.RightBound = ScreenLocation.X + Margins.Right;
	NewZone.BottomBound = ScreenLocation.Y + Margins.Bottom;

	NewZone.ClampBounds();

	return NewZone;
}

FFramingZone FFramingZone::FromPoints(TConstArrayView<FVector2d> ScreenPoints)
{
	FFramingZone NewZone;

	if (!ensure(ScreenPoints.Num() > 0))
	{
		return NewZone;
	}

	const FVector2d& FirstPoint = ScreenPoints[0];
	NewZone.LeftBound = FirstPoint.X;
	NewZone.TopBound = FirstPoint.Y;
	NewZone.RightBound = FirstPoint.X;
	NewZone.BottomBound = FirstPoint.Y;

	for (int32 Index = 1; Index < ScreenPoints.Num(); ++Index)
	{
		const FVector2d& Point(ScreenPoints[Index]);
		NewZone.LeftBound = FMath::Min(NewZone.LeftBound, Point.X);
		NewZone.TopBound = FMath::Min(NewZone.TopBound, Point.Y);
		NewZone.RightBound = FMath::Max(NewZone.RightBound, Point.X);
		NewZone.BottomBound = FMath::Max(NewZone.BottomBound, Point.Y);
	}

	return NewZone;
}

FFramingZone::FFramingZone()
{
	LeftBound = 0;
	TopBound = 0;
	RightBound = 1;
	BottomBound = 1;
}

bool FFramingZone::IsValid() const
{
	return LeftBound >= 0 && LeftBound <= 1 &&
		TopBound >= 0 && TopBound <= 1 &&
		RightBound >= 0 && RightBound <= 1 &&
		BottomBound >= 0 && BottomBound <= 1 &&
		LeftBound <= RightBound &&
		TopBound <= BottomBound;
}

void FFramingZone::ClampBounds()
{
	LeftBound = FMath::Clamp(LeftBound, 0.0, 1.0);
	TopBound = FMath::Clamp(TopBound, 0.0, 1.0);
	RightBound = FMath::Clamp(RightBound, 0.0, 1.0);
	BottomBound = FMath::Clamp(BottomBound, 0.0, 1.0);

	RightBound = FMath::Max(LeftBound, RightBound);
	BottomBound = FMath::Max(TopBound, BottomBound);
}

void FFramingZone::ClampBounds(const FVector2d& MustContain)
{
	LeftBound = FMath::Clamp(LeftBound, 0.0, MustContain.X);
	TopBound = FMath::Clamp(TopBound, 0.0, MustContain.Y);
	RightBound = FMath::Clamp(RightBound, MustContain.X, 1.0);
	BottomBound = FMath::Clamp(BottomBound, MustContain.Y, 1.0);
}

void FFramingZone::ClampBounds(const FVector2d& MustContain, double Margin)
{
	LeftBound = FMath::Clamp(LeftBound, 0.0, MustContain.X - Margin);
	TopBound = FMath::Clamp(TopBound, 0.0, MustContain.Y - Margin);
	RightBound = FMath::Clamp(RightBound, MustContain.X + Margin, 1.0);
	BottomBound = FMath::Clamp(BottomBound, MustContain.Y + Margin, 1.0);
}

void FFramingZone::ClampBounds(const FFramingZone& MustContain)
{
	LeftBound = FMath::Clamp(LeftBound, 0.0, MustContain.LeftBound);
	TopBound = FMath::Clamp(TopBound, 0.0, MustContain.TopBound);
	RightBound = FMath::Clamp(RightBound, MustContain.RightBound, 1.0);
	BottomBound = FMath::Clamp(BottomBound, MustContain.BottomBound, 1.0);
}

void FFramingZone::Add(const FFramingZone& Other)
{
	LeftBound = FMath::Min(LeftBound, Other.LeftBound);
	TopBound = FMath::Min(TopBound, Other.TopBound);
	RightBound = FMath::Max(RightBound, Other.RightBound);
	BottomBound = FMath::Max(BottomBound, Other.BottomBound);
}

bool FFramingZone::Contains(const FVector2d& Point) const
{
	return Point.X >= LeftBound && Point.X <= RightBound &&
		Point.Y >= TopBound && Point.Y <= BottomBound;
}

FVector2d FFramingZone::ComputeClosestIntersection(const FVector2d& Origin, const FVector2d& LineDir, bool bLineDirIsNormalized) const
{
	// Points along the line are of the form of:
	//
	//		P = Orig + Dir*d
	//
	// We test this equation against a result that yields an intersection with
	// one of the unbounded lines of the zone. For instance, to see where it
	// intersects with the top bound, we have:
	//
	//		P.y = TopBound
	//		P.y = Orig.y + Dir.y*d
	//		d = (P.y - Orig.y) / Dir.y
	//
	//		P.x = Orig.x + Dir.x*d
	//		P.x = Orig.x + Dir.x*(P.y - Orig.y) / Dir.y
	//		P.x = Orig.x + (TopBound - Orig.y) * (Dir.x / Dir.y)
	//
	// If P.x ends up being between LeftBound and RightBound, we have an intersection
	// there. Otherwise, it misses the zone. Repeat for all four edges, and pick the
	// closest intersection.

	TArray<FVector2d, TInlineAllocator<2>> Intersections;
	const FVector2d Dir = bLineDirIsNormalized ? LineDir : LineDir.GetSafeNormal();

	// Intersection with TopBound/BottomBound.
	if (Dir.Y != 0)
	{
		const double Slope = (Dir.X / Dir.Y);
		// TopBound
		{
			FVector2d Intersection(0, TopBound);
			Intersection.X = Origin.X + (TopBound - Origin.Y) * Slope;
			if (Intersection.X >= LeftBound && Intersection.X <= RightBound)
			{
				Intersections.Add(Intersection);
			}
		}
		// BottomBound
		{
			FVector2d Intersection(0, BottomBound);
			Intersection.X = Origin.X + (BottomBound - Origin.Y) * Slope;
			if (Intersection.X >= LeftBound && Intersection.X <= RightBound)
			{
				Intersections.Add(Intersection);
			}
		}
	}
	// Intersection with LeftBound/RightBound.
	if (Dir.X != 0)
	{
		const double Slope = (Dir.Y / Dir.X);
		// LeftBound
		{
			FVector2d Intersection(LeftBound, 0);
			Intersection.Y = Origin.Y + (LeftBound - Origin.X) * Slope;
			if (Intersection.Y >= TopBound && Intersection.Y <= BottomBound)
			{
				Intersections.Add(Intersection);
			}
		}
		// RightBound
		{
			FVector2d Intersection(RightBound, 0);
			Intersection.Y = Origin.Y + (RightBound - Origin.X) * Slope;
			if (Intersection.Y >= TopBound && Intersection.Y <= BottomBound)
			{
				Intersections.Add(Intersection);
			}
		}
	}

	const int32 NumIntersections = Intersections.Num();
	ensure(NumIntersections > 0 && NumIntersections <= 2);
	if (NumIntersections > 0)
	{
		if (NumIntersections > 1)
		{
			Algo::SortBy(Intersections, [Origin](const FVector2d& Intersection)
					{
						return FVector2d::DistSquared(Origin, Intersection);
					});
		}
		return Intersections[0];
	}
	else
	{
		return FVector2d::ZeroVector;
	}
}

FVector4d FFramingZone::GetNormalizedBounds() const
{
	// Returned margins are negative if in the left or upper halves, and
	// positive if in the right or lower halves.
	return FVector4d(
		GetNormalizedBound(LeftBound),
		GetNormalizedBound(TopBound),
		GetNormalizedBound(RightBound),
		GetNormalizedBound(BottomBound));
}

double FFramingZone::GetNormalizedBound(double Bound)
{
	return (Bound - 0.5) * 2.0;
}

FVector2d FFramingZone::GetCanvasPosition(const FVector2d& CanvasSize) const
{
	return FVector2d(LeftBound * CanvasSize.X, TopBound * CanvasSize.Y);
}

FVector2d FFramingZone::GetCanvasSize(const FVector2d& CanvasSize) const
{
	return FVector2d((RightBound - LeftBound) * CanvasSize.X, (BottomBound - TopBound) * CanvasSize.Y);
}

void FFramingZone::Serialize(FArchive& Ar)
{
	Ar << LeftBound;
	Ar << TopBound;
	Ar << RightBound;
	Ar << BottomBound;
}

FArchive& operator <<(FArchive& Ar, FFramingZone& FramingZone)
{
	FramingZone.Serialize(Ar);
	return Ar;
}

}  // namespace UE::Cameras

