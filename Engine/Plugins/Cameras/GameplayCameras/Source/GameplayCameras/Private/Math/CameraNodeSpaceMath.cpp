// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/CameraNodeSpaceMath.h"

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraPose.h"
#include "Core/CameraRigJoints.h"
#include "Core/CameraSystemEvaluator.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Math/Axis.h"

namespace UE::Cameras
{

FCameraNodeSpaceParams::FCameraNodeSpaceParams(const FCameraNodeEvaluationParams& InEvaluationParams, const FCameraNodeEvaluationResult& InEvaluationResult)
	: EvaluationParams(InEvaluationParams)
	, EvaluationResult(InEvaluationResult)
{
}

TSharedPtr<const FCameraEvaluationContext> FCameraNodeSpaceParams::GetActiveContext() const
{
	if (ensure(EvaluationParams.Evaluator))
	{
		const FCameraEvaluationContextStack& ContextStack = EvaluationParams.Evaluator->GetEvaluationContextStack();
		return ContextStack.GetActiveContext();
	}
	return nullptr;
}

TSharedPtr<const FCameraEvaluationContext> FCameraNodeSpaceParams::GetOwningContext() const
{
	return EvaluationParams.EvaluationContext;
}

bool FCameraNodeSpaceParams::FindPivotTransform(FTransform3d& OutPivotTransform) const
{
	const FBuiltInCameraVariables& BuiltInVariables = FBuiltInCameraVariables::Get();
	TArrayView<const FCameraRigJoint> Joints = EvaluationResult.CameraRigJoints.GetJoints();
	const FCameraRigJoint* PivotJoint = Joints.FindByPredicate([&BuiltInVariables](const FCameraRigJoint& Joint)
			{
				return Joint.VariableID == BuiltInVariables.YawPitchDefinition;
			});
	if (PivotJoint)
	{
		OutPivotTransform = PivotJoint->Transform;
		return true;
	}
	return false;
}

bool FCameraNodeSpaceMath::GetCameraNodeOriginPosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, ECameraNodeOriginPosition InOriginPosition, FVector3d& OutOrigin)
{
	FCameraNodeSpaceParams SpaceParams(Params, Result);
	return GetCameraNodeOriginPosition(SpaceParams, InOriginPosition, OutOrigin);
}

bool FCameraNodeSpaceMath::GetCameraNodeOriginPosition(const FCameraNodeSpaceParams& Params, ECameraNodeOriginPosition InOriginPosition, FVector3d& OutOrigin)
{
	switch (InOriginPosition)
	{
		case ECameraNodeOriginPosition::CameraPose:
			{
				OutOrigin = Params.EvaluationResult.CameraPose.GetLocation();
				return true;
			}
		case ECameraNodeOriginPosition::ActiveContext:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
				OutOrigin = InitialResult.CameraPose.GetLocation();
				return true;
			}
			break;
		case ECameraNodeOriginPosition::OwningContext:
			if (TSharedPtr<const FCameraEvaluationContext> OwningContext = Params.GetOwningContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = OwningContext->GetInitialResult();
				OutOrigin = InitialResult.CameraPose.GetLocation();
				return true;
			}
			break;
		case ECameraNodeOriginPosition::Pivot:
			{
				FTransform PivotTransform;
				if (Params.FindPivotTransform(PivotTransform))
				{
					OutOrigin = PivotTransform.GetLocation();
					return true;
				}
				else if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
				{
					const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
					OutOrigin = InitialResult.CameraPose.GetLocation();
					return true;
				}
			}
			break;
		case ECameraNodeOriginPosition::Pawn:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				if (APlayerController* PlayerController = ActiveContext->GetPlayerController())
				{
					if (APawn* Pawn = PlayerController->GetPawnOrSpectator())
					{
						OutOrigin = Pawn->GetActorTransform().GetLocation();
						return true;
					}
				}
			}
			break;
		default:
			ensureMsgf(false, TEXT("Unsupported camera node space."));
			break;
	}

	return false;
}

bool FCameraNodeSpaceMath::GetCameraNodeSpaceTransform(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, ECameraNodeSpace InSpace, FTransform3d& OutTransform)
{
	FCameraNodeSpaceParams SpaceParams(Params, Result);
	return GetCameraNodeSpaceTransform(SpaceParams, InSpace, OutTransform);
}

bool FCameraNodeSpaceMath::GetCameraNodeSpaceTransform(const FCameraNodeSpaceParams& Params, ECameraNodeSpace InSpace, FTransform3d& OutTransform)
{
	switch (InSpace)
	{
		case ECameraNodeSpace::CameraPose:
			{
				OutTransform = Params.EvaluationResult.CameraPose.GetTransform();
				return true;
			}
		case ECameraNodeSpace::ActiveContext:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
				OutTransform = InitialResult.CameraPose.GetTransform();
				return true;
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (TSharedPtr<const FCameraEvaluationContext> OwningContext = Params.GetOwningContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = OwningContext->GetInitialResult();
				OutTransform = InitialResult.CameraPose.GetTransform();
				return true;
			}
			break;
		case ECameraNodeSpace::Pivot:
			{
				FTransform PivotTransform;
				if (Params.FindPivotTransform(PivotTransform))
				{
					OutTransform = PivotTransform;
					return true;
				}
				else if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
				{
					const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
					OutTransform = InitialResult.CameraPose.GetTransform();
					return true;
				}
			}
			break;
		case ECameraNodeSpace::Pawn:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				if (APlayerController* PlayerController = ActiveContext->GetPlayerController())
				{
					if (APawn* Pawn = PlayerController->GetPawnOrSpectator())
					{
						OutTransform = Pawn->GetActorTransform();
						return true;
					}
				}
			}
			break;
		case ECameraNodeSpace::World:
			{
				OutTransform = FTransform3d::Identity;
				return true;
			}
		default:
			ensureMsgf(false, TEXT("Unsupported camera node space."));
			break;
	}

	return false;
}

bool FCameraNodeSpaceMath::OffsetCameraNodeSpacePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, const FVector3d& InPosition, const FVector3d& InOffset, ECameraNodeSpace InSpace, FVector3d& OutPosition)
{
	FCameraNodeSpaceParams SpaceParams(Params, Result);
	return OffsetCameraNodeSpacePosition(SpaceParams, InPosition, InOffset, InSpace, OutPosition);
}

bool FCameraNodeSpaceMath::OffsetCameraNodeSpacePosition(const FCameraNodeSpaceParams& Params, const FVector3d& InPosition, const FVector3d& InOffset, ECameraNodeSpace InSpace, FVector3d& OutPosition)
{
	FVector3d WorldOffset = FVector3d::ZeroVector;
	bool bGotWorldOffset = false;

	switch (InSpace)
	{
		case ECameraNodeSpace::CameraPose:
			{
				WorldOffset = Params.EvaluationResult.CameraPose.GetRotation().RotateVector(InOffset);
				bGotWorldOffset = true;
			}
			break;
		case ECameraNodeSpace::ActiveContext:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
				WorldOffset = InitialResult.CameraPose.GetRotation().RotateVector(InOffset);
				bGotWorldOffset = true;
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (TSharedPtr<const FCameraEvaluationContext> OwningContext = Params.GetOwningContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = OwningContext->GetInitialResult();
				WorldOffset = InitialResult.CameraPose.GetRotation().RotateVector(InOffset);
				bGotWorldOffset = true;
			}
			break;
		case ECameraNodeSpace::Pivot:
			{
				FTransform PivotTransform;
				if (Params.FindPivotTransform(PivotTransform))
				{
					WorldOffset = PivotTransform.TransformVectorNoScale(InOffset);
					bGotWorldOffset = true;
				}
				else if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
				{
					const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
					WorldOffset = InitialResult.CameraPose.GetRotation().RotateVector(InOffset);
					bGotWorldOffset = true;
				}
			}
			break;
		case ECameraNodeSpace::Pawn:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				if (APlayerController* PlayerController = ActiveContext->GetPlayerController())
				{
					if (APawn* Pawn = PlayerController->GetPawnOrSpectator())
					{
						const FTransform3d& PawnTransform = Pawn->GetActorTransform();
						WorldOffset = PawnTransform.TransformVectorNoScale(InOffset);
						bGotWorldOffset = true;
					}
				}
			}
			break;
		case ECameraNodeSpace::World:
			{
				WorldOffset = InOffset;
				bGotWorldOffset = true;
			}
			break;
		default:
			ensureMsgf(false, TEXT("Unsupported camera node space."));
			break;
	}

	if (bGotWorldOffset)
	{
		OutPosition = InPosition + WorldOffset;
	}

	return bGotWorldOffset;
}

bool FCameraNodeSpaceMath::OffsetCameraNodeSpaceTransform(const FCameraNodeSpaceParams& Params, const FTransform3d& InTransform, const FVector3d& InLocationOffset, const FRotator3d& InRotationOffset, ECameraNodeSpace InSpace, FTransform3d& OutTransform)
{
	FTransform3d LocalSpace;
	bool bApplyLocalSpaceTransform = true;
	bool bIsValid = false;

	switch (InSpace)
	{
		case ECameraNodeSpace::CameraPose:
		default:
			{
				OutTransform = FTransform3d(InRotationOffset, InLocationOffset) * InTransform;
				bApplyLocalSpaceTransform = false;
				bIsValid = true;
			}
			break;
		case ECameraNodeSpace::ActiveContext:
			if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
			{
				const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
				LocalSpace = InitialResult.CameraPose.GetTransform();
				bIsValid = true;
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (TSharedPtr<const FCameraEvaluationContext> OwningContext = Params.GetOwningContext())
			{ 
				const FCameraNodeEvaluationResult& InitialResult = OwningContext->GetInitialResult();
				LocalSpace = InitialResult.CameraPose.GetTransform();
				bIsValid = true;
			}
			break;
		case ECameraNodeSpace::Pivot:
			{
				FTransform PivotTransform;
				if (Params.FindPivotTransform(PivotTransform))
				{
					LocalSpace = PivotTransform;
					bIsValid = true;
				}
				else if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = Params.GetActiveContext())
				{
					const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
					LocalSpace = InitialResult.CameraPose.GetTransform();
					bIsValid = true;
				}
			}
			break;
		case ECameraNodeSpace::World:
			{
				OutTransform = InTransform;
				OutTransform.SetTranslation(InLocationOffset + InTransform.GetTranslation());
				OutTransform.SetRotation(InRotationOffset.Quaternion() * InTransform.GetRotation());
				bApplyLocalSpaceTransform = false;
				bIsValid = true;
			}
			break;
	}

	if (bIsValid && bApplyLocalSpaceTransform)
	{
		const FVector3d WorldTranslationOffset = LocalSpace.TransformVector(InLocationOffset);

		const FVector3d ContextForward = LocalSpace.GetUnitAxis(EAxis::X);
		const FVector3d ContextRight = LocalSpace.GetUnitAxis(EAxis::Y);
		const FVector3d ContextUp = LocalSpace.GetUnitAxis(EAxis::Z);
		const FQuat WorldRotationOffset = 
			FQuat(ContextUp, FMath::DegreesToRadians(InRotationOffset.Yaw)) * 
			FQuat(ContextRight, -FMath::DegreesToRadians(InRotationOffset.Pitch)) *
			FQuat(ContextForward, -FMath::DegreesToRadians(InRotationOffset.Roll));

		OutTransform = InTransform;
		OutTransform.SetTranslation(WorldTranslationOffset + InTransform.GetTranslation());
		OutTransform.SetRotation(WorldRotationOffset * InTransform.GetRotation());
	}

	return bIsValid;
}

}  // namespace UE::Cameras

