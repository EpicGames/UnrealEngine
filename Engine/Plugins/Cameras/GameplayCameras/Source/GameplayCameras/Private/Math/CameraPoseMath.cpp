// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/CameraPoseMath.h"

#include "Core/CameraPose.h"
#include "Core/CameraEvaluationContext.h"
#include "CoreGlobals.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/TranslationMatrix.h"
#include "SceneView.h"

namespace UE::Cameras
{

FArchive& operator<< (FArchive& Ar, FCameraFieldsOfView& FieldsOfView)
{
	Ar << FieldsOfView.HorizontalFieldOfView;
	Ar << FieldsOfView.VerticalFieldOfView;
	return Ar;
}

FCameraFieldsOfView FCameraPoseMath::GetEffectiveFieldsOfView(const FCameraPose& CameraPose)
{
	return GetEffectiveFieldsOfView(CameraPose, CameraPose.GetSensorAspectRatio(), CameraPose.GetEffectiveAspectRatioAxisConstraint());
}

FCameraFieldsOfView FCameraPoseMath::GetEffectiveFieldsOfView(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	const double AspectRatio = GetEffectiveAspectRatio(CameraPose, EvaluationContext);
	EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint = GetDefaultAspectRatioAxisConstraint(EvaluationContext);
	return GetEffectiveFieldsOfView(CameraPose, AspectRatio, DefaultAspectRatioAxisConstraint);
}

FCameraFieldsOfView FCameraPoseMath::GetEffectiveFieldsOfView(const FCameraPose& CameraPose, double ActualAspectRatio, EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint)
{
	const double HorizontalFOV = CameraPose.GetEffectiveFieldOfView();
	const double IdealAspectRatio = CameraPose.GetSensorAspectRatio();

	// Check the sort of aspect ratio axis constraint we have.
	const EAspectRatioAxisConstraint Constraint(CameraPose.GetEffectiveAspectRatioAxisConstraint(DefaultAspectRatioAxisConstraint));
	if (Constraint == AspectRatio_MaintainYFOV || (Constraint == AspectRatio_MajorAxisFOV && ActualAspectRatio < 1.0))
	{
		FCameraFieldsOfView FOVs;

		// We need to maintain vertical FOV... the horizontal FOV we have is for our "ideal" aspect ratio,
		// i.e. our sensor's aspect ratio. Now we need to compute the vertical FOV in this ideal situation
		// and re-compute the effective horizontal FOV using the effective aspect ratio.
		//
		// Given H as horizontal FOV, V as vertical FOV, L the focal distance, X the focal plane's width, and Y the
		// focal plane's height:
		//
		//     tan(H/2) = (X/2)/L
		//     tan(V/2) = (Y/2)/L
		//
		// So L = (X/2)/tan(H/2)
		// And we know that Y = X/A, where A is the aspect ratio (the "ideal" one for now).
		//
		//     tan(V/2) = (Y/2)/(X/2)*tan(H/2)
		//     tan(V/2) = (1/A)*tan(H/2)
		//     V/2 = atan(tan(H/2)/A)
		//
		// Then keep V (since we want to maintain vertical FOV) and recompute H given the "actual" aspect ratio:
		//
		//     tan(H/2) = (X/2)/(Y/2)*tan(V/2)
		//     tan(H/2) = A*tan(V/2)
		//     H/2 = atan(tan(V/2)*A)
		//
		// (where A is the "actual" aspect ratio this time)
		//
		const double TanHorizontalHalfFOV = FMath::Tan(FMath::DegreesToRadians(HorizontalFOV / 2.0));
		const double TanVerticalHalfFOV = TanHorizontalHalfFOV / IdealAspectRatio;
		const double VerticalHalfFOV = FMath::Atan(TanVerticalHalfFOV);
		const double HorizontalHalfFOV = FMath::Atan(TanVerticalHalfFOV * ActualAspectRatio);

		FOVs.HorizontalFieldOfView = FMath::RadiansToDegrees(2.0 * HorizontalHalfFOV);
		FOVs.VerticalFieldOfView = FMath::RadiansToDegrees(2.0 * VerticalHalfFOV);

		return FOVs;
	}
	else
	{
		// Our horizontal FOV is the effective one, so just compute the vertical FOV.
		FCameraFieldsOfView FOVs;

		FOVs.HorizontalFieldOfView = HorizontalFOV;
		FOVs.VerticalFieldOfView = FMath::RadiansToDegrees(2.0 * FMath::Atan(
					FMath::Tan(FMath::DegreesToRadians(HorizontalFOV / 2.0)) / ActualAspectRatio));

		return FOVs;
	}
}

double FCameraPoseMath::GetEffectiveAspectRatio(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	if (CameraPose.GetConstrainAspectRatio() || !EvaluationContext)
	{
		return CameraPose.GetSensorAspectRatio();
	}
	else
	{
		FIntPoint ViewportSize = EvaluationContext->GetViewportSize();
		if (ViewportSize.X > 0 && ViewportSize.Y > 0)
		{
			return (double)ViewportSize.X / (double)ViewportSize.Y;
		}
		else
		{
			return CameraPose.GetSensorAspectRatio();
		}
	}
}

EAspectRatioAxisConstraint FCameraPoseMath::GetDefaultAspectRatioAxisConstraint(TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	if (EvaluationContext)
	{
		if (APlayerController* PlayerController = EvaluationContext->GetPlayerController())
		{
			if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
			{
				return LocalPlayer->AspectRatioAxisConstraint;
			}
		}
	}
	// Same default as UCameraComponent.
	return EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;
}

void FCameraPoseMath::BuildProjectionData(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		FSceneViewProjectionData& OutProjectionData)
{
	// Use some arbitrary defaults: 1080p 16/9 screen, and XFOV constraint like in UCameraComponent.
	static const FIntPoint DefaultViewportSize(1920, 1080);
	EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint = AspectRatio_MaintainXFOV;

	FIntPoint ViewportSize(EForceInit::ForceInit);
	if (EvaluationContext)
	{
		ViewportSize = EvaluationContext->GetViewportSize();

		if (APlayerController* PlayerController = EvaluationContext->GetPlayerController())
		{
			if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
			{
				DefaultAspectRatioAxisConstraint = LocalPlayer->AspectRatioAxisConstraint;
			}
		}
	}
	if (ViewportSize.X == 0 || ViewportSize.Y == 0)
	{
		ViewportSize = DefaultViewportSize;
	}

	BuildProjectionData(CameraPose, ViewportSize, DefaultAspectRatioAxisConstraint, OutProjectionData);
}

void FCameraPoseMath::BuildProjectionData(
		const FCameraPose& CameraPose,
		const FIntPoint& ViewportSize,
		EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint,
		FSceneViewProjectionData& OutProjectionData)
{
	FMinimalViewInfo ViewInfo;
	CameraPose.GetViewInfo(ViewInfo);

	const FIntRect ViewportRect(FIntPoint::ZeroValue, ViewportSize);
	OutProjectionData.SetViewRectangle(ViewportRect);

	OutProjectionData.ViewOrigin = ViewInfo.Location;
	OutProjectionData.ViewRotationMatrix = FInverseRotationMatrix(ViewInfo.Rotation) * FMatrix(
			FPlane(0,	0,	1,	0),
			FPlane(1,	0,	0,	0),
			FPlane(0,	1,	0,	0),
			FPlane(0,	0,	0,	1));
	FMinimalViewInfo::CalculateProjectionMatrixGivenViewRectangle(
			ViewInfo, DefaultAspectRatioAxisConstraint, ViewportRect, OutProjectionData);
	
	ensure(!OutProjectionData.ProjectionMatrix.ContainsNaN());
	ensure(!OutProjectionData.ViewRotationMatrix.ContainsNaN());
}

FMatrix FCameraPoseMath::BuildProjectionMatrix(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	FSceneViewProjectionData ProjectionData;
	BuildProjectionData(CameraPose, EvaluationContext, ProjectionData);
	return ProjectionData.ProjectionMatrix;
}

FMatrix FCameraPoseMath::BuildViewProjectionMatrix(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	FSceneViewProjectionData ProjectionData;
	BuildProjectionData(CameraPose, EvaluationContext, ProjectionData);
	return ProjectionData.ComputeViewProjectionMatrix();
}

FMatrix FCameraPoseMath::BuildLocalViewProjectionMatrix(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	FCameraPose IdentityPose(CameraPose);
	IdentityPose.SetLocation(FVector3d::ZeroVector);
	IdentityPose.SetRotation(FRotator3d::ZeroRotator);

	FSceneViewProjectionData ProjectionData;
	BuildProjectionData(IdentityPose, EvaluationContext, ProjectionData);
	return ProjectionData.ComputeViewProjectionMatrix();
}

TOptional<FVector2d> FCameraPoseMath::ProjectWorldToScreen(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector3d& WorldLocation, bool bForceLocationInsideFrustum)
{
	const FMatrix ViewProjectionMatrix = BuildViewProjectionMatrix(CameraPose, EvaluationContext);
	return ProjectToScreen(ViewProjectionMatrix, WorldLocation, bForceLocationInsideFrustum);
}

TOptional<FVector2d> FCameraPoseMath::ProjectCameraToScreen(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector3d& CameraSpaceLocation, bool bForceLocationInsideFrustum)
{
	const FMatrix ProjectionMatrix = BuildProjectionMatrix(CameraPose, EvaluationContext);
	return ProjectToScreen(ProjectionMatrix, CameraSpaceLocation, bForceLocationInsideFrustum);
}

TOptional<FVector2d> FCameraPoseMath::ProjectToScreen(
		const FMatrix& ViewProjectionMatrix, const FVector3d& Location, bool bForceLocationInsideFrustum)
{
	const FVector4d ProjectedLocation = ViewProjectionMatrix.TransformFVector4(FVector4(Location, 1.0));

	// See if we need to handle the case of a point outside of the view frustum.
	const bool bIsInsideFrustum = (ProjectedLocation.W > 0);
	double W = ProjectedLocation.W;
	if (!bIsInsideFrustum)
	{
		if (!bForceLocationInsideFrustum)
		{
			return TOptional<FVector2d>();
		}

		W = FMath::Abs(W);
	}

	// The result of this will be coordinates in -1..1 projection space.
	const double RHW = 1.0 / W;
	const FVector4d ScreenSpaceLocation(
			ProjectedLocation.X * RHW, 
			ProjectedLocation.Y * RHW, 
			ProjectedLocation.Z * RHW, 
			ProjectedLocation.W);

	// Move from projection space to normalized 0..1 UI space.
	const double ScreenSpaceX = (ScreenSpaceLocation.X / 2.0) + 0.5;
	const double ScreenSpaceY = 1.0 - (ScreenSpaceLocation.Y / 2.0) - 0.5;

	return FVector2d(ScreenSpaceX, ScreenSpaceY);
}

FRay3d FCameraPoseMath::UnprojectScreenToCamera(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector2D& ScreenSpacePoint)
{
	const FMatrix ProjectionMatrix = BuildProjectionMatrix(CameraPose, EvaluationContext);
	const FMatrix InvProjectionMatrix = ProjectionMatrix.InverseFast();
	return UnprojectFromScreen(InvProjectionMatrix, ScreenSpacePoint);
}

FVector3d FCameraPoseMath::UnprojectScreenToCamera(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector2D& ScreenSpacePoint,
		double PredictedDistance)
{
	const FRay3d UnprojectedRay = UnprojectScreenToCamera(CameraPose, EvaluationContext, ScreenSpacePoint);
	const FVector3d WorldPoint = UnprojectedRay.PointAt(PredictedDistance);
	return WorldPoint;
}

FRay3d FCameraPoseMath::UnprojectScreenToWorld(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector2D& ScreenSpacePoint)
{
	const FMatrix ViewProjectionMatrix = BuildViewProjectionMatrix(CameraPose, EvaluationContext);
	const FMatrix InvViewProjectionMatrix = ViewProjectionMatrix.InverseFast();
	return UnprojectFromScreen(InvViewProjectionMatrix, ScreenSpacePoint);
}

FVector3d FCameraPoseMath::UnprojectScreenToWorld(
		const FCameraPose& CameraPose,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		const FVector2D& ScreenSpacePoint,
		double PredictedDistance)
{
	const FRay3d UnprojectedRay = UnprojectScreenToWorld(CameraPose, EvaluationContext, ScreenSpacePoint);
	const FVector3d WorldPoint = UnprojectedRay.PointAt(PredictedDistance);
	return WorldPoint;
}

FRay3d FCameraPoseMath::UnprojectFromScreen(const FMatrix& InverseViewProjectionMatrix, const FVector2D& ScreenSpacePoint)
{
	// Convert the given screen-space point from 0..1 UI space to -1..1 projection space.
	const double ScreenSpaceX = (ScreenSpacePoint.X - 0.5) * 2.0;
	const double ScreenSpaceY = ((1.0 - ScreenSpacePoint.Y) - 0.5) * 2.0;

	// Build a ray from the front of the frustum to the back of the frustum, starting at the screen-space point.
	// We use reverse-Z projection matrices for better precision, so near is Z=1, and far is Z=0.
	const FVector4 RayStartProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 1.0, 1.0);
	const FVector4 RayEndProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 0.01, 1.0);

	// Unproject the ray points and normalize them.
	const FVector4 RayStartProjected = InverseViewProjectionMatrix.TransformFVector4(RayStartProjectionSpace);
	const FVector4 RayEndProjected = InverseViewProjectionMatrix.TransformFVector4(RayEndProjectionSpace);

	FVector RayStartWorldSpace(RayStartProjected.X, RayStartProjected.Y, RayStartProjected.Z);
	if (RayStartProjected.W != 0)
	{
		RayStartWorldSpace /= RayStartProjected.W;
	}
	FVector RayEndWorldSpace(RayEndProjected.X, RayEndProjected.Y, RayEndProjected.Z);
	if (RayEndProjected.W != 0)
	{
		RayEndWorldSpace /= RayEndProjected.W;
	}

	// Make the 3D ray.
	const bool bDirectionIsNormalized = true;
	const FVector RayDirWorldSpace = (RayEndWorldSpace - RayStartWorldSpace).GetSafeNormal();
	return FRay3d(RayStartWorldSpace, RayDirWorldSpace, bDirectionIsNormalized);
}

FVector3d FCameraPoseMath::UnprojectFromScreen(const FMatrix& InverseViewProjectionMatrix, const FVector2D& ScreenSpacePoint, double PredictedDistance)
{
	const FRay3d UnprojectedRay = UnprojectFromScreen(InverseViewProjectionMatrix, ScreenSpacePoint);
	const FVector3d WorldPoint = UnprojectedRay.PointAt(PredictedDistance);
	return WorldPoint;
}

FMatrix FCameraPoseMath::InverseProjectionMatrix(const FMatrix& ProjectionMatrix)
{
	// Stolen from SceneView.h
	if (ProjectionMatrix.M[1][0] == 0.0f &&
		ProjectionMatrix.M[3][0] == 0.0f &&
		ProjectionMatrix.M[0][1] == 0.0f &&
		ProjectionMatrix.M[3][1] == 0.0f &&
		ProjectionMatrix.M[0][2] == 0.0f &&
		ProjectionMatrix.M[1][2] == 0.0f &&
		ProjectionMatrix.M[0][3] == 0.0f &&
		ProjectionMatrix.M[1][3] == 0.0f &&
		ProjectionMatrix.M[2][3] == 1.0f &&
		ProjectionMatrix.M[3][3] == 0.0f)
	{
		double a = ProjectionMatrix.M[0][0];
		double b = ProjectionMatrix.M[1][1];
		double c = ProjectionMatrix.M[2][2];
		double d = ProjectionMatrix.M[3][2];
		double s = ProjectionMatrix.M[2][0];
		double t = ProjectionMatrix.M[2][1];

		return FMatrix(
			FPlane(1.0 / a, 0.0f, 0.0f, 0.0f),
			FPlane(0.0f, 1.0 / b, 0.0f, 0.0f),
			FPlane(0.0f, 0.0f, 0.0f, 1.0 / d),
			FPlane(-s/a, -t/b, 1.0f, -c/d)
		);
	}
	else
	{
		return ProjectionMatrix.Inverse();
	}
}

}  // namespace UE::Cameras

