// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/CriticalDamper.h"

#include "HAL/IConsoleManager.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Cameras
{

float GGameplayCamerasCriticalDamperStabilizationThreshold = 0.00001f;
static FAutoConsoleVariableRef CVarGameplayCamerasCriticalDamperStabilizationThreshold(
	TEXT("GameplayCameras.CriticalDamper.StabilizationThreshold"),
	GGameplayCamerasCriticalDamperStabilizationThreshold,
	TEXT("(Default: 0.00001f (in meters and meters/sec). Determines when to clamp a critical damper to 0 to stabilize it.")
	);

FCriticalDamper::FCriticalDamper()
{
	Reset(0, 0);
	SetW0(-1);
}

FCriticalDamper::FCriticalDamper(float InW0)
{
	Reset(0, 0);
	SetW0(InW0);
}

void FCriticalDamper::Reset(float InX0, float InX0Derivative)
{
	X0 = InX0;
	X0Derivative = InX0Derivative;
}

float FCriticalDamper::Update(float DeltaTime)
{
	return Update(X0, DeltaTime);
}

float FCriticalDamper::Update(float X, float DeltaTime)
{
	if (DeltaTime != 0.f)
	{
		// Last frame we were at X0. This frame we are at X, so something forcibly moved
		// us by the difference.
		const float ForcedMovement = (X - X0);
		InternalUpdate(ForcedMovement, DeltaTime);
	}
	return X0;
}

float FCriticalDamper::Update(float PreviousDamped, float NextTarget, float DeltaTime)
{
	const float X = (NextTarget - PreviousDamped);
	Update(X, DeltaTime);
	// Return the position of the system relative to current frame's target.
	return NextTarget - X0;
}

void FCriticalDamper::InternalUpdate(float ForcedMovement, float DeltaTime)
{
	ensure(DeltaTime != 0.f);
	if (W0 > 0.f)
	{
		// We need to move the base of the spring by ForcedMovement, and then run the
		// simulation of the spring-mass system. This first step is because we typically 
		// have the base of the spring attached to a moving object (the player character,
		// the player's moving vehicle, etc.) This means we effectively have a "driven
		// harmonic oscillator" system.
		//
		// References:
		//		https://en.wikipedia.org/wiki/Damping
		//      https://en.wikipedia.org/wiki/Harmonic_oscillator
		//		https://mathworld.wolfram.com/CriticallyDampedSimpleHarmonicMotion.html
		//		https://tutorial.math.lamar.edu/classes/de/nonhomogeneousde.aspx
		//
		// Notations:
		//		x' is the first derivative of x, aka dx/dt
		//		x'' is the second derivative of x, aka (d^2)x/d(t^2)
		//
		// Starting almost from first principles, we know that the movement of the mass in
		// the spring-mass system is:
		//
		//		m*x'' = sum(forces)
		//
		// One force is the spring, so:
		//
		//		m*x'' = -k*x - c*x' + ExternalForces
		//		m*x'' + c*x' + k*x = ExternalForces
		//		x'' + c/m*x' + k/m*x = ExternalForces/m
		//
		// ...where m is the mass, k is the spring constant, and c is the viscous damping
		// coefficient.
		//
		// Let's introduce some notations:
		//
		//		w0 = sqrt(k/m)  aka the natural frequency of the system
		//		zeta = c/(2*sqrt(m*k))  aka the damping ratio
		//
		// Our equation becomes:
		//
		//		x'' + 2*zeta*w0*x' + w0^2*x = ExternalForces/m
		//
		// We want a critically damped system, which means we want to force zeta=1:
		//
		//		x'' + 2*w0*x' + w0^2*x = ExternalForces/m
		// 
		// Now let's look at ExternalForces. These exist when ForcedMovement is non
		// zero, i.e. we have some external force that moves the base of the spring.
		// This typically happens when the spring is attached to some moving player
		// controlled entity like a character or a vehicle. The acceleration or
		// deceleration of that entity expand or compresses the spring.
		//
		// When the base of the spring isn't moving ExternalForces is zero, and our 
		// equation is:
		//
		//		x'' + 2*w0*x' + w0^2*x = 0
		//
		// This is a homogenous linear differential equation. Let's put a pin in that,
		// and call it HE.
		//
		// Now, when the base of the spring is indeed moving, the force introduced in 
		// the system is based on the spring itself:
		//
		//		ExternalForces = -k*D
		//
		// ...where D is the distance forced by the movement of the base. This is the
		// equation for the added pull/push of the spring on the mass due to how much
		// the base moved.
		//
		// We assume that this forced movement happened at constant speed over 
		// DeltaTime (it may not have, but that's irrelevant since DeltaTime is our
		// sampling rate) so:
		//
		//		ExternalForces = -k*(D0 + Dv*t)
		//
		// ...where D0 is the previous position of the spring base, Dv is the speed
		// of the spring base, and t is the time. In practice D0 is always 0 since 
		// we reset our simulation space to converge towards 0 every frame. So we
		// can rewrite our general equation as:
		//
		//		x'' + 2*w0*x' + w0^2*x = -k*Dv*t/m
		//		x'' + 2*w0*x' + w0^2*x = -w0^2*Dv*t
		//
		// This is a non-homogeneous linear differential equation. Let's call it NHE.
		// The homogeneous one that we mentioned earlier (that we called HE, when 
		// ExternalForces are 0), is therefore its "associated" homogeneous differential 
		// equation, or "complementary" homogeneous differential equation.
		//
		// There's a theorem that says, very roughly, that if YP is a "particular 
		// solution" to the NHE, we can just add it to the general solution of the HE
		// and obtain a general solution to the NHE.
		//
		// Let's start by finding YP, a particular solution to the NHE. Since the 
		// constant in the NHE is of the form c*t where c is a constant, we can use the
		// "method of undertermined coefficients" to make an educated guess as to what
		// a possible solution is. We can imagine that one solution is of the form:
		//
		//		x = P0 + P1*t
		//
		// To test this theory, let's derive this form and plug the result in the NHE.
		// By the way, I'd like to thank Matt Peters for this extra element that is 
		// easily missed from spring/mass system equations (because most solutions found 
		// online don't need to handle a moving target, but we do).
		//
		//		x' = P1
		//		x'' = 0
		//		x'' + 2*w0*x' + w0^2*x = -w0^2*Dv*t
		//		0 + 2*w0*P1 + w0^2*(P0 + P1*t) = -w0^2*Dv*t
		//		2*w0*P1 + w0^2*(P0 + P1*t) = -w0^2*Dv*t
		//		
		// We solve this by setting t=0:
		//
		//		2*w0*P1 + w0^2*P0 = 0
		//		P0 = -2*w0*P1/w0^2
		//		P0 = -2*P1/w0
		//
		//	And so:
		//
		//		2*w0*P1 + w0^2*(P0 + P1*t) = -w0^2*Dv*t
		//		2*w0*P1 + w0^2*(-2*P1/w0 + P1*t) = -w0^2*Dv*t
		//		2*w0*P1 - 2*w0*P1 + w0^2*P1*t = -w0^2*Dv*t
		//		w0^2*P1*t = -w0^2*Dv*t
		//		P1 = -Dv
		//
		// Therefore:
		//
		//		P0 = 2*Dv/w0
		//
		// Going back to the full solution:
		//
		//		x = (A + B*t)*e^(-w0*t) + (P0 + P1*t)
		//
		// We derive this to get the velocity:
		//
		//		x' = -w0*A*e^(-w0*t) + B*e^(-w0*t) - w0*B*t*e^(-w0*t) + P1
		//		x' = (-w0*A + B - w0*B*t)*e^(-w0*t) + P1
		//
		// Now we go back to figuring out A and B. Wolfram (see references) did that by
		// using the initial conditions, so let's do that too with this solution that has
		// the extra elements at the end from the NHE. At t=0, our two equations for x
		// and x' become:
		//
		//		x(0) = A + P0
		//		x'(0) = (-w0*A + B) + P1
		//
		// Solving for A and B:
		//
		//		A = x(0) - P0
		//		B = x'(0) + w0*A - P1
		//
		// Now all we need to do is assemble all those pieces and write the code!

		const float Dv = (ForcedMovement != 0.f) ? (ForcedMovement / DeltaTime) : 0.f;

		const float P0 = 2.f * Dv / W0;
		const float P1 = -Dv;

		const float ExpMinusW0Dt = FMath::Exp(-W0 * DeltaTime);
		const float A = X0 - P0;
		const float B = X0Derivative + W0 * A - P1;

		const float Xt = (A + B * DeltaTime) * ExpMinusW0Dt + P0 + P1 * DeltaTime;
		const float XtDerivative = (-W0 * A + B - W0 * B * DeltaTime) * ExpMinusW0Dt + P1;

		// Let's set the evaluation result of this frame as X0 and X0Derivative, which
		// both store the last evaluated state, and also serve as the "initial conditions" 
		// for next frame's evaluation.
		//
		// Note that we add ForcedMovement to X0 because the base of the spring has moved
		// by that amount during this frame. This means that X0 is now relative to this
		// new base, which used to be located at the origin at the beginning of the frame.
		// We want X0 to be instead relative to 0.
		X0 = Xt + ForcedMovement;
		X0Derivative = XtDerivative;

		// Floating point precision isn't good enough to let us mathematically converge
		// towards 0 in an optimal manner, without overshooting. So overshooting by very
		// small fractions can occur. We just stop the spring-mass system dead in its
		// tracks when we figure that it's very close... since we test both position and
		// velocity, this will only happen when it was about the settle anyway.
		if (FMath::Abs(X0) <= GGameplayCamerasCriticalDamperStabilizationThreshold &&
				FMath::Abs(X0Derivative) <= GGameplayCamerasCriticalDamperStabilizationThreshold)
		{
			X0 = 0;
			X0Derivative = 0;
		}
	}
	else if (W0 <= 0.f)
	{
		// The spring-mass system is disabled... let's just stick exactly to our target.
		X0 = 0;
		X0Derivative = 0;
	}
}

void FCriticalDamper::Serialize(FArchive& Ar)
{
	Ar << W0;
	Ar << X0;
	Ar << X0Derivative;
}

FArchive& operator <<(FArchive& Ar, FCriticalDamper& Damper)
{
	Damper.Serialize(Ar);
	return Ar;
}

}  // namespace UE::Cameras

