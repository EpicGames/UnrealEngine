// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/PerlinNoise.h"

#include "HAL/IConsoleManager.h"
#include "Math/Interpolation.h"
#include "Math/UnrealMathUtility.h"
#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerlinNoise)

namespace UE::Cameras
{

bool GCubicPerlinNoise = true;
static FAutoConsoleVariableRef CVarCubicPerlinNoise(
	TEXT("GameplayCameras.CubicPerlinNoise"),
	GCubicPerlinNoise,
	TEXT("(Default: true) Enables cubic splines for perlin noise generator."));

float GCubicPerlinNoiseTension = 0.1f;
static FAutoConsoleVariableRef CVarCubicPerlinNoiseTension(
	TEXT("GameplayCameras.CubicPerlinNoiseTension"),
	GCubicPerlinNoiseTension,
	TEXT("(Default: 0.1) Sets the tension for the cubic splines of the perlin noise generator."));

FPerlinNoise::FPerlinNoise()
{
	Initialize(1.f);
}

FPerlinNoise::FPerlinNoise(const FPerlinNoiseData& InData, uint8 InOctaves)
	: Amplitude(InData.Amplitude)
	, NumOctaves(FMath::Clamp(InOctaves, 1, MAX_OCTAVES))
{
	Initialize(1.f);
}

FPerlinNoise::FPerlinNoise(float InAmplitude, float InFrequency, uint8 InOctaves)
	: Amplitude(InAmplitude)
	, NumOctaves(FMath::Clamp(InOctaves, 1, MAX_OCTAVES))
{
	Initialize(InFrequency);
}

void FPerlinNoise::Initialize(float InFrequency)
{
	float OctaveFrequency = InFrequency;

	for (int32 Index = 0; Index < NumOctaves; ++Index)
	{
		FSinglePerlinNoise& Octave(Octaves[Index]);
		Octave.Frequency = OctaveFrequency;
		Octave.SecondPrev = FMath::FRandRange(-1.f, 1.f);
		Octave.Prev = FMath::FRandRange(-1.f, 1.f);
		Octave.Next = FMath::FRandRange(-1.f, 1.f);
		Octave.SecondNext = FMath::FRandRange(-1.f, 1.f);

		OctaveFrequency *= Lacunarity;
	}
}

FVector2f FPerlinNoise::ComputeTangent(float InPrev, float InNext, float Interval, float Tension)
{
	return (1.f - Tension) * (FVector2f(Interval, InNext) - FVector2f(-Interval, InPrev)) / (2.f * Interval);
}

void FPerlinNoise::SetFrequency(float InFrequency)
{
	if (InFrequency == Octaves[0].Frequency)
	{
		return;
	}

	// Move our current time to be in the same relative place inside the new interval
	// compared to the old interval. So for instance if we were at 70% between the 
	// noise peaks using the old frequency, let's move ourselves to be at 70% between
	// the noise peaks of the new frequency.
	// This loses the amount of accumulated time, but we don't really need it.

	float NewOctaveFrequency = InFrequency;

	for (int32 Index = 0; Index < NumOctaves; ++Index)
	{
		FSinglePerlinNoise& Octave(Octaves[Index]);

		const float OldInterval = (Octave.Frequency > 0 ? 1.f / Octave.Frequency : 1.f);
		const float OldIntervalFactor = FMath::Fractional(Octave.CurTime / OldInterval);

		Octave.Frequency = NewOctaveFrequency;

		const float NewInterval = (Octave.Frequency > 0 ? 1.f / Octave.Frequency : 1.f);
		Octave.CurTime = NewInterval * OldIntervalFactor;

		NewOctaveFrequency *= Lacunarity;
	}
}

void FPerlinNoise::SetNumOctaves(uint8 InNumOctaves)
{
	InNumOctaves = FMath::Clamp(InNumOctaves, 1, MAX_OCTAVES);
	if (InNumOctaves < NumOctaves)
	{
		NumOctaves = InNumOctaves;
	}
	else if (InNumOctaves > NumOctaves)
	{
		float OctaveFrequency = Octaves[NumOctaves - 1].Frequency * Lacunarity;

		for (int32 Index = NumOctaves; Index < InNumOctaves; ++Index)
		{
			FSinglePerlinNoise& Octave(Octaves[Index]);
			Octave.Frequency = OctaveFrequency;
			Octave.CurTime = 0.f;
			Octave.SecondPrev = FMath::FRandRange(-1.f, 1.f);
			Octave.Prev = FMath::FRandRange(-1.f, 1.f);
			Octave.Next = FMath::FRandRange(-1.f, 1.f);
			Octave.SecondNext = FMath::FRandRange(-1.f, 1.f);

			OctaveFrequency *= Lacunarity;
		}

		NumOctaves = InNumOctaves;
	}
}

float FPerlinNoise::GenerateValue(float DeltaTime)
{
	float Value = 0.f;
	float OctaveAmplitude = Amplitude;

	const float Tension = FMath::Clamp(GCubicPerlinNoiseTension, 0.f, 1.f);

	for (int32 Index = 0; Index < NumOctaves; ++Index)
	{
		FSinglePerlinNoise& Octave(Octaves[Index]);

		const float Interval = (Octave.Frequency > 0 ? 1.f / Octave.Frequency : 1.f);

		const float PrevNumIntervals = (Octave.CurTime / Interval);
		const float NextNumIntervals = ((Octave.CurTime + DeltaTime) / Interval);

		// If we are going over the end of the current interval, generate
		// some new value for the next interval.
		// NOTE: for large delta-times we might skip over intervals, but here we behave as if
		//       only going to the next interval.
		if ((int32)NextNumIntervals > (int32)PrevNumIntervals)
		{
			Octave.SecondPrev = Octave.Prev;
			Octave.Prev = Octave.Next;
			Octave.Next = Octave.SecondNext;
			Octave.SecondNext = FMath::FRandRange(-1.f, 1.f);
		}

		Octave.CurTime += DeltaTime;

		const float IntervalFactor = (NextNumIntervals - FMath::TruncToFloat(NextNumIntervals));
		if (GCubicPerlinNoise)
		{
			const FVector2f PrevTangent = ComputeTangent(Octave.SecondPrev, Octave.Next, Interval, Tension);
			const FVector2f NextTangent = ComputeTangent(Octave.Prev, Octave.SecondNext, Interval, Tension);

			const FVector2f InterpPoint = FMath::CubicInterp(
					FVector2f(0.f, Octave.Prev), PrevTangent,
					FVector2f(Interval, Octave.Next), NextTangent,
					IntervalFactor);
			Value += OctaveAmplitude * InterpPoint.Y;
		}
		else
		{
			const float InterpFactor = SmoothStep(IntervalFactor);
			Value += OctaveAmplitude * FMath::Lerp(Octave.Prev, Octave.Next, InterpFactor);
		}

		OctaveAmplitude *= OctaveGain;
	}

	return Value;
}

void FPerlinNoise::Serialize(FArchive& Ar)
{
	Ar << Amplitude;
	Ar << Lacunarity;
	Ar << OctaveGain;

	Ar << NumOctaves;

	for (int32 Index = 0; Index < NumOctaves; ++Index)
	{
		FSinglePerlinNoise& Octave(Octaves[Index]);
		Ar.Serialize(static_cast<void*>(&Octave), sizeof(FSinglePerlinNoise));
	}
}

}  // namespace UE::Cameras

