// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneCameraFramingZonePropertySystem.h"

#include "MovieScene/MovieSceneGameplayCamerasComponentTypes.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraFramingZonePropertySystem)

UMovieSceneCameraFramingZonePropertySystem::UMovieSceneCameraFramingZonePropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::Cameras::FMovieSceneGameplayCamerasComponentTypes::Get()->CameraFramingZone);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneCameraFramingZonePropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

