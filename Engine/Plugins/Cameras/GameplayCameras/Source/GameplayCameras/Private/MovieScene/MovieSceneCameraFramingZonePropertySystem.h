// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Framing/CameraFramingZone.h"
#include "Systems/MovieScenePropertySystem.h"

#include "MovieSceneCameraFramingZonePropertySystem.generated.h"

Expose_TNameOf(FCameraFramingZone);

UCLASS(MinimalAPI)
class UMovieSceneCameraFramingZonePropertySystem : public UMovieScenePropertySystem
{
public:

	GENERATED_BODY()
	
	UMovieSceneCameraFramingZonePropertySystem(const FObjectInitializer& ObjInit);

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

