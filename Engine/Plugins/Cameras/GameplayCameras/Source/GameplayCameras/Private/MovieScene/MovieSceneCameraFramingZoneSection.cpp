// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneCameraFramingZoneSection.h"

#include "MovieScene/MovieSceneCameraFramingZoneTrack.h"
#include "MovieScene/MovieSceneGameplayCamerasComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraFramingZoneSection)

#define LOCTEXT_NAMESPACE "MovieSceneCameraFramingZoneSection"

#if WITH_EDITOR

struct FCameraFramingZoneSectionEditorData
{
	FCameraFramingZoneSectionEditorData()
	{
		MetaData[0].SetIdentifiers("Left", LOCTEXT("LeftText", "Left"));
		MetaData[0].SubPropertyPath = MetaData[0].Name;
		MetaData[0].SortOrder = 0;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Top", LOCTEXT("TopText", "Top"));
		MetaData[1].SubPropertyPath = MetaData[1].Name;
		MetaData[1].SortOrder = 1;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Right", LOCTEXT("RightText", "Right"));
		MetaData[2].SubPropertyPath = MetaData[2].Name;
		MetaData[2].SortOrder = 2;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Bottom", LOCTEXT("BottomText", "Bottom"));
		MetaData[3].SubPropertyPath = MetaData[3].Name;
		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = ExtractLeftChannel;
		ExternalValues[1].OnGetExternalValue = ExtractTopChannel;
		ExternalValues[2].OnGetExternalValue = ExtractRightChannel;
		ExternalValues[3].OnGetExternalValue = ExtractBottomChannel;
	}

	static TOptional<double> ExtractLeftChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FCameraFramingZone>(InObject).Left : TOptional<double>();
	}
	static TOptional<double> ExtractTopChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FCameraFramingZone>(InObject).Top : TOptional<double>();
	}
	static TOptional<double> ExtractRightChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FCameraFramingZone>(InObject).Right : TOptional<double>();
	}
	static TOptional<double> ExtractBottomChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FCameraFramingZone>(InObject).Bottom : TOptional<double>();
	}

	FMovieSceneChannelMetaData      MetaData[4];
	TMovieSceneExternalValue<double> ExternalValues[4];
};

#endif	// WITH_EDITOR

UMovieSceneCameraFramingZoneSection::UMovieSceneCameraFramingZoneSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

	BlendType = EMovieSceneBlendType::Absolute;

	FMovieSceneChannelProxyData Channels;
	bSupportsInfiniteRange = true;

#if WITH_EDITOR

	static const FCameraFramingZoneSectionEditorData EditorData;
	Channels.Add(LeftMarginCurve, EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(TopMarginCurve, EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(RightMarginCurve, EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(BottomMarginCurve, EditorData.MetaData[3], EditorData.ExternalValues[3]);

#else

	Channels.Add(LeftMarginCurve);
	Channels.Add(TopMarginCurve);
	Channels.Add(RightMarginCurve);
	Channels.Add(BottomMarginCurve);

#endif
	
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

bool UMovieSceneCameraFramingZoneSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneCameraFramingZoneSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::Cameras;
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneGameplayCamerasComponentTypes* GameplayCamerasComponents = FMovieSceneGameplayCamerasComponentTypes::Get();

	FPropertyTrackEntityImportHelper(GameplayCamerasComponents->CameraFramingZone)
		.AddConditional(Components->DoubleChannel[0], &LeftMarginCurve, LeftMarginCurve.HasAnyData())
		.AddConditional(Components->DoubleChannel[1], &TopMarginCurve, TopMarginCurve.HasAnyData())
		.AddConditional(Components->DoubleChannel[2], &RightMarginCurve, RightMarginCurve.HasAnyData())
		.AddConditional(Components->DoubleChannel[3], &BottomMarginCurve, BottomMarginCurve.HasAnyData())
		.Commit(this, Params, OutImportedEntity);
}

#undef LOCTEXT_NAMESPACE

