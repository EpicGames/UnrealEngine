// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneCameraFramingZoneTrack.h"
#include "MovieScene/MovieSceneCameraFramingZoneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraFramingZoneTrack)

UMovieSceneCameraFramingZoneTrack::UMovieSceneCameraFramingZoneTrack(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneCameraFramingZoneTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCameraFramingZoneSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCameraFramingZoneTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCameraFramingZoneSection>(this, NAME_None, RF_Transactional);
}

