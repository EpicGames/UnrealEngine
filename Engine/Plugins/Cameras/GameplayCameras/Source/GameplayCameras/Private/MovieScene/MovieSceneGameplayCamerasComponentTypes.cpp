// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneGameplayCamerasComponentTypes.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)

namespace UE::MovieScene
{
	void UnpackChannelsFromOperational(FCameraFramingZone InZone, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 0, InZone, Left));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 1, InZone, Top));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 2, InZone, Right));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 3, InZone, Bottom));
	}
}

#endif  // UE >= 5.7.0

namespace UE::Cameras
{

static bool GGameplayCamerasComponentTypesDestroyed = false;
static TUniquePtr<FMovieSceneGameplayCamerasComponentTypes> GGameplayCamerasComponentTypes;

FMovieSceneGameplayCamerasComponentTypes::FMovieSceneGameplayCamerasComponentTypes()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	CameraParameterOverrideID = ComponentRegistry->NewComponentType<FGuid>(TEXT("Camera Parameter ID"), EComponentTypeFlags::CopyToChildren);

	ComponentRegistry->NewPropertyType(CameraFramingZone, TEXT("Camera Framing Zone"));

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(CameraFramingZone, TEXT("Apply FCameraFramingZone Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FCameraFramingZone::Left)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FCameraFramingZone::Top)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FCameraFramingZone::Right)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FCameraFramingZone::Bottom)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&CustomCameraFramingZoneAccessors)
	.Commit();
}

FMovieSceneGameplayCamerasComponentTypes::~FMovieSceneGameplayCamerasComponentTypes()
{
}

void FMovieSceneGameplayCamerasComponentTypes::Destroy()
{
	GGameplayCamerasComponentTypes.Reset();
	GGameplayCamerasComponentTypesDestroyed = true;
}

FMovieSceneGameplayCamerasComponentTypes* FMovieSceneGameplayCamerasComponentTypes::Get()
{
	if (!GGameplayCamerasComponentTypes.IsValid())
	{
		check(!GGameplayCamerasComponentTypesDestroyed);
		GGameplayCamerasComponentTypes.Reset(new FMovieSceneGameplayCamerasComponentTypes);
	}
	return GGameplayCamerasComponentTypes.Get();
}

} // namespace UE::Cameras

