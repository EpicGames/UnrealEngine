// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyTraits.h"
#include "Misc/EngineVersionComparison.h"
#include "Nodes/Framing/CameraFramingZone.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
#include "EntitySystem/MovieScenePropertyMetaData.h"
#else
#include "EntitySystem/MovieScenePropertyMetaDataTraits.h"
#endif  // UE >= 5.7.0

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)

namespace UE::MovieScene
{
	void UnpackChannelsFromOperational(FCameraFramingZone InZone, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
}

#endif  // UE >= 5.7.0

namespace UE::Cameras
{

using FCameraFramingZoneTraits = UE::MovieScene::TDirectPropertyTraits<FCameraFramingZone>;

struct FMovieSceneGameplayCamerasComponentTypes
{
	GAMEPLAYCAMERAS_API ~FMovieSceneGameplayCamerasComponentTypes();

	UE::MovieScene::TComponentTypeID<FGuid> CameraParameterOverrideID;

	UE::MovieScene::TPropertyComponents<FCameraFramingZoneTraits> CameraFramingZone;

	UE::MovieScene::TCustomPropertyRegistration<FCameraFramingZoneTraits, 1> CustomCameraFramingZoneAccessors;

	static GAMEPLAYCAMERAS_API void Destroy();

	static GAMEPLAYCAMERAS_API FMovieSceneGameplayCamerasComponentTypes* Get();

private:

	FMovieSceneGameplayCamerasComponentTypes();
};

} // namespace UE::Cameras

