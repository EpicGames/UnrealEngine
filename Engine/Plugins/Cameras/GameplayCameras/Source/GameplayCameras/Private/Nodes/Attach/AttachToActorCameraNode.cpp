// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Attach/AttachToActorCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttachToActorCameraNode)

namespace UE::Cameras
{

class FAttachToActorCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FAttachToActorCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	FCameraActorAttachmentInfoReader AttachmentReader;
	TCameraParameterReader<bool> AttachToLocationReader;
	TCameraParameterReader<bool> AttachToRotationReader;

	TOptional<FTransform3d> LastAttachTransform;
	bool bIsAttachValid = true;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FAttachToActorCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FAttachToActorCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FTransform3d, AttachTransform);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, AttachInfo);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FAttachToActorCameraDebugBlock)

void FAttachToActorCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UAttachToActorCameraNode* AttachNode = GetCameraNodeAs<UAttachToActorCameraNode>();
	AttachmentReader.Initialize(AttachNode->Attachment, AttachNode->AttachmentDataID);
	AttachToLocationReader.Initialize(AttachNode->AttachToLocation);
	AttachToRotationReader.Initialize(AttachNode->AttachToRotation);
}

void FAttachToActorCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FTransform3d AttachTransform;
	const bool bWasAttachValid = bIsAttachValid;
	bIsAttachValid = AttachmentReader.GetAttachmentTransform(OutResult.ContextDataTable, AttachTransform);
	if (!bIsAttachValid)
	{
		// If we just became invalid, log a warning.
		if (bWasAttachValid)
		{
			UE_LOGF(LogCameraSystem, Warning, 
					"AttachToActorCameraNode: Couldn't resolve attachment! The camera will stay in place.");
		}

		// Stay in the last known place, if any, otherwise completely bail out.
		if (LastAttachTransform.IsSet())
		{
			AttachTransform = LastAttachTransform.GetValue();
		}
		else
		{
			return;
		}
	}

	const bool bAttachToLocation = AttachToLocationReader.Get(OutResult.VariableTable);
	const bool bAttachToRotation = AttachToRotationReader.Get(OutResult.VariableTable);
	
	if (bAttachToLocation)
	{
		const FVector3d AttachLocation = AttachTransform.GetLocation();
		OutResult.CameraPose.SetLocation(AttachLocation);
	}

	if (bAttachToRotation)
	{
		const FRotator3d AttachRotation = AttachTransform.GetRotation().Rotator();
		OutResult.CameraPose.SetRotation(AttachRotation);
	}

	if (bIsAttachValid)
	{
		LastAttachTransform = AttachTransform;
	}
}

void FAttachToActorCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << LastAttachTransform;
	Ar << bIsAttachValid;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FAttachToActorCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FAttachToActorCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FAttachToActorCameraDebugBlock>();
	DebugBlock.AttachTransform = LastAttachTransform.Get(FTransform3d::Identity);
	DebugBlock.AttachInfo = AttachmentReader.RenderAttachmentInfo();
}

void FAttachToActorCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(AttachInfo);

	if (Renderer.IsExternalRendering())
	{
		Renderer.DrawPoint(AttachTransform.GetLocation(), 2.f, FColorList::NeonBlue, 2.f);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UAttachToActorCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAttachToActorCameraNodeEvaluator>();
}

