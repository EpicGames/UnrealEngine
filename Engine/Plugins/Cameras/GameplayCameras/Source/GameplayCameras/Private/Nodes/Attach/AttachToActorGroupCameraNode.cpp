// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Attach/AttachToActorGroupCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttachToActorGroupCameraNode)

namespace UE::Cameras
{

class FAttachToActorGroupCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FAttachToActorGroupCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

#if WITH_EDITOR
	void UpdateAttachmentReaders();
#endif  // WITH_EDITOR

private:

	FCameraActorAttachmentInfoArrayReader AttachmentsReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FAttachToActorGroupCameraNodeEvaluator)

void FAttachToActorGroupCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UAttachToActorGroupCameraNode* AttachNode = GetCameraNodeAs<UAttachToActorGroupCameraNode>();
	AttachmentsReader.Initialize(AttachNode->Attachments, AttachNode->AttachmentsDataID);
}

void FAttachToActorGroupCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
#if WITH_EDITOR
	UpdateAttachmentReaders();
#endif  // WITH_EDITOR

	FTransform3d AttachTransform;
	if (AttachmentsReader.GetAttachmentTransform(OutResult.ContextDataTable, AttachTransform))
	{
		const FVector3d AttachLocation = AttachTransform.GetLocation();
		OutResult.CameraPose.SetLocation(AttachLocation);
	}
}

#if WITH_EDITOR

void FAttachToActorGroupCameraNodeEvaluator::UpdateAttachmentReaders()
{
	//const UAttachToActorGroupCameraNode* AttachNode = GetCameraNodeAs<UAttachToActorGroupCameraNode>();
	//if (AttachNode->Attachments.Num() != AttachmentReaders.Num())
	//{
	//	AttachmentReaders.Reset();
	//	for (const FCameraActorAttachmentInfo& Attachment : AttachNode->Attachments)
	//	{
	//		AttachmentReaders.Emplace(Attachment, FCameraContextDataID());
	//	}
	//}
	//else
	//{
	//	for (int32 Index = 0; Index < AttachmentReaders.Num(); ++Index)
	//	{
	//		FCameraActorAttachmentInfoReader& Reader(AttachmentReaders[Index]);
	//		const FCameraActorAttachmentInfo& Attachment(AttachNode->Attachments[Index]);
	//		Reader.Initialize(Attachment, FCameraContextDataID());
	//	}
	//}
}

#endif  // WITH_EDITOR

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UAttachToActorGroupCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAttachToActorGroupCameraNodeEvaluator>();
}

