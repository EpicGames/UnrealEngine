// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Attach/AttachToPlayerPawnCameraNode.h"

#include "Components/SkeletalMeshComponent.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraSystemEvaluator.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttachToPlayerPawnCameraNode)

namespace UE::Cameras
{

class FAttachToPlayerPawnCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FAttachToPlayerPawnCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	USkeletalMeshComponent* CachedSkeletalMeshComponent = nullptr;
	FName CachedBoneName;

	TCameraParameterReader<bool> AttachToLocationReader;
	TCameraParameterReader<bool> AttachToRotationReader;

	bool bHasValidPlayerController = true;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FAttachToPlayerPawnCameraNodeEvaluator)

void FAttachToPlayerPawnCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UAttachToPlayerPawnCameraNode* AttachNode = GetCameraNodeAs<UAttachToPlayerPawnCameraNode>();
	AttachToLocationReader.Initialize(AttachNode->AttachToLocation);
	AttachToRotationReader.Initialize(AttachNode->AttachToRotation);

	APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController();
	if (!PlayerController)
	{
		if (Params.Evaluator->GetRole() == ECameraSystemEvaluatorRole::Game)
		{
			UE_LOGF(LogCameraSystem, Error, 
					"Can't run AttatchToPlayerPawn camera node because no player controller was found on the context.");
		}
		bHasValidPlayerController = false;
		return;
	}

	APawn* Pawn = PlayerController->GetPawnOrSpectator();
	if (!Pawn)
	{
		UE_LOGF(LogCameraSystem, Error, 
				"Can't run AttatchToPlayerPawn camera node because the player controller has no pawn.");
		bHasValidPlayerController = false;
		return;
	}

	if (!AttachNode->SocketName.IsNone() || !AttachNode->BoneName.IsNone())
	{
		CachedSkeletalMeshComponent = Pawn->FindComponentByClass<USkeletalMeshComponent>();
	}

	CachedBoneName = NAME_None;
	if (CachedSkeletalMeshComponent)
	{
		CachedBoneName = AttachNode->BoneName;
		if (!AttachNode->SocketName.IsNone())
		{
			CachedBoneName = CachedSkeletalMeshComponent->GetSocketBoneName(AttachNode->SocketName);
		}
	}
}

void FAttachToPlayerPawnCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
#if WITH_EDITOR
	if (Params.Evaluator && Params.Evaluator->GetRole() == ECameraSystemEvaluatorRole::EditorPreview)
	{
		// No player pawn in editor preview.
		return;
	}
#endif

	if (!bHasValidPlayerController)
	{
		return;
	}

	APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController();
	if (!ensure(PlayerController))
	{
		bHasValidPlayerController = false;
		return;
	}

	const APawn* Pawn = PlayerController->GetPawnOrSpectator();
	if (!ensure(Pawn))
	{
		return;
	}

	const bool bAttachToLocation = AttachToLocationReader.Get(OutResult.VariableTable);
	const bool bAttachToRotation = AttachToRotationReader.Get(OutResult.VariableTable);

	FTransform3d AttachTransform = Pawn->GetActorTransform();
	if (CachedSkeletalMeshComponent && !CachedBoneName.IsNone())
	{
		AttachTransform = CachedSkeletalMeshComponent->GetBoneTransform(CachedBoneName);
	}
	
	if (bAttachToLocation)
	{
		OutResult.CameraPose.SetLocation(AttachTransform.GetLocation());
	}

	if (bAttachToRotation)
	{
		OutResult.CameraPose.SetRotation(AttachTransform.GetRotation().Rotator());
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UAttachToPlayerPawnCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAttachToPlayerPawnCameraNodeEvaluator>();
}

