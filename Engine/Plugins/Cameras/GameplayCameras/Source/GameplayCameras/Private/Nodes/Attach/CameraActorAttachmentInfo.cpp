// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Attach/CameraActorAttachmentInfo.h"

#include "Algo/Accumulate.h"
#include "Components/SkeletalMeshComponent.h"
#include "Core/CameraContextDataTable.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraActorAttachmentInfo)

namespace UE::Cameras
{

FCameraActorAttachmentInfoReader::FCameraActorAttachmentInfoReader(const FCameraActorAttachmentInfo& InAttachmentInfo, FCameraContextDataID InDataID)
{
	Initialize(InAttachmentInfo, InDataID);
}

void FCameraActorAttachmentInfoReader::Initialize(const FCameraActorAttachmentInfo& InAttachmentInfo, FCameraContextDataID InDataID)
{
	DefaultAttachmentInfo = InAttachmentInfo;
	DataID = InDataID;

	CacheAttachmentInfo(DefaultAttachmentInfo);
}

void FCameraActorAttachmentInfoReader::CacheAttachmentInfo(const FCameraActorAttachmentInfo& InAttachmentInfo)
{
	if (CachedAttachmentInfo != InAttachmentInfo)
	{
		CachedAttachmentInfo = InAttachmentInfo;

		CachedSkeletalMeshComponent = nullptr;
		if (InAttachmentInfo.Actor && (!InAttachmentInfo.SocketName.IsNone() || !InAttachmentInfo.BoneName.IsNone()))
		{
			CachedSkeletalMeshComponent = InAttachmentInfo.Actor->FindComponentByClass<USkeletalMeshComponent>();
		}

		CachedBoneName = NAME_None;
		if (CachedSkeletalMeshComponent)
		{
			CachedBoneName = InAttachmentInfo.BoneName;
			if (!InAttachmentInfo.SocketName.IsNone())
			{
				CachedBoneName = CachedSkeletalMeshComponent->GetSocketBoneName(InAttachmentInfo.SocketName);
			}
		}
	}
}

bool FCameraActorAttachmentInfoReader::GetAttachmentTransform(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform)
{
	if (DataID.IsValid())
	{
		if (const FCameraActorAttachmentInfo* NewAttachmentInfo = ContextDataTable.TryGetData<FCameraActorAttachmentInfo>(DataID))
		{
			CacheAttachmentInfo(*NewAttachmentInfo);
		}
		else
		{
			CacheAttachmentInfo(DefaultAttachmentInfo);
		}
	}

	if (CachedSkeletalMeshComponent && !CachedBoneName.IsNone())
	{
		OutTransform = CachedSkeletalMeshComponent->GetBoneTransform(CachedBoneName);
		return true;
	}
	else if (CachedAttachmentInfo.Actor)
	{
		OutTransform = CachedAttachmentInfo.Actor->GetTransform();
		return true;
	}
	return false;
}

FCameraActorAttachmentInfoArrayReader::FCameraActorAttachmentInfoArrayReader(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos, FCameraContextDataID InDataID)
{
	Initialize(InAttachmentInfos, InDataID);
}

void FCameraActorAttachmentInfoArrayReader::Initialize(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos, FCameraContextDataID InDataID)
{
	DataID = InDataID;

	CacheAttachmentInfos(InAttachmentInfos);
}

void FCameraActorAttachmentInfoArrayReader::CacheAttachmentInfos(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos)
{
	Readers.SetNum(InAttachmentInfos.Num());

	for (int32 Index = 0; Index < InAttachmentInfos.Num(); ++Index)
	{
		Readers[Index].CacheAttachmentInfo(InAttachmentInfos[Index]);
	}
}

bool FCameraActorAttachmentInfoArrayReader::GetAttachmentTransform(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform)
{
	if (DataID.IsValid())
	{
		TConstArrayView<FCameraActorAttachmentInfo> NewAttachmentInfos;
		if (ContextDataTable.TryGetArrayData<FCameraActorAttachmentInfo>(DataID, NewAttachmentInfos))
		{
			CacheAttachmentInfos(NewAttachmentInfos);
		}
	}

	if (Readers.IsEmpty())
	{
		return false;
	}

	struct FComputedAttachmentInfo
	{
		FTransform3d Transform;
		float Weight = 1.f;
	};

	TArray<FComputedAttachmentInfo> ComputedAttachments;
	ComputedAttachments.SetNum(Readers.Num());

	for (int32 Index = 0; Index < Readers.Num(); ++Index)
	{
		FCameraActorAttachmentInfoReader& Reader(Readers[Index]);
		FComputedAttachmentInfo& ComputedAttachment(ComputedAttachments[Index]);
		const bool bGotAttachment = Reader.GetAttachmentTransform(ContextDataTable, ComputedAttachment.Transform);
		ComputedAttachment.Weight = bGotAttachment ? Reader.CachedAttachmentInfo.Weight : 0.f;
	}

	float TotalWeight = Algo::Accumulate(
			ComputedAttachments, 
			0.f,
			[](float Cur, const FComputedAttachmentInfo& Item) { return Cur + Item.Weight; });
	if (TotalWeight == 0.f)
	{
		return false;
	}

	OutTransform = FTransform3d::Identity;
	for (const FComputedAttachmentInfo& ComputedAttachment : ComputedAttachments)
	{
		if (ComputedAttachment.Weight > 0.f)
		{
			OutTransform.BlendWith(ComputedAttachment.Transform, ComputedAttachment.Weight / TotalWeight);
		}
	}

	return true;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

FString FCameraActorAttachmentInfoReader::RenderAttachmentInfo() const
{
	return FString::Printf(TEXT("Actor '%s' (Bone '%s')"), *GetNameSafe(CachedAttachmentInfo.Actor), *CachedBoneName.ToString());
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

