// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/EasingBlendCameraNode.h"

#include "Core/CameraContextDataReader.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EasingBlendCameraNode)

namespace UE::Cameras
{

namespace Internal
{

float SinOut(float InTime)
{
	return FMath::Sin(.5f * UE_PI * InTime);
}

float SinIn(float InTime)
{
	return 1.f - SinOut(1.f - InTime);
}

float PowIn(float InTime, float Power)
{
	return FMath::Pow(InTime, Power);
}

float PowOut(float InTime, float Power)
{
	return 1.f - FMath::Pow(1.f - InTime, Power);
}

float ExpIn(float InTime)
{
	return FMath::Pow(2, 10*(InTime - 1.f));
}

float ExpOut(float InTime)
{
	return 1.f - ExpIn(1.f - InTime);
}

float CircIn(float InTime)
{
	return 1.f - FMath::Sqrt(1-InTime*InTime);
}

float CircOut(float InTime)
{
	return 1.f - CircIn(1.f - InTime);
}

float EvaluateEasing(EEasingCameraBlendType BlendType, float Factor)
{
	// The algorithms below match the ones from Sequencer (see UMovieSceneBuiltInEasingFunction).
	const float InTime = Factor * 2.f;
	const float OutTime = (Factor - .5f) * 2.f;

	switch (BlendType)
	{
		case EEasingCameraBlendType::SinIn:
			return SinIn(Factor);
		case EEasingCameraBlendType::SinOut:
			return SinOut(Factor);
		case EEasingCameraBlendType::SinInOut:
			return InTime < 1.f ? .5f * SinIn(InTime) : .5f + .5f * SinOut(OutTime);

		case EEasingCameraBlendType::QuadIn:
			return PowIn(Factor,2);
		case EEasingCameraBlendType::QuadOut:
			return PowOut(Factor,2);
		case EEasingCameraBlendType::QuadInOut:
			return InTime < 1.f ? .5f * PowIn(InTime, 2) : .5f + .5f * PowOut(OutTime, 2);

		case EEasingCameraBlendType::Cubic:
			return FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, Factor), 0.f, 1.f);
		case EEasingCameraBlendType::CubicIn:
			return PowIn(Factor,3);
		case EEasingCameraBlendType::CubicOut:
			return PowOut(Factor,3);
		case EEasingCameraBlendType::CubicInOut:
			return InTime < 1.f ? .5f * PowIn(InTime,3) : .5f + .5f * PowOut(OutTime, 3);

		case EEasingCameraBlendType::HermiteCubicInOut:
			return FMath::Clamp<float>(FMath::SmoothStep(0.0f, 1.0f, Factor), 0.0f, 1.0f);

		case EEasingCameraBlendType::QuartIn:
			return PowIn(Factor, 4);
		case EEasingCameraBlendType::QuartOut:
			return PowOut(Factor, 4);
		case EEasingCameraBlendType::QuartInOut:
			return InTime < 1.f ? .5f * PowIn(InTime, 4) : .5f + .5f * PowOut(OutTime, 4);

		case EEasingCameraBlendType::QuintIn:
			return PowIn(Factor,5);
		case EEasingCameraBlendType::QuintOut:
			return PowOut(Factor,5);
		case EEasingCameraBlendType::QuintInOut:
			return InTime < 1.f ? .5f * PowIn(InTime, 5) : .5f + .5f * PowOut(OutTime, 5);

		case EEasingCameraBlendType::ExpoIn:
			return ExpIn(Factor);
		case EEasingCameraBlendType::ExpoOut:
			return ExpOut(Factor);
		case EEasingCameraBlendType::ExpoInOut:
			return InTime < 1.f ? .5f * ExpIn(InTime) : .5f + .5f * ExpOut(OutTime);

		case EEasingCameraBlendType::CircIn:
			return CircIn(Factor);
		case EEasingCameraBlendType::CircOut:
			return CircOut(Factor);
		case EEasingCameraBlendType::CircInOut:
			return InTime < 1.f ? .5f * CircIn(InTime) : .5f + .5f * CircOut(OutTime);

		case EEasingCameraBlendType::Linear:
		default:
			return Factor;
	}
}

}  // namespace Internal

class FEasingBlendCameraNodeEvaluator : public FSimpleFixedTimeBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FEasingBlendCameraNodeEvaluator, FSimpleFixedTimeBlendCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult) override;

private:

	EEasingCameraBlendType BlendType;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FEasingBlendCameraNodeEvaluator)

void FEasingBlendCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	const UEasingBlendCameraNode* BlendNode = GetCameraNodeAs<UEasingBlendCameraNode>();
	TCameraContextDataReader<EEasingCameraBlendType> BlendTypeReader;
	BlendTypeReader.Initialize(&BlendNode->BlendType, BlendNode->BlendTypeDataID);
	BlendType = BlendTypeReader.Get(OutResult.ContextDataTable);
}

void FEasingBlendCameraNodeEvaluator::OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult)
{
	using namespace UE::Cameras::Internal;

	const float TimeFactor = GetTimeFactor();
	OutResult.BlendFactor = EvaluateEasing(BlendType, TimeFactor);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UEasingBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FEasingBlendCameraNodeEvaluator>();
}

