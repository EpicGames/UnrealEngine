// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/InterruptedBlendCameraNode.h"

namespace UE::Cameras
{

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FInterruptedBlendCameraNodeEvaluator)

FInterruptedBlendCameraNodeEvaluator::FInterruptedBlendCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
}

FInterruptedBlendCameraNodeEvaluator::FInterruptedBlendCameraNodeEvaluator(FBlendCameraNodeEvaluator* InChildBlend, FBlendCameraNodeEvaluator* InFrozenBlend) 
	: ChildBlend(InChildBlend)
	, FrozenBlend(InFrozenBlend)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
}

FCameraNodeEvaluatorChildrenView FInterruptedBlendCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView{ ChildBlend, FrozenBlend };
}

void FInterruptedBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (ChildBlend)
	{
		ChildBlend->Run(Params, OutResult);
	}
	// Don't run the frozen blend... it's... frozen.
}

void FInterruptedBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	FrozenResult.VariableTable.OverrideAll(OutResult.VariableTable, true);

	if (FrozenBlend)
	{
		FCameraNodePreBlendResult FrozenPreBlendResult(FrozenResult.VariableTable);
		FrozenBlend->BlendParameters(Params, FrozenPreBlendResult);
	}
	if (ChildBlend)
	{
		FCameraNodePreBlendParams FrozenPreBlendParams(Params.EvaluationParams, Params.LastCameraPose, FrozenResult.VariableTable);
		ChildBlend->BlendParameters(FrozenPreBlendParams, OutResult);
	}
}

void FInterruptedBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	FrozenResult.OverrideAll(OutResult.BlendedResult, true);

	if (FrozenBlend)
	{
		FCameraNodeBlendResult FrozenBlendResult(FrozenResult);
		FrozenBlend->BlendResults(Params, FrozenBlendResult);
	}
	if (ChildBlend)
	{
		FCameraNodeBlendParams FrozenBlendParams(Params.ChildParams, FrozenResult);
		ChildBlend->BlendResults(FrozenBlendParams, OutResult);
	}
}

}  // namespace UE::Cameras

