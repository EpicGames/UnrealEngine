// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/LinearBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinearBlendCameraNode)

namespace UE::Cameras
{

class FLinearBlendCameraNodeEvaluator : public FSimpleFixedTimeBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FLinearBlendCameraNodeEvaluator, FSimpleFixedTimeBlendCameraNodeEvaluator)

protected:
	virtual void OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult) override;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FLinearBlendCameraNodeEvaluator)

void FLinearBlendCameraNodeEvaluator::OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult)
{
	const ULinearBlendCameraNode* BlendNode = GetCameraNodeAs<ULinearBlendCameraNode>();
	OutResult.BlendFactor = FMath::Lerp(0.f, 1.f, GetTimeFactor());
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr ULinearBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FLinearBlendCameraNodeEvaluator>();
}

