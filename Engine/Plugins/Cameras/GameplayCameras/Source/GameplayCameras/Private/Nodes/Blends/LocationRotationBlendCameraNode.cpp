// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/LocationRotationBlendCameraNode.h"

#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Nodes/Blends/PopBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocationRotationBlendCameraNode)

namespace UE::Cameras
{

class FLocationRotationBlendCameraNodeEvaluator : public FBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FLocationRotationBlendCameraNodeEvaluator)

public:

	FLocationRotationBlendCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
	}

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) override;
	virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	FSimpleBlendCameraNodeEvaluator* LocationBlendEvaluator = nullptr;
	FSimpleBlendCameraNodeEvaluator* RotationBlendEvaluator = nullptr;
	FSimpleBlendCameraNodeEvaluator* OtherBlendEvaluator = nullptr;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FLocationRotationBlendCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FLocationRotationBlendCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, LocationBlendFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, RotationBlendFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, OtherBlendFactor);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FLocationRotationBlendCameraDebugBlock)

void FLocationRotationBlendCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const ULocationRotationBlendCameraNode* LocationRotationBlend = GetCameraNodeAs<ULocationRotationBlendCameraNode>();
	if (LocationRotationBlend->LocationBlend)
	{
		LocationBlendEvaluator = Params.BuildEvaluatorAs<FSimpleBlendCameraNodeEvaluator>(LocationRotationBlend->LocationBlend);
	}
	if (LocationRotationBlend->RotationBlend)
	{
		RotationBlendEvaluator = Params.BuildEvaluatorAs<FSimpleBlendCameraNodeEvaluator>(LocationRotationBlend->RotationBlend);
	}
	if (LocationRotationBlend->OtherBlend)
	{
		OtherBlendEvaluator = Params.BuildEvaluatorAs<FSimpleBlendCameraNodeEvaluator>(LocationRotationBlend->OtherBlend);
	}
}

FCameraNodeEvaluatorChildrenView FLocationRotationBlendCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ LocationBlendEvaluator, RotationBlendEvaluator, OtherBlendEvaluator });
}

void FLocationRotationBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	if (OtherBlendEvaluator)
	{
		OtherBlendEvaluator->BlendParameters(Params, OutResult);
	}
	else
	{
		FPopBlendCameraNodeHelper::PopParameters(Params, OutResult);
	}
}

void FLocationRotationBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (LocationBlendEvaluator)
	{
		LocationBlendEvaluator->Run(Params, OutResult);
	}
	if (RotationBlendEvaluator)
	{
		RotationBlendEvaluator->Run(Params, OutResult);
	}
	if (OtherBlendEvaluator)
	{
		OtherBlendEvaluator->Run(Params, OutResult);
	}
}

void FLocationRotationBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	const FCameraNodeEvaluationResult& ChildResult(Params.ChildResult);
	FCameraNodeEvaluationResult& BlendedResult(OutResult.BlendedResult);

	// Save the "from" location and rotation.
	const FVector FromLocation = BlendedResult.CameraPose.GetLocation();
	const FRotator FromRotation = BlendedResult.CameraPose.GetRotation();

	// Blend the whole result. We will overwrite location and rotation with our own logic.
	bool bOtherBlendFull = true;
	bool bOtherBlendFinished = true;
	if (OtherBlendEvaluator)
	{
		OtherBlendEvaluator->BlendResults(Params, OutResult);

		bOtherBlendFull = OutResult.bIsBlendFull;
		bOtherBlendFinished = OutResult.bIsBlendFinished;
	}
	else
	{
		FPopBlendCameraNodeHelper::PopResults(Params, OutResult);
	}

	bool bLocationBlendFull = bOtherBlendFull;
	bool bLocationBlendFinished = bOtherBlendFinished;
	if (LocationBlendEvaluator)
	{
		const float LocationBlendFactor = LocationBlendEvaluator->GetBlendFactor();
		const FVector ToLocation = ChildResult.CameraPose.GetLocation();
		const FVector BlendedLocation = FMath::Lerp(FromLocation, ToLocation, LocationBlendFactor);
		BlendedResult.CameraPose.SetLocation(BlendedLocation);

		bLocationBlendFull = LocationBlendEvaluator->IsBlendFull();
		bLocationBlendFinished = LocationBlendEvaluator->IsBlendFinished();
	}

	bool bRotationBlendFull = bOtherBlendFull;
	bool bRotationBlendFinished = bOtherBlendFinished;
	if (RotationBlendEvaluator)
	{
		const float RotationBlendFactor = RotationBlendEvaluator->GetBlendFactor();
		const FRotator ToRotation = ChildResult.CameraPose.GetRotation();
		const FRotator BlendedRotation = FMath::Lerp(FromRotation, ToRotation, RotationBlendFactor);
		BlendedResult.CameraPose.SetRotation(BlendedRotation);

		bRotationBlendFull = RotationBlendEvaluator->IsBlendFull();
		bRotationBlendFinished = RotationBlendEvaluator->IsBlendFinished();
	}

	OutResult.bIsBlendFull = (bOtherBlendFull && bLocationBlendFull && bRotationBlendFull);
	OutResult.bIsBlendFinished = (bOtherBlendFinished && bLocationBlendFinished && bRotationBlendFinished);
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FLocationRotationBlendCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FLocationRotationBlendCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FLocationRotationBlendCameraDebugBlock>();
	DebugBlock.LocationBlendFactor = LocationBlendEvaluator ? LocationBlendEvaluator->GetBlendFactor() : -1.f;
	DebugBlock.RotationBlendFactor = RotationBlendEvaluator ? RotationBlendEvaluator->GetBlendFactor() : -1.f;
	DebugBlock.OtherBlendFactor = OtherBlendEvaluator ? OtherBlendEvaluator->GetBlendFactor() : -1.f;
}

void FLocationRotationBlendCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (LocationBlendFactor >= 0.f)
	{
		Renderer.AddText(TEXT("location %.2f%% "), LocationBlendFactor * 100.f);
	}
	if (RotationBlendFactor >= 0.f)
	{
		Renderer.AddText(TEXT("rotation %.2f%% "), RotationBlendFactor * 100.f);
	}
	if (OtherBlendFactor >= 0.f)
	{
		Renderer.AddText(TEXT("other %.2f%% "), OtherBlendFactor * 100.f);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

ULocationRotationBlendCameraNode::ULocationRotationBlendCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView ULocationRotationBlendCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView({ LocationBlend, RotationBlend, OtherBlend });
}

FCameraNodeEvaluatorPtr ULocationRotationBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FLocationRotationBlendCameraNodeEvaluator>();
}

