// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/OrbitBlendCameraNode.h"

#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OrbitBlendCameraNode)

namespace UE::Cameras
{

float GOrbitBlendDeactivationSmoothingTime = 0.3f;
static FAutoConsoleVariableRef CVarOrbitBlendDeactivationSmoothingTime(
	TEXT("GameplayCameras.OrbitBlend.DeactivationSmoothingTime"),
	GOrbitBlendDeactivationSmoothingTime,
	TEXT("(Default: 0.3 seconds. The time to smooth out any differences between a deactivated orbit blend and its underlying blend."));

float GOrbitBlendDeactivationSmoothingMinTime = 0.01f;

class FOrbitBlendCameraNodeEvaluator : public FBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FOrbitBlendCameraNodeEvaluator)

public:

	FOrbitBlendCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);
	}

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) override;
	virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	static bool ClosestPoints(const FRay3d& A, const FRay3d& B, double& OutParameterA, double& OutParameterB);

private:

	enum class FState
	{
		Active,
		SmoothingOut,
		Inactive
	};

	FSimpleBlendCameraNodeEvaluator* DrivingBlendEvaluator = nullptr;

	FVector3d DeltaLocation = FVector3d::ZeroVector;
	FRotator3d DeltaRotation = FRotator3d::ZeroRotator;
	float SmoothingTimeLeft = -1.f;
	FState State = FState::Active;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FOrbitBlendCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FOrbitBlendCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, DeltaLocation);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, DeltaRotation);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, SmoothingTimeLeft);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsActive);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FOrbitBlendCameraDebugBlock)

void FOrbitBlendCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UOrbitBlendCameraNode* OrbitBlend = GetCameraNodeAs<UOrbitBlendCameraNode>();
	if (OrbitBlend->DrivingBlend)
	{
		DrivingBlendEvaluator = Params.BuildEvaluatorAs<FSimpleBlendCameraNodeEvaluator>(OrbitBlend->DrivingBlend);
	}
}

FCameraNodeEvaluatorChildrenView FOrbitBlendCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ DrivingBlendEvaluator });
}

void FOrbitBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	if (DrivingBlendEvaluator)
	{
		DrivingBlendEvaluator->BlendParameters(Params, OutResult);
	}
	else
	{
		const FCameraVariableTable& ChildVariableTable(Params.ChildVariableTable);
		OutResult.VariableTable.Override(ChildVariableTable, Params.VariableTableFilter);

		OutResult.bIsBlendFinished = true;
		OutResult.bIsBlendFull = true;
	}
}

void FOrbitBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (DrivingBlendEvaluator)
	{
		DrivingBlendEvaluator->Run(Params, OutResult);
	}
}

void FOrbitBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	const FCameraNodeEvaluationResult& ChildResult(Params.ChildResult);
	FCameraNodeEvaluationResult& BlendedResult(OutResult.BlendedResult);

	// If we don't have a driving blend, just cut to the new camera rig.
	if (!DrivingBlendEvaluator)
	{
		BlendedResult.OverrideAll(ChildResult);

		BlendedResult.bIsCameraCut = true;

		OutResult.bIsBlendFinished = true;
		OutResult.bIsBlendFull = true;

		return;
	}
	
	// Let our underlying blend do most of the blending, but overwrite the camera transform with our
	// own blending algorithm.
	//
	// But first, remember a few things about the original camera poses.
	const FRay3d FromAim = BlendedResult.CameraPose.GetAimRay();
	const FRay3d ToAim = ChildResult.CameraPose.GetAimRay();

	const FVector3d FromLocation = BlendedResult.CameraPose.GetLocation();
	const FVector3d ToLocation = ChildResult.CameraPose.GetLocation();

	// Run the underlying blend.
	DrivingBlendEvaluator->BlendResults(Params, OutResult);

	// If the blend reached 100%, we're done.
	if (OutResult.bIsBlendFull)
	{
		return;
	}

	if (State == FState::Active)
	{
		// Find the points on each line of sight that are the closest to each other. If successful,
		// start blending around an interpolating mid-point between the two.
		double FromClosestParam, ToClosestParam;
		const bool bFoundClosestPoints = ClosestPoints(FromAim, ToAim, FromClosestParam, ToClosestParam);

		// We only do the orbit blend if we (1) found the appropriate points and (2) they are both in
		// front of us.
		// (1) may be false if both lines of sight are parallel.
		// (2) may be false if the lines of sight aren't converging (i.e. the two cameras are looking
		//     away from each other, even if just slighly).
		const bool bDoOrbitBlend = bFoundClosestPoints && FromClosestParam > 0 && ToClosestParam > 0;

		if (bDoOrbitBlend)
		{
			const float Factor = DrivingBlendEvaluator->GetBlendFactor();

			const FVector3d BlendedLocation = FMath::Lerp(FromLocation, ToLocation, Factor);

			// Rotate around a point that is interpolating from the first line of sight to the other
			// line of sight.
			const FVector3d FromOrbitCenter = FromAim.PointAt(FromClosestParam);
			const FVector3d ToOrbitCenter = ToAim.PointAt(ToClosestParam);
			const FVector3d BlendedOrbitCenter = FMath::Lerp(FromOrbitCenter, ToOrbitCenter, Factor);

			// The aim direction will get us a yaw/pitch orientation only, so we need to also get the
			// blended roll from the underlying blend's result.
			const FVector3d BlendedAimDir = (BlendedOrbitCenter - BlendedLocation).GetUnsafeNormal();
			const FRotator3d BlendedRotationNoRoll = BlendedAimDir.ToOrientationRotator();
			const float BlendedRoll = BlendedResult.CameraPose.GetRotation().Roll;
			const FRotator3d BlendedRotation(BlendedRotationNoRoll.Pitch, BlendedRotationNoRoll.Yaw, BlendedRoll);

			const double FromOrbitCenterDistance = FVector3d::Distance(FromLocation, FromOrbitCenter);
			const double ToOrbitCenterDistance = FVector3d::Distance(ToLocation, ToOrbitCenter);
			const double BlendedTargetDistance = FMath::Lerp(FromOrbitCenterDistance, ToOrbitCenterDistance, Factor);

			// So, instead of interpolating between the two positions and letting the target move
			// forwards or backwards as TargetDistance interpolates, we do the opposite: we "anchor"
			// the target at the orbit center, and push or pull the position based on the interpolated
			// TargetDistance.
			const FRay3d BlendedReverseAim(BlendedOrbitCenter, -BlendedAimDir, true);
			const FVector3d OrbitingLocation = BlendedReverseAim.PointAt(BlendedTargetDistance);

			// Remember our offset from the underlying blend before we apply our orbit blend.
			DeltaLocation = OrbitingLocation - BlendedResult.CameraPose.GetLocation();
			DeltaRotation = BlendedRotation - BlendedResult.CameraPose.GetRotation();

			// Apply the orbit blend!
			BlendedResult.CameraPose.SetLocation(OrbitingLocation);
			BlendedResult.CameraPose.SetRotation(BlendedRotation);
		}
		else
		{
			// The orbit blend was deactivated for one of the reasons mentioned above. We are going to
			// smooth out the difference betwen us and our underlying driving blend over a short time
			// to prevent creating artefacts... that is, unless we somehow have zero difference.
			const bool bHasDelta = (!DeltaLocation.IsZero() || !DeltaRotation.IsZero());
			State = bHasDelta ? FState::SmoothingOut : FState::Inactive;

			const float SmoothingTime = FMath::Max(GOrbitBlendDeactivationSmoothingTime, GOrbitBlendDeactivationSmoothingMinTime);
			SmoothingTimeLeft = SmoothingTime;

			// NOTE: once we deactivated, we never try to reactivate, to keep things simple.
		}
	}

	if (State == FState::SmoothingOut)
	{

		SmoothingTimeLeft -= Params.ChildParams.DeltaTime;
		if (SmoothingTimeLeft > 0.f)
		{
			// Continue smoothing out the difference linearly.
			const float SmoothingTime = FMath::Max(GOrbitBlendDeactivationSmoothingTime, GOrbitBlendDeactivationSmoothingMinTime);
			const float Alpha = (SmoothingTime - SmoothingTimeLeft) / SmoothingTime;
			const FVector3d CurDeltaLocation = FMath::Lerp(DeltaLocation, FVector3d::ZeroVector, Alpha);
			const FRotator3d CurDeltaRotation = FMath::Lerp(DeltaRotation, FRotator3d::ZeroRotator, Alpha);

			BlendedResult.CameraPose.SetLocation(BlendedResult.CameraPose.GetLocation() + CurDeltaLocation);
			BlendedResult.CameraPose.SetRotation(BlendedResult.CameraPose.GetRotation() + CurDeltaRotation);

			// Even if our underlying blend is finished, we still have some smoothing out to do.
			OutResult.bIsBlendFull = false;
			OutResult.bIsBlendFinished = false;
		}
		else
		{
			// We are done.
			DeltaLocation = FVector3d::ZeroVector;
			DeltaRotation = FRotator3d::ZeroRotator;
			SmoothingTimeLeft = -1.f;
			State = FState::Inactive;
		}
	}
}

void FOrbitBlendCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << DeltaLocation;
	Ar << DeltaRotation;
	Ar << SmoothingTimeLeft;
	Ar << State;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FOrbitBlendCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FOrbitBlendCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FOrbitBlendCameraDebugBlock>();
	DebugBlock.DeltaLocation = DeltaLocation;
	DebugBlock.DeltaRotation = DeltaRotation;
	DebugBlock.SmoothingTimeLeft = SmoothingTimeLeft;
	DebugBlock.bIsActive = (State == FState::Active);
}

void FOrbitBlendCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bIsActive)
	{
		Renderer.AddText(TEXT("orbiting (delta = %s)"), *DeltaLocation.ToString());
	}
	else
	{
		Renderer.AddText(TEXT("INACTIVE (delta = %s  smoothing time %f)"), *DeltaLocation.ToString(), SmoothingTimeLeft);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

bool FOrbitBlendCameraNodeEvaluator::ClosestPoints(const FRay3d& A, const FRay3d& B, double& OutParameterA, double& OutParameterB)
{
	// The points closest to each other on rays A and B are named T1 and T1. They are such that the
	// vector T1T2 is orthogonal to both A and B's direction vectors. So the dot products should be zero:
	//
	//    (T2 - T1).A = 0
	//    (T2 - T1).B = 0
	//
	// We can define T1 and T2 using the parametric equations of the rays:
	//
	//    T1 = O1 + x1*A 
	//    T2 = O2 + x2*B
	//
	// Where O1 and O2 are the origin points of the rays, and x1 and x2 are the linear parameters.
	//
	// So we can rewrite our conditions:
	//
	//    (O2 + x2*B - O1 - x1*A).A = 0
	//    (O2 + x2*B - O1 - x1*A).B = 0
	//
	//    (O2 - O1).A + x2*(B.A) - x1*(A.A) = 0
	//    (O2 - O1).B + x2*(B.B) - x1*(A.B) = 0
	//
	// A and B are unit vectors so A.A and B.B equal 1.
	// Also, let's note D = (O2 - O1) and C = (A.B)
	//
	//    D.A + x2*C - x1 = 0
	//    D.B + x2 - x1*C = 0
	//
	// Let's solve for x2:
	//
	//    x1 = D.A + x2*C
	//    D.B + x2 - (D.A + x2*C)*C = 0
	//    D.B + x2 - (D.A)*C - x2*C*C = 0
	//	  x2 = ((D.A)*C - D.B) / (1 - C*C)
	//
	// And x1:
	//
	//    x2 = x1*C - D.B
	//    D.A + (x1*C - D.B)*C - x1 = 0
	//	  D.A + x1*C*C - (D.B)*C - x1 = 0
	//	  (D.A - (D.B)*C) / (1 - C*C) = x1
	//
	// We can see that there is no solution if C*C == 1, which corresponds to parallel rays.
	//
	
	const FVector3d D = (B.Origin - A.Origin);

	const FVector3d DirA = A.Direction.GetUnsafeNormal();
	const FVector3d DirB = B.Direction.GetUnsafeNormal();
	const double C = FVector3d::DotProduct(DirA, DirB);

	const double OneMinusCC = (1.0 - C * C);

	if (OneMinusCC != 0.0)
	{
		OutParameterA = (D.Dot(DirA) - (D.Dot(DirB)*C)) / OneMinusCC;
		OutParameterB = ((D.Dot(DirA)*C) - D.Dot(DirB)) / OneMinusCC;

		return true;
	}

	return false;
}

}  // namespace UE::Cameras

UOrbitBlendCameraNode::UOrbitBlendCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView UOrbitBlendCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView({ DrivingBlend });
}

FCameraNodeEvaluatorPtr UOrbitBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FOrbitBlendCameraNodeEvaluator>();
}

