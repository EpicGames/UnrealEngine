// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/PopBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PopBlendCameraNode)

namespace UE::Cameras
{

class FPopBlendCameraNodeEvaluator : public FBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FPopBlendCameraNodeEvaluator)

public:

	FPopBlendCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
	}

protected:

	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) override;
	virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) override;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FPopBlendCameraNodeEvaluator)

void FPopBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
}

void FPopBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	FPopBlendCameraNodeHelper::PopParameters(Params, OutResult);
}

void FPopBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	FPopBlendCameraNodeHelper::PopResults(Params, OutResult);
}

void FPopBlendCameraNodeHelper::PopParameters(const UE::Cameras::FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	const FCameraVariableTable& ChildVariableTable(Params.ChildVariableTable);
	OutResult.VariableTable.Override(ChildVariableTable, Params.VariableTableFilter);

	OutResult.bIsBlendFull = true;
	OutResult.bIsBlendFinished = true;
}

void FPopBlendCameraNodeHelper::PopResults(const UE::Cameras::FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	const FCameraNodeEvaluationResult& ChildResult(Params.ChildResult);
	FCameraNodeEvaluationResult& BlendedResult(OutResult.BlendedResult);

	BlendedResult.OverrideAll(ChildResult);
	
	if (ChildResult.bIsCameraCut || Params.ChildParams.bIsFirstFrame)
	{
		BlendedResult.bIsCameraCut = true;
	}

	OutResult.bIsBlendFull = true;
	OutResult.bIsBlendFinished = true;
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UPopBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPopBlendCameraNodeEvaluator>();
}

