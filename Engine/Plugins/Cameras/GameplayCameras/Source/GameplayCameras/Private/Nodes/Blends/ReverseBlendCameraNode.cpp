// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/ReverseBlendCameraNode.h"

namespace UE::Cameras
{

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FReverseBlendCameraNodeEvaluator)

FReverseBlendCameraNodeEvaluator::FReverseBlendCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
}

FReverseBlendCameraNodeEvaluator::FReverseBlendCameraNodeEvaluator(FBlendCameraNodeEvaluator* InChildBlend) 
	: ChildBlend(InChildBlend)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
}

FCameraNodeEvaluatorChildrenView FReverseBlendCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView{ ChildBlend };
}

void FReverseBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (ChildBlend)
	{
		ChildBlend->Run(Params, OutResult);
	}
}

void FReverseBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	if (ChildBlend)
	{
		FCameraNodePreBlendParams ReversedParams(Params.EvaluationParams, Params.LastCameraPose, OutResult.VariableTable);
		FCameraNodePreBlendResult ReversedResult(const_cast<FCameraVariableTable&>(Params.ChildVariableTable));
		ChildBlend->BlendParameters(ReversedParams, ReversedResult);
	}
}

void FReverseBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	if (ChildBlend)
	{
		FCameraNodeBlendParams ReversedParams(Params.ChildParams, OutResult.BlendedResult);
		FCameraNodeBlendResult ReversedResult(const_cast<FCameraNodeEvaluationResult&>(Params.ChildResult));
		ChildBlend->BlendResults(ReversedParams, ReversedResult);
	}
}

bool FReverseBlendCameraNodeEvaluator::OnInitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params)
{
	if (ChildBlend)
	{
		return ChildBlend->InitializeFromInterruption(Params);
	}
	return true;
}

}  // namespace UE::Cameras

