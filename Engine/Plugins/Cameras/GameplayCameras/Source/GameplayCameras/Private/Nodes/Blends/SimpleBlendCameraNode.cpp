// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleBlendCameraNode)

namespace UE::Cameras
{

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FSimpleBlendCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FSimpleBlendCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, BlendFactor)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FSimpleBlendCameraDebugBlock)

FSimpleBlendCameraNodeEvaluator::FSimpleBlendCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);
}

void FSimpleBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FSimpleBlendCameraNodeEvaluationResult FactorResult;
	OnComputeBlendFactor(Params, FactorResult);
	BlendFactor = FMath::Clamp(FactorResult.BlendFactor, 0.f, 1.f);
	if (bReverse)
	{
		BlendFactor = 1.f - BlendFactor;
	}
}

void FSimpleBlendCameraNodeEvaluator::OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult)
{
	const FCameraVariableTable& ChildVariableTable(Params.ChildVariableTable);
	OutResult.VariableTable.Lerp(ChildVariableTable, Params.VariableTableFilter, BlendFactor);

	OutResult.bIsBlendFull = (bReverse ? BlendFactor <= 0.f : BlendFactor >= 1.f);
	OutResult.bIsBlendFinished = bIsBlendFinished;
}

void FSimpleBlendCameraNodeEvaluator::OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult)
{
	const FCameraNodeEvaluationResult& ChildResult(Params.ChildResult);
	FCameraNodeEvaluationResult& BlendedResult(OutResult.BlendedResult);

	BlendedResult.LerpAll(ChildResult, BlendFactor);

	OutResult.bIsBlendFull = (bReverse ? BlendFactor <= 0.f : BlendFactor >= 1.f);
	OutResult.bIsBlendFinished = bIsBlendFinished;
}

void FSimpleBlendCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << BlendFactor;
	Ar << bIsBlendFinished;
	Ar << bReverse;
}

bool FSimpleBlendCameraNodeEvaluator::OnSetReversed(bool bInReverse)
{
	bReverse = bInReverse;
	return true;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FSimpleBlendCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FSimpleBlendCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FSimpleBlendCameraDebugBlock>();
	DebugBlock.BlendFactor = BlendFactor;
}

void FSimpleBlendCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("blend %.2f%%"), BlendFactor * 100.f);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FSimpleFixedTimeBlendCameraNodeEvaluator)

void FSimpleFixedTimeBlendCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	const USimpleFixedTimeBlendCameraNode* BlendNode = GetCameraNodeAs<USimpleFixedTimeBlendCameraNode>();
	BlendTimeReader.Initialize(BlendNode->BlendTime);
	TotalTime = BlendTimeReader.Get(OutResult.VariableTable);
}

void FSimpleFixedTimeBlendCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	CurrentTime += Params.DeltaTime;
	if (CurrentTime >= TotalTime)
	{
		CurrentTime = TotalTime;
		SetBlendFinished();
	}

	FSimpleBlendCameraNodeEvaluator::OnRun(Params, OutResult);
}

void FSimpleFixedTimeBlendCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << CurrentTime;
}

bool FSimpleFixedTimeBlendCameraNodeEvaluator::OnInitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params)
{
	// If we are interrupting a fixed-time blend, adjust our own time to the complementary time ratio.
	// That is: if we interrupted a blend that was 70% complete, let's set reduce our blend time to 70% 
	// of its original value.
	if (Params.InterruptedBlend)
	{
		const FSimpleFixedTimeBlendCameraNodeEvaluator* InterruptedBlend = Params.InterruptedBlend->CastThis<FSimpleFixedTimeBlendCameraNodeEvaluator>();
		if (InterruptedBlend)
		{
			const float InterruptedTimeFactor = InterruptedBlend->GetTimeFactor();
			TotalTime = TotalTime * InterruptedTimeFactor;
		}
	}

	// We still want to be wrapped in an interrupted blend.
	return false;
}

float FSimpleFixedTimeBlendCameraNodeEvaluator::GetTimeFactor() const
{
	if (TotalTime > 0.f)
	{
		return CurrentTime / TotalTime;
	}
	return 1.f;
}

}  // namespace UE::Cameras

