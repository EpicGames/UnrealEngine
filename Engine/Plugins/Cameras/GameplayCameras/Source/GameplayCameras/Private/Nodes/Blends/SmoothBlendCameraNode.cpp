// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Blends/SmoothBlendCameraNode.h"

#include "Math/Interpolation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmoothBlendCameraNode)

namespace UE::Cameras
{

class FSmoothBlendCameraNodeEvaluator : public FSimpleFixedTimeBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FSmoothBlendCameraNodeEvaluator, FSimpleFixedTimeBlendCameraNodeEvaluator)

protected:
	virtual void OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult) override;
};

UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(FSmoothBlendCameraNodeEvaluator)

void FSmoothBlendCameraNodeEvaluator::OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult)
{
	using namespace UE::Cameras;

	const USmoothBlendCameraNode* BlendNode = GetCameraNodeAs<USmoothBlendCameraNode>();
	const float t = GetTimeFactor();
	switch (BlendNode->BlendType)
	{
		case ESmoothCameraBlendType::SmoothStep:
			OutResult.BlendFactor = SmoothStep(t);
			break;
		case ESmoothCameraBlendType::SmootherStep:
			OutResult.BlendFactor = SmootherStep(t);
			break;
		default:
			OutResult.BlendFactor = 1.f;
			break;
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr USmoothBlendCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSmoothBlendCameraNodeEvaluator>();
}

