// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Collision/CollisionPushCameraNode.h"

#include "CollisionQueryParams.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraDebugRenderer.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCameras.h"
#include "Math/CameraNodeSpaceMath.h"
#include "Misc/AssertionMacros.h"
#include "WorldCollision.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionPushCameraNode)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCollisionPushCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FCollisionPushCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bCollisionEnabled)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bGotSafePosition)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bGotSafePositionOffset)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(ECollisionSafePosition, SafePositionType)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(ECollisionSafePositionOffsetSpace, SafePositionOffsetSpace)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, PushFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, DampedPushFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsPulling)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bFoundHit)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, HitObjectName)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, SafePosition)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FCollisionPushCameraDebugBlock)

void FCollisionPushCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UCollisionPushCameraNode* CollisionPushNode = GetCameraNodeAs<UCollisionPushCameraNode>();

	EnableCollisionReader.Initialize(CollisionPushNode->EnableCollision, true);
	CustomSafePositionReader.Initialize(CollisionPushNode->CustomSafePosition);

	CollisionSphereRadiusReader.Initialize(CollisionPushNode->CollisionSphereRadius);
	SafePositionOffsetReader.Initialize(CollisionPushNode->SafePositionOffset);

	PushInterpolator = CollisionPushNode->PushInterpolator ?
		CollisionPushNode->PushInterpolator->BuildDoubleInterpolator() :
		MakeUnique<TPopValueInterpolator<double>>();
	PullInterpolator = CollisionPushNode->PullInterpolator ?
		CollisionPushNode->PullInterpolator->BuildDoubleInterpolator() :
		MakeUnique<TPopValueInterpolator<double>>();
}

void FCollisionPushCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (!ensure(Params.EvaluationContext))
	{
		return;
	}

	UWorld* World = Params.EvaluationContext->GetWorld();
	APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController();
	if (!World || !PlayerController)
	{
		return;
	}

	// Get the safe position... bail out if we don't have any.
	TOptional<FVector3d> SafePosition = GetFinalSafePosition(Params, OutResult);
	if (!SafePosition.IsSet())
	{
		return;
	}

	// See if collision is enabled. If not, handle it as if we didn't collide with anything.
	const bool bEnableCollision = EnableCollisionReader.Get(OutResult.VariableTable);
#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugCollisionEnabled = bEnableCollision;
#endif
	if (!bEnableCollision)
	{
		HandleDisabledCollision(SafePosition.GetValue(), Params, OutResult);
		return;
	}

	if (Params.EvaluationType != ECameraNodeEvaluationType::Standard)
	{
		// Don't run collision traces during IK/stateless updates.
		// Push the camera by the same amount as last time we updated properly, if possible.
		if (SafePosition.IsSet())
		{
			const FVector3d CameraPoseLocation = OutResult.CameraPose.GetLocation();
			const FVector3d PushedLocation = CameraPoseLocation + (SafePosition.GetValue() - CameraPoseLocation) * LastDampedPushFactor;
			OutResult.CameraPose.SetLocation(PushedLocation);
		}
		return;
	}

	// Actually run some collision tests!
	HandleAsyncCollisionTraceResult(World, SafePosition.GetValue(), Params, OutResult);
	RunCollisionTrace(World, PlayerController, SafePosition.GetValue(), Params, OutResult);
}

TOptional<FVector3d> FCollisionPushCameraNodeEvaluator::GetFinalSafePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult)
{
	if (!ensure(Params.Evaluator))
	{
		return TOptional<FVector3d>();
	}

	if (!ensure(Params.EvaluationContext))
	{
		return TOptional<FVector3d>();
	}

	// Get the safe position itself first.
	TOptional<FVector3d> OptSafePosition = GetSafePosition(Params, OutResult);	
	if (!OptSafePosition.IsSet())
	{
		return TOptional<FVector3d>();
	}

	// Apply the offset in the specified space.
	FVector3d SafePosition = OptSafePosition.GetValue();
	bool bGotSafePositionOffset = true;
	const FVector3d SafePositionOffset = SafePositionOffsetReader.Get(OutResult.VariableTable);
	if (!SafePositionOffset.IsZero())
	{
		ECameraNodeSpace SafePositionOffsetSpace;
		const UCollisionPushCameraNode* ThisNode = GetCameraNodeAs<UCollisionPushCameraNode>();
		switch (ThisNode->SafePositionOffsetSpace)
		{
			case ECollisionSafePositionOffsetSpace::ActiveContext:
				SafePositionOffsetSpace = ECameraNodeSpace::ActiveContext;
				break;
			case ECollisionSafePositionOffsetSpace::OwningContext:
				SafePositionOffsetSpace = ECameraNodeSpace::OwningContext;
				break;
			case ECollisionSafePositionOffsetSpace::Pivot:
				SafePositionOffsetSpace = ECameraNodeSpace::Pivot;
				break;
			case ECollisionSafePositionOffsetSpace::CameraPose:
				SafePositionOffsetSpace = ECameraNodeSpace::CameraPose;
				break;
			case ECollisionSafePositionOffsetSpace::Pawn:
				SafePositionOffsetSpace = ECameraNodeSpace::Pawn;
				break;
			default:
				ensure(false);
				SafePositionOffsetSpace = ECameraNodeSpace::Pivot;
				break;
		}

		bGotSafePositionOffset = FCameraNodeSpaceMath::OffsetCameraNodeSpacePosition(
				Params, OutResult, SafePosition, SafePositionOffset, SafePositionOffsetSpace, SafePosition);
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugGotSafePositionOffset = bGotSafePositionOffset;
#endif

	return SafePosition;
}

TOptional<FVector3d> FCollisionPushCameraNodeEvaluator::GetSafePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult)
{
	const UCollisionPushCameraNode* ThisNode = GetCameraNodeAs<UCollisionPushCameraNode>();

#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugGotSafePosition = false;
#endif

	// See if we have a custom safe position.
	FVector3d SafePosition;
	if (CustomSafePositionReader.TryGet(OutResult.VariableTable, SafePosition))
	{
#if UE_GAMEPLAY_CAMERAS_DEBUG
		bDebugGotSafePosition = false;
#endif
		return TOptional<FVector3d>(SafePosition);
	}
	
	// Compute the base safe position.
	ECameraNodeOriginPosition OriginPosition;
	switch (ThisNode->SafePosition)
	{
		case ECollisionSafePosition::ActiveContext:
			OriginPosition = ECameraNodeOriginPosition::ActiveContext;
			break;
		case ECollisionSafePosition::OwningContext:
			OriginPosition = ECameraNodeOriginPosition::OwningContext;
			break;
		case ECollisionSafePosition::Pivot:
			OriginPosition = ECameraNodeOriginPosition::Pivot;
			break;
		case ECollisionSafePosition::Pawn:
			OriginPosition = ECameraNodeOriginPosition::Pawn;
			break;
		default:
			ensure(false);
			OriginPosition = ECameraNodeOriginPosition::Pivot;
			break;
	}

	const bool bGotSafePosition = FCameraNodeSpaceMath::GetCameraNodeOriginPosition(Params, OutResult, OriginPosition, SafePosition);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugGotSafePosition = bGotSafePosition;
#endif
	if (bGotSafePosition)
	{
		return TOptional<FVector3d>(SafePosition);
	}
	else
	{
		return TOptional<FVector3d>();
	}
}

void FCollisionPushCameraNodeEvaluator::RunCollisionTrace(UWorld* World, APlayerController* PlayerController, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCollisionPushCameraNode* CollisionPushNode = GetCameraNodeAs<UCollisionPushCameraNode>();
	ECollisionChannel CollisionChannel = CollisionPushNode->CollisionChannel;

	const float CollisionSphereRadius = CollisionSphereRadiusReader.Get(OutResult.VariableTable);
	const FVector3d SafePositionOffset = SafePositionOffsetReader.Get(OutResult.VariableTable);

	const FVector3d TraceStart(SafePosition);
	const FVector3d TraceEnd(OutResult.CameraPose.GetLocation());

	double TraceLength = FVector3d::Distance(TraceStart, TraceEnd);
	if (TraceLength <= 0)
	{
		return;
	}

	FCollisionTraceParams TraceParams;
	TraceParams.World = World;
	TraceParams.Pawn = PlayerController->GetPawn();
	TraceParams.SafePosition = SafePosition;
	TraceParams.TraceStart = TraceStart;
	TraceParams.TraceEnd = TraceEnd;
	TraceParams.CollisionSphereRadius = CollisionSphereRadius;
	TraceParams.CollisionChannel = CollisionChannel;
	TraceParams.bRequestedAsyncCollision = CollisionPushNode->bRunAsyncCollision;

	FCollisionTraceResult TraceResult;

	RunCollisionTrace(Params, TraceParams, OutResult, TraceResult);

	CollisionTraceHandle = TraceResult.AsyncTraceHandle;
	if (!CollisionTraceHandle.IsValid())
	{
		HandleCollisionTraceResult(World, TraceResult.HitResults, SafePosition, Params, OutResult);
	}
}

void FCollisionPushCameraNodeEvaluator::RunCollisionTrace(const FCameraNodeEvaluationParams& Params, const FCollisionTraceParams& TraceParams, FCameraNodeEvaluationResult& OutResult, FCollisionTraceResult& OutTraceResult)
{
	static FName CollisionTraceTag(TEXT("CameraCollision"));
	static FName CollisionTraceOwnerTag(TEXT("CollisionPushCameraNode"));

	FCollisionShape SweepShape = FCollisionShape::MakeSphere(TraceParams.CollisionSphereRadius);
	// Ignore the player pawn by default.
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(StartCollisionSweep), false, TraceParams.Pawn);
	QueryParams.TraceTag = CollisionTraceTag;
	QueryParams.OwnerTag = CollisionTraceOwnerTag;

	if (TraceParams.bRequestedAsyncCollision)
	{
		OutTraceResult.AsyncTraceHandle = TraceParams.World->AsyncSweepByChannel(
			EAsyncTraceType::Single,
			TraceParams.TraceStart, TraceParams.TraceEnd, FQuat::Identity,
			TraceParams.CollisionChannel,
			SweepShape,
			QueryParams,
			FCollisionResponseParams::DefaultResponseParam);
	}
	else
	{
		TraceParams.World->SweepMultiByChannel(
			OutTraceResult.HitResults,
			TraceParams.TraceStart, TraceParams.TraceEnd, FQuat::Identity,
			TraceParams.CollisionChannel,
			SweepShape,
			QueryParams,
			FCollisionResponseParams::DefaultResponseParam);
	}
}

void FCollisionPushCameraNodeEvaluator::HandleAsyncCollisionTraceResult(UWorld* World, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCollisionPushCameraNode* CollisionPushNode = GetCameraNodeAs<UCollisionPushCameraNode>();
	if (!CollisionPushNode->bRunAsyncCollision)
	{
		return;
	}

	if (!CollisionTraceHandle.IsValid())
	{
		return;
	}

	FTraceDatum TraceDatum;
	if (!World->QueryTraceData(CollisionTraceHandle, TraceDatum))
	{
		return;
	}

	HandleCollisionTraceResult(World, TraceDatum.OutHits, SafePosition, Params, OutResult);
}

void FCollisionPushCameraNodeEvaluator::HandleCollisionTraceResult(UWorld* World, TArrayView<const FHitResult> HitResults, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Find any relevant hit in the trace results.
	bool bFoundHit = false;
	float CurrentPushFactor = 0.f;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugHitObjectName.Empty();
	DebugSafePosition = SafePosition;
#endif

	for (const FHitResult& Hit : HitResults)
	{
		if (!Hit.bBlockingHit)
		{
			continue;
		}

		double TraceLength = FVector3d::Distance(Hit.TraceStart, Hit.TraceEnd);
		double DistanceToHit = FVector3d::Distance(Hit.TraceEnd, Hit.Location);
		if (ensure(TraceLength > 0))
		{
			CurrentPushFactor = (float)(DistanceToHit / TraceLength);
			bFoundHit = true;
#if UE_GAMEPLAY_CAMERAS_DEBUG
			if (UObject* HitPhysicsObjectOwner = Hit.PhysicsObjectOwner.Get())
			{
				DebugHitObjectName = GetNameSafe(HitPhysicsObjectOwner);
			}
			else
			{
				DebugHitObjectName = TEXT("<no physics object owner>");
			}
#endif
			break;
		}
	}

	UpdatePushFactor(bFoundHit, CurrentPushFactor, SafePosition, Params, OutResult);
}

void FCollisionPushCameraNodeEvaluator::HandleDisabledCollision(const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugHitObjectName.Empty();
	DebugSafePosition = SafePosition;
#endif

	UpdatePushFactor(false, 0.f, SafePosition, Params, OutResult);
}

void FCollisionPushCameraNodeEvaluator::UpdatePushFactor(bool bFoundHit, float CurrentPushFactor, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugFoundHit = bFoundHit;
#endif

	// If we weren't pushed last frame, and we have no reason to push this frame either, then
	// we can bail out early.
	if (!bFoundHit && LastDampedPushFactor < UE_SMALL_NUMBER)
	{
		LastDirection = ECameraCollisionDirection::Pushing;
		LastPushFactor = 0;
		LastDampedPushFactor = 0;
		return;
	}

	// Figure out if we're pulling or pushing towards the safe position.
	// If we had no hit, the current push factor is zero.
	ECameraCollisionDirection CurrentDirection = LastDirection;
	if (LastPushFactor < CurrentPushFactor)
	{
		CurrentDirection = ECameraCollisionDirection::Pushing;
	}
	else if (LastPushFactor > CurrentPushFactor)
	{
		CurrentDirection = ECameraCollisionDirection::Pulling;
	}
	// else, push factor hasn't changed, keep the same direction as before.

	// Interpolate the push factor to make camera movements smoother.
	float CurrentDampedPushFactor = LastDampedPushFactor;

	FCameraValueInterpolationParams InterpParams;
	InterpParams.bIsCameraCut = Params.bIsFirstFrame;
	InterpParams.DeltaTime = Params.DeltaTime;
	FCameraValueInterpolationResult InterpResult(OutResult.VariableTable);
	switch (CurrentDirection)
	{
		case ECameraCollisionDirection::Pushing:
			PushInterpolator->Reset(LastDampedPushFactor, CurrentPushFactor);
			CurrentDampedPushFactor = PushInterpolator->Run(InterpParams, InterpResult);
			break;
		case ECameraCollisionDirection::Pulling:
			PullInterpolator->Reset(LastDampedPushFactor, CurrentPushFactor);
			CurrentDampedPushFactor = PullInterpolator->Run(InterpParams, InterpResult);
			break;
	}

	// Push the camera!
	if (CurrentDampedPushFactor > 0)
	{
		const FVector3d CameraPoseLocation = OutResult.CameraPose.GetLocation();
		const FVector3d PushedLocation = CameraPoseLocation + (SafePosition - CameraPoseLocation) * CurrentDampedPushFactor;
		OutResult.CameraPose.SetLocation(PushedLocation);
	}

	LastPushFactor = CurrentPushFactor;
	LastDampedPushFactor = CurrentDampedPushFactor;
	LastDirection = CurrentDirection;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCollisionPushCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FCollisionPushCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FCollisionPushCameraDebugBlock>();

	DebugBlock.bCollisionEnabled = bDebugCollisionEnabled;

	const UCollisionPushCameraNode* ThisNode = GetCameraNodeAs<UCollisionPushCameraNode>();
	DebugBlock.bGotSafePosition = bDebugGotSafePosition;
	DebugBlock.bGotSafePositionOffset = bDebugGotSafePositionOffset;
	DebugBlock.SafePositionType = ThisNode->SafePosition;
	DebugBlock.SafePositionOffsetSpace = ThisNode->SafePositionOffsetSpace;

	DebugBlock.bFoundHit = bDebugFoundHit;
	DebugBlock.HitObjectName = DebugHitObjectName;
	DebugBlock.SafePosition = DebugSafePosition;
	DebugBlock.bIsPulling = (LastDirection == ECameraCollisionDirection::Pulling);
	DebugBlock.PushFactor = LastPushFactor;
	DebugBlock.DampedPushFactor = LastDampedPushFactor;
}

void FCollisionPushCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (!bCollisionEnabled)
	{
		Renderer.AddText(TEXT("collision disabled"));
		return;
	}

	if (DampedPushFactor > 0)
	{
		Renderer.AddText(TEXT("need to push by %.2f%%, currently %.2f%% [%s]"),
				PushFactor * 100.f, DampedPushFactor * 100.f,
				bIsPulling ? TEXT("pulling") : TEXT("pushing"));
		if (bFoundHit)
		{
			Renderer.AddText(TEXT(" (colliding with '%s')"), *HitObjectName);
		}
	}
	else
	{
		Renderer.AddText(TEXT("not pushing"));
	}

	const FCameraDebugColors& Colors = FCameraDebugColors::Get();
	if (!bGotSafePosition)
	{
		Renderer.NewLine();
		Renderer.SetTextColor(Colors.Error);
		Renderer.AddText(TEXT("can't get safe position: "));
		switch (SafePositionType)
		{
			case ECollisionSafePosition::ActiveContext:
				Renderer.AddText(TEXT("no active context"));
				break;
			case ECollisionSafePosition::Pivot:
				Renderer.AddText(TEXT("no pivot nor active context"));
				break;
			case ECollisionSafePosition::Pawn:
				Renderer.AddText(TEXT("no active context nor player controller"));
				break;
			default:
				Renderer.AddText(TEXT("unknown error"));
				break;
		}
		Renderer.SetTextColor(Colors.Default);
	}

	if (!bGotSafePositionOffset)
	{
		Renderer.NewLine();
		Renderer.SetTextColor(Colors.Error);
		Renderer.AddText(TEXT("can't get safe position offset space: "));
		switch (SafePositionOffsetSpace)
		{
			case ECollisionSafePositionOffsetSpace::ActiveContext:
				Renderer.AddText(TEXT("no active context"));
				break;
			case ECollisionSafePositionOffsetSpace::Pivot:
				Renderer.AddText(TEXT("no pivot nor active context"));
				break;
			case ECollisionSafePositionOffsetSpace::Pawn:
				Renderer.AddText(TEXT("no active context nor player controller"));
				break;
			default:
				Renderer.AddText(TEXT("unknown error"));
				break;
		}
		Renderer.SetTextColor(Colors.Default);
	}

	if (Renderer.IsExternalRendering() && bGotSafePosition)
	{
		const FVector2d TextOffset(-20, -20);
		Renderer.DrawPoint(SafePosition, 2.f, FLinearColor::Gray, 2.f);
		Renderer.DrawTextView(SafePosition, TextOffset, TEXT("Safe Position"), FLinearColor::Gray, GEngine->GetTinyFont());
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UCollisionPushCameraNode::UCollisionPushCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CollisionSphereRadius.Value = 10.f;
}

FCameraNodeEvaluatorPtr UCollisionPushCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCollisionPushCameraNodeEvaluator>();
}

