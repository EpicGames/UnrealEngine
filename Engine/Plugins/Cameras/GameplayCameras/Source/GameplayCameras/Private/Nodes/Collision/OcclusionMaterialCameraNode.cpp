// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Collision/OcclusionMaterialCameraNode.h"

#include "CollisionQueryParams.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraSystemEvaluator.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameplayCameras.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/CameraNodeSpaceMath.h"
#include "Misc/AssertionMacros.h"
#include "WorldCollision.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OcclusionMaterialCameraNode)

namespace UE::Cameras
{

struct FOcclusionMaterialOverrideInfo
{
	TArray<UMaterialInterface*> OriginalMaterials;
};

class FOcclusionMaterialCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FOcclusionMaterialCameraNodeEvaluator)

public:

	~FOcclusionMaterialCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	void RunOcclusionTrace(UWorld* World, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void HandleOcclusionTraceResult(UWorld* World);

	void FindAllPrimitiveComponents(UPrimitiveComponent* InPrimitiveComponent, TSet<UPrimitiveComponent*>& OutPrimitiveComponents);

	void ApplyOcclusionMaterial(TSet<UPrimitiveComponent*> PrimitiveComponents);
	void RemoveOcclusionMaterial(TSet<UPrimitiveComponent*> PrimitiveComponents);

	void ResolveWeakPrimitiveComponents(TSet<TWeakObjectPtr<UPrimitiveComponent>> WeakPrimitiveComponents, TSet<UPrimitiveComponent*>& OutPrimitiveComponents);

private:

	TCameraParameterReader<float> OcclusionSphereRadiusReader;
	TCameraParameterReader<FVector3d> OcclusionTargetOffsetReader;

	FTraceHandle OcclusionTraceHandle;
	TObjectPtr<UMaterialInstanceDynamic> OverrideMaterialInstance;
	TSet<TWeakObjectPtr<UPrimitiveComponent>> CurrentlyOccludedPrimitiveComponents;
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FOcclusionMaterialOverrideInfo> AppliedMaterialOverrides;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FOcclusionMaterialCameraNodeEvaluator)

FOcclusionMaterialCameraNodeEvaluator::~FOcclusionMaterialCameraNodeEvaluator()
{
	// Make sure any occluded meshes are released when our camera rig is deactivated it.
	TSet<UPrimitiveComponent*> PrimitiveComponents;
	ResolveWeakPrimitiveComponents(CurrentlyOccludedPrimitiveComponents, PrimitiveComponents);
	RemoveOcclusionMaterial(PrimitiveComponents);
}

void FOcclusionMaterialCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(OverrideMaterialInstance);
}

void FOcclusionMaterialCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UOcclusionMaterialCameraNode* OcclusionMaterialNode = GetCameraNodeAs<UOcclusionMaterialCameraNode>();
	OcclusionSphereRadiusReader.Initialize(OcclusionMaterialNode->OcclusionSphereRadius);
	OcclusionTargetOffsetReader.Initialize(OcclusionMaterialNode->OcclusionTargetOffset);

	if (OcclusionMaterialNode->OcclusionTransparencyMaterial)
	{
		UObject* OuterObject = Params.EvaluationContext->GetOwner();
		OverrideMaterialInstance = UMaterialInstanceDynamic::Create(OcclusionMaterialNode->OcclusionTransparencyMaterial, OuterObject);
	}
	else
	{
		UE_LOGF(LogCameraSystem, Error, 
				"OcclusionMaterialCameraNode: no occlusion transparency material set on '%ls'",
				*GetNameSafe(OcclusionMaterialNode));
	}
}

void FOcclusionMaterialCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (!ensure(Params.EvaluationContext))
	{
		return;
	}

	if (Params.EvaluationType != ECameraNodeEvaluationType::Standard)
	{
		// Don't run occlusion traces during IK/stateless updates.
		return;
	}

	UWorld* World = Params.EvaluationContext->GetWorld();
	if (!World)
	{
		return;
	}

	HandleOcclusionTraceResult(World);
	RunOcclusionTrace(World, Params, OutResult);
}

void FOcclusionMaterialCameraNodeEvaluator::RunOcclusionTrace(UWorld* World, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	static FName OcclusionTraceTag(TEXT("CameraOcclusion"));
	static FName OcclusionTraceOwnerTag(TEXT("OcclusionMaterialCameraNode"));

	const UOcclusionMaterialCameraNode* OcclusionMaterialNode = GetCameraNodeAs<UOcclusionMaterialCameraNode>();

	const FCameraNodeSpaceParams SpaceParams(Params, OutResult);

	FVector3d OcclusionTarget;
	const bool bGotOcclusionTarget = FCameraNodeSpaceMath::GetCameraNodeOriginPosition(
			SpaceParams, OcclusionMaterialNode->OcclusionTargetPosition, OcclusionTarget);
	if (!bGotOcclusionTarget)
	{
		return;
	}

	const FVector3d OcclusionTargetOffset = OcclusionTargetOffsetReader.Get(OutResult.VariableTable);
	if (!OcclusionTargetOffset.IsZero())
	{
		FCameraNodeSpaceMath::OffsetCameraNodeSpacePosition(
				SpaceParams,
				OcclusionTarget, OcclusionTargetOffset, OcclusionMaterialNode->OcclusionTargetOffsetSpace,
				OcclusionTarget);
	}

	ECollisionChannel OcclusionChannel = OcclusionMaterialNode->OcclusionChannel;

	const float OcclusionSphereRadius = OcclusionSphereRadiusReader.Get(OutResult.VariableTable);

	const FVector3d TraceStart(OutResult.CameraPose.GetLocation());
	const FVector3d TraceEnd(OcclusionTarget);

	// Ignore the player pawn by default.
	APawn* Pawn = nullptr;
	if (TSharedPtr<const FCameraEvaluationContext> ActiveContext = SpaceParams.GetActiveContext())
	{
		if (APlayerController* PlayerController = ActiveContext->GetPlayerController())
		{
			Pawn = PlayerController->GetPawn();
		}
	}

	FCollisionShape SweepShape = FCollisionShape::MakeSphere(OcclusionSphereRadius);
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(StartOcclusionSweep), false, Pawn);
	QueryParams.TraceTag = OcclusionTraceTag;
	QueryParams.OwnerTag = OcclusionTraceOwnerTag;

	OcclusionTraceHandle = World->AsyncSweepByChannel(
			EAsyncTraceType::Multi, 
			TraceStart, TraceEnd, FQuat::Identity, 
			OcclusionChannel, 
			SweepShape,
			QueryParams, 
			FCollisionResponseParams::DefaultResponseParam);
}

void FOcclusionMaterialCameraNodeEvaluator::HandleOcclusionTraceResult(UWorld* World)
{
	// Do some basic validation... right now we just bail out if we can't get the trace result
	// without figuring out if it's too old, still running, or whatever else. This is because
	// we're supposed to be running only once a frame, so our trace should have run last frame
	// and be available now. We'll have to better handle error cases when we start doing multi
	// evaluations.
	if (!OcclusionTraceHandle.IsValid())
	{
		return;
	}

	FTraceDatum TraceDatum;
	if (!World->QueryTraceData(OcclusionTraceHandle, TraceDatum))
	{
		return;
	}

	// Get the list of meshes collected by the occlusion trace, and figure out which ones are
	// new, and which ones got out of the way.
	TSet<UPrimitiveComponent*> PrimitiveComponents;
	for (const FHitResult& Hit : TraceDatum.OutHits)
	{
		FindAllPrimitiveComponents(Hit.GetComponent(), PrimitiveComponents);
	}

	TSet<UPrimitiveComponent*> CurrentPrimitiveComponents;
	ResolveWeakPrimitiveComponents(CurrentlyOccludedPrimitiveComponents, CurrentPrimitiveComponents);

	TSet<UPrimitiveComponent*> NewPrimitiveComponents = PrimitiveComponents.Difference(CurrentPrimitiveComponents);
	TSet<UPrimitiveComponent*> OldPrimitiveComponents = CurrentPrimitiveComponents.Difference(PrimitiveComponents);

	CurrentlyOccludedPrimitiveComponents.Reset();
	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		CurrentlyOccludedPrimitiveComponents.Add(PrimitiveComponent);
	}

	// Apply occlusion material changes to new/old components.
	ApplyOcclusionMaterial(NewPrimitiveComponents);
	RemoveOcclusionMaterial(OldPrimitiveComponents);

	OcclusionTraceHandle.Invalidate();
}

void FOcclusionMaterialCameraNodeEvaluator::FindAllPrimitiveComponents(UPrimitiveComponent* InPrimitiveComponent, TSet<UPrimitiveComponent*>& OutPrimitiveComponents)
{
	if (InPrimitiveComponent)
	{
		OutPrimitiveComponents.Add(InPrimitiveComponent);

		for (USceneComponent* AttachedChild : InPrimitiveComponent->GetAttachChildren())
		{
			if (UPrimitiveComponent* AttachedPrimitiveComponent = Cast<UPrimitiveComponent>(AttachedChild))
			{
				FindAllPrimitiveComponents(AttachedPrimitiveComponent, OutPrimitiveComponents);
			}
		}
	}
}

void FOcclusionMaterialCameraNodeEvaluator::ApplyOcclusionMaterial(TSet<UPrimitiveComponent*> PrimitiveComponents)
{
	if (!OverrideMaterialInstance)
	{
		return;
	}

	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (!PrimitiveComponent || AppliedMaterialOverrides.Contains(PrimitiveComponent))
		{
			continue;
		}

		FOcclusionMaterialOverrideInfo& MaterialOverride = AppliedMaterialOverrides.Add(PrimitiveComponent);
		const int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
		{
			UMaterialInterface* OriginalMaterial = PrimitiveComponent->GetMaterial(MaterialIndex);
			MaterialOverride.OriginalMaterials.Add(OriginalMaterial);
			PrimitiveComponent->SetMaterial(MaterialIndex, OverrideMaterialInstance);
		}
	}
}

void FOcclusionMaterialCameraNodeEvaluator::RemoveOcclusionMaterial(TSet<UPrimitiveComponent*> PrimitiveComponents)
{
	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (!PrimitiveComponent)
		{
			continue;
		}

		FOcclusionMaterialOverrideInfo MaterialOverrides;
		const bool bFoundEntry = AppliedMaterialOverrides.RemoveAndCopyValue(PrimitiveComponent, MaterialOverrides);
		if (bFoundEntry)
		{
			const int32 NumMaterials = MaterialOverrides.OriginalMaterials.Num();
			for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
			{
				PrimitiveComponent->SetMaterial(MaterialIndex, MaterialOverrides.OriginalMaterials[MaterialIndex]);
			}
		}
	}
}

void FOcclusionMaterialCameraNodeEvaluator::ResolveWeakPrimitiveComponents(TSet<TWeakObjectPtr<UPrimitiveComponent>> WeakPrimitiveComponents, TSet<UPrimitiveComponent*>& OutPrimitiveComponents)
{
	for (TWeakObjectPtr<UPrimitiveComponent> WeakPrimitiveComponent : WeakPrimitiveComponents)
	{
		if (UPrimitiveComponent* PrimitiveComponent = WeakPrimitiveComponent.Get())
		{
			OutPrimitiveComponents.Add(PrimitiveComponent);
		}
	}
}

}  // namespace UE::Cameras

UOcclusionMaterialCameraNode::UOcclusionMaterialCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	OcclusionSphereRadius.Value = 10.f;
}

FCameraNodeEvaluatorPtr UOcclusionMaterialCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FOcclusionMaterialCameraNodeEvaluator>();
}

