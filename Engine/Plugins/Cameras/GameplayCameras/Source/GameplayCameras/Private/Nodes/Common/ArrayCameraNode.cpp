// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/ArrayCameraNode.h"

#include "Core/CameraNodeEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ArrayCameraNode)

namespace UE::Cameras
{

class FArrayCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FArrayCameraNodeEvaluator)

public:

	FArrayCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
	}

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TArray<FCameraNodeEvaluator*> Children;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FArrayCameraNodeEvaluator)

FCameraNodeEvaluatorChildrenView FArrayCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView(Children);
}

void FArrayCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UArrayCameraNode* ArrayNode = GetCameraNodeAs<UArrayCameraNode>();
	for (const UCameraNode* Child : ArrayNode->Children)
	{
		if (Child)
		{
			FCameraNodeEvaluator* ChildEvaluator = Params.BuildEvaluator(Child);
			Children.Add(ChildEvaluator);
		}
	}
}

void FArrayCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	for (FCameraNodeEvaluator* Child : Children)
	{
		if (Child)
		{
			Child->Run(Params, OutResult);
		}
	}
}

}  // namespace UE::Cameras

UArrayCameraNode::UArrayCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView UArrayCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView(Children);
}

FCameraNodeEvaluatorPtr UArrayCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FArrayCameraNodeEvaluator>();
}

TTuple<int32, int32> UArrayCameraNode::GetEvaluatorAllocationInfo()
{
	using namespace UE::Cameras;
	return { sizeof(FArrayCameraNodeEvaluator), alignof(FArrayCameraNodeEvaluator) };
}

