// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/AutoFocusCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Core/CameraVariableReferenceReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameplayCameras.h"
#include "Math/CriticalDamper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutoFocusCameraNode)

namespace UE::Cameras
{

class FAutoFocusCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FAutoFocusCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	TCameraVariableReferenceReader<bool> EnableAutoFocusReader;
	TCameraParameterReader<float> AutoFocusDampingFactorReader;

	FCriticalDamper AutoFocusDamper;
	double LastUndampedFocusDistance;
	double LastDampedFocusDistance;
	bool bLastEnableAutoFocus;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FAutoFocusCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FAutoFocusCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, UndampedFocusDistance);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, DampedFocusDistance);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, AutoFocusDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bEnableAutoFocus);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FAutoFocusCameraDebugBlock)

void FAutoFocusCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UAutoFocusCameraNode* AutoFocusNode = GetCameraNodeAs<UAutoFocusCameraNode>();

	EnableAutoFocusReader.Initialize(AutoFocusNode->EnableAutoFocus);
	AutoFocusDampingFactorReader.Initialize(AutoFocusNode->AutoFocusDampingFactor);
}

void FAutoFocusCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	bLastEnableAutoFocus = true;
	if (EnableAutoFocusReader.IsDriven())
	{
		bLastEnableAutoFocus = EnableAutoFocusReader.Get(OutResult.VariableTable);
	}

	if (bLastEnableAutoFocus)
	{
		const float AutoFocusDampingFactor = AutoFocusDampingFactorReader.Get(OutResult.VariableTable);
		AutoFocusDamper.SetW0(AutoFocusDampingFactor);

		FCameraPose& OutCameraPose = OutResult.CameraPose;
		LastUndampedFocusDistance = OutCameraPose.GetTargetDistance();

		if (Params.bIsFirstFrame)
		{
			LastDampedFocusDistance = LastUndampedFocusDistance;
			OutCameraPose.SetFocusDistance(LastDampedFocusDistance);
		}
		else
		{
			const double NewFocusDistance = AutoFocusDamper.Update(
					LastDampedFocusDistance, OutCameraPose.GetTargetDistance(), Params.DeltaTime);
			LastDampedFocusDistance = NewFocusDistance;
			OutCameraPose.SetFocusDistance(NewFocusDistance);
		}
	}
}

void FAutoFocusCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << AutoFocusDamper;
	Ar << LastUndampedFocusDistance;
	Ar << LastDampedFocusDistance;
	Ar << bLastEnableAutoFocus;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FAutoFocusCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FAutoFocusCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FAutoFocusCameraDebugBlock>();

	DebugBlock.bEnableAutoFocus = bLastEnableAutoFocus;
	DebugBlock.UndampedFocusDistance = LastUndampedFocusDistance;
	DebugBlock.DampedFocusDistance = LastDampedFocusDistance;
	DebugBlock.AutoFocusDampingFactor = AutoFocusDamper.GetW0();
}

void FAutoFocusCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bEnableAutoFocus)
	{
		Renderer.AddText(
				TEXT("target distance: %.3f  focus distance: %.3f  (damping factor %.1f)"),
				UndampedFocusDistance,
				DampedFocusDistance,
				AutoFocusDampingFactor);
	}
	else
	{
		Renderer.AddText(TEXT("auto-focused DISABLED"));
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UAutoFocusCameraNode::UAutoFocusCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AutoFocusDampingFactor.Value = 0;
}

FCameraNodeEvaluatorPtr UAutoFocusCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAutoFocusCameraNodeEvaluator>();
}

