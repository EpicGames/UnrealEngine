// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/BodyParametersCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Core/CameraPose.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodyParametersCameraNode)

namespace UE::Cameras
{

class FBodyParametersCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBodyParametersCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<float> ShutterSpeedReader;
	TCameraParameterReader<float> ISOReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBodyParametersCameraNodeEvaluator)

void FBodyParametersCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UBodyParametersCameraNode* BodyParametersNode = GetCameraNodeAs<UBodyParametersCameraNode>();
	ShutterSpeedReader.Initialize(BodyParametersNode->ShutterSpeed);
	ISOReader.Initialize(BodyParametersNode->ISO);
}

void FBodyParametersCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& OutPose = OutResult.CameraPose;

	float ShutterSpeed = ShutterSpeedReader.Get(OutResult.VariableTable);
	if (ShutterSpeed > 0)
	{
		OutPose.SetShutterSpeed(ShutterSpeed);
	}
	float ISO = ISOReader.Get(OutResult.VariableTable);
	if (ISO > 0)
	{
		OutPose.SetISO(ISO);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UBodyParametersCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FBodyParametersCameraNodeEvaluator>();
}

