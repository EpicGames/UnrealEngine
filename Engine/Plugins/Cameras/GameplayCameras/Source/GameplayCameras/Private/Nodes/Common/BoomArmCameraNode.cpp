// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/BoomArmCameraNode.h"

#include "Core/CameraRigJoints.h"
#include "Core/CameraValueInterpolator.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Nodes/Input/DrivenControlRotationCameraNode.h"
#include "Nodes/Input/Input2DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoomArmCameraNode)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBoomArmCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FBoomArmCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, BoomYawPitch)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, CumulativePull)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bDidClampPull)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bHasBoomLengthInterpolator)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FBoomArmCameraDebugBlock)

void FBoomArmCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UBoomArmCameraNode* BoomArmNode = GetCameraNodeAs<UBoomArmCameraNode>();
	if (BoomArmNode->InputSlot)
	{
		InputSlotEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(BoomArmNode->InputSlot);
	}
	else
	{
		const UDrivenControlRotationCameraNode* PlaceholderInput = NewObject<UDrivenControlRotationCameraNode>();
		InputSlotEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(PlaceholderInput);
	}
	if (BoomArmNode->BoomLengthInterpolator)
	{
		BoomLengthInterpolator = BoomArmNode->BoomLengthInterpolator->BuildDoubleInterpolator();
	}
}

void FBoomArmCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UBoomArmCameraNode* BoomArmNode = GetCameraNodeAs<UBoomArmCameraNode>();
	BoomOffsetReader.Initialize(BoomArmNode->BoomOffset);
	MaxForwardInterpolationFactorReader.Initialize(BoomArmNode->MaxForwardInterpolationFactor);
	MaxBackwardInterpolationFactorReader.Initialize(BoomArmNode->MaxBackwardInterpolationFactor);

	LastPivotLocation = FVector3d::ZeroVector;
	CumulativePull = 0.0;
}

FCameraNodeEvaluatorChildrenView FBoomArmCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ InputSlotEvaluator });
}

FRotator FBoomArmCameraNodeEvaluator::ComputeBoomRotation(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FRotator3d BoomRotation = FRotator3d::ZeroRotator;
	if (ensure(InputSlotEvaluator))
	{
		InputSlotEvaluator->Run(Params, OutResult);
		const FVector2d YawPitch = InputSlotEvaluator->GetInputValue();
		BoomRotation = FRotator3d(YawPitch.Y, YawPitch.X, 0);
	}
	return BoomRotation;
}

void FBoomArmCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FRotator3d BoomRotation = ComputeBoomRotation(Params, OutResult);

	const UBoomArmCameraNode* BoomArmNode = GetCameraNodeAs<UBoomArmCameraNode>();
	if (BoomArmNode->bAdditiveRotation)
	{
		BoomRotation = OutResult.CameraPose.GetRotation() + BoomRotation;
	}

	// Here we want to logically apply transform in this order:
	//
	// FinalTransform = BoomOffset * BoomRotation * CameraPose.Location
	//
	// Since FTransform3d applies rotation first and translation second, we can save one multiplication
	// by using the fact that BoomRotation is, well, just a rotation, and CameraPose.Location is of
	// course just a translation. So we can put them both in the same transform:
	const FTransform3d BoomPivot(BoomRotation, OutResult.CameraPose.GetLocation());
	const FVector3d BoomOffset(BoomOffsetReader.Get(OutResult.VariableTable));

	FTransform3d FinalTransform(FTransform3d(BoomOffset) * BoomPivot);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugYawPitch = FVector2d(BoomRotation.Yaw, BoomRotation.Pitch);
	bDebugDidClampPull = false;
#endif

	// If we have an interpolator for the boom length, let's run it now. The way we use it is that we
	// keep track of the "pull" on the boom arm, i.e. how much the boom gets pulled in various directions
	// as the pivot moves around (such as when running around with the player character or driving a
	// vehicle). We compute the amount of pull along the pivot<->camera line, and then ask the interpolator
	// to converge towards zero. This creates a sort of rubber-band/spring effect on the boom arm.
	const double DefaultBoomLength = BoomOffset.Length();
	if (BoomLengthInterpolator && DefaultBoomLength > 0)
	{
		if (!Params.bIsFirstFrame && !OutResult.bIsCameraCut)
		{
			// The pull this frame is how much the base (pivot) of the boom arm has moved. We add that
			// to our cumulative tally of the pull.
			// Note that pull is positive when the pivot is moving forwards (away from the camera) and 
			// negative when moving backwards (towards the camera).
			const FVector3d PivotMovement = BoomPivot.GetLocation() - LastPivotLocation;
			const FVector3d ForwardBoomOrientation = BoomRotation.RotateVector(FVector3d::ForwardVector);
			const double PullThisFrame = PivotMovement.Dot(ForwardBoomOrientation);
			CumulativePull += PullThisFrame;

			// Update the interpolator to try and get back to zero pull.
			BoomLengthInterpolator->Reset(CumulativePull, 0);

			FCameraValueInterpolationParams InterpParams;
			InterpParams.DeltaTime = Params.DeltaTime;
			FCameraValueInterpolationResult InterpResult(OutResult.VariableTable);
			double NewCumulativePull = BoomLengthInterpolator->Run(InterpParams, InterpResult);

			// Clamp the cumulative pull to any maximums defined by the user.
			double ClampedPull = NewCumulativePull;
			if (ClampedPull < 0)
			{
				const double MaxForwardInterpolationFactor = MaxForwardInterpolationFactorReader.Get(OutResult.VariableTable);
				if (MaxForwardInterpolationFactor > 0)
				{
					const double MaxForwardPush = DefaultBoomLength * MaxForwardInterpolationFactor;
					ClampedPull = FMath::Max(-MaxForwardPush, ClampedPull);
				}
			}
			else if (ClampedPull > 0)
			{
				const double MaxBackwardInterpolationFactor = MaxBackwardInterpolationFactorReader.Get(OutResult.VariableTable);
				if (MaxBackwardInterpolationFactor > 0)
				{
					const double MaxBackwardPull = DefaultBoomLength * MaxBackwardInterpolationFactor;
					ClampedPull = FMath::Min(MaxBackwardPull, ClampedPull);
				}
			}

			// Add the pull to final transform. This effectively distorts the boom offset, since we move the 
			// camera forwards/backwards based on the boom orientation, not the offset's orientation.
			FinalTransform.SetLocation(FinalTransform.GetLocation() - ForwardBoomOrientation * ClampedPull);

#if UE_GAMEPLAY_CAMERAS_DEBUG
			bDebugDidClampPull = (ClampedPull != NewCumulativePull);
#endif

			CumulativePull = ClampedPull;
		}
		else if (!Params.bIsFirstFrame && OutResult.bIsCameraCut)
		{
			// On camera cuts, we re-use last frame's cumulative pull without updating it.
			const FVector3d ForwardBoomOrientation = BoomRotation.RotateVector(FVector3d::ForwardVector);
			FinalTransform.SetLocation(FinalTransform.GetLocation() - ForwardBoomOrientation * CumulativePull);

			// Leave bDebugDidClampPull to what it was last frame.
		}
		else if (Params.bIsFirstFrame)
		{
			CumulativePull = 0.0;
		}

		LastPivotLocation = BoomPivot.GetLocation();
	}

	OutResult.CameraPose.SetTransform(FinalTransform);
	
	OutResult.CameraRigJoints.AddYawPitchJoint(BoomPivot);
}

void FBoomArmCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	if (BoomLengthInterpolator)
	{
		FCameraValueInterpolatorSerializeParams InterpolatorParams;
		BoomLengthInterpolator->Serialize(InterpolatorParams, Ar);
	}

	Ar << LastPivotLocation;
	Ar << CumulativePull;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FBoomArmCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FBoomArmCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FBoomArmCameraDebugBlock>();

	DebugBlock.BoomYawPitch = DebugYawPitch;
	DebugBlock.CumulativePull = CumulativePull;
	DebugBlock.bDidClampPull = bDebugDidClampPull;
	DebugBlock.bHasBoomLengthInterpolator = BoomLengthInterpolator.IsValid();
}

void FBoomArmCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("yaw: %.3f pitch: %.3f"), BoomYawPitch.X, BoomYawPitch.Y);

	if (bHasBoomLengthInterpolator)
	{
		Renderer.AddText(TEXT(" (pull: %.3f)"), CumulativePull);
		if (bDidClampPull)
		{
			Renderer.AddText(TEXT(" [CLAMPING]"));
		}
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UBoomArmCameraNode::UBoomArmCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView UBoomArmCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView({ InputSlot });
}

FCameraNodeEvaluatorPtr UBoomArmCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FBoomArmCameraNodeEvaluator>();
}

