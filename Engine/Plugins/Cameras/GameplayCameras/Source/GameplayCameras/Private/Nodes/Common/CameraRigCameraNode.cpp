// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/CameraRigCameraNode.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "Logging/TokenizedMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigCameraNode)

#define LOCTEXT_NAMESPACE "CameraRigCameraNode"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraRigCameraNodeEvaluator)

FCameraRigCameraNodeEvaluator::FCameraRigCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsParameterUpdate);
}

FCameraNodeEvaluatorChildrenView FCameraRigCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ CameraRigRootEvaluator });
}

void FCameraRigCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UCameraRigCameraNode* CameraRigNode = GetCameraNodeAs<UCameraRigCameraNode>();
	const UCameraRigAsset* OuterCameraRig = CameraRigNode->GetTypedOuter<UCameraRigAsset>();
	if (const UCameraRigAsset* CameraRig = CameraRigNode->CameraRigReference.GetCameraRig())
	{
		if (CameraRig->RootNode && CameraRig != OuterCameraRig)
		{
			CameraRigRootEvaluator = Params.BuildEvaluator(CameraRig->RootNode);
		}
	}
}

void FCameraRigCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraRigCameraNode* PrefabNode = GetCameraNodeAs<UCameraRigCameraNode>();

	// In editor and other uncooked builds, we only have the allocation info for the variables and data of our own 
	// camera rig.  So we need to make room for the referenced camera rig's stuff.
	// With cooked data we would have all that info already.
	PrefabNode->CameraRigReference.EnsureAllocationInfo(OutResult);
	
	// Apply overrides and defaults right away.
	PrefabNode->CameraRigReference.ApplyParameterOverridesAndDefaults(OutResult);
}

void FCameraRigCameraNodeEvaluator::OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	// Keep applying overrides in case they are driven by a variable.
	const UCameraRigCameraNode* PrefabNode = GetCameraNodeAs<UCameraRigCameraNode>();
	PrefabNode->CameraRigReference.ApplyParameterOverrides(OutResult.VariableTable, false);
}

void FCameraRigCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (CameraRigRootEvaluator)
	{
		CameraRigRootEvaluator->Run(Params, OutResult);
	}
}

}  // namespace UE::Cameras

void UCameraRigCameraNode::OnPreBuild(FCameraBuildContext& BuildContext)
{
	// Build the inner camera rig. Silently skip it if it's not set or invalid... but we will
	// report an error in OnBuild about it.
	if (UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig())
	{
		const bool bNeedsBuild = CameraRig->BuildStatus == ECameraBuildStatus::Dirty;
		const UCameraRigAsset* OuterCameraRig = GetTypedOuter<UCameraRigAsset>();
		if (OuterCameraRig != CameraRig && bNeedsBuild)
		{
			CameraRig->BuildCameraRig(BuildContext);
		}
	}

	// Make sure the property bag of the camera rig reference is up to date.
	CameraRigReference.RebuildParametersIfNeeded();
}

void UCameraRigCameraNode::OnBuild(FCameraObjectBuildContext& BuildContext)
{
	using namespace UE::Cameras;

	UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig();
	if (!CameraRig)
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Warning, this, 
				LOCTEXT("MissingCameraRig", "No camera rig specified on camera rig node."));
		return;
	}

	const bool bDidModify = CameraRigReference.PostBuild();
	if (bDidModify)
	{
		Modify();
	}

	const UCameraRigAsset* OuterCameraRig = GetTypedOuter<UCameraRigAsset>();
	if (OuterCameraRig != CameraRig)
	{
		if (BuildContext.IsCooking())
		{
			// Whatever allocations our inner camera rig needs for its evaluators and
			// their camera variables, we add that to our camera rig's allocation info.
			BuildContext.AllocationInfo.Append(CameraRig->AllocationInfo);
		}
	}
	else
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, this, 
				LOCTEXT("SelfReferenceError", "Circular camera rig references are forbidden."));
	}
}

void UCameraRigCameraNode::GatherPackages(FCameraRigPackages& OutPackages) const
{
	if (const UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig())
	{
		const UCameraRigAsset* OuterCameraRig = GetTypedOuter<UCameraRigAsset>();
		if (OuterCameraRig != CameraRig)
		{
#if WITH_EDITOR
			CameraRig->GatherPackages(OutPackages);
#endif  // WITH_EDITOR
		}
	}
}

void UCameraRigCameraNode::GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos)
{
	CameraRigReference.GetCustomCameraNodeParameters(OutParameterInfos);
}

FCameraNodeEvaluatorPtr UCameraRigCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCameraRigCameraNodeEvaluator>();
}

#if WITH_EDITOR

void UCameraRigCameraNode::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UCameraRigCameraNode, CameraRigReference) &&
			PropertyChangedEvent.GetPropertyName() == TEXT("CameraRig"))
	{
		OnCustomCameraNodeParametersChanged(this);
	}
}

EObjectTreeGraphObjectSupportFlags UCameraRigCameraNode::GetSupportFlags(FName InGraphName) const
{
	return (Super::GetSupportFlags(InGraphName) | EObjectTreeGraphObjectSupportFlags::CustomTitle);
}

void UCameraRigCameraNode::GetGraphNodeName(FName InGraphName, FText& OutName) const
{
	const UCameraRigAsset* CameraRig = CameraRigReference.GetCameraRig();
	const FText CameraRigName = CameraRig ? FText::FromString(CameraRig->GetName()) : LOCTEXT("None", "None");
	OutName = FText::Format(LOCTEXT("GraphNodeNameFormat", "Camera Rig ({0})"), CameraRigName);
}

#endif  // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

