// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/ClippingPlanesCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClippingPlanesCameraNode)

namespace UE::Cameras
{

class FClippingPlanesCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FClippingPlanesCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<double> NearPlaneReader;
	TCameraParameterReader<double> FarPlaneReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FClippingPlanesCameraNodeEvaluator)

void FClippingPlanesCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UClippingPlanesCameraNode* ClippingPlanesNode = GetCameraNodeAs<UClippingPlanesCameraNode>();
	NearPlaneReader.Initialize(ClippingPlanesNode->NearPlane);
	FarPlaneReader.Initialize(ClippingPlanesNode->FarPlane);
}

void FClippingPlanesCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& OutPose = OutResult.CameraPose;

	float NearPlane = NearPlaneReader.Get(OutResult.VariableTable);
	if (NearPlane > 0)
	{
		OutPose.SetNearClippingPlane(NearPlane);
	}
	float FarPlane = FarPlaneReader.Get(OutResult.VariableTable);
	if (FarPlane > 0)
	{
		OutPose.SetFarClippingPlane(FarPlane);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UClippingPlanesCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FClippingPlanesCameraNodeEvaluator>();
}

