// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/DampenPositionCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameplayCameras.h"
#include "HAL/IConsoleManager.h"
#include "Math/CriticalDamper.h"
#include "Templates/Tuple.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DampenPositionCameraNode)

namespace UE::Cameras
{

bool GGameplayCamerasDebugDampingShowLocalSpace = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugDampingShowLocalSpace(
	TEXT("GameplayCameras.Debug.Damping.ShowLocalSpace"),
	GGameplayCamerasDebugDampingShowLocalSpace,
	TEXT(""));

class FDampenPositionCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FDampenPositionCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	using FAxisDamper = TTuple<FVector3d, FCriticalDamper*>;
	using FAxisDampers = FAxisDamper[3];
	void ComputeAxisDampers(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult, FAxisDampers& OutAxisDampers);

private:

	TCameraParameterReader<float> ForwardDampingFactorReader;
	TCameraParameterReader<float> LateralDampingFactorReader;
	TCameraParameterReader<float> VerticalDampingFactorReader;

	FCriticalDamper ForwardDamper;
	FCriticalDamper LateralDamper;
	FCriticalDamper VerticalDamper;

	FVector3d PreviousLocation;
	FVector3d PreviousLagVector;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector3d DebugLastUndampedPosition;
	FVector3d DebugLastDampedPosition;
	FRotator3d DebugLastDampingRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FDampenPositionCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FDampenPositionCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, ForwardX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, LateralX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, VerticalX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, ForwardDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, LateralDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, VerticalDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, UndampedPosition);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, DampedPosition);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, DampingRotation);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDampenPositionCameraDebugBlock)

void FDampenPositionCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UDampenPositionCameraNode* DampenNode = GetCameraNodeAs<UDampenPositionCameraNode>();

	ForwardDampingFactorReader.Initialize(DampenNode->ForwardDampingFactor);
	LateralDampingFactorReader.Initialize(DampenNode->LateralDampingFactor);
	VerticalDampingFactorReader.Initialize(DampenNode->VerticalDampingFactor);

	ForwardDamper.SetW0(ForwardDampingFactorReader.Get(OutResult.VariableTable));
	ForwardDamper.Reset(0, 0);

	LateralDamper.SetW0(LateralDampingFactorReader.Get(OutResult.VariableTable));
	LateralDamper.Reset(0, 0);

	VerticalDamper.SetW0(VerticalDampingFactorReader.Get(OutResult.VariableTable));
	VerticalDamper.Reset(0, 0);

	const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
	PreviousLocation = InitialResult.CameraPose.GetLocation();
}

void FDampenPositionCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Update our damping factors if they're driven by a camera variable (which means they
	// could change every frame). Also update them if we are in the editor, since the 
	// user might tweak them live.
#if WITH_EDITOR
	const bool bUpdateForwardDampingFactor = true;
	const bool bUpdateLateralDampingFactor = true;
	const bool bUpdateVerticalDampingFactor = true;
#else
	const bool bUpdateForwardDampingFactor = ForwardDampingFactorReader.IsDriven();
	const bool bUpdateLateralDampingFactor = LateralDampingFactorReader.IsDriven();
	const bool bUpdateVerticalDampingFactor = VerticalDampingFactorReader.IsDriven();
#endif
	if (bUpdateForwardDampingFactor)
	{
		ForwardDamper.SetW0(ForwardDampingFactorReader.Get(OutResult.VariableTable));
	}
	if (bUpdateLateralDampingFactor)
	{
		LateralDamper.SetW0(LateralDampingFactorReader.Get(OutResult.VariableTable));
	}
	if (bUpdateVerticalDampingFactor)
	{
		VerticalDamper.SetW0(VerticalDampingFactorReader.Get(OutResult.VariableTable));
	}

	// We want the dampen the given camera position, which means it's trying
	// to converge towards the one given in the result (which we set as our 
	// next target), but will be lagging behind.
	const FVector3d NextTarget = OutResult.CameraPose.GetLocation();
	FVector3d NextLocation = NextTarget;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugLastDampingRotation = FRotator3d::ZeroRotator;
#endif

	// Figure out the coordinate system in which we are damping movement.
	FAxisDampers AxisDampers;
	ComputeAxisDampers(Params, OutResult, AxisDampers);

	if (!Params.bIsFirstFrame && !OutResult.bIsCameraCut)
	{
		// The next target has moved further away compared to the previous target,
		// so we're lagging behind even more than before. Compute this new lag vector.
		const FVector3d NewLagVector = NextTarget - PreviousLocation;
		// Let's start at our previous (dampened) location, and see by how much we
		// can catch up on our lag this frame.
		FVector3d NewDampedLocation = PreviousLocation;

		for (FAxisDamper& AxisDamper : AxisDampers)
		{
			FVector3d Axis(AxisDamper.Key);
			FCriticalDamper* Damper(AxisDamper.Value);

			// Compute lag on the forward/lateral/vertical axis, and pass this new
			// lag distance as the new position of the damper. Update it to know 
			// how much we catch up, and offset last frame's position by that amount.
			double NewLagDistance = FVector3d::DotProduct(NewLagVector, Axis);
			// TODO: use GetWorld()->GetWorldSettings()->WorldToMeters
			Damper->Update(NewLagDistance / 100.0, Params.DeltaTime);
			NewDampedLocation += Axis * (NewLagDistance - Damper->GetX0() * 100.0);
		}
		
		NextLocation = NewDampedLocation;
		PreviousLagVector = NextTarget - NextLocation;
	}
	else if (!Params.bIsFirstFrame && OutResult.bIsCameraCut)
	{
		// On camera cuts, we don't update the damping, and just re-use whatever lag 
		// we previously had.
		NextLocation = NextTarget - PreviousLagVector;
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugLastUndampedPosition = NextTarget;
	DebugLastDampedPosition = NextLocation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	PreviousLocation = NextLocation;

	OutResult.CameraPose.SetLocation(NextLocation);
}

void FDampenPositionCameraNodeEvaluator::ComputeAxisDampers(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, FAxisDampers& OutAxisDampers)
{
	// Figure out the desired coordinate system.
	FRotator3d AxesRotation(EForceInit::ForceInit);
	const UDampenPositionCameraNode* DampenNode = GetCameraNodeAs<UDampenPositionCameraNode>();
	switch (DampenNode->DampenSpace)
	{
		case ECameraNodeSpace::CameraPose:
			{
				const FRotator3d CameraRotation = Result.CameraPose.GetRotation();
				AxesRotation = CameraRotation;
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (Params.EvaluationContext)
			{
				const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
				const FRotator3d ContextRotation = InitialResult.CameraPose.GetRotation();
				AxesRotation = ContextRotation;
			}
			else
			{
				UE_LOGF(LogCameraSystem, Error,
						"DampenPositionCameraNode: cannot dampen in context space when there is "
							"no current context set.");
			}
			break;
		case ECameraNodeSpace::World:
			break;
	}
#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugLastDampingRotation = AxesRotation;
#endif

	OutAxisDampers[0] = { AxesRotation.RotateVector(FVector3d::ForwardVector), &ForwardDamper };
	OutAxisDampers[1] = { AxesRotation.RotateVector(FVector3d::RightVector),& LateralDamper };
	OutAxisDampers[2] = { AxesRotation.RotateVector(FVector3d::UpVector), &VerticalDamper };
}

void FDampenPositionCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << ForwardDamper;
	Ar << LateralDamper;
	Ar << VerticalDamper;

	Ar << PreviousLocation;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	Ar << DebugLastUndampedPosition;
	Ar << DebugLastDampedPosition;
	Ar << DebugLastDampingRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FDampenPositionCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FDampenPositionCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FDampenPositionCameraDebugBlock>();

	DebugBlock.ForwardX0 = ForwardDamper.GetX0();
	DebugBlock.LateralX0 = LateralDamper.GetX0();
	DebugBlock.VerticalX0 = VerticalDamper.GetX0();

	DebugBlock.ForwardDampingFactor = ForwardDamper.GetW0();
	DebugBlock.LateralDampingFactor = LateralDamper.GetW0();
	DebugBlock.VerticalDampingFactor = VerticalDamper.GetW0();

	DebugBlock.UndampedPosition = DebugLastUndampedPosition;
	DebugBlock.DampedPosition = DebugLastDampedPosition;
	DebugBlock.DampingRotation = DebugLastDampingRotation;
}

void FDampenPositionCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(
			TEXT("forward %.3f (factor %.3f)  lateral %.3f (factor %.3f)  vertical %.3f (factor %.3f)"),
			ForwardX0, ForwardDampingFactor,
			LateralX0, LateralDampingFactor,
			VerticalX0, VerticalDampingFactor);

	if (Renderer.IsExternalRendering())
	{
		if (GGameplayCamerasDebugDampingShowLocalSpace)
		{
			const double DampingAxesLength = 100.f;
			Renderer.DrawLine(
					UndampedPosition, 
					UndampedPosition + DampingRotation.RotateVector(FVector3d::ForwardVector * DampingAxesLength),
					FLinearColor::Red);
			Renderer.DrawLine(
					UndampedPosition, 
					UndampedPosition + DampingRotation.RotateVector(FVector3d::RightVector * DampingAxesLength),
					FLinearColor::Green);
			Renderer.DrawLine(
					UndampedPosition, 
					UndampedPosition + DampingRotation.RotateVector(FVector3d::UpVector * DampingAxesLength),
					FLinearColor::Blue);
		}

		Renderer.DrawLine(UndampedPosition, DampedPosition, FLinearColor::Yellow);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UDampenPositionCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FDampenPositionCameraNodeEvaluator>();
}

