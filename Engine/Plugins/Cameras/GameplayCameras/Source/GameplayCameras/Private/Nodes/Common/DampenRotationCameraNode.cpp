// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/DampenRotationCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameplayCameras.h"
#include "Math/CriticalDamper.h"
#include "Templates/Tuple.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DampenRotationCameraNode)

namespace UE::Cameras
{

class FDampenRotationCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FDampenRotationCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	TCameraParameterReader<float> YawDampingFactorReader;
	TCameraParameterReader<float> PitchDampingFactorReader;
	TCameraParameterReader<float> RollDampingFactorReader;

	FCriticalDamper YawDamper;
	FCriticalDamper PitchDamper;
	FCriticalDamper RollDamper;

	FRotator3d PreviousRotation;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FRotator3d DebugLastUndampedRotation;
	FRotator3d DebugLastDampedRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FDampenRotationCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FDampenRotationCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, YawX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, PitchX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, RollX0);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, YawDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, PitchDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, RollDampingFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, UndampedRotation);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, DampedRotation);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDampenRotationCameraDebugBlock)

void FDampenRotationCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UDampenRotationCameraNode* DampenNode = GetCameraNodeAs<UDampenRotationCameraNode>();

	YawDampingFactorReader.Initialize(DampenNode->YawDampingFactor);
	PitchDampingFactorReader.Initialize(DampenNode->PitchDampingFactor);
	RollDampingFactorReader.Initialize(DampenNode->RollDampingFactor);

	YawDamper.SetW0(YawDampingFactorReader.Get(OutResult.VariableTable));
	YawDamper.Reset(0, 0);

	PitchDamper.SetW0(PitchDampingFactorReader.Get(OutResult.VariableTable));
	PitchDamper.Reset(0, 0);

	RollDamper.SetW0(RollDampingFactorReader.Get(OutResult.VariableTable));
	RollDamper.Reset(0, 0);

	const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
	PreviousRotation = InitialResult.CameraPose.GetRotation();
}

void FDampenRotationCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// Update our damping factors if they're driven by a camera variable (which means they
	// could change every frame). Also update them if we are in the editor, since the 
	// user might tweak them live.
#if WITH_EDITOR
	const bool bUpdateYawDampingFactor = true;
	const bool bUpdatePitchDampingFactor = true;
	const bool bUpdateRollDampingFactor = true;
#else
	const bool bUpdateYawDampingFactor = YawDampingFactorReader.IsDriven();
	const bool bUpdatePitchDampingFactor = PitchDampingFactorReader.IsDriven();
	const bool bUpdateRollDampingFactor = RollDampingFactorReader.IsDriven();
#endif
	if (bUpdateYawDampingFactor)
	{
		YawDamper.SetW0(YawDampingFactorReader.Get(OutResult.VariableTable));
	}
	if (bUpdatePitchDampingFactor)
	{
		PitchDamper.SetW0(PitchDampingFactorReader.Get(OutResult.VariableTable));
	}
	if (bUpdateRollDampingFactor)
	{
		RollDamper.SetW0(RollDampingFactorReader.Get(OutResult.VariableTable));
	}

	// We want to dampen the given camera rotation, which means it's trying
	// to converge towards the one given in the result (which we set as our 
	// next target), but will be lagging behind.
	const FRotator3d NextIdealRotation = OutResult.CameraPose.GetRotation();
	FRotator3d NextRotation = NextIdealRotation;

	if (!Params.bIsFirstFrame && !OutResult.bIsCameraCut)
	{
		// Let's see by how much we are lagging behind the ideal rotation. Update
		// the dampers to try and close the gap.
		const FRotator3d DeltaRotation = (PreviousRotation - NextIdealRotation).GetNormalized();
		YawDamper.Update(DeltaRotation.Yaw, Params.DeltaTime);
		PitchDamper.Update(DeltaRotation.Pitch, Params.DeltaTime);
		RollDamper.Update(DeltaRotation.Roll, Params.DeltaTime);

		NextRotation = NextIdealRotation + FRotator3d(PitchDamper.GetX0(), YawDamper.GetX0(), RollDamper.GetX0());
	}
	else if (!Params.bIsFirstFrame && OutResult.bIsCameraCut)
	{
		// On camera cuts, we don't update the damping, and just re-use whatever lag 
		// we previously had.
		NextRotation = NextIdealRotation + FRotator3d(PitchDamper.GetX0(), YawDamper.GetX0(), RollDamper.GetX0());
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugLastUndampedRotation = NextIdealRotation;
	DebugLastDampedRotation = NextRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	PreviousRotation = NextRotation;

	OutResult.CameraPose.SetRotation(NextRotation);
}

void FDampenRotationCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << YawDamper;
	Ar << PitchDamper;
	Ar << RollDamper;

	Ar << PreviousRotation;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	Ar << DebugLastUndampedRotation;
	Ar << DebugLastDampedRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FDampenRotationCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FDampenRotationCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FDampenRotationCameraDebugBlock>();

	DebugBlock.YawX0 = YawDamper.GetX0();
	DebugBlock.PitchX0 = PitchDamper.GetX0();
	DebugBlock.RollX0 = RollDamper.GetX0();

	DebugBlock.YawDampingFactor = YawDamper.GetW0();
	DebugBlock.PitchDampingFactor = PitchDamper.GetW0();
	DebugBlock.RollDampingFactor = RollDamper.GetW0();

	DebugBlock.UndampedRotation = DebugLastUndampedRotation;
	DebugBlock.DampedRotation = DebugLastDampedRotation;
}

void FDampenRotationCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(
			TEXT("yaw %.3f (factor %.3f)  pitch %.3f (factor %.3f)  roll %.3f (factor %.3f)"),
			YawX0, YawDampingFactor,
			PitchX0, PitchDampingFactor,
			RollX0, RollDampingFactor);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UDampenRotationCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FDampenRotationCameraNodeEvaluator>();
}

