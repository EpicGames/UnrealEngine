// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/FieldOfViewCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldOfViewCameraNode)

namespace UE::Cameras
{

class FFieldOfViewCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FFieldOfViewCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<float> FieldOfViewReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FFieldOfViewCameraNodeEvaluator)

void FFieldOfViewCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UFieldOfViewCameraNode* FieldOfViewNode = GetCameraNodeAs<UFieldOfViewCameraNode>();
	FieldOfViewReader.Initialize(FieldOfViewNode->FieldOfView);
}

void FFieldOfViewCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const float FieldOfView = FieldOfViewReader.Get(OutResult.VariableTable);
	OutResult.CameraPose.SetFieldOfView(FieldOfView);
	OutResult.CameraPose.SetFocalLength(-1);
}

}  // namespace UE::Cameras

UFieldOfViewCameraNode::UFieldOfViewCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	FieldOfView.Value = 90.f;
}

FCameraNodeEvaluatorPtr UFieldOfViewCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FFieldOfViewCameraNodeEvaluator>();
}

