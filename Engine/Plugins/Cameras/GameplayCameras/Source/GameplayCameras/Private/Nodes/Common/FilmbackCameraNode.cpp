// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/FilmbackCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Core/CameraPose.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilmbackCameraNode)

namespace UE::Cameras
{

class FFilmbackCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FFilmbackCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<float> SensorWidthReader;
	TCameraParameterReader<float> SensorHeightReader;
	TCameraParameterReader<float> SensorHorizontalOffsetReader;
	TCameraParameterReader<float> SensorVerticalOffsetReader;
	TCameraParameterReader<float> OverscanReader;
	TCameraParameterReader<bool> ConstrainAspectRatioReader;
	TCameraParameterReader<bool> OverrideAspectRatioAxisConstraintReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FFilmbackCameraNodeEvaluator)

void FFilmbackCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UFilmbackCameraNode* FilmbackNode = GetCameraNodeAs<UFilmbackCameraNode>();
	SensorWidthReader.Initialize(FilmbackNode->SensorWidth);
	SensorHeightReader.Initialize(FilmbackNode->SensorHeight);
	SensorHorizontalOffsetReader.Initialize(FilmbackNode->SensorHorizontalOffset);
	SensorVerticalOffsetReader.Initialize(FilmbackNode->SensorVerticalOffset);
	OverscanReader.Initialize(FilmbackNode->Overscan);
	ConstrainAspectRatioReader.Initialize(FilmbackNode->ConstrainAspectRatio);
	OverrideAspectRatioAxisConstraintReader.Initialize(FilmbackNode->OverrideAspectRatioAxisConstraint);
}

void FFilmbackCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& OutPose = OutResult.CameraPose;

	float SensorWidth = SensorWidthReader.Get(OutResult.VariableTable);
	if (SensorWidth > 0)
	{
		OutPose.SetSensorWidth(SensorWidth);
	}
	float SensorHeight = SensorHeightReader.Get(OutResult.VariableTable);
	if (SensorHeight > 0)
	{
		OutPose.SetSensorHeight(SensorHeight);
	}
	float SensorHorizontalOffset = SensorHorizontalOffsetReader.Get(OutResult.VariableTable);
	if (SensorHorizontalOffset > 0)
	{
		OutPose.SetSensorHorizontalOffset(SensorHorizontalOffset);
	}
	float SensorVerticalOffset = SensorVerticalOffsetReader.Get(OutResult.VariableTable);
	if (SensorVerticalOffset > 0)
	{
		OutPose.SetSensorVerticalOffset(SensorVerticalOffset);
	}

	OutPose.SetOverscan(OverscanReader.Get(OutResult.VariableTable));

	OutPose.SetConstrainAspectRatio(ConstrainAspectRatioReader.Get(OutResult.VariableTable));
	OutPose.SetOverrideAspectRatioAxisConstraint(OverrideAspectRatioAxisConstraintReader.Get(OutResult.VariableTable));

	// TODO: add support for enum camera parameters.
	const UFilmbackCameraNode* FilmbackNode = GetCameraNodeAs<UFilmbackCameraNode>();
	OutPose.SetAspectRatioAxisConstraint(FilmbackNode->AspectRatioAxisConstraint);
}

}  // namespace UE::Cameras

UFilmbackCameraNode::UFilmbackCameraNode(const FObjectInitializer& ObjectInitializer)
{
	FCameraPose::GetDefaultSensorSize(SensorWidth.Value, SensorHeight.Value);
}

FCameraNodeEvaluatorPtr UFilmbackCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FFilmbackCameraNodeEvaluator>();
}

