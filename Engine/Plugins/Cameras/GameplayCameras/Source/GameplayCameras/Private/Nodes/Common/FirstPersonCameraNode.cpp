// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/FirstPersonCameraNode.h"

#include "Core/CameraParameterReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FirstPersonCameraNode)

namespace UE::Cameras
{

class FFirstPersonCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FFirstPersonCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<bool> EnableFirstPersonReader;
	TCameraParameterReader<float> FirstPersonFieldOfViewReader;
	TCameraParameterReader<float> FirstPersonScaleReader;

};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FFirstPersonCameraNodeEvaluator)

void FFirstPersonCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UFirstPersonCameraNode* FirstPersonCameraNode = GetCameraNodeAs<UFirstPersonCameraNode>();
	EnableFirstPersonReader.Initialize(FirstPersonCameraNode->EnableFirstPerson);
	FirstPersonFieldOfViewReader.Initialize(FirstPersonCameraNode->FirstPersonFieldOfView);
	FirstPersonScaleReader.Initialize(FirstPersonCameraNode->FirstPersonScale);
}

void FFirstPersonCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UFirstPersonCameraNode* FirstPersonCameraNode = GetCameraNodeAs<UFirstPersonCameraNode>();

	const bool bEnableFirstPerson = EnableFirstPersonReader.Get(OutResult.VariableTable);
	OutResult.CameraPose.SetEnableFirstPerson(bEnableFirstPerson);

	const float FirstPersonFieldOfView = FirstPersonFieldOfViewReader.Get(OutResult.VariableTable);
	if (FirstPersonCameraNode->bEnableFirstPersonFieldOfView && FirstPersonFieldOfView > 0.f)
	{
		OutResult.CameraPose.SetFirstPersonFieldOfView(FirstPersonFieldOfView);
	}

	const float FirstPersonScale = FirstPersonScaleReader.Get(OutResult.VariableTable);
	if (FirstPersonCameraNode->bEnableFirstPersonScale && FirstPersonScale > 0.f)
	{
		OutResult.CameraPose.SetFirstPersonScale(FirstPersonScale);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UFirstPersonCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FFirstPersonCameraNodeEvaluator>();
}

