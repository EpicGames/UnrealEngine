// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/LensParametersCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "GameplayCamerasCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensParametersCameraNode)

namespace UE::Cameras
{

class FLensParametersCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FLensParametersCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<float> FocalLengthReader;
	TCameraParameterReader<float> ApertureReader;
	TCameraParameterReader<float> FocusDistanceReader;
	TCameraParameterReader<bool> EnablePhysicalCameraReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FLensParametersCameraNodeEvaluator)

void FLensParametersCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const ULensParametersCameraNode* LensParametersNode = GetCameraNodeAs<ULensParametersCameraNode>();
	FocalLengthReader.Initialize(LensParametersNode->FocalLength);
	ApertureReader.Initialize(LensParametersNode->Aperture);
	FocusDistanceReader.Initialize(LensParametersNode->FocusDistance);
	EnablePhysicalCameraReader.Initialize(LensParametersNode->EnablePhysicalCamera);
}

void FLensParametersCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const ULensParametersCameraNode* LensParametersNode = GetCameraNodeAs<ULensParametersCameraNode>();
	FCameraPose& OutPose = OutResult.CameraPose;

	float FocalLength = FocalLengthReader.Get(OutResult.VariableTable);
	if (LensParametersNode->bEnableFocalLength && FocalLength > 0)
	{
		OutPose.SetFocalLength(FocalLength);
		OutPose.SetFieldOfView(-1);
	}
	float Aperture = ApertureReader.Get(OutResult.VariableTable);
	if (LensParametersNode->bEnableAperture && Aperture > 0)
	{
		OutPose.SetAperture(Aperture);
	}
	float FocusDistance = FocusDistanceReader.Get(OutResult.VariableTable);
	if (LensParametersNode->bEnableFocusDistance && FocusDistance > 0)
	{
		OutPose.SetFocusDistance(FocusDistance);
	}

	const bool bEnablePhysicalCamera = EnablePhysicalCameraReader.Get(OutResult.VariableTable);
	OutPose.SetEnablePhysicalCamera(bEnablePhysicalCamera);
	OutPose.SetPhysicalCameraBlendWeight(bEnablePhysicalCamera ? 1.f : 0.f);
}

}  // namespace UE::Cameras

void ULensParametersCameraNode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FGameplayCamerasCustomVersion::GUID);

	// If this node was saved before the parameter flags were added, set the properties to their old default values,
	// then do the serialization, which will give us the non-zero values.
	const bool bUpgradeParameterFlags = (
			Ar.IsLoading() &&
			Ar.CustomVer(FGameplayCamerasCustomVersion::GUID) < FGameplayCamerasCustomVersion::AddLensNodeParameterFlags);
	if (bUpgradeParameterFlags)
	{
		FocalLength.Value = 0;
		Aperture.Value = 0;
		FocusDistance.Value = 0;

		bEnableFocalLength = false;
		bEnableAperture = false;
		bEnableFocusDistance = false;
	}

	Super::Serialize(Ar);

	// Now that we have the non-zero values, auto-set the appropriate flags, and restore the new defaults.
	// These defaults should match the ones in the header file!
	if (bUpgradeParameterFlags)
	{
		bEnableFocalLength = FocalLength.Value > 0.f || FocalLength.Variable || FocalLength.VariableID;
		bEnableAperture = Aperture.Value > 0.f || Aperture.Variable || Aperture.VariableID;
		bEnableFocusDistance = FocusDistance.Value > 0.f || FocusDistance.Variable || FocusDistance.VariableID;

		if (FocalLength.Value <= 0.f)
		{
			FocalLength.Value = 35.f;
		}
		if (Aperture.Value <= 0.f)
		{
			Aperture.Value = 16.f;
		}
		if (FocusDistance.Value <= 0.f)
		{
			FocusDistance.Value = 1000.f;
		}
	}
}

FCameraNodeEvaluatorPtr ULensParametersCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FLensParametersCameraNodeEvaluator>();
}

