// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/OffsetCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Math/CameraNodeSpaceMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OffsetCameraNode)

namespace UE::Cameras
{

class FOffsetCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FOffsetCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<FVector3d> TranslationReader;
	TCameraParameterReader<FRotator3d> RotationReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FOffsetCameraNodeEvaluator)

void FOffsetCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UOffsetCameraNode* OffsetNode = GetCameraNodeAs<UOffsetCameraNode>();
	TranslationReader.Initialize(OffsetNode->TranslationOffset);
	RotationReader.Initialize(OffsetNode->RotationOffset);
}

void FOffsetCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const FVector3d TranslationOffset = TranslationReader.Get(OutResult.VariableTable);
	const FRotator3d RotationOffset = RotationReader.Get(OutResult.VariableTable);

	const UOffsetCameraNode* OffsetNode = GetCameraNodeAs<UOffsetCameraNode>();

	FTransform3d OutTransform;
	const bool bSuccess = FCameraNodeSpaceMath::OffsetCameraNodeSpaceTransform(
			FCameraNodeSpaceParams(Params, OutResult),
			OutResult.CameraPose.GetTransform(),
			TranslationOffset,
			RotationOffset,
			OffsetNode->OffsetSpace,
			OutTransform);
	if (bSuccess)
	{
		OutResult.CameraPose.SetTransform(OutTransform);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UOffsetCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FOffsetCameraNodeEvaluator>();
}

TTuple<int32, int32> UOffsetCameraNode::GetEvaluatorAllocationInfo()
{
	using namespace UE::Cameras;
	return { sizeof(FOffsetCameraNodeEvaluator), alignof(FOffsetCameraNodeEvaluator) };
}

