// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraTypes.h"
#include "Nodes/Common/OrthographicCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OrthographicCameraNode)

namespace UE::Cameras
{

class FOrthographicCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FOrthographicCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<bool> EnableOrthographicModeReader;
	TCameraParameterReader<float> OrthographicWidthReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FOrthographicCameraNodeEvaluator)

void FOrthographicCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UOrthographicCameraNode* OrthographicNode = GetCameraNodeAs<UOrthographicCameraNode>();
	EnableOrthographicModeReader.Initialize(OrthographicNode->EnableOrthographicMode);
	OrthographicWidthReader.Initialize(OrthographicNode->OrthographicWidth);
}

void FOrthographicCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& OutPose = OutResult.CameraPose;

	if (EnableOrthographicModeReader.Get(OutResult.VariableTable))
	{
		OutPose.SetProjectionMode(ECameraProjectionMode::Orthographic);
	}

	const float OrthographicWidth = OrthographicWidthReader.Get(OutResult.VariableTable);
	OutPose.SetOrthographicWidth(OrthographicWidth);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UOrthographicCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FOrthographicCameraNodeEvaluator>();
}

