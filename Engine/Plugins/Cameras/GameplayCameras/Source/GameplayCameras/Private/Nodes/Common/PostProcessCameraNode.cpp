// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/PostProcessCameraNode.h"

#include "Core/CameraContextDataReader.h"
#include "Core/CameraPose.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PostProcessCameraNode)

namespace UE::Cameras
{

class FPostProcessCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FPostProcessCameraNodeEvaluator)

public:

	FPostProcessCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
	}

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraContextDataReader<FPostProcessSettings> PostProcessSettingsReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FPostProcessCameraNodeEvaluator)

void FPostProcessCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UPostProcessCameraNode* PostProcessNode = GetCameraNodeAs<UPostProcessCameraNode>();
	PostProcessSettingsReader.Initialize(&PostProcessNode->PostProcessSettings, PostProcessNode->PostProcessSettingsDataID);
}

void FPostProcessCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& OutPose = OutResult.CameraPose;

	const FPostProcessSettings& PostProcessSettings = PostProcessSettingsReader.GetRef(OutResult.ContextDataTable);
	OutResult.PostProcessSettings.OverrideChanged(PostProcessSettings);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UPostProcessCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPostProcessCameraNodeEvaluator>();
}

