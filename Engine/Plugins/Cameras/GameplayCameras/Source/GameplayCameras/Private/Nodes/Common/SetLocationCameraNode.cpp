// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/SetLocationCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetLocationCameraNode)

namespace UE::Cameras
{

class FSetLocationCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSetLocationCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<FVector3d> LocationReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FSetLocationCameraNodeEvaluator)

void FSetLocationCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const USetLocationCameraNode* LocationNode = GetCameraNodeAs<USetLocationCameraNode>();
	LocationReader.Initialize(LocationNode->Location);
}

void FSetLocationCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const FVector3d NewLocation = LocationReader.Get(OutResult.VariableTable);

	const USetLocationCameraNode* LocationNode = GetCameraNodeAs<USetLocationCameraNode>();
	switch (LocationNode->OffsetSpace)
	{
		case ECameraNodeSpace::CameraPose:
		default:
			{
				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform = FTransform3d(NewLocation) * Transform;
				OutResult.CameraPose.SetTransform(Transform);
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (Params.EvaluationContext)
			{ 
				const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
				const FTransform3d ContextTransform = InitialResult.CameraPose.GetTransform();

				const FVector3d WorldLocation = ContextTransform.TransformVector(NewLocation);

				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform.SetLocation(WorldLocation);
				OutResult.CameraPose.SetTransform(Transform);
			}
			else
			{
				UE_LOGF(LogCameraSystem, Error, 
						"SetLocationCameraNode: cannot offset in context space when there is "
							 "no current context set.");
				return;
			}
			break;
		case ECameraNodeSpace::World:
			{
				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform.SetLocation(NewLocation);
				OutResult.CameraPose.SetTransform(Transform);
			}
			break;
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr USetLocationCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSetLocationCameraNodeEvaluator>();
}

