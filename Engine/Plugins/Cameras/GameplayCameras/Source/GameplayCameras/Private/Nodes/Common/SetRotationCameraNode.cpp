// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/SetRotationCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetRotationCameraNode)

namespace UE::Cameras
{

class FSetRotationCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSetRotationCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<FRotator3d> RotationReader;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FSetRotationCameraNodeEvaluator)

void FSetRotationCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const USetRotationCameraNode* RotationNode = GetCameraNodeAs<USetRotationCameraNode>();
	RotationReader.Initialize(RotationNode->Rotation);
}

void FSetRotationCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const FRotator3d NewRotation = RotationReader.Get(OutResult.VariableTable);

	const USetRotationCameraNode* RotationNode = GetCameraNodeAs<USetRotationCameraNode>();
	switch (RotationNode->OffsetSpace)
	{
		case ECameraNodeSpace::CameraPose:
		default:
			{
				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform = FTransform3d(NewRotation) * Transform;
				OutResult.CameraPose.SetTransform(Transform);
			}
			break;
		case ECameraNodeSpace::OwningContext:
			if (Params.EvaluationContext)
			{ 
				const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
				const FTransform3d ContextTransform = InitialResult.CameraPose.GetTransform();

				const FQuat4d WorldRotation = ContextTransform.TransformRotation(NewRotation.Quaternion());

				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform.SetRotation(WorldRotation);
				OutResult.CameraPose.SetTransform(Transform);
			}
			else
			{
				UE_LOGF(LogCameraSystem, Error, 
						"SetRotationCameraNode: cannot offset in context space when there is "
							 "no current context set.");
				return;
			}
			break;
		case ECameraNodeSpace::World:
			{
				FTransform3d Transform = OutResult.CameraPose.GetTransform();
				Transform.SetRotation(NewRotation.Quaternion());
				OutResult.CameraPose.SetTransform(Transform);
			}
			break;
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr USetRotationCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSetRotationCameraNodeEvaluator>();
}

