// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/SplineFieldOfViewCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "GameplayCameras.h"
#include "IGameplayCamerasLiveEditListener.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineFieldOfViewCameraNode)

namespace UE::Cameras
{

class FSplineFieldOfViewCameraNodeEvaluator 
	: public FCameraNodeEvaluator
#if WITH_EDITOR
    , public IGameplayCamerasLiveEditListener
#endif  // WITH_EDITOR
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSplineFieldOfViewCameraNodeEvaluator)

public:

	~FSplineFieldOfViewCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	void RebuildCurve();

private:

	TCameraParameterReader<float> SplineInputReader;

	FCompressedRichCurve FieldOfViewSpline;

	bool bHasAnyValues = false;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FSplineFieldOfViewCameraNodeEvaluator)

void FSplineFieldOfViewCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const USplineFieldOfViewCameraNode* FieldOfViewNode = GetCameraNodeAs<USplineFieldOfViewCameraNode>();
	SplineInputReader.Initialize(FieldOfViewNode->SplineInput);

	RebuildCurve();

#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->AddListener(GetCameraNode(), this);
	}
#endif  // WITH_EDITOR
}

FSplineFieldOfViewCameraNodeEvaluator::~FSplineFieldOfViewCameraNodeEvaluator()
{
#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->RemoveListener(this);
	}
#endif  // WITH_EDITOR
}

void FSplineFieldOfViewCameraNodeEvaluator::RebuildCurve()
{
	const USplineFieldOfViewCameraNode* FieldOfViewNode = GetCameraNodeAs<USplineFieldOfViewCameraNode>();

	bHasAnyValues = FieldOfViewNode->FieldOfViewSpline.HasAnyData();

	FieldOfViewNode->FieldOfViewSpline.Curve.CompressCurve(FieldOfViewSpline);
}

void FSplineFieldOfViewCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (bHasAnyValues)
	{
		const float SplineInput = SplineInputReader.Get(OutResult.VariableTable);
		const float FieldOfView = FieldOfViewSpline.Eval(SplineInput);
		OutResult.CameraPose.SetFieldOfView(FieldOfView);
		OutResult.CameraPose.SetFocalLength(-1);
	}
}

}  // namespace UE::Cameras

USplineFieldOfViewCameraNode::USplineFieldOfViewCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

FCameraNodeEvaluatorPtr USplineFieldOfViewCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSplineFieldOfViewCameraNodeEvaluator>();
}

