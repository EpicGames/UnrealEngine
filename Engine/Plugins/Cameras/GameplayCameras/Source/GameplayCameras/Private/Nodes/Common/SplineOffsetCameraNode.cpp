// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/SplineOffsetCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "IGameplayCamerasModule.h"
#include "IGameplayCamerasLiveEditListener.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "Math/CameraNodeSpaceMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineOffsetCameraNode)

namespace UE::Cameras
{

class FSplineOffsetCameraNodeEvaluator
	: public FCameraNodeEvaluator
#if WITH_EDITOR
    , public IGameplayCamerasLiveEditListener
#endif  // WITH_EDITOR
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSplineOffsetCameraNodeEvaluator)

public:
	
	~FSplineOffsetCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if WITH_EDITOR
	virtual void OnPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) override;
#endif  // WITH_EDITOR

private:

	void RebuildCurves();

private:

	TCameraParameterReader<float> SplineInputReader;

	FCompressedRichCurve LocationOffsetSpline[3];
	FCompressedRichCurve RotationOffsetSpline[3];

	bool bHasAnyLocationOffset = false;
	bool bHasAnyRotationOffset = false;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FSplineOffsetCameraNodeEvaluator)

void FSplineOffsetCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const USplineOffsetCameraNode* SplineOffsetNode = GetCameraNodeAs<USplineOffsetCameraNode>();
	SplineInputReader.Initialize(SplineOffsetNode->SplineInput);

	RebuildCurves();

#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->AddListener(GetCameraNode(), this);
	}
#endif  // WITH_EDITOR
}

FSplineOffsetCameraNodeEvaluator::~FSplineOffsetCameraNodeEvaluator()
{
#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->RemoveListener(this);
	}
#endif  // WITH_EDITOR
}

void FSplineOffsetCameraNodeEvaluator::RebuildCurves()
{
	const USplineOffsetCameraNode* SplineOffsetNode = GetCameraNodeAs<USplineOffsetCameraNode>();

	bHasAnyLocationOffset = SplineOffsetNode->TranslationOffsetSpline.HasAnyData();
	bHasAnyRotationOffset = SplineOffsetNode->RotationOffsetSpline.HasAnyData();

	SplineOffsetNode->TranslationOffsetSpline.Curves[0].CompressCurve(LocationOffsetSpline[0]);
	SplineOffsetNode->TranslationOffsetSpline.Curves[1].CompressCurve(LocationOffsetSpline[1]);
	SplineOffsetNode->TranslationOffsetSpline.Curves[2].CompressCurve(LocationOffsetSpline[2]);

	SplineOffsetNode->RotationOffsetSpline.Curves[0].CompressCurve(RotationOffsetSpline[0]);
	SplineOffsetNode->RotationOffsetSpline.Curves[1].CompressCurve(RotationOffsetSpline[1]);
	SplineOffsetNode->RotationOffsetSpline.Curves[2].CompressCurve(RotationOffsetSpline[2]);
}

void FSplineOffsetCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const float SplineInput = SplineInputReader.Get(OutResult.VariableTable);

	const FVector3d TranslationOffset = bHasAnyLocationOffset ?
		FVector3d(
			LocationOffsetSpline[0].Eval(SplineInput),
			LocationOffsetSpline[1].Eval(SplineInput),
			LocationOffsetSpline[2].Eval(SplineInput))
		:
		FVector3d::ZeroVector;

	const FRotator3d RotationOffset = bHasAnyRotationOffset ?
		FRotator3d(
			RotationOffsetSpline[0].Eval(SplineInput),
			RotationOffsetSpline[1].Eval(SplineInput),
			RotationOffsetSpline[2].Eval(SplineInput))
		:
		FRotator3d::ZeroRotator;

	const USplineOffsetCameraNode* OffsetNode = GetCameraNodeAs<USplineOffsetCameraNode>();

	FTransform3d OutTransform;
	const bool bSuccess = FCameraNodeSpaceMath::OffsetCameraNodeSpaceTransform(
			FCameraNodeSpaceParams(Params, OutResult),
			OutResult.CameraPose.GetTransform(),
			TranslationOffset,
			RotationOffset,
			OffsetNode->OffsetSpace,
			OutTransform);
	if (bSuccess)
	{
		OutResult.CameraPose.SetTransform(OutTransform);
	}
}

#if WITH_EDITOR

void FSplineOffsetCameraNodeEvaluator::OnPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(USplineOffsetCameraNode, TranslationOffsetSpline) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(USplineOffsetCameraNode, RotationOffsetSpline))
	{
		RebuildCurves();
	}
}

#endif  // WITH_EDITOR

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr USplineOffsetCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSplineOffsetCameraNodeEvaluator>();
}

