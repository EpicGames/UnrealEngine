// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/SplineOrbitCameraNode.h"

#include "Core/CameraParameterReader.h"
#include "Core/CameraRigJoints.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"
#include "IGameplayCamerasModule.h"
#include "IGameplayCamerasLiveEditListener.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "Math/CameraNodeSpaceMath.h"
#include "Math/Ray.h"
#include "Nodes/Input/DrivenControlRotationCameraNode.h"
#include "Nodes/Input/Input2DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineOrbitCameraNode)

namespace UE::Cameras
{

bool GGameplayCamerasSplineOrbitShowLocationOffsetSpline = true;
static FAutoConsoleVariableRef CVarGameplayCamerasSplineOrbitShowLocationOffsetSpline(
	TEXT("GameplayCameras.SplineOrbit.ShowLocationOffsetSpline"),
	GGameplayCamerasSplineOrbitShowLocationOffsetSpline,
	TEXT("(Default: 1. Whether to show the camera's spline trajectory."));

bool GGameplayCamerasSplineOrbitShowLocationOffsetOrbits = true;
static FAutoConsoleVariableRef CVarGameplayCamerasSplineOrbitShowLocationOffsetOrbits(
	TEXT("GameplayCameras.SplineOrbit.ShowLocationOffsetOrbits"),
	GGameplayCamerasSplineOrbitShowLocationOffsetOrbits,
	TEXT("Default: 1. Whether to show the control points' orbits."));

class FSplineOrbitCameraNodeEvaluator 
	: public FCameraNodeEvaluator
#if WITH_EDITOR
    , public IGameplayCamerasLiveEditListener
#endif  // WITH_EDITOR
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSplineOrbitCameraNodeEvaluator)

public:

	~FSplineOrbitCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if WITH_EDITOR
	virtual void OnPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) override;
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void RebuildCurves();

private:

	FInput2DCameraNodeEvaluator* InputSlotEvaluator = nullptr;

	FCompressedRichCurve LocationOffsetSpline[3];
	FCompressedRichCurve TargetOffsetSpline[3];
	FCompressedRichCurve RotationOffsetSpline[3];

	TCameraParameterReader<float> LocationOffsetMultiplierReader;

	bool bHasAnyTargetOffset = false;
	bool bHasAnyRotationOffset = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector2d DebugYawPitch;
	FTransform DebugPivotTransform;
	FVector3d DebugLocationOffset;
	FVector3d DebugWorldTargetOffset;
	FRotator3d DebugRotationOffset;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FSplineOrbitCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FSplineOrbitCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, OrbitYawPitch);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FTransform3d, PivotTransform);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, LocationOffset);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, WorldTargetOffset);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, RotationOffset);
#if UE_GAMEPLAY_CAMERAS_DEBUG
	// Pointer to the spline orbit node for rendering the orbit spline in debug cameras.
	// TODO: This won't get serialized when recording gameplay (e.g. with RewindDebugger). We need to add
	//		 support for global data shared between frames, otherwise would have to serialize the curves 
	//		 each frame.
	public:
		TWeakObjectPtr<const USplineOrbitCameraNode> WeakSplineOrbitNode;
	private:
		void RenderLocationOffsetSpline(const USplineOrbitCameraNode* SplineOrbitNode, FCameraDebugRenderer& Renderer);
		void RenderLocationOffsetOrbits(const USplineOrbitCameraNode* SplineOrbitNode, FCameraDebugRenderer& Renderer);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FSplineOrbitCameraDebugBlock)

void FSplineOrbitCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const USplineOrbitCameraNode* SplineOrbitNode = GetCameraNodeAs<USplineOrbitCameraNode>();
	if (SplineOrbitNode->InputSlot)
	{
		InputSlotEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(SplineOrbitNode->InputSlot);
	}
	else
	{
		const UDrivenControlRotationCameraNode* PlaceholderInput = NewObject<UDrivenControlRotationCameraNode>();
		InputSlotEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(PlaceholderInput);
	}
}

void FSplineOrbitCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	RebuildCurves();

	const USplineOrbitCameraNode* SplineOrbitNode = GetCameraNodeAs<USplineOrbitCameraNode>();
	LocationOffsetMultiplierReader.Initialize(SplineOrbitNode->LocationOffsetMultiplier);

#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->AddListener(GetCameraNode(), this);
	}
#endif  // WITH_EDITOR
}

FSplineOrbitCameraNodeEvaluator::~FSplineOrbitCameraNodeEvaluator()
{
#if WITH_EDITOR
	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	if (TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager = GameplayCamerasModule.GetLiveEditManager())
	{
		LiveEditManager->RemoveListener(this);
	}
#endif  // WITH_EDITOR
}

void FSplineOrbitCameraNodeEvaluator::RebuildCurves()
{
	const USplineOrbitCameraNode* SplineOrbitNode = GetCameraNodeAs<USplineOrbitCameraNode>();

	bHasAnyTargetOffset = SplineOrbitNode->TargetOffsetSpline.HasAnyData();
	bHasAnyRotationOffset = SplineOrbitNode->RotationOffsetSpline.HasAnyData();

	SplineOrbitNode->LocationOffsetSpline.Curves[0].CompressCurve(LocationOffsetSpline[0]);
	SplineOrbitNode->LocationOffsetSpline.Curves[1].CompressCurve(LocationOffsetSpline[1]);
	SplineOrbitNode->LocationOffsetSpline.Curves[2].CompressCurve(LocationOffsetSpline[2]);

	SplineOrbitNode->TargetOffsetSpline.Curves[0].CompressCurve(TargetOffsetSpline[0]);
	SplineOrbitNode->TargetOffsetSpline.Curves[1].CompressCurve(TargetOffsetSpline[1]);
	SplineOrbitNode->TargetOffsetSpline.Curves[2].CompressCurve(TargetOffsetSpline[2]);

	SplineOrbitNode->RotationOffsetSpline.Curves[0].CompressCurve(RotationOffsetSpline[0]);
	SplineOrbitNode->RotationOffsetSpline.Curves[1].CompressCurve(RotationOffsetSpline[1]);
	SplineOrbitNode->RotationOffsetSpline.Curves[2].CompressCurve(RotationOffsetSpline[2]);
}

FCameraNodeEvaluatorChildrenView FSplineOrbitCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ InputSlotEvaluator });
}

void FSplineOrbitCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FRotator3d OrbitRotation = FRotator3d::ZeroRotator;
	if (ensure(InputSlotEvaluator))
	{
		InputSlotEvaluator->Run(Params, OutResult);
		const FVector2d YawPitch = InputSlotEvaluator->GetInputValue();
		OrbitRotation = FRotator3d(YawPitch.Y, YawPitch.X, 0);
	}
	OrbitRotation.Normalize();

	const USplineOrbitCameraNode* SplineOrbitNode = GetCameraNodeAs<USplineOrbitCameraNode>();

	// Compute the camera transform similarly to the boom arm (see FBoomArmCameraNodeEvaluator).
	const FTransform3d OrbitPivot(OrbitRotation, OutResult.CameraPose.GetLocation());
	const FVector3d LocationOffset = FVector3d(
			LocationOffsetSpline[0].Eval(OrbitRotation.Pitch),
			LocationOffsetSpline[1].Eval(OrbitRotation.Pitch),
			LocationOffsetSpline[2].Eval(OrbitRotation.Pitch));
	const float LocationOffsetMultiplier = LocationOffsetMultiplierReader.Get(OutResult.VariableTable);
	FTransform3d OrbitTransform(FTransform3d(LocationOffset * LocationOffsetMultiplier) * OrbitPivot);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugYawPitch = FVector2d(OrbitRotation.Yaw, OrbitRotation.Pitch);
	DebugPivotTransform = OrbitPivot;
	DebugLocationOffset = LocationOffset;
	DebugWorldTargetOffset = FVector3d::ZeroVector;
	DebugRotationOffset = FRotator3d::ZeroRotator;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// Now rotate the camera according to the target offset, if any.
	if (bHasAnyTargetOffset)
	{
		const FVector3d TargetOffset = FVector3d(
				TargetOffsetSpline[0].Eval(OrbitRotation.Pitch),
				TargetOffsetSpline[1].Eval(OrbitRotation.Pitch),
				TargetOffsetSpline[2].Eval(OrbitRotation.Pitch));
		if (!TargetOffset.IsNearlyZero())
		{
			// Project the orbit center onto the line of sight. It would already be on it if there wasn't
			// any lateral or vertical offset in LocationOffset, but most often there is, so the line of sight
			// is offset from the center.
			const FRay LineOfSight(OrbitTransform.GetLocation(), OrbitTransform.GetRotation().GetForwardVector());
			const FVector3d ProjectedOrbitPivot = LineOfSight.ClosestPoint(OrbitPivot.GetLocation());

			// Now use this projected point as the "target" of the camera for the purposes of orbiting.
			// Offset that target and make the camera look at the new target.
			FVector3d NewTarget;
			FCameraNodeSpaceParams SpaceMathParams(Params, OutResult);
			FCameraNodeSpaceMath::OffsetCameraNodeSpacePosition(
					SpaceMathParams, ProjectedOrbitPivot, TargetOffset, SplineOrbitNode->TargetOffsetSpace, NewTarget);

			const FVector3d NewLineOfSight = (NewTarget - LineOfSight.Origin);
			FRotator3d NewOrientation = NewLineOfSight.ToOrientationRotator();
			NewOrientation.Roll = OrbitTransform.Rotator().Roll;
			OrbitTransform.SetRotation(NewOrientation.Quaternion());

#if UE_GAMEPLAY_CAMERAS_DEBUG
			DebugWorldTargetOffset = (NewTarget - ProjectedOrbitPivot);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
		}
	}

	// Add any extra rotation if needed.
	if (bHasAnyRotationOffset)
	{
		const FVector3d RotationOffset = FVector3d(
				RotationOffsetSpline[0].Eval(OrbitRotation.Pitch),
				RotationOffsetSpline[1].Eval(OrbitRotation.Pitch),
				RotationOffsetSpline[2].Eval(OrbitRotation.Pitch));
		if (!RotationOffset.IsNearlyZero())
		{
			const FTransform3d RotationOffsetTransform(FRotator3d::MakeFromEuler(RotationOffset));
			OrbitTransform = RotationOffsetTransform * OrbitTransform;

#if UE_GAMEPLAY_CAMERAS_DEBUG
			DebugRotationOffset = FRotator3d::MakeFromEuler(RotationOffset);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
		}
	}

	OutResult.CameraPose.SetTransform(OrbitTransform);

	OutResult.CameraRigJoints.AddYawPitchJoint(OrbitPivot);
}

#if WITH_EDITOR

void FSplineOrbitCameraNodeEvaluator::OnPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(USplineOrbitCameraNode, LocationOffsetSpline) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(USplineOrbitCameraNode, TargetOffsetSpline) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(USplineOrbitCameraNode, RotationOffsetSpline))
	{
		RebuildCurves();
	}
}

#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FSplineOrbitCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FSplineOrbitCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FSplineOrbitCameraDebugBlock>();

	DebugBlock.OrbitYawPitch = DebugYawPitch;
	DebugBlock.PivotTransform = DebugPivotTransform;
	DebugBlock.LocationOffset = DebugLocationOffset;
	DebugBlock.WorldTargetOffset = DebugWorldTargetOffset;
	DebugBlock.RotationOffset = DebugRotationOffset;
	DebugBlock.WeakSplineOrbitNode = GetCameraNodeAs<USplineOrbitCameraNode>();
}

void FSplineOrbitCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("yaw: %.3f pitch: %.3f"), OrbitYawPitch.X, OrbitYawPitch.Y);
	Renderer.NewLine();
	Renderer.AddIndent();
	{
		Renderer.AddText(TEXT("location offset: %s\n"), *LocationOffset.ToString());
		Renderer.AddText(TEXT("target offset: %s\n"), *WorldTargetOffset.ToString());
		Renderer.AddText(TEXT("rotation offset: %s\n"), *RotationOffset.ToString());
	}
	Renderer.RemoveIndent();

	if (Renderer.IsExternalRendering())
	{
		// TODO: right now we need to keep a pointer to the camera node to read the spline data (see previous comment).
		if (const USplineOrbitCameraNode* SplineOrbitNode = WeakSplineOrbitNode.Get())
		{
			RenderLocationOffsetSpline(SplineOrbitNode, Renderer);	
			RenderLocationOffsetOrbits(SplineOrbitNode, Renderer);

			// Render the orbit pivot.
			Renderer.DrawSphere(PivotTransform.GetLocation(), 1.f, 8, FColorList::Brass, 1.f);
		}
	}
}

void FSplineOrbitCameraDebugBlock::RenderLocationOffsetSpline(const USplineOrbitCameraNode* SplineOrbitNode, FCameraDebugRenderer& Renderer)
{
	if (!GGameplayCamerasSplineOrbitShowLocationOffsetSpline)
	{
		return;
	}

	// Sample the curve for pitch values inside a sensible range.
	// We use the uncompressed curves directly from the camera node here because the compressed curves
	// from the evaluator may not be available if we recorded the debug data and the evaluator is gone.
	auto ComputeLocationOffsetSample = [this, SplineOrbitNode](float InPitchAngle) -> FVector3d
	{
		const FRotator3d CurOrbitRotation(InPitchAngle, OrbitYawPitch.X, 0.f);
		const FTransform3d CurPivotTransform(CurOrbitRotation, PivotTransform.GetLocation());

		const FVector3d CurLocationOffset = SplineOrbitNode->LocationOffsetSpline.GetValue(InPitchAngle);
		const FTransform3d CurOrbitTransform(FTransform3d(CurLocationOffset) * CurPivotTransform);
		return CurOrbitTransform.GetLocation();
	};

	const FLinearColor SplineColor(FColorList::OrangeRed);
	FVector3d PrevSamplePoint = ComputeLocationOffsetSample(-89.f);

	for (float CurPitchAngle = -89.f + 2.f; CurPitchAngle <= 89.f; CurPitchAngle += 2.f)
	{
		FVector3d NextSamplePoint = ComputeLocationOffsetSample(CurPitchAngle);

		Renderer.DrawLine(PrevSamplePoint, NextSamplePoint, SplineColor);

		PrevSamplePoint = NextSamplePoint;
	}
}

void FSplineOrbitCameraDebugBlock::RenderLocationOffsetOrbits(const USplineOrbitCameraNode* SplineOrbitNode, FCameraDebugRenderer& Renderer)
{
	if (!GGameplayCamerasSplineOrbitShowLocationOffsetOrbits)
	{
		return;
	}

	// Gather the control points' pitch values. We'll draw an ellipse at each of them.
	TArray<float, TInlineAllocator<32>> PitchValues;
	for (int32 CurveIndex = 0; CurveIndex < 3; ++CurveIndex)
	{
		for (auto It = SplineOrbitNode->LocationOffsetSpline.Curves[CurveIndex].GetKeyIterator(); It; ++It)
		{
			const float PitchValue = It->Time;
			if (PitchValue >= -89.f && PitchValue <= 89.f)
			{
				PitchValues.Add(It->Time);
			}
		}
	}

	// Now sample the location offset all around the pivot (360 degrees) for each pitch value of
	// the control points.
	auto ComputeLocationOffsetSample = [this, SplineOrbitNode](
			float InYawAngle, float InPitchAngle, const FVector3d& InLocationOffset) -> FVector3d
	{
		const FRotator3d CurOrbitRotation(InPitchAngle, InYawAngle, 0.f);
		const FTransform3d CurPivotTransform(CurOrbitRotation, PivotTransform.GetLocation());

		const FTransform3d CurOrbitTransform(FTransform3d(InLocationOffset) * CurPivotTransform);
		return CurOrbitTransform.GetLocation();
	};

	const float OrbitAngleStep = 2.f;
	const FLinearColor EllipseColor(FColorList::Brass);
	for (float PitchValue : PitchValues)
	{
		const FVector3d CurLocationOffset = SplineOrbitNode->LocationOffsetSpline.GetValue(PitchValue);

		FVector3d  PrevSamplePoint = ComputeLocationOffsetSample(0.f, PitchValue, CurLocationOffset);
		for (float CurEllipseAngle = OrbitAngleStep; CurEllipseAngle < 360.f; CurEllipseAngle += OrbitAngleStep)
		{
			FVector3d NextSamplePoint = ComputeLocationOffsetSample(CurEllipseAngle, PitchValue, CurLocationOffset);

			Renderer.DrawLine(PrevSamplePoint, NextSamplePoint, EllipseColor);

			PrevSamplePoint = NextSamplePoint;
		}
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

USplineOrbitCameraNode::USplineOrbitCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);
}

FCameraNodeChildrenView USplineOrbitCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView({ InputSlot });
}

FCameraNodeEvaluatorPtr USplineOrbitCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FSplineOrbitCameraNodeEvaluator>();
}

