// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Common/TargetRayCastCameraNode.h"

#include "CollisionQueryParams.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "WorldCollision.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetRayCastCameraNode)

namespace UE::Cameras
{

float GTargetRayCastLength = 100000.0;
static FAutoConsoleVariableRef CVarTargetRayCastLength(
	TEXT("GameplayCameras.TargetRayCastLength"),
	GTargetRayCastLength,
	TEXT("(Default: 100000cm. Sets the length of the line trace test that determines the camera's target distance."));

class FTargetRayCastCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FTargetRayCastCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void RunLineTrace(UWorld* World, APlayerController* PlayerController, FCameraNodeEvaluationResult& OutResult);

private:

	TCameraParameterReader<bool> AutoFocusReader;

	double LastHitResultDistance = 1000.0; // Same as the default CameraPose target distance.

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector3d LastHitResultLocation;
	FString LastHitResultName;
	bool bGotLastHitResult;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FTargetRayCastCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FTargetRayCastCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, HitResultLocation);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, HitResultDistance);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, HitResultName);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bGotHitResult);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FTargetRayCastCameraDebugBlock)

void FTargetRayCastCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UTargetRayCastCameraNode* RayCastNode = GetCameraNodeAs<UTargetRayCastCameraNode>();
	AutoFocusReader.Initialize(RayCastNode->AutoFocus);
}

void FTargetRayCastCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (!ensure(Params.EvaluationContext))
	{
		return;
	}

	if (Params.EvaluationType != ECameraNodeEvaluationType::Standard)
	{
		// Don't run actual ray-casts during IK/stateless updates.
		OutResult.CameraPose.SetTargetDistance(LastHitResultDistance);
		return;
	}

	UWorld* World = Params.EvaluationContext->GetWorld();
	if (!ensure(World))
	{
		return;
	}

	APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController();
	RunLineTrace(World, PlayerController, OutResult);
}

void FTargetRayCastCameraNodeEvaluator::RunLineTrace(UWorld* World, APlayerController* PlayerController, FCameraNodeEvaluationResult& OutResult)
{
	static FName LineTraceTag(TEXT("TargetRayCast"));
	static FName LineTraceOwnerTag(TEXT("TargetRayCastCameraNode"));

	APawn* Pawn = PlayerController ? PlayerController->GetPawn().Get() : nullptr;

	FCameraPose& CameraPose = OutResult.CameraPose;
	const FVector3d RayStart = CameraPose.GetLocation();
	const FVector3d RayEnd = RayStart + CameraPose.GetAimDir() * GTargetRayCastLength;

	const UTargetRayCastCameraNode* RayCastNode = GetCameraNodeAs<UTargetRayCastCameraNode>();
	ECollisionChannel TraceChannel = RayCastNode->TraceChannel;

	FHitResult HitResult;

	// Ignore the player pawn by default.
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(TargetLineTrace), false, Pawn);
	QueryParams.TraceTag = LineTraceTag;
	QueryParams.OwnerTag = LineTraceOwnerTag;

	const bool bGotHit = World->LineTraceSingleByChannel(
			HitResult, 
			RayStart, RayEnd,
			TraceChannel,
			QueryParams);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	bGotLastHitResult = bGotHit;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	if (bGotHit)
	{
		const double TargetDistance = (HitResult.ImpactPoint - RayStart).Length();
		CameraPose.SetTargetDistance(TargetDistance);
		LastHitResultDistance = TargetDistance;

#if UE_GAMEPLAY_CAMERAS_DEBUG
		LastHitResultLocation = HitResult.ImpactPoint;
		LastHitResultName = GetNameSafe(HitResult.GetActor());
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	}
	else
	{
		CameraPose.SetTargetDistance(GTargetRayCastLength);
	}

	if (AutoFocusReader.Get(OutResult.VariableTable))
	{
		CameraPose.SetFocusDistance(CameraPose.GetTargetDistance());
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FTargetRayCastCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FTargetRayCastCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FTargetRayCastCameraDebugBlock>();
	DebugBlock.HitResultLocation = LastHitResultLocation;
	DebugBlock.HitResultDistance = LastHitResultDistance;
	DebugBlock.HitResultName = LastHitResultName;
	DebugBlock.bGotHitResult = bGotLastHitResult;
}

void FTargetRayCastCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bGotHitResult)
	{
		Renderer.AddText(TEXT("hit '%s', distance %.3f"), *HitResultName, HitResultDistance);
	}
	else
	{
		Renderer.AddText(TEXT("no hit"));
	}

	Renderer.DrawSphere(HitResultLocation, 1.f, 8, FLinearColor::Blue, 1.f);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UTargetRayCastCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FTargetRayCastCameraNodeEvaluator>();
}

