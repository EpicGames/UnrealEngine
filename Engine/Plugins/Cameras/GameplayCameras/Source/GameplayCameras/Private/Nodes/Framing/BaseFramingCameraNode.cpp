// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Framing/BaseFramingCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCameras.h"
#include "HAL/IConsoleManager.h"
#include "Math/CameraPoseMath.h"
#include "Math/ColorList.h"
#include "Math/Interpolation.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Nodes/Framing/CameraActorTargetInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseFramingCameraNode)

namespace UE::Cameras
{

float GFramingIdealReachedEpsilon = 0.001f;
static FAutoConsoleVariableRef CVarFramingUnlockRadiusEpsilon(
	TEXT("GameplayCameras.Framing.IdealReachedEpsilon"),
	GFramingIdealReachedEpsilon,
	TEXT("(Default: 0.001) The epsilon to determine whether we have reached ideal screen framing."));

float GFramingExtrapolationEpsilon = 0.001f;
static FAutoConsoleVariableRef CVarFramingExtrapolationEpsilon(
	TEXT("GameplayCameras.Framing.ExtrapolationEpsilon"),
	GFramingExtrapolationEpsilon,
	TEXT("(Default: 0.001) The epsilon to determine whether target movement extrapolation should be included."));

float GFramingMinDampingFactor = 0.0001f;
static FAutoConsoleVariableRef CVarFramingMinDampingFactor(
	TEXT("GameplayCameras.Framing.MinDampingFactor"),
	GFramingMinDampingFactor,
	TEXT("(Default: 0.0001) The minimimum reframe damping factor possible once all factors have been taken into account."));

float GFramingTargetRestEpsilon = 0.001f;
static FAutoConsoleVariableRef CVarFramingTargetRestEpsilon(
	TEXT("GameplayCameras.Framing.TargetRestEpsilon"),
	GFramingTargetRestEpsilon,
	TEXT("(Default: 0.001) The screen-space speed under which we consider the target has stopped moving."));

int32 GFramingNumTargetMovementSamples = 10;
static FAutoConsoleVariableRef CVarFramingNumTargetMovementSamples(
	TEXT("GameplayCameras.Framing.NumTargetMovementSamples"),
	GFramingNumTargetMovementSamples,
	TEXT("(Default: 10) The number of samples to use to extrapolate target movement."));

bool GFramingShowEffectiveDeadZone = false;
static FAutoConsoleVariableRef CVarFramingShowEffectiveDeadZone(
	TEXT("GameplayCameras.Framing.ShowEffectiveDeadZone"),
	GFramingShowEffectiveDeadZone,
	TEXT("(Default: false) Show the effective dead zone"));

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FBaseFramingCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FBaseFramingCameraNodeEvaluator::FState, State);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FBaseFramingCameraNodeEvaluator::FDesired, Desired);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FBaseFramingCameraNodeEvaluator::FWorldTargetInfos, WorldTargets);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FBaseFramingCameraDebugBlock)

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBaseFramingCameraNodeEvaluator)

void FBaseFramingCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UBaseFramingCameraNode* BaseFramingNode = GetCameraNodeAs<UBaseFramingCameraNode>();

	Readers.TargetInfos.Initialize(BaseFramingNode->TargetInfos, BaseFramingNode->TargetInfosDataID);

	Readers.IdealFramingLocation.Initialize(BaseFramingNode->IdealFramingLocation);
	Readers.InitializeWithIdealFraming.Initialize(BaseFramingNode->InitializeWithIdealFraming);
	Readers.SetTargetDistance.Initialize(BaseFramingNode->SetTargetDistance);

	Readers.ReframeDampingFactor.Initialize(BaseFramingNode->ReframeDampingFactor);
	Readers.LowReframeDampingFactor.Initialize(BaseFramingNode->LowReframeDampingFactor);
	Readers.ReengageTime.Initialize(BaseFramingNode->ReengageTime);
	Readers.DisengageTime.Initialize(BaseFramingNode->DisengageTime);
	Readers.TargetMovementAnticipationTime.Initialize(BaseFramingNode->TargetMovementAnticipationTime);

	Readers.DeadZone.Initialize(BaseFramingNode->DeadZone);
	Readers.SoftZone.Initialize(BaseFramingNode->SoftZone);

	ScreenTargetHistory.History.Reserve(GFramingNumTargetMovementSamples);

	TArray<FCameraActorComputedTargetInfo> AcquiredTargetInfos;
	const bool bAcquireSuccess = AcquireTargetInfo(Params.EvaluationContext, OutResult, AcquiredTargetInfos);
	WorldTargets.TargetInfos = AcquiredTargetInfos;
}

TOptional<FVector3d> FBaseFramingCameraNodeEvaluator::GetInitialDesiredWorldTarget(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult)
{
	if (Params.bIsFirstFrame && 
			Readers.InitializeWithIdealFraming.Get(OutResult.VariableTable) && 
			!WorldTargets.TargetInfos.IsEmpty())
	{
		FVector3d ApproximatedWorldTarget(EForceInit::ForceInitToZero);
		for (const FCameraActorComputedTargetInfo& TargetInfo : WorldTargets.TargetInfos)
		{
			ApproximatedWorldTarget += TargetInfo.Transform.GetLocation() * TargetInfo.NormalizedWeight;
		}
		return ApproximatedWorldTarget;
	}
	return TOptional<FVector3d>();
}

void FBaseFramingCameraNodeEvaluator::UpdateFramingState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult, const FTransform3d& LastFraming)
{
	TArray<FCameraActorComputedTargetInfo> AcquiredTargetInfos;
	const bool bAcquireSuccess = AcquireTargetInfo(Params.EvaluationContext, OutResult, AcquiredTargetInfos);
	WorldTargets.TargetInfos = AcquiredTargetInfos;
	if (bAcquireSuccess && AcquiredTargetInfos.Num() > 0)
	{
		ComputeCurrentState(Params, OutResult, LastFraming);
		ComputeDesiredState(Params, OutResult);
	}
}

void FBaseFramingCameraNodeEvaluator::EndFramingUpdate(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (Readers.SetTargetDistance.Get(OutResult.VariableTable))
	{
		FCameraPose& OutCameraPose = OutResult.CameraPose;
		const double TargetDistance = FVector3d::Distance(State.WorldTarget, OutCameraPose.GetLocation());
		OutCameraPose.SetTargetDistance(TargetDistance);
	}
}

bool FBaseFramingCameraNodeEvaluator::AcquireTargetInfo(TSharedPtr<const FCameraEvaluationContext> EvaluationContext, const FCameraNodeEvaluationResult& InResult, TArray<FCameraActorComputedTargetInfo>& OutInfos)
{
	const UBaseFramingCameraNode* FramingNode = GetCameraNodeAs<UBaseFramingCameraNode>();
	if (FramingNode->TargetLocation.IsValid())
	{
		const FVector3d* TargetLocation = FramingNode->TargetLocation.GetValue(InResult.VariableTable);
		if (TargetLocation)
		{
			FCameraActorComputedTargetInfo OutInfo;
			OutInfo.Transform = FTransform3d(*TargetLocation);
			OutInfo.LocalBounds = FBoxSphereBounds3d(EForceInit::ForceInit);
			OutInfos.Add(OutInfo);
			return true;
		}
	}
	else if (FramingNode->TargetInfos.Num() > 0 || FramingNode->TargetInfosDataID.IsValid())
	{
		if (Readers.TargetInfos.ComputeTargetInfos(InResult.ContextDataTable, OutInfos))
		{
			return true;
		}
	}
	else if (APlayerController* PlayerController = EvaluationContext->GetPlayerController())
	{
		if (APawn* Pawn = PlayerController->GetPawn())
		{
			FCameraActorComputedTargetInfo OutInfo;
			OutInfo.Transform = FTransform3d(Pawn->GetActorLocation());
			OutInfo.LocalBounds = FBoxSphereBounds3d(EForceInit::ForceInit);
			if (USceneComponent* RootComponent = Pawn->GetRootComponent())
			{
				OutInfo.LocalBounds = RootComponent->Bounds;
			}
			OutInfos.Add(OutInfo);
			return true;
		}
	}

	return false;
}

void FBaseFramingCameraNodeEvaluator::ComputeCurrentState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult, const FTransform3d& LastFraming)
{
	// Get screen-space coordinates of the ideal framing point. These are in 0..1 UI space.
	State.IdealTarget = Readers.IdealFramingLocation.Get(OutResult.VariableTable);

	// Update the damping factors and reengage/disengage times in case they are driven by a variable.
	State.ReframeDampingFactor = Readers.ReframeDampingFactor.Get(OutResult.VariableTable);
	State.LowReframeDampingFactor = Readers.LowReframeDampingFactor.Get(OutResult.VariableTable);
	State.ReengageTime = Readers.ReengageTime.Get(OutResult.VariableTable);
	State.DisengageTime = Readers.DisengageTime.Get(OutResult.VariableTable);
	State.TargetMovementAnticipationTime = Readers.TargetMovementAnticipationTime.Get(OutResult.VariableTable);

	// Get the effective margins of the framing zones for this frame.
	const FCameraFramingZone DeadZone = Readers.DeadZone.Get(OutResult.VariableTable);
	const FCameraFramingZone SoftZone = Readers.SoftZone.Get(OutResult.VariableTable);

	// Compute the UI space coordinates of the framing zones.
	State.DeadZone = FFramingZone::FromRelativeMargins(State.IdealTarget, DeadZone);
	State.SoftZone = FFramingZone::FromScreenMargins(SoftZone);

	// Make sure our framing zones are hierarchically correct: soft zone contains the dead zone, and the
	// dead zone contains the ideal target.
	State.DeadZone.ClampBounds(State.IdealTarget);
	State.SoftZone.ClampBounds(State.DeadZone);

	// We are going to reframe things iteratively, so we'll use a temporary pose defined by last frame's
	// shot transform.
	FCameraPose TempPose(OutResult.CameraPose);
	TempPose.SetTransform(LastFraming);
	const FMatrix TempPoseViewProjectionMatrix = FCameraPoseMath::BuildViewProjectionMatrix(TempPose, Params.EvaluationContext);

	// Process our targets and figure out the weighted average we should be aiming at.
	FVector2d NewScreenTarget;
	FVector3d OldWorldTarget = State.WorldTarget;
	ComputeFinalTargetInfo(TempPose, TempPoseViewProjectionMatrix, State.WorldTarget, NewScreenTarget, State.ScreenTargetBounds);

	// See if we need to extrapolate where the target will be in "anticipation time" seconds.
	State.ScreenTarget = ComputeAnticipatedScreenTarget(Params.DeltaTime, State.ScreenTarget, NewScreenTarget);

	// Compute how fast the target has moved over the last frame.
	State.TargetScreenSpaceSpeed = ComputeScreenTargetSpeed(Params.DeltaTime, TempPoseViewProjectionMatrix, OldWorldTarget, State.WorldTarget);

	// Compute the effective dead-zone, which is the subset of the dead-zone that encompasses as much
	// of the target's bound as possible.
	State.EffectiveDeadZone = ComputeEffectiveDeadZone();

#if UE_GAMEPLAY_CAMERAS_DEBUG
	State.DebugDeadZoneEdgePoint = State.ScreenTarget;
	State.DebugHardZoneEdgePoint = State.ScreenTarget;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// Update the reframe damper's damping factor.
	if (State.LowReframeDampingFactor <= 0)
	{
		// There's no high/low factors, so just set the standard one at 100%.
		State.ReframeDamper.SetW0(State.ReframeDampingFactor);
		State.ReframeDampingFactorAlpha = 1.f;
	}
	else
	{
		// Make a line between the ideal target and the current target. Note how it intersects the 
		// boundaries of the dead zone and the hard zone. We will interpolate the damping factors from 
		// the first intersection to the second intersection.
		const FVector2d IdealToCurrent(State.ScreenTarget - State.IdealTarget);
		const double IdealToCurrentDistance = IdealToCurrent.Length();
		if (IdealToCurrentDistance > 0.0)
		{
			const FVector2d IdealToCurrentNormalized = (IdealToCurrent / IdealToCurrentDistance);
			const FVector2d DeadEdgePoint = State.DeadZone.ComputeClosestIntersection(State.IdealTarget, IdealToCurrentNormalized, true);
			const FVector2d HardEdgePoint = State.SoftZone.ComputeClosestIntersection(State.IdealTarget, IdealToCurrentNormalized, true);

			const double DeadToCurrent = FVector2d::Distance(DeadEdgePoint, State.ScreenTarget);
			const double DeadToHardEdge = FVector2d::Distance(DeadEdgePoint, HardEdgePoint);

			const double Alpha = FMath::Clamp(DeadToCurrent / DeadToHardEdge, 0.0, 1.0);
			State.ReframeDampingFactorAlpha = Alpha;

#if UE_GAMEPLAY_CAMERAS_DEBUG
			State.DebugDeadZoneEdgePoint = DeadEdgePoint;
			State.DebugHardZoneEdgePoint = HardEdgePoint;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
		}
		else
		{
			State.ReframeDampingFactorAlpha = 0.f;
		}
	}

	const bool bIsInSoftZone = State.SoftZone.Contains(State.ScreenTarget);
	const bool bIsInDeadZone = State.EffectiveDeadZone.Contains(State.ScreenTarget);
	const bool bTargetStillMoving = (State.TargetScreenSpaceSpeed > GFramingTargetRestEpsilon);
	if (!bIsInSoftZone)
	{
		// Target is out of view or outside the soft zone -- it's therefore in the hard zone and we will
		// do everything we can to put it back in the soft zone ASAP.
		State.TargetFramingState = ETargetFramingState::InHardZone;
		State.ToggleEngageTimeLeft = 0.f;
		State.ToggleEngageAlpha = 1.f;
		State.bIsReframingTarget = true;
	}
	else if (!bIsInDeadZone || bTargetStillMoving)
	{
		// Target is in the soft zone, or in the dead-zone but still moving enough that we don't want to
		// disengage the reframing. We will gently reframe it towards the ideal framing.
		const bool bWasInDeadZone = (State.TargetFramingState == ETargetFramingState::InDeadZone);
		State.TargetFramingState = ETargetFramingState::InSoftZone;

		// We just exited the dead zone, so start the reengage timer. If we were still disengaging
		// inside the dead zone, restart the timer from an equivalent time.
		if (bWasInDeadZone)
		{
			if (State.ToggleEngageTimeLeft > 0 && State.DisengageTime > 0.0)
			{
				// Example: if we had 60% of time left to disengage, we would want to restart at 60%
				//			of reengagement, with 40% left to go.
				const float PreviousAlpha = State.ToggleEngageAlpha;
				const float TimeRatioLeft = FMath::Clamp(State.ToggleEngageTimeLeft / State.DisengageTime, 0.f, 1.f);
				State.ToggleEngageTimeLeft = (1.f - TimeRatioLeft) * State.ReengageTime;
				ensure(FMath::IsNearlyEqual(PreviousAlpha, 1.0f - State.ToggleEngageTimeLeft / State.ReengageTime, 1.e-4f));
			}
			else
			{
				State.ToggleEngageTimeLeft = State.ReengageTime;
			}
		}
		// If we are reengaging, continue doing so.
		// The ratio between ToggleEngageTimeLeft and ReengageTime will go from 1 to 0, but we we want
		// to reengage, i.e. go from 0% to 100%, hence the "one minus..." below.
		if (State.ToggleEngageTimeLeft > 0.0)
		{
			State.ToggleEngageTimeLeft = FMath::Max(0, State.ToggleEngageTimeLeft - Params.DeltaTime);
			State.ToggleEngageAlpha = 1.0f - FMath::Clamp(State.ToggleEngageTimeLeft / State.ReengageTime, 0.f, 1.f);
		}
		else
		{
			State.ToggleEngageAlpha = 1.f;
		}
		
		// Put us in reframing mode as soon as we get into the soft zone.
		State.bIsReframingTarget = true;
	}
	else
	{
		// Target is in the dead zone.
		const bool bWasInSoftZone = (State.TargetFramingState == ETargetFramingState::InSoftZone);
		State.TargetFramingState = ETargetFramingState::InDeadZone;

		// We just entered the dead zone, so start the disengage timer. If we were still reengaging
		// inside the soft zone, restart the timer from an equivalent time.
		if (bWasInSoftZone)
		{
			if (State.ToggleEngageTimeLeft > 0 && State.ReengageTime > 0.0)
			{
				// Example: if we had 30% of time left to reengage, we would want to restart at 30% of 
				//		    disengagement, with 70% left to go.
				const float PreviousAlpha = State.ToggleEngageAlpha;
				const float TimeRatioLeft = FMath::Clamp(State.ToggleEngageTimeLeft / State.ReengageTime, 0.f, 1.f);
				State.ToggleEngageTimeLeft = (1.f - TimeRatioLeft) * State.DisengageTime;
				ensure(FMath::IsNearlyEqual(PreviousAlpha, State.ToggleEngageTimeLeft / State.DisengageTime, 1.e-4f));
			}
			else
			{
				State.ToggleEngageTimeLeft = State.DisengageTime;
			}
		}
		// If we are disengaging, continue doing so.
		// The ratio between ToggleEngageTimeLeft and DisengageTime will go from 1 to 0, which is what
		// we want since we want ramp down from 100% to 0%.
		if (State.ToggleEngageTimeLeft > 0.0)
		{
			State.ToggleEngageTimeLeft = FMath::Max(0, State.ToggleEngageTimeLeft - Params.DeltaTime);
			State.ToggleEngageAlpha = FMath::Clamp(State.ToggleEngageTimeLeft / State.DisengageTime, 0.f, 1.f);
		}
		else
		{
			State.ToggleEngageAlpha = 0.f;
		}

		// Only truly disengage reframing once we reach the end of the disengagement time.
		State.bIsReframingTarget = (State.ToggleEngageAlpha > 0);
	}
}

bool FBaseFramingCameraNodeEvaluator::ComputeFinalTargetInfo(const FCameraPose& CameraPose, const FMatrix& ViewProjectionMatrix, FVector3d& OutWorldTarget, FVector2d& OutScreenTarget, FFramingZone& OutScreenBounds)
{
	TConstArrayView<FCameraActorComputedTargetInfo> TargetInfos(WorldTargets.TargetInfos);

	if (TargetInfos.IsEmpty())
	{
		return false;
	}

	// Start with projecting all the targets, and their bounds, on screen.
	struct FComputedTargetScreenInfo
	{
		FVector3d WorldTarget;
		FVector2d ScreenTarget;
		FFramingZone ScreenBounds;
		double WorldTargetDistance = 0.f;
		float NormalizedWeight = 1.f;
	};

	TArray<FComputedTargetScreenInfo> TargetScreenInfos;
	TargetScreenInfos.SetNum(TargetInfos.Num());
#if UE_GAMEPLAY_CAMERAS_DEBUG
	State.DebugAllScreenTargetBounds.SetNum(TargetInfos.Num());
#endif

	for (int32 Index = 0; Index < TargetInfos.Num(); ++Index)
	{
		const FCameraActorComputedTargetInfo& TargetInfo(TargetInfos[Index]);
		FComputedTargetScreenInfo& TargetScreenInfo(TargetScreenInfos[Index]);

		const FVector3d& WorldTarget = TargetInfo.Transform.GetLocation();
		const TOptional<FVector2d> ScreenTarget = FCameraPoseMath::ProjectToScreen(ViewProjectionMatrix, WorldTarget, true);

		TargetScreenInfo.WorldTarget = WorldTarget;
		TargetScreenInfo.ScreenTarget = ScreenTarget.Get(FVector2d(0.5, 0.5));
		TargetScreenInfo.ScreenBounds = ComputeScreenTargetBounds(ViewProjectionMatrix, TargetInfo.Transform, TargetInfo.LocalBounds);
		TargetScreenInfo.WorldTargetDistance = FVector3d::Distance(CameraPose.GetLocation(), WorldTarget);
		TargetScreenInfo.NormalizedWeight = TargetInfo.NormalizedWeight;

#if UE_GAMEPLAY_CAMERAS_DEBUG
		State.DebugAllScreenTargetBounds[Index] = TargetScreenInfo.ScreenBounds;
#endif
	}

	// If we only have one target, just use that.
	if (TargetInfos.Num() == 1)
	{
		const FComputedTargetScreenInfo& TargetScreenInfo(TargetScreenInfos[0]);
		OutScreenTarget = TargetScreenInfo.ScreenTarget;
		OutScreenBounds = TargetScreenInfo.ScreenBounds;
		OutWorldTarget = TargetScreenInfo.WorldTarget;
		return true;
	}

	// Compute the weighted average of the screen target, and combine the screen bounds.
	FVector2d FinalScreenTarget(EForceInit::ForceInit);
	FFramingZone FinalScreenBounds(TargetScreenInfos[0].ScreenBounds);
	double FinalWorldTargetDistance = 0.f;
	for (int32 Index = 0; Index < TargetScreenInfos.Num(); ++Index)
	{
		const FComputedTargetScreenInfo& TargetScreenInfo(TargetScreenInfos[Index]);
		FinalScreenTarget += TargetScreenInfo.ScreenTarget * TargetScreenInfo.NormalizedWeight;
		FinalScreenBounds.Add(TargetScreenInfo.ScreenBounds);
		FinalWorldTargetDistance += TargetScreenInfo.WorldTargetDistance * TargetScreenInfo.NormalizedWeight;
	}

	// Unproject the final screen target, and use the weigted average distance to get, roughly, what
	// world-space target we might be looking at.
	const FMatrix InvViewProjectionMatrix = ViewProjectionMatrix.InverseFast();
	const FVector3d FinalWorldTarget = FCameraPoseMath::UnprojectFromScreen(InvViewProjectionMatrix, FinalScreenTarget, FinalWorldTargetDistance);

	OutScreenTarget = FinalScreenTarget;
	OutScreenBounds = FinalScreenBounds;
	OutWorldTarget = FinalWorldTarget;

	return true;
}

float FBaseFramingCameraNodeEvaluator::ComputeScreenTargetSpeed(float DeltaTime, const FMatrix& ViewProjectionMatrix, const FVector3d& OldWorldTarget, const FVector3d& NewWorldTarget)
{
	if (DeltaTime <= 0.f)
	{
		// Return last frame's speed.
		return State.TargetScreenSpaceSpeed;
	}

	TOptional<FVector2d> OldScreenTarget = FCameraPoseMath::ProjectToScreen(ViewProjectionMatrix, OldWorldTarget, true);
	TOptional<FVector2d> NewScreenTarget = FCameraPoseMath::ProjectToScreen(ViewProjectionMatrix, NewWorldTarget, true);
	if (!OldScreenTarget.IsSet() || !NewScreenTarget.IsSet())
	{
		return State.TargetScreenSpaceSpeed;
	}

	return (FVector2d::Distance(OldScreenTarget.GetValue(), NewScreenTarget.GetValue()) / DeltaTime);
}

FVector2d FBaseFramingCameraNodeEvaluator::ComputeAnticipatedScreenTarget(float DeltaTime, const FVector2d& InPreviousAnticipatedScreenTarget, const FVector2d& InScreenTarget)
{
	if (State.TargetMovementAnticipationTime <= 0)
	{
		return InScreenTarget;
	}
	if (DeltaTime <= 0)
	{
		return InPreviousAnticipatedScreenTarget;
	}

	using FHistoryEntry = TTuple<FVector2d, float>;

	const int32 MaxNumSamples = ScreenTargetHistory.History.Max();
	if (ScreenTargetHistory.History.Num() == MaxNumSamples)
	{
		ScreenTargetHistory.History.RemoveAt(0, EAllowShrinking::No);
	}
	ScreenTargetHistory.History.Add({ InScreenTarget, DeltaTime });

	FVector2d AverageMoveStep(EForceInit::ForceInit);
	for (int32 Index = 1; Index < ScreenTargetHistory.History.Num(); ++Index)
	{
		const FHistoryEntry& Entry(ScreenTargetHistory.History[Index]);
		const FHistoryEntry& PrevEntry(ScreenTargetHistory.History[Index - 1]);
		const FVector2d NormalizedMoveStep = ((Entry.Key - PrevEntry.Key) / Entry.Value);
		AverageMoveStep += NormalizedMoveStep;
	}
	AverageMoveStep /= ScreenTargetHistory.History.Num();

	if (AverageMoveStep.Length() >= GFramingExtrapolationEpsilon)
	{
		const FVector2d ExtrapolatedMovement = AverageMoveStep * State.TargetMovementAnticipationTime;
		const FVector2d ExtrapolatedScreenTarget = InScreenTarget + ExtrapolatedMovement;
		return ExtrapolatedScreenTarget;
	}
	else
	{
		return InScreenTarget;
	}
}

FFramingZone FBaseFramingCameraNodeEvaluator::ComputeEffectiveDeadZone()
{
	const FVector2d& CurTarget = State.ScreenTarget;
	const FFramingZone& CurTargetBounds = State.ScreenTargetBounds;

	FFramingZone RelativeTargetBounds;
	RelativeTargetBounds.LeftBound = FMath::Max(0.0, CurTarget.X - CurTargetBounds.LeftBound);
	RelativeTargetBounds.TopBound = FMath::Max(0.0, CurTarget.Y - CurTargetBounds.TopBound);
	RelativeTargetBounds.RightBound = FMath::Max(0.0, CurTargetBounds.RightBound - CurTarget.X);
	RelativeTargetBounds.BottomBound = FMath::Max(0.0, CurTargetBounds.BottomBound - CurTarget.Y);

	FFramingZone EffectiveDeadZone = State.DeadZone;
	EffectiveDeadZone.LeftBound += RelativeTargetBounds.LeftBound;
	EffectiveDeadZone.TopBound += RelativeTargetBounds.TopBound;
	EffectiveDeadZone.RightBound -= RelativeTargetBounds.RightBound;
	EffectiveDeadZone.BottomBound -= RelativeTargetBounds.BottomBound;

	EffectiveDeadZone.ClampBounds(State.IdealTarget, UE_DOUBLE_KINDA_SMALL_NUMBER);
	
	return EffectiveDeadZone;
}

FFramingZone FBaseFramingCameraNodeEvaluator::ComputeScreenTargetBounds(const FMatrix& ViewProjectionMatrix, const FTransform3d& TargetTransform, const FBoxSphereBounds3d& LocalBounds)
{
	const FVector3d BoxExtent = LocalBounds.BoxExtent;
	FVector3d BoxCorners[8];
	BoxCorners[0] = TargetTransform.TransformPositionNoScale({ BoxExtent.X, BoxExtent.Y, BoxExtent.Z });
	BoxCorners[1] = TargetTransform.TransformPositionNoScale({ -BoxExtent.X, BoxExtent.Y, BoxExtent.Z });
	BoxCorners[2] = TargetTransform.TransformPositionNoScale({ BoxExtent.X, -BoxExtent.Y, BoxExtent.Z });
	BoxCorners[3] = TargetTransform.TransformPositionNoScale({ -BoxExtent.X, -BoxExtent.Y, BoxExtent.Z });
	BoxCorners[4] = TargetTransform.TransformPositionNoScale({ BoxExtent.X, BoxExtent.Y, -BoxExtent.Z });
	BoxCorners[5] = TargetTransform.TransformPositionNoScale({ -BoxExtent.X, BoxExtent.Y, -BoxExtent.Z });
	BoxCorners[6] = TargetTransform.TransformPositionNoScale({ BoxExtent.X, -BoxExtent.Y, -BoxExtent.Z });
	BoxCorners[7] = TargetTransform.TransformPositionNoScale({ -BoxExtent.X, -BoxExtent.Y, -BoxExtent.Z });

	FVector2d ScreenBoxCorners[8];
	for (int32 Index = 0; Index < 8; ++Index)
	{
		TOptional<FVector2d> ScreenCorner = FCameraPoseMath::ProjectToScreen(ViewProjectionMatrix, BoxCorners[Index], true);
		ScreenBoxCorners[Index] = ScreenCorner.GetValue();
	}

	return FFramingZone::FromPoints(ScreenBoxCorners);
}

void FBaseFramingCameraNodeEvaluator::ComputeDesiredState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult)
{
	// If we don't have any reframing to do, bail out.
	const double ReframingSpeed = FMath::Abs(State.ReframeDamper.GetX0Derivative());
	if (!State.bIsReframingTarget && (ReframingSpeed <= GFramingIdealReachedEpsilon || State.ToggleEngageAlpha <= 0))
	{
		Desired.ScreenTarget = State.ScreenTarget;
		Desired.FramingCorrection = FVector2d::ZeroVector;
		Desired.bHasCorrection = false;
		return;
	}

	// We may need to jump directly to the ideal framing without interpolating.
	bool bJumpToIdealFraming = false;
	if (Params.bIsFirstFrame && Readers.InitializeWithIdealFraming.Get(OutResult.VariableTable))
	{
		Desired.ScreenTarget = State.IdealTarget;
		Desired.FramingCorrection = Desired.ScreenTarget - State.ScreenTarget;
		Desired.bHasCorrection = true;
		return;
	}

	FVector2d IdealToTarget(State.ScreenTarget - State.IdealTarget);
	double DistanceToGo = IdealToTarget.Length();
	if (State.TargetFramingState == ETargetFramingState::InHardZone)
	{
		// Bring the target immediately to edge of the soft zone, in the direction of the 
		// ideal position. From there, follow-up with applying the soft zone effect.
		Desired.ScreenTarget = GetHardReframeCoords();

		IdealToTarget = (Desired.ScreenTarget - State.IdealTarget);
		DistanceToGo = IdealToTarget.Length();
	}

	// Figure out the damping factor for this frame. We might have interpolation between
	// the low and normal (high) damping factors if that option is enabled.
	float EffectiveDampingFactor = State.ReframeDampingFactor;
	if (State.LowReframeDampingFactor > 0)
	{
		EffectiveDampingFactor = FMath::Lerp(
				State.LowReframeDampingFactor, State.ReframeDampingFactor, State.ReframeDampingFactorAlpha);
	}
	EffectiveDampingFactor = FMath::Max(GFramingMinDampingFactor, EffectiveDampingFactor);
	State.ReframeDamper.SetW0(EffectiveDampingFactor);

	// Move the target towards the ideal framing using damping.
	double NewDistanceToGo = State.ReframeDamper.Update(DistanceToGo, Params.DeltaTime);

	// Slowly shorten the distance left to go if we're trying to disengage, or just starting
	// to re-engage.
	NewDistanceToGo = FMath::Lerp(DistanceToGo, NewDistanceToGo, SmoothStep(State.ToggleEngageAlpha));

	// Compute where we want the target this frame.
	if (DistanceToGo != 0.f)
	{
		const FVector2d InvReframeDir(IdealToTarget / DistanceToGo);
		const FVector2d NewScreenTarget = State.IdealTarget + InvReframeDir * NewDistanceToGo;
		Desired.ScreenTarget = NewScreenTarget;
	}
	else
	{
		Desired.ScreenTarget = State.IdealTarget;
	}

	Desired.FramingCorrection = Desired.ScreenTarget - State.ScreenTarget;
	Desired.bHasCorrection = true;
}

FVector2d FBaseFramingCameraNodeEvaluator::GetHardReframeCoords() const
{
	// The target is in the hard zone and must be brought back to the edge of the soft zone.
	// Let's compute the diagonal between the target and the ideal framing point, and bring
	// the target where that diagonal intersects the soft zone.
	const FVector2d Diagonal(State.IdealTarget - State.ScreenTarget);
	if (Diagonal.IsZero())
	{
		// Somehow we're already on the desired framing. This shouldn't happen, we're supposed
		// to be in the hard zone right now...
		ensure(false);
		return State.ScreenTarget;
	}

	return State.SoftZone.ComputeClosestIntersection(State.ScreenTarget, Diagonal, false);
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FBaseFramingCameraNodeEvaluator::DrawFramingState(const FState& State, const FDesired& Desired, FCameraDebugRenderer& Renderer)
{
	if (!Renderer.IsExternalRendering() && Renderer.HasCanvas())
	{
		const FVector2D CanvasSize = Renderer.GetCanvasSize();

		Renderer.Draw2DBox(
				State.SoftZone.GetCanvasPosition(CanvasSize), 
				State.SoftZone.GetCanvasSize(CanvasSize),
				FColorList::Red,
				1.f);
		Renderer.Draw2DBox(
				State.DeadZone.GetCanvasPosition(CanvasSize), 
				State.DeadZone.GetCanvasSize(CanvasSize),
				FColorList::Green,
				1.f);

		Renderer.Draw2DBox(
				State.ScreenTargetBounds.GetCanvasPosition(CanvasSize),
				State.ScreenTargetBounds.GetCanvasSize(CanvasSize),
				FColorList::LightGrey,
				1.f);

		if (State.DebugAllScreenTargetBounds.Num() > 1)
		{
			for (const FFramingZone& SingleTargetBounds : State.DebugAllScreenTargetBounds)
			{
				Renderer.Draw2DBox(
						SingleTargetBounds.GetCanvasPosition(CanvasSize),
						SingleTargetBounds.GetCanvasSize(CanvasSize),
						FColorList::LightGrey,
						1.f);
			}
		}

		if (GFramingShowEffectiveDeadZone)
		{
			Renderer.Draw2DBox(
					State.EffectiveDeadZone.GetCanvasPosition(CanvasSize), 
					State.EffectiveDeadZone.GetCanvasSize(CanvasSize),
					FColorList::LightGrey,
					1.f);
		}

		if (State.LowReframeDampingFactor > 0)
		{
			Renderer.Draw2DLine(State.DebugDeadZoneEdgePoint, State.DebugHardZoneEdgePoint, FLinearColor(FColorList::LightGrey), 1.f);
		}

		const FVector2D ScreenTarget(State.ScreenTarget.X * CanvasSize.X, State.ScreenTarget.Y * CanvasSize.Y);
		const FVector2D NextScreenTarget(Desired.ScreenTarget.X * CanvasSize.X, Desired.ScreenTarget.Y * CanvasSize.Y);
		const FVector2D IdealTarget(State.IdealTarget.X * CanvasSize.X, State.IdealTarget.Y * CanvasSize.Y);

		Renderer.Draw2DLine(ScreenTarget, NextScreenTarget, FLinearColor(FColorList::Salmon), 1.f);
		Renderer.Draw2DCircle(ScreenTarget, 2.f, FLinearColor(FColorList::Orange), 2.f);
		Renderer.Draw2DCircle(IdealTarget, 2.f, FLinearColor::Green, 2.f);
	}
}

#endif

#if WITH_EDITOR

void FBaseFramingCameraNodeEvaluator::OnDrawEditorPreview(const FCameraEditorPreviewDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	DrawFramingState(State, Desired, Renderer);
}

#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FBaseFramingCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FBaseFramingCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FBaseFramingCameraDebugBlock>();

	DebugBlock.State = State;
	DebugBlock.Desired = Desired;
	DebugBlock.WorldTargets = WorldTargets;
}

void FBaseFramingCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	FString StateString(TEXT("Unknown"));
	switch (State.TargetFramingState)
	{
		case FBaseFramingCameraNodeEvaluator::ETargetFramingState::InDeadZone:
			StateString = TEXT("Dead Zone");
			break;
		case FBaseFramingCameraNodeEvaluator::ETargetFramingState::InSoftZone:
			StateString = TEXT("Soft Zone");
			break;
		case FBaseFramingCameraNodeEvaluator::ETargetFramingState::InHardZone:
			StateString = TEXT("Hard Zone");
			break;
	}

	Renderer.AddText(TEXT("state [%s]"), *StateString);
	if (State.bIsReframingTarget)
	{
		Renderer.AddText(TEXT("[REFRAMING]"));
	}
	Renderer.NewLine();

	Renderer.AddIndent();
	{
		if (Renderer.HasCanvas())
		{
			const FVector2D CanvasSize = Renderer.GetCanvasSize();

			const FVector2D FramingCorrection(Desired.FramingCorrection.X * CanvasSize.X, Desired.FramingCorrection.Y * CanvasSize.Y);
			Renderer.AddText(TEXT("correction (%0.1f ; %0.1f) "),
					State.ReframeDamper.GetX0(),
					FramingCorrection.X, FramingCorrection.Y);

			const FVector2D ScreenTarget(State.ScreenTarget.X * CanvasSize.X, State.ScreenTarget.Y * CanvasSize.Y);
			Renderer.AddText(TEXT("target (%0.1f; %0.1f) "), ScreenTarget.X, ScreenTarget.Y);
		}

		Renderer.AddText(TEXT("damping %0.3f (factor = %0.1f)\n"), State.ReframeDamper.GetX0(), State.ReframeDamper.GetW0());

		Renderer.AddText(TEXT("reengage/disengage time left %.3f (damping factor alpha = %.1f)\n"),
				State.ToggleEngageTimeLeft,
				State.ToggleEngageAlpha);

		Renderer.AddText(TEXT("interpolation = %.1f\n"), State.ReframeDampingFactorAlpha);
	}
	Renderer.RemoveIndent();

	FBaseFramingCameraNodeEvaluator::DrawFramingState(State, Desired, Renderer);

	for (const FCameraActorComputedTargetInfo& TargetInfo : WorldTargets.TargetInfos)
	{
		Renderer.DrawBox(TargetInfo.Transform, TargetInfo.LocalBounds.BoxExtent, FColorList::LightSteelBlue, 0.1f);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

void FBaseFramingCameraNodeEvaluator::FState::Serialize(FArchive& Ar)
{
	Ar << IdealTarget;
	Ar << ReframeDampingFactor;
	Ar << LowReframeDampingFactor;
	Ar << ReframeDampingFactorAlpha;
	Ar << ReengageTime;
	Ar << DisengageTime;
	Ar << ToggleEngageTimeLeft;
	Ar << ToggleEngageAlpha;
	Ar << TargetMovementAnticipationTime;
	Ar << DeadZone;
	Ar << SoftZone;

	Ar << WorldTarget;
	Ar << ScreenTarget;
	Ar << ScreenTargetBounds;
	Ar << EffectiveDeadZone;

	Ar << TargetFramingState;
	Ar << TargetScreenSpaceSpeed;
	Ar << bIsReframingTarget;

	Ar << ReframeDamper;
}

void FBaseFramingCameraNodeEvaluator::FDesired::Serialize(FArchive& Ar)
{
	Ar << ScreenTarget;
	Ar << FramingCorrection;
	Ar << bHasCorrection;
}

void FBaseFramingCameraNodeEvaluator::FWorldTargetInfos::Serialize(FArchive& Ar)
{
	Ar << TargetInfos;
}

FArchive& operator <<(FArchive& Ar, FBaseFramingCameraNodeEvaluator::FState& State)
{
	State.Serialize(Ar);
	return Ar;
}

FArchive& operator <<(FArchive& Ar, FBaseFramingCameraNodeEvaluator::FDesired& Desired)
{
	Desired.Serialize(Ar);
	return Ar;
}

FArchive& operator <<(FArchive& Ar, FBaseFramingCameraNodeEvaluator::FWorldTargetInfos& WorldTargets)
{
	WorldTargets.Serialize(Ar);
	return Ar;
}

}  // namespace UE::Cameras

UBaseFramingCameraNode::UBaseFramingCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
	, DeadZone(0.2)
	, SoftZone(0.05)
{
	SetTargetDistance.Value = true;
	IdealFramingLocation.Value = FVector2d(0.5, 0.5);
	ReframeDampingFactor.Value = 10.f;
	LowReframeDampingFactor.Value = -1.f;
	ReengageTime.Value = 1.f;
	DisengageTime.Value = 1.f;
}

void UBaseFramingCameraNode::PostLoad()
{
	Super::PostLoad();

	if (TargetInfos.IsEmpty() && TargetInfo_DEPRECATED.IsValid())
	{
		TargetInfos.Add(TargetInfo_DEPRECATED);
		TargetInfo_DEPRECATED = FCameraActorTargetInfo();
	}
}

void UBaseFramingCameraNode::GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos)
{
	OutParameterInfos.AddBlendableParameter(
			GET_MEMBER_NAME_CHECKED(UBaseFramingCameraNode, DeadZone),
			ECameraVariableType::BlendableStruct,
			FCameraFramingZone::StaticStruct(),
			reinterpret_cast<const uint8*>(&DeadZone.Value),
			&DeadZone.VariableID);

	OutParameterInfos.AddBlendableParameter(
			GET_MEMBER_NAME_CHECKED(UBaseFramingCameraNode, SoftZone),
			ECameraVariableType::BlendableStruct,
			FCameraFramingZone::StaticStruct(),
			reinterpret_cast<const uint8*>(&SoftZone.Value),
			&SoftZone.VariableID);
}

