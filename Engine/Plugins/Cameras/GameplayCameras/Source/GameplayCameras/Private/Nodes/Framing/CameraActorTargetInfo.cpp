// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Framing/CameraActorTargetInfo.h"

#include "Algo/Accumulate.h"
#include "Components/SkeletalMeshComponent.h"
#include "Core/CameraContextDataTable.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraActorTargetInfo)

namespace UE::Cameras
{

FCameraActorTargetInfoReader::FCameraActorTargetInfoReader(const FCameraActorTargetInfo& InTargetInfo, FCameraContextDataID InDataID)
{
	Initialize(InTargetInfo, InDataID);
}

void FCameraActorTargetInfoReader::Initialize(const FCameraActorTargetInfo& InTargetInfo, FCameraContextDataID InDataID)
{
	DefaultTargetInfo = InTargetInfo;
	DataID = InDataID;

	CacheTargetInfo(DefaultTargetInfo);
}

void FCameraActorTargetInfoReader::CacheTargetInfo(const FCameraActorTargetInfo& InTargetInfo)
{
	if (CachedTargetInfo != InTargetInfo)
	{
		CachedTargetInfo = InTargetInfo;

		CachedSkeletalMeshComponent = nullptr;
		AActor* TargetActor = InTargetInfo.Actor.Get();
		if (TargetActor && (!InTargetInfo.SocketName.IsNone() || !InTargetInfo.BoneName.IsNone()))
		{
			CachedSkeletalMeshComponent = TargetActor->FindComponentByClass<USkeletalMeshComponent>();
		}

		CachedBoneName = NAME_None;
		CachedParentBoneName = NAME_None;
		if (CachedSkeletalMeshComponent)
		{
			CachedBoneName = InTargetInfo.BoneName;
			if (!InTargetInfo.SocketName.IsNone())
			{
				CachedBoneName = CachedSkeletalMeshComponent->GetSocketBoneName(InTargetInfo.SocketName);
			}
			if (!CachedBoneName.IsNone())
			{
				CachedParentBoneName = CachedSkeletalMeshComponent->GetParentBone(CachedBoneName);
			}
		}
	}
}

bool FCameraActorTargetInfoReader::GetTargetInfo(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform, FBoxSphereBounds3d& OutBounds)
{
	if (DataID.IsValid())
	{
		if (const FCameraActorTargetInfo* NewTargetInfo = ContextDataTable.TryGetData<FCameraActorTargetInfo>(DataID))
		{
			CacheTargetInfo(*NewTargetInfo);
		}
		else
		{
			CacheTargetInfo(DefaultTargetInfo);
		}
	}

	if (CachedSkeletalMeshComponent && !CachedBoneName.IsNone())
	{
		OutTransform = CachedSkeletalMeshComponent->GetBoneTransform(CachedBoneName);
		ComputeTargetBounds(OutTransform.GetLocation(), OutBounds);
		return true;
	}
	else if (AActor* TargetActor = CachedTargetInfo.Actor.Get())
	{
		OutTransform = TargetActor->GetTransform();
		ComputeTargetBounds(OutTransform.GetLocation(), OutBounds);
		return true;
	}
	return false;
}

void FCameraActorTargetInfoReader::ComputeTargetBounds(const FVector3d& TargetLocation, FBoxSphereBounds3d& OutBounds)
{
	switch (CachedTargetInfo.TargetShape)
	{
		case ECameraTargetShape::Point:
			OutBounds = FBoxSphereBounds3d(EForceInit::ForceInit);
			break;
		case ECameraTargetShape::AutomaticBounds:
			if (CachedSkeletalMeshComponent && !CachedParentBoneName.IsNone())
			{
				const FVector3d ParentBoneLocation = CachedSkeletalMeshComponent->GetBoneLocation(CachedParentBoneName);
				const FVector3d ParentToBone((TargetLocation - ParentBoneLocation).GetAbs());
				const float ParentToBoneLength(ParentToBone.Length());
				OutBounds = FBoxSphereBounds3d(FVector3d::ZeroVector, ParentToBone, ParentToBoneLength);
			}
			else
			{
				OutBounds = FBoxSphereBounds3d(EForceInit::ForceInit);
				if (AActor* TargetActor = CachedTargetInfo.Actor.Get())
				{
					if (USceneComponent* RootComponent = TargetActor->GetRootComponent())
					{
						OutBounds = RootComponent->Bounds;
					}
				}
			}
			break;
		case ECameraTargetShape::ManualBounds:
			{
				float TargetSize = FMath::Max(0.f, CachedTargetInfo.TargetSize);
				OutBounds = FBoxSphereBounds3d(FVector3d::ZeroVector, FVector3d(TargetSize), TargetSize);
			}
			break;
	}
}

FCameraActorTargetInfoArrayReader::FCameraActorTargetInfoArrayReader(TConstArrayView<FCameraActorTargetInfo> InTargetInfos, FCameraContextDataID InDataID)
{
	Initialize(InTargetInfos, InDataID);
}

void FCameraActorTargetInfoArrayReader::Initialize(TConstArrayView<FCameraActorTargetInfo> InTargetInfos, FCameraContextDataID InDataID)
{
	DataID = InDataID;

	CacheTargetInfos(InTargetInfos);
}

void FCameraActorTargetInfoArrayReader::CacheTargetInfos(TConstArrayView<FCameraActorTargetInfo> InTargetInfos)
{
	Readers.SetNum(InTargetInfos.Num());

	for (int32 Index = 0; Index < InTargetInfos.Num(); ++Index)
	{
		Readers[Index].CacheTargetInfo(InTargetInfos[Index]);
	}
}

bool FCameraActorTargetInfoArrayReader::ComputeTargetInfos(const FCameraContextDataTable& ContextDataTable, TArray<FCameraActorComputedTargetInfo>& ComputedTargets)
{
	if (DataID.IsValid())
	{
		TConstArrayView<FCameraActorTargetInfo> NewTargetInfos;
		if (ContextDataTable.TryGetArrayData<FCameraActorTargetInfo>(DataID, NewTargetInfos))
		{
			CacheTargetInfos(NewTargetInfos);
		}
	}

	if (Readers.IsEmpty())
	{
		return false;
	}

	ComputedTargets.SetNum(Readers.Num());

	for (int32 Index = 0; Index < Readers.Num(); ++Index)
	{
		FCameraActorTargetInfoReader& Reader(Readers[Index]);
		FCameraActorComputedTargetInfo& ComputedTarget(ComputedTargets[Index]);
		Reader.GetTargetInfo(ContextDataTable, ComputedTarget.Transform, ComputedTarget.LocalBounds);
		ComputedTarget.NormalizedWeight = Reader.CachedTargetInfo.Weight;
	}

	float TotalWeight = Algo::Accumulate(
			ComputedTargets, 
			0.f,
			[](float Cur, const FCameraActorComputedTargetInfo& Item) { return Cur + Item.NormalizedWeight; });
	if (TotalWeight == 0.f)
	{
		return false;
	}

	for (int32 Index = 0; Index < Readers.Num(); ++Index)
	{
		FCameraActorComputedTargetInfo& ComputedTarget(ComputedTargets[Index]);
		ComputedTarget.NormalizedWeight /= TotalWeight;
	}

	return true;
}

#if WITH_EDITOR

void FCameraActorTargetInfoArrayReader::Refresh(TConstArrayView<FCameraActorTargetInfo> InTargetInfos)
{
	CacheTargetInfos(InTargetInfos);
}

#endif  // WITH_EDITOR

FArchive& operator <<(FArchive& Ar, FCameraActorComputedTargetInfo& TargetInfo)
{
	Ar << TargetInfo.Transform;
	Ar << TargetInfo.LocalBounds;
	Ar << TargetInfo.NormalizedWeight;
	return Ar;
}

}  // namespace UE::Cameras

