// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Framing/CameraFramingZone.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraFramingZone)

void FCameraFramingZone::TypeErasedInterpolate(uint8* From, const uint8* To, float Alpha)
{
	FCameraFramingZone& FromZone = *reinterpret_cast<FCameraFramingZone*>(From);
	const FCameraFramingZone& ToZone = *reinterpret_cast<const FCameraFramingZone*>(To);

	FromZone.Left = FMath::Lerp(FromZone.Left, ToZone.Left, Alpha);
	FromZone.Top = FMath::Lerp(FromZone.Top, ToZone.Top, Alpha);
	FromZone.Right = FMath::Lerp(FromZone.Right, ToZone.Right, Alpha);
	FromZone.Bottom = FMath::Lerp(FromZone.Bottom, ToZone.Bottom, Alpha);
}

FString FCameraFramingZone::ToString() const
{
	return FString::Printf(TEXT("<< %f ^^ %f >> %f vv %f"), Left, Top, Right, Bottom);
}

