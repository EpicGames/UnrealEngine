// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Framing/DollyFramingCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameplayCameras.h"
#include "Math/CameraAimingMath.h"
#include "Math/CameraPoseMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DollyFramingCameraNode)

namespace UE::Cameras
{

class FDollyFramingCameraNodeEvaluator : public FBaseFramingCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FDollyFramingCameraNodeEvaluator, FBaseFramingCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	FTransform3d BuildDollyShotTransform(const FCameraPose& CameraPose) const;
	FVector3d ComputeFramingTranslation(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

private:

	TCameraParameterReader<bool> CanMoveLaterallyReader;
	TCameraParameterReader<bool> CanMoveVerticallyReader;

	FVector2d DollyPosition;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector3d DebugNextDesiredTarget;
	FVector2d DebugDollyCorrection;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FDollyFramingCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FDollyFramingCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, DollyPosition);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, DollyCorrection);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, WorldTarget);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector3d, NextWorldTarget);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDollyFramingCameraDebugBlock)

void FDollyFramingCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	const UDollyFramingCameraNode* DollyNode = GetCameraNodeAs<UDollyFramingCameraNode>();
	CanMoveLaterallyReader.Initialize(DollyNode->CanMoveLaterally);
	CanMoveVerticallyReader.Initialize(DollyNode->CanMoveVertically);

	DollyPosition = FVector2d::ZeroVector;
}

void FDollyFramingCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// If this is the first frame, we may want to help frame the targets dead-on.
	if (Params.bIsFirstFrame)
	{
		TOptional<FVector3d> ApproximatedWorldTarget = GetInitialDesiredWorldTarget(Params, OutResult);
		if (ApproximatedWorldTarget.IsSet())
		{
			const FTransform3d InitialPose = OutResult.CameraPose.GetTransform();
			const FVector3d InitialLocalTarget = InitialPose.InverseTransformPositionNoScale(ApproximatedWorldTarget.GetValue());
			DollyPosition.X = InitialLocalTarget.Y;
			DollyPosition.Y = InitialLocalTarget.Z;
		}
	}

	// Let the base class figure out all the screen-space framing stuff.
	const FTransform3d LastShotTransform = BuildDollyShotTransform(OutResult.CameraPose);
	UpdateFramingState(Params, OutResult, LastShotTransform);

	// If we need to reframe the target this tick, figure out how much we need to move the dolly
	// to accomplish that.
	if (Desired.bHasCorrection)
	{
		FCameraPose LastShotPose(OutResult.CameraPose);
		LastShotPose.SetTransform(LastShotTransform);

		FVector3d DesiredLocalOffset = ComputeFramingTranslation(LastShotPose, Params.EvaluationContext);

		// We never bring the dolly forward or backward (we only move it vertically or horizontally).
		DesiredLocalOffset.X = 0.f;

		if (!CanMoveLaterallyReader.Get(OutResult.VariableTable))
		{
			DesiredLocalOffset.Y = 0.f;
		}
		if (!CanMoveVerticallyReader.Get(OutResult.VariableTable))
		{
			DesiredLocalOffset.Z = 0.f;
		}

		const FVector2d DollyCorrection = FVector2d(DesiredLocalOffset.Y, DesiredLocalOffset.Z);
		DollyPosition += DollyCorrection;

#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugDollyCorrection = DollyCorrection;
#endif  //UE_GAMEPLAY_CAMERAS_DEBUG
	}
	else
	{
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugDollyCorrection = FVector2d::ZeroVector;
#endif  //UE_GAMEPLAY_CAMERAS_DEBUG
	}

	const FTransform3d NewShotTransform = BuildDollyShotTransform(OutResult.CameraPose);
	OutResult.CameraPose.SetTransform(NewShotTransform);

	EndFramingUpdate(Params, OutResult);
}

FTransform3d FDollyFramingCameraNodeEvaluator::BuildDollyShotTransform(const FCameraPose& CameraPose) const
{
	FTransform3d Transform = CameraPose.GetTransform();
	Transform = FTransform3d(FVector3d(0, DollyPosition.X, DollyPosition.Y)) * Transform;
	return Transform;
}

FVector3d FDollyFramingCameraNodeEvaluator::ComputeFramingTranslation(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	// Get the position of the current target in camera space.
	const FTransform3d InverseCameraTransform = CameraPose.GetTransform().Inverse();
	const FVector3d TargetInCameraSpace = InverseCameraTransform.TransformPosition(State.WorldTarget);

	// Get the direction, relative to the aiming vector, for the desired target position.
	const FMatrix LocalViewProjectionMatrix = FCameraPoseMath::BuildLocalViewProjectionMatrix(CameraPose, EvaluationContext);
	const FMatrix InvLocalViewProjectionMatrix = LocalViewProjectionMatrix.InverseFast();
	const FRay3d DesiredAim = FCameraPoseMath::UnprojectFromScreen(InvLocalViewProjectionMatrix, Desired.ScreenTarget);

	// We want the distance to the desired target to be so that it sits on the same Y/Z plane as the current target.
	// That is, we want to only move laterally and vertically. This means that:
	//
	//     DesiredAim.Direction . LocalUnitX = TargetInCameraSpace.X
	// 
	// Since DesiredAim's direction vector is a unit vector, we can therefore do:
	//
	const double CosAngle = FVector3d::DotProduct(DesiredAim.Direction, FVector3d::UnitX());
	ensure(CosAngle != 0.0);
	const double DesiredAimLength = (CosAngle != 0) ? TargetInCameraSpace.X / CosAngle : TargetInCameraSpace.X;
	const FVector3d DesiredInCameraSpace = DesiredAim.Direction * DesiredAimLength;

	// Now we can figure out the desired camera-space offset that the dolly needs to move by. Remember
	// that, for instance, moving the camera to the *right* will result in the target moving to the
	// *left* on screen, so that's why we move by Desired->Current, and not the other way around.
	FVector3d DesiredLocalOffset = (TargetInCameraSpace - DesiredInCameraSpace);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugNextDesiredTarget = CameraPose.GetTransform().TransformPosition(DesiredInCameraSpace);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	return DesiredLocalOffset;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FDollyFramingCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	Super::OnBuildDebugBlocks(Params, Builder);

	FDollyFramingCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FDollyFramingCameraDebugBlock>();
	DebugBlock.DollyPosition = DollyPosition;
	DebugBlock.DollyCorrection = DebugDollyCorrection;
	DebugBlock.WorldTarget = State.WorldTarget;
	DebugBlock.NextWorldTarget = DebugNextDesiredTarget;
}

void FDollyFramingCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(
			TEXT("dolly position (%0.1f ; %0.1f)  correction (%0.1f ; %0.1f)"), 
			DollyPosition.X, DollyPosition.Y,
			DollyCorrection.X, DollyCorrection.Y);

	Renderer.DrawLine(WorldTarget, NextWorldTarget, FLinearColor(FColorList::LightGrey), 1.f);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UDollyFramingCameraNode::UDollyFramingCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CanMoveLaterally.Value = true;
	CanMoveVertically.Value = true;
}

FCameraNodeEvaluatorPtr UDollyFramingCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FDollyFramingCameraNodeEvaluator>();
}

