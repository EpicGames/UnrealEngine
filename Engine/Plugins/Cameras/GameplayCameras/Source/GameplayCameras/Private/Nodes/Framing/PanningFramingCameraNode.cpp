// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Framing/PanningFramingCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraVariableAssets.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameplayCameras.h"
#include "Math/CameraAimingMath.h"
#include "Math/CameraPoseMath.h"
#include "Math/InverseRotationMatrix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PanningFramingCameraNode)

namespace UE::Cameras
{

class FPanningFramingCameraNodeEvaluator : public FBaseFramingCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FPanningFramingCameraNodeEvaluator, FBaseFramingCameraNodeEvaluator)

protected:

	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	FTransform3d BuildPanningShotTransform(const FCameraPose& CameraPose) const;

private:

	TCameraParameterReader<bool> CanPanLaterallyReader;
	TCameraParameterReader<bool> CanPanVerticallyReader;

	FRotator3d PanningRotation;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FRotator3d DebugPanningCorrection;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FPanningFramingCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FPanningFramingCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, PanningRotation);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, PanningCorrection);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FPanningFramingCameraDebugBlock)

void FPanningFramingCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	Super::OnInitialize(Params, OutResult);

	const UPanningFramingCameraNode* DollyNode = GetCameraNodeAs<UPanningFramingCameraNode>();
	CanPanLaterallyReader.Initialize(DollyNode->CanPanLaterally);
	CanPanVerticallyReader.Initialize(DollyNode->CanPanVertically);

	PanningRotation = FRotator3d::ZeroRotator;
}

void FPanningFramingCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// If this is the first frame, we may want to help frame the targets dead-on.
	if (Params.bIsFirstFrame)
	{
		TOptional<FVector3d> ApproximatedWorldTarget = GetInitialDesiredWorldTarget(Params, OutResult);
		if (ApproximatedWorldTarget.IsSet())
		{
			const FVector3d InitialDesiredAim = ApproximatedWorldTarget.GetValue() - OutResult.CameraPose.GetLocation();
			if (!InitialDesiredAim.IsNearlyZero())
			{
				const FVector3d InitialPoseAim = OutResult.CameraPose.GetAimDir();
				const FRotator3d NewPanningRotation = InitialDesiredAim.Rotation() - InitialPoseAim.Rotation();
				if (ensure(!NewPanningRotation.ContainsNaN()))
				{
					PanningRotation = NewPanningRotation;
				}
			}
		}
	}

	// Let the base class figure out all the screen-space framing stuff.
	const FTransform3d LastShotTransform = BuildPanningShotTransform(OutResult.CameraPose);
	UpdateFramingState(Params, OutResult, LastShotTransform);

	// If we need to reframe the target this tick, figure out how much we need to rotate the camera
	// to accomplish that.
	if (Desired.bHasCorrection)
	{
		FCameraPose LastShotPose(OutResult.CameraPose);
		LastShotPose.SetTransform(LastShotTransform);
		const FMatrix LocalViewProjectionMatrix = FCameraPoseMath::BuildLocalViewProjectionMatrix(LastShotPose, Params.EvaluationContext);
		const FMatrix InvLocalViewProjectionMatrix = LocalViewProjectionMatrix.InverseFast();

		// Get the direction for where the target is currently at, and where we want it to be at.
		const FRay3d TargetAim = FCameraPoseMath::UnprojectFromScreen(InvLocalViewProjectionMatrix, State.ScreenTarget);
		const FRay3d DesiredAim = FCameraPoseMath::UnprojectFromScreen(InvLocalViewProjectionMatrix, Desired.ScreenTarget);

		// Compute the yaw/pitch correction needed. Remember that if we want the target to move left on the screen, we
		// need to turn to the right. This is why we compute the Desired->Current angle, and not the other way around.
		const FRotator3d PanningCorrection = FCameraAimingMath::GetNoRollRotationBetween(DesiredAim.Direction, TargetAim.Direction);
		if (ensure(!PanningCorrection.ContainsNaN() && PanningCorrection.Roll == 0.0))
		{
			PanningRotation += PanningCorrection;
		}

#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugPanningCorrection = PanningCorrection;
#endif  //UE_GAMEPLAY_CAMERAS_DEBUG
	}
	else
	{
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugPanningCorrection = FRotator3d::ZeroRotator;
#endif  //UE_GAMEPLAY_CAMERAS_DEBUG
	}

	const FTransform3d NewShotTransform = BuildPanningShotTransform(OutResult.CameraPose);
	OutResult.CameraPose.SetTransform(NewShotTransform);

	EndFramingUpdate(Params, OutResult);
}

FTransform3d FPanningFramingCameraNodeEvaluator::BuildPanningShotTransform(const FCameraPose& CameraPose) const
{
	FTransform3d Transform = CameraPose.GetTransform();
	Transform = FTransform3d(PanningRotation) * Transform;
	return Transform;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FPanningFramingCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	Super::OnBuildDebugBlocks(Params, Builder);

	FPanningFramingCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FPanningFramingCameraDebugBlock>();
	DebugBlock.PanningRotation = PanningRotation;
	DebugBlock.PanningCorrection = DebugPanningCorrection;
}

void FPanningFramingCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(
			TEXT("pan yaw/pitch (%0.1f ; %0.1f)  correction (%0.1f ; %0.1f)"), 
			PanningRotation.Yaw, PanningRotation.Pitch,
			PanningCorrection.Yaw, PanningCorrection.Pitch);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UPanningFramingCameraNode::UPanningFramingCameraNode(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	CanPanLaterally.Value = true;
	CanPanVertically.Value = true;
}

FCameraNodeEvaluatorPtr UPanningFramingCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPanningFramingCameraNodeEvaluator>();
}

