// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/AutoRotateInput2DCameraNode.h"

#include "Build/CameraObjectBuildContext.h"
#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraOperation.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraValueInterpolator.h"
#include "Core/CameraVariableReferenceReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameFramework/PlayerController.h"
#include "HAL/IConsoleManager.h"
#include "ValueInterpolators/CriticalDamperValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutoRotateInput2DCameraNode)

namespace UE::Cameras
{

float GGameplayCamerasAutoRotateSnapThreshold = 0.5f;
static FAutoConsoleVariableRef CVarGameplayCamerasAutoRotateSnapThreshold(
	TEXT("GameplayCameras.AutoRotate.SnapThreshold"),
	GGameplayCamerasAutoRotateSnapThreshold,
	TEXT(""));

class FAutoRotateInput2DCameraNodeEvaluator : public FInput2DCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FAutoRotateInput2DCameraNodeEvaluator, FInput2DCameraNodeEvaluator)

public:

	FAutoRotateInput2DCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void DeactivateAutoRotate(FCameraNodeEvaluationResult& OutResult);
	APlayerController* GetPlayerController(TSharedPtr<const FCameraEvaluationContext> EvaluationContext) const;

private:

	TCameraVariableReferenceReader<FVector3d> DirectionVectorReader;
	TCameraParameterReader<float> WaitTimeReader;
	TCameraParameterReader<float> DeactivationThresholdReader;
	TCameraParameterReader<bool> FreezeControlRotationReader;
	TCameraParameterReader<bool> EnableAutoRotateReader;
	TCameraParameterReader<bool> AutoRotateYawReader;
	TCameraParameterReader<bool> AutoRotatePitchReader;

	TUniquePtr<FCameraDoubleValueInterpolator> Interpolator;

	FInput2DCameraNodeEvaluator* InputNodeEvaluator = nullptr;

	FVector2d LastInputValue;

	FVector3d LastContextLocation;
	FVector2d OriginalInputValue;
	double RemainingWaitTime = 0.f;
	bool bIsAutoRotating = false;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FAutoRotateInput2DCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FAutoRotateInput2DCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, RemainingWaitTime);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, InterpolationFactor);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(double, InterpolationTarget);
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsAutoRotating);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FAutoRotateInput2DCameraDebugBlock)

FAutoRotateInput2DCameraNodeEvaluator::FAutoRotateInput2DCameraNodeEvaluator()
{
}

void FAutoRotateInput2DCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UAutoRotateInput2DCameraNode* AutoRotateNode = GetCameraNodeAs<UAutoRotateInput2DCameraNode>();
	if (AutoRotateNode->InputNode)
	{
		InputNodeEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(AutoRotateNode->InputNode);
	}
}

FCameraNodeEvaluatorChildrenView FAutoRotateInput2DCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ InputNodeEvaluator });
}

void FAutoRotateInput2DCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
	LastContextLocation = InitialResult.CameraPose.GetLocation();

	LastInputValue = FVector2d::ZeroVector;
	if (InputNodeEvaluator)
	{
		LastInputValue = InputNodeEvaluator->GetInputValue();
	}
	else if (APlayerController* PlayerController = GetPlayerController(Params.EvaluationContext))
	{
		const FRotator3d ControlRotation = PlayerController->GetControlRotation();
		LastInputValue = FVector2d(ControlRotation.Yaw, ControlRotation.Pitch);
	}

	const UAutoRotateInput2DCameraNode* AutoRotateNode = GetCameraNodeAs<UAutoRotateInput2DCameraNode>();

	DirectionVectorReader.Initialize(AutoRotateNode->DirectionVector);
	WaitTimeReader.Initialize(AutoRotateNode->WaitTime);
	DeactivationThresholdReader.Initialize(AutoRotateNode->DeactivationThreshold);
	FreezeControlRotationReader.Initialize(AutoRotateNode->FreezeControlRotation);
	EnableAutoRotateReader.Initialize(AutoRotateNode->EnableAutoRotate);
	AutoRotateYawReader.Initialize(AutoRotateNode->AutoRotateYaw);
	AutoRotatePitchReader.Initialize(AutoRotateNode->AutoRotatePitch);

	RemainingWaitTime = WaitTimeReader.Get(OutResult.VariableTable);
	bIsAutoRotating = false;
}

void FAutoRotateInput2DCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UAutoRotateInput2DCameraNode* AutoRotateNode = GetCameraNodeAs<UAutoRotateInput2DCameraNode>();

	// Evaluate our inner input node and grab the new input value.
	if (InputNodeEvaluator)
	{
		InputNodeEvaluator->Run(Params, OutResult);

		InputValue = InputNodeEvaluator->GetInputValue();
	}
	else if (APlayerController* PlayerController = GetPlayerController(Params.EvaluationContext))
	{
		const FRotator3d ControlRotation = PlayerController->GetControlRotation();
		InputValue = FVector2d(ControlRotation.Yaw, ControlRotation.Pitch);
	}

	// Bail out if auto-rotate is disabled.
	const bool bAutoRotateEnabled = EnableAutoRotateReader.Get(OutResult.VariableTable);
	const bool bAutoRotateYaw = AutoRotateYawReader.Get(OutResult.VariableTable);
	const bool bAutoRotatePitch = AutoRotatePitchReader.Get(OutResult.VariableTable);
	if (!bAutoRotateEnabled || (!bAutoRotateYaw && !bAutoRotatePitch))
	{
		DeactivateAutoRotate(OutResult);
		return;
	}

	// Keep track of the context's movement this frame.
	const FCameraNodeEvaluationResult& InitialResult = Params.EvaluationContext->GetInitialResult();
	FVector3d ContextMovement = FVector3d::ZeroVector;
	FVector3d CurrentContextLocation = InitialResult.CameraPose.GetLocation();
	if (!Params.bIsFirstFrame && Params.DeltaTime > 0)
	{
		ContextMovement = (CurrentContextLocation - LastContextLocation);
	}
	LastContextLocation = CurrentContextLocation;

	// Check if the input value changed from under us, and if it changed enough for
	// us to deactivate auto-rotate.
	const double YawChange = FMath::Abs(InputValue.X - LastInputValue.X);
	const double PitchChange = FMath::Abs(InputValue.Y - LastInputValue.Y);
	const float DeactivationThreshold = DeactivationThresholdReader.Get(OutResult.VariableTable);
	LastInputValue = InputValue;
	if ((bAutoRotateYaw && YawChange >= DeactivationThreshold) || (bAutoRotatePitch && PitchChange >= DeactivationThreshold))
	{
		DeactivateAutoRotate(OutResult);
		return;
	}

	// Figure out which direction we should auto-rotate towards.
	bool bHasAutoRotateDir = false;
	const FVector3d ContextAimDir = InitialResult.CameraPose.GetAimDir();
	FVector3d AutoRotateDir = ContextAimDir;
	if (DirectionVectorReader.IsDriven())
	{
		bHasAutoRotateDir = DirectionVectorReader.TryGet(OutResult.VariableTable, AutoRotateDir);
	}
	else
	{
		switch (AutoRotateNode->Direction)
		{
			case ECameraAutoRotateDirection::Facing:
			default:
				bHasAutoRotateDir = true;
				break;
			case ECameraAutoRotateDirection::Movement:
				if (!ContextMovement.IsNearlyZero())
				{
					AutoRotateDir = ContextMovement.GetSafeNormal(UE_SMALL_NUMBER, AutoRotateDir);
					bHasAutoRotateDir = true;
				}
				break;
			case ECameraAutoRotateDirection::MovementOrFacing:
				if (!ContextMovement.IsNearlyZero())
				{
					AutoRotateDir = ContextMovement.GetSafeNormal(UE_SMALL_NUMBER, AutoRotateDir);
				}
				bHasAutoRotateDir = true;
				break;
			case ECameraAutoRotateDirection::Zero:
				AutoRotateDir = FVector3d(1, 0, 0);
				bHasAutoRotateDir = true;
				break;
		}
	}
	if (!bHasAutoRotateDir)
	{
		DeactivateAutoRotate(OutResult);
		return;
	}

	const FRotator3d AutoRotateRot = AutoRotateDir.ToOrientationRotator();

	// Figure out how much work we have to do.
	const double DeltaYaw = FRotator::NormalizeAxis(bAutoRotateYaw ? (AutoRotateRot.Yaw - InputValue.X) : 0.0);
	const double DeltaPitch = FRotator::NormalizeAxis(bAutoRotatePitch ? (AutoRotateRot.Pitch - InputValue.Y) : 0.0);
	const double DeltaThreshold = GGameplayCamerasAutoRotateSnapThreshold;
	if (FMath::Abs(DeltaYaw) < DeltaThreshold && FMath::Abs(DeltaPitch) < DeltaThreshold)
	{
		DeactivateAutoRotate(OutResult);
		return;
	}

	// We are almost good to auto-rotate... but maybe we need to wait a bit longer.
	if (RemainingWaitTime > 0.f)
	{
		RemainingWaitTime -= Params.DeltaTime;
		if (RemainingWaitTime > 0.f)
		{
			return;
		}
	}

	// We will interpolate the length of the vector that represents the delta yaw/pitch
	// that we need to compensate with.
	const FVector2d DeltaVector = FVector2d(DeltaYaw, DeltaPitch);
	const double DeltaMagnitude = DeltaVector.Length();

	// Create our interpolator and update it.
	if (!bIsAutoRotating)
	{
		if (AutoRotateNode->Interpolator)
		{
			Interpolator = AutoRotateNode->Interpolator->BuildDoubleInterpolator();
		}
		else
		{
			Interpolator = MakeUnique<TPopValueInterpolator<double>>();
		}

		OriginalInputValue = FVector2d::ZeroVector;
		if (InputNodeEvaluator)
		{
			OriginalInputValue = InputNodeEvaluator->GetInputValue();
		}

		bIsAutoRotating = true;
	}
	check(Interpolator);

	Interpolator->Reset(DeltaMagnitude, 0);

	FCameraValueInterpolationParams InterpParams;
	InterpParams.DeltaTime = Params.DeltaTime;
	FCameraValueInterpolationResult InterpResult(OutResult.VariableTable);
	const double NewDeltaMagnitude = Interpolator->Run(InterpParams, InterpResult);

	// Get the new delta yaw/pitch and try to adjust our inner input node.
	const FVector2d NewDeltaVector = DeltaVector * (1.0 - NewDeltaMagnitude / DeltaMagnitude);
	bool bDeactivateAutoRotate = false;

	if (InputNodeEvaluator)
	{
		FCameraOperationParams OperationParams;
		OperationParams.EvaluationContext = Params.EvaluationContext;
		OperationParams.Evaluator = Params.Evaluator;

		FYawPitchCameraOperation YawPitchOperation;
		YawPitchOperation.Yaw = FConsumableDouble::Delta(NewDeltaVector.X);
		YawPitchOperation.Pitch = FConsumableDouble::Delta(NewDeltaVector.Y);

		InputNodeEvaluator->ExecuteOperation(OperationParams, YawPitchOperation);
		InputValue = InputNodeEvaluator->GetInputValue();
		LastInputValue = InputValue;

		if (YawPitchOperation.Yaw.HasValue() || YawPitchOperation.Pitch.HasValue())
		{
			bDeactivateAutoRotate = true;
		}
	}
	else if (APlayerController* PlayerController = GetPlayerController(Params.EvaluationContext))
	{
		const FVector2d NewInputValue(InputValue.X + NewDeltaVector.X, InputValue.Y + NewDeltaVector.Y);
		const FRotator3d NewControlRotation(NewInputValue.Y, NewInputValue.X, 0.0);
		PlayerController->SetControlRotation(NewControlRotation);

		LastInputValue = NewInputValue;
	}

	if (Interpolator->IsFinished() || bDeactivateAutoRotate)
	{
		DeactivateAutoRotate(OutResult);
	}
	else if (FreezeControlRotationReader.Get(OutResult.VariableTable))
	{
		const FBuiltInCameraVariables& BuiltInVariables = FBuiltInCameraVariables::Get();
		const FRotator3d OriginalControlRotation(OriginalInputValue.Y, OriginalInputValue.X, 0.0);
		OutResult.VariableTable.SetValue<FRotator3d>(BuiltInVariables.ControlRotationDefinition, OriginalControlRotation);
		OutResult.VariableTable.SetValue<bool>(BuiltInVariables.FreezeControlRotationDefinition, true);
	}
}

void FAutoRotateInput2DCameraNodeEvaluator::DeactivateAutoRotate(FCameraNodeEvaluationResult& OutResult)
{
	if (bIsAutoRotating)
	{
		bIsAutoRotating = false;
		Interpolator.Reset();
		RemainingWaitTime = WaitTimeReader.Get(OutResult.VariableTable);

		if (FreezeControlRotationReader.Get(OutResult.VariableTable))
		{
			// TODO: need Deinit callback to set this back to false also
			const FBuiltInCameraVariables& BuiltInVariables = FBuiltInCameraVariables::Get();
			OutResult.VariableTable.SetValue<bool>(BuiltInVariables.FreezeControlRotationDefinition, false);
		}
	}
}

APlayerController* FAutoRotateInput2DCameraNodeEvaluator::GetPlayerController(TSharedPtr<const FCameraEvaluationContext> EvaluationContext) const
{
	if (EvaluationContext)
	{
		return EvaluationContext->GetPlayerController();
	}
	return nullptr;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FAutoRotateInput2DCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FAutoRotateInput2DCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FAutoRotateInput2DCameraDebugBlock>();

	DebugBlock.RemainingWaitTime = RemainingWaitTime;
	DebugBlock.bIsAutoRotating = bIsAutoRotating;
	DebugBlock.InterpolationFactor = Interpolator ? Interpolator->GetCurrentValue() : 0.0f;
	DebugBlock.InterpolationTarget = Interpolator ? Interpolator->GetTargetValue() : 0.0f;
}

void FAutoRotateInput2DCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bIsAutoRotating)
	{
		Renderer.AddText(TEXT("rotating: %.3f -> %.3f"), InterpolationFactor, InterpolationTarget);
	}
	else
	{
		Renderer.AddText(TEXT("waiting: %.3fsec"), RemainingWaitTime);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

UAutoRotateInput2DCameraNode::UAutoRotateInput2DCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AddNodeFlags(ECameraNodeFlags::CustomGetChildren);

	UCriticalDamperValueInterpolator* DefaultInterpolator = ObjInit.CreateDefaultSubobject<UCriticalDamperValueInterpolator>(this, "Interpolator");
	DefaultInterpolator->DampingFactor = 10.f;
	Interpolator = DefaultInterpolator;
}

FCameraNodeChildrenView UAutoRotateInput2DCameraNode::OnGetChildren()
{
	return FCameraNodeChildrenView({ InputNode });
}

void UAutoRotateInput2DCameraNode::OnBuild(FCameraObjectBuildContext& BuildContext)
{
	using namespace UE::Cameras;
	const FBuiltInCameraVariables& BuiltInVariables = FBuiltInCameraVariables::Get();
	BuildContext.AllocationInfo.VariableTableInfo.VariableDefinitions.Add(BuiltInVariables.ControlRotationDefinition);
	BuildContext.AllocationInfo.VariableTableInfo.VariableDefinitions.Add(BuiltInVariables.FreezeControlRotationDefinition);
}

FCameraNodeEvaluatorPtr UAutoRotateInput2DCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FAutoRotateInput2DCameraNodeEvaluator>();
}

