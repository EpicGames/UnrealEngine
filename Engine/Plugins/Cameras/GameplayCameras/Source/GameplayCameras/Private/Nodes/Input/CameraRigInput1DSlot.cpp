// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/CameraRigInput1DSlot.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableTable.h"
#include "Nodes/Input/Input1DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigInput1DSlot)

#define LOCTEXT_NAMESPACE "CameraRigInputSlot"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraRigInput1DSlotEvaluator)

FCameraRigInput1DSlotEvaluator::FCameraRigInput1DSlotEvaluator()
{
	SetNodeEvaluatorFlags(
			ECameraNodeEvaluatorFlags::NeedsParameterUpdate | 
			ECameraNodeEvaluatorFlags::NeedsSerialize | 
			ECameraNodeEvaluatorFlags::SupportsOperations);
}

void FCameraRigInput1DSlotEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraRigInput1DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput1DSlot>();

	RevertAxisReader.Initialize(SlotNode->RevertAxis);
	SpeedReader.Initialize(SlotNode->Speed);

	DeltaInputValue = 0.f;
	InputValue = 0.f;

	if (SlotNode->GetVariableID().IsValid() && Params.LastActiveCameraRigInfo.LastResult)
	{
		const FCameraVariableTable& LastActiveRigVariableTable = Params.LastActiveCameraRigInfo.LastResult->VariableTable;
		LastActiveRigVariableTable.TryGetValue<double>(SlotNode->GetVariableID(), InputValue);
	}
}

void FCameraRigInput1DSlotEvaluator::OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	const UCameraRigInput1DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput1DSlot>();
	if (SlotNode->bIsPreBlended)
	{
		const double Speed = SpeedReader.Get(OutResult.VariableTable);
		OutResult.VariableTable.SetValue<double>(SlotNode->GetSpeedVariableID(), Speed);
	}
}

void FCameraRigInput1DSlotEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraRigInput1DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput1DSlot>();

	if (!bIsLockedThisFrame)
	{
		const double Speed = 
			SlotNode->bIsPreBlended ?
			OutResult.VariableTable.GetValue<double>(SlotNode->GetSpeedVariableID()) :
			SpeedReader.Get(OutResult.VariableTable);

		double FinalDelta(DeltaInputValue * Speed * Params.DeltaTime);

		if (bIsAccumulated)
		{
			InputValue += FinalDelta;
		}
		else
		{
			InputValue = FinalDelta;
		}

		InputValue = SlotNode->Normalize.NormalizeValue(InputValue);

		InputValue = SlotNode->Clamp.ClampValue(InputValue);
	}

	OutResult.VariableTable.SetValue<double>(SlotNode->GetVariableID(), InputValue);

	bIsLockedThisFrame = false;	
}

void FCameraRigInput1DSlotEvaluator::OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	if (FSingleValueCameraOperation* YawPitchOp = Operation.CastOperation<FSingleValueCameraOperation>())
	{
		const UCameraRigInput1DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput1DSlot>();

		double MinValue, MaxValue;
		SlotNode->Clamp.GetEffectiveClamping(MinValue, MaxValue);

		InputValue = YawPitchOp->Value.Apply(InputValue, MinValue, MaxValue);
	}
	else if (FLockUserInputCameraOperation* LockOp = Operation.CastOperation<FLockUserInputCameraOperation>())
	{
		bIsLockedThisFrame = LockOp->bLockThisFrame;
	}
}

void FCameraRigInput1DSlotEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << DeltaInputValue;
}

}  // namespace UE::Cameras

void UCameraRigInput1DSlot::PostLoad()
{
	if (!InputSlotParameters_DEPRECATED.bIsPreBlended)
	{
		InputSlotParameters_DEPRECATED.bIsPreBlended = true;
		bIsPreBlended = false;
	}

	Super::PostLoad();
}

void UCameraRigInput1DSlot::OnBuild(FCameraObjectBuildContext& BuildContext) 
{
	using namespace UE::Cameras;

	FCameraVariableDefinition VariableDefinition;

	if (BuiltInVariable != EBuiltInDoubleCameraVariable::None)
	{
		VariableDefinition = FBuiltInCameraVariables::Get().GetDefinition(BuiltInVariable);
	}
	else if (CustomVariable.HasVariable())
	{
		VariableDefinition = CustomVariable.Variable->GetVariableDefinition();
	}
	else if (bIsPreBlended)
	{
		BuildContext.BuildLog.AddMessage(
				EMessageSeverity::Error,
				this,
				LOCTEXT("PreBlendedInputSlotRequiresVariable",
					"An input slot with pre-blend enabled must specify a variable (built-in or custom) "
					"to blend with other input slots"));
	}

	if (VariableDefinition.IsValid())
	{
		VariableDefinition.bIsInput = true;

		FCameraVariableTableAllocationInfo& VariableTableInfo = BuildContext.AllocationInfo.VariableTableInfo;
		VariableTableInfo.VariableDefinitions.Add(VariableDefinition);

		FCameraVariableDefinition SpeedVariableDefinition = VariableDefinition.CreateVariant(TEXT("Speed"));
		VariableTableInfo.VariableDefinitions.Add(SpeedVariableDefinition);

		VariableID = VariableDefinition.VariableID;
		SpeedVariableID = SpeedVariableDefinition.VariableID;
	}
}

FCameraNodeEvaluatorPtr UCameraRigInput1DSlot::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	return Builder.BuildEvaluator<UE::Cameras::FCameraRigInput1DSlotEvaluator>();
}

#undef LOCTEXT_NAMESPACE

