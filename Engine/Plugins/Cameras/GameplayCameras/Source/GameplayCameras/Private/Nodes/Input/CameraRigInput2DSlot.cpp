// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/CameraRigInput2DSlot.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableTable.h"
#include "Nodes/Input/Input2DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigInput2DSlot)

#define LOCTEXT_NAMESPACE "CameraRigInputSlot"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraRigInput2DSlotEvaluator)

FCameraRigInput2DSlotEvaluator::FCameraRigInput2DSlotEvaluator()
{
	SetNodeEvaluatorFlags(
			ECameraNodeEvaluatorFlags::NeedsParameterUpdate | 
			ECameraNodeEvaluatorFlags::NeedsSerialize | 
			ECameraNodeEvaluatorFlags::SupportsOperations);
}

void FCameraRigInput2DSlotEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraRigInput2DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput2DSlot>();

	RevertAxisXReader.Initialize(SlotNode->RevertAxisX);
	RevertAxisYReader.Initialize(SlotNode->RevertAxisY);
	SpeedReader.Initialize(SlotNode->Speed);

	DeltaInputValue = FVector2d::ZeroVector;
	InputValue = FVector2d::ZeroVector;

	if (SlotNode->GetVariableID().IsValid() && Params.LastActiveCameraRigInfo.LastResult)
	{
		const FCameraVariableTable& LastActiveRigVariableTable = Params.LastActiveCameraRigInfo.LastResult->VariableTable;
		LastActiveRigVariableTable.TryGetValue<FVector2d>(SlotNode->GetVariableID(), InputValue);
	}
}

void FCameraRigInput2DSlotEvaluator::OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	const UCameraRigInput2DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput2DSlot>();
	if (SlotNode->bIsPreBlended)
	{
		const FVector2d Speed = SpeedReader.Get(OutResult.VariableTable);
		OutResult.VariableTable.SetValue<FVector2d>(SlotNode->GetSpeedVariableID(), Speed);
	}
}

void FCameraRigInput2DSlotEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraRigInput2DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput2DSlot>();

	if (!bIsLockedThisFrame)
	{
		const FVector2d Speed = 
			SlotNode->bIsPreBlended ?
			OutResult.VariableTable.GetValue<FVector2d>(SlotNode->GetSpeedVariableID()) :
			SpeedReader.Get(OutResult.VariableTable);

		FVector2d FinalDelta(DeltaInputValue.X * Speed.X * Params.DeltaTime, DeltaInputValue.Y * Speed.Y * Params.DeltaTime);

		if (RevertAxisXReader.Get(OutResult.VariableTable))
		{
			FinalDelta.X = -FinalDelta.X;
		}
		if (RevertAxisYReader.Get(OutResult.VariableTable))
		{
			FinalDelta.Y = -FinalDelta.Y;
		}

		if (bIsAccumulated)
		{
			InputValue += FinalDelta;
		}
		else
		{
			InputValue = FinalDelta;
		}

		InputValue.X = SlotNode->NormalizeX.NormalizeValue(InputValue.X);
		InputValue.Y = SlotNode->NormalizeY.NormalizeValue(InputValue.Y);

		InputValue.X = SlotNode->ClampX.ClampValue(InputValue.X);
		InputValue.Y = SlotNode->ClampY.ClampValue(InputValue.Y);
	}

	OutResult.VariableTable.SetValue<FVector2d>(SlotNode->GetVariableID(), InputValue);

	bIsLockedThisFrame = false;
}

void FCameraRigInput2DSlotEvaluator::OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	if (FYawPitchCameraOperation* YawPitchOp = Operation.CastOperation<FYawPitchCameraOperation>())
	{
		const UCameraRigInput2DSlot* SlotNode = GetCameraNodeAs<UCameraRigInput2DSlot>();

		double MinValueX, MaxValueX;
		SlotNode->ClampX.GetEffectiveClamping(MinValueX, MaxValueX);

		double MinValueY, MaxValueY;
		SlotNode->ClampY.GetEffectiveClamping(MinValueY, MaxValueY);

		InputValue.X = YawPitchOp->Yaw.Apply(InputValue.X, MinValueX, MaxValueX);
		InputValue.Y = YawPitchOp->Pitch.Apply(InputValue.Y, MinValueY, MaxValueY);
	}
	else if (FLockUserInputCameraOperation* LockOp = Operation.CastOperation<FLockUserInputCameraOperation>())
	{
		bIsLockedThisFrame = LockOp->bLockThisFrame;
	}
}

void FCameraRigInput2DSlotEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << DeltaInputValue;
}

}  // namespace UE::Cameras

void UCameraRigInput2DSlot::PostLoad()
{
	if (!InputSlotParameters_DEPRECATED.bIsPreBlended)
	{
		InputSlotParameters_DEPRECATED.bIsPreBlended = true;
		bIsPreBlended = false;
	}

	Super::PostLoad();
}

void UCameraRigInput2DSlot::OnBuild(FCameraObjectBuildContext& BuildContext)
{
	using namespace UE::Cameras;

	FCameraVariableDefinition VariableDefinition;

	if (BuiltInVariable != EBuiltInVector2dCameraVariable::None)
	{
		VariableDefinition = FBuiltInCameraVariables::Get().GetDefinition(BuiltInVariable);
	}
	else if (CustomVariable.HasVariable())
	{
		VariableDefinition = CustomVariable.Variable->GetVariableDefinition();
	}
	else if (bIsPreBlended)
	{
		BuildContext.BuildLog.AddMessage(
				EMessageSeverity::Error,
				this,
				LOCTEXT("PreBlendedInputSlotRequiresVariable",
					"An input slot with pre-blend enabled must specify a variable (built-in or custom) "
					"to blend with other input slots"));
	}

	if (VariableDefinition.IsValid())
	{
		VariableDefinition.bIsInput = true;

		FCameraVariableTableAllocationInfo& VariableTableInfo = BuildContext.AllocationInfo.VariableTableInfo;
		VariableTableInfo.VariableDefinitions.Add(VariableDefinition);

		FCameraVariableDefinition SpeedVariableDefinition = VariableDefinition.CreateVariant(TEXT("Speed"));
		VariableTableInfo.VariableDefinitions.Add(SpeedVariableDefinition);

		VariableID = VariableDefinition.VariableID;
		SpeedVariableID = SpeedVariableDefinition.VariableID;
	}
}

FCameraNodeEvaluatorPtr UCameraRigInput2DSlot::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	return Builder.BuildEvaluator<UE::Cameras::FCameraRigInput2DSlotEvaluator>();
}

#undef LOCTEXT_NAMESPACE

