// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/CameraRigInputSlotTypes.h"

#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigInputSlotTypes)

double FCameraParameterClamping::ClampValue(double Value) const
{
	if (bClampMin && Value < MinValue)
	{
		Value = MinValue;
	}
	if (bClampMax && Value > MaxValue)
	{
		Value = MaxValue;
	}
	return Value;
}

void FCameraParameterClamping::GetEffectiveClamping(double& OutMinValue, double& OutMaxValue) const
{
	OutMinValue = bClampMin ? MinValue : TNumericLimits<double>::Lowest();
	OutMaxValue = bClampMax ? MaxValue : TNumericLimits<double>::Max();
}

double FCameraParameterNormalization::NormalizeValue(double Value) const
{
	if (bNormalize)
	{
		Value = FMath::Fmod(Value, MaxValue);
		if (Value < 0.0)
		{
			Value += MaxValue;
		}
	}
	return Value;
}

