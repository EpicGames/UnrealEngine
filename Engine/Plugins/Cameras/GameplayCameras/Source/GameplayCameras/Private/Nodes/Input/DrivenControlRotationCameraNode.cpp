// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/DrivenControlRotationCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraOperation.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrivenControlRotationCameraNode)

namespace UE::Cameras
{

class FDrivenControlRotationHelperService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(, FDrivenControlRotationHelperService)

public:

	void AddMonitoredContext(TSharedPtr<const FCameraEvaluationContext> InContext);
	void RemoveMonitoredContext(TSharedPtr<const FCameraEvaluationContext> InContext);

	FRotator3d GetCachedControlRotation(TSharedPtr<const FCameraEvaluationContext> InContext) const;

protected:

	// FCameraEvaluationService interface.
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) override;
	virtual void OnPreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	struct FMonitoredContext
	{
		TWeakObjectPtr<APlayerController> WeakPlayerController;
		FRotator3d CachedControlRotation = FRotator3d::ZeroRotator;
		uint8 MonitoringRequests = 0;
	};

	using FMonitoredContexts = TMap<TSharedPtr<const FCameraEvaluationContext>, FMonitoredContext>;
	FMonitoredContexts MonitoredContexts;
};

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FDrivenControlRotationHelperService)

void FDrivenControlRotationHelperService::OnInitialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPreUpdate);
}

void FDrivenControlRotationHelperService::OnPreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	for (auto It = MonitoredContexts.CreateIterator(); It; ++It)
	{
		FMonitoredContext& MonitoredContext(It->Value);
		if (APlayerController* PlayerController = MonitoredContext.WeakPlayerController.Get())
		{
			MonitoredContext.CachedControlRotation = PlayerController->GetControlRotation();
		}
		else
		{
			It.RemoveCurrent();
		}
	}
}

void FDrivenControlRotationHelperService::AddMonitoredContext(TSharedPtr<const FCameraEvaluationContext> InContext)
{
	if (FMonitoredContext* ExistingEntry = MonitoredContexts.Find(InContext))
	{
		++ExistingEntry->MonitoringRequests;
		return;
	}

	if (APlayerController* PlayerController = InContext->GetPlayerController())
	{
		FMonitoredContext& NewEntry = MonitoredContexts.Add(InContext);
		NewEntry.WeakPlayerController = PlayerController;
		NewEntry.CachedControlRotation = PlayerController->GetControlRotation();
		NewEntry.MonitoringRequests = 1;
	}
}

void FDrivenControlRotationHelperService::RemoveMonitoredContext(TSharedPtr<const FCameraEvaluationContext> InContext)
{
	if (FMonitoredContext* ExistingEntry = MonitoredContexts.Find(InContext))
	{
		--ExistingEntry->MonitoringRequests;
		if (ExistingEntry->MonitoringRequests == 0)
		{
			MonitoredContexts.Remove(InContext);
		}
	}
}

FRotator3d FDrivenControlRotationHelperService::GetCachedControlRotation(TSharedPtr<const FCameraEvaluationContext> InContext) const
{
	if (const FMonitoredContext* ExistingEntry = MonitoredContexts.Find(InContext))
	{
		return ExistingEntry->CachedControlRotation;
	}
	return FRotator3d::ZeroRotator;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(, FDrivenControlRotationHelperDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TArray<FString>, ContextNames)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TArray<FRotator3d>, CachedControlRotations)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TArray<uint8>, MonitoringRequests)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDrivenControlRotationHelperDebugBlock)

void FDrivenControlRotationHelperService::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FDrivenControlRotationHelperDebugBlock& DebugBlock = Builder.AttachDebugBlock<FDrivenControlRotationHelperDebugBlock>();
	for (const FMonitoredContexts::ElementType& Pair : MonitoredContexts)
	{
		TSharedPtr<const FCameraEvaluationContext> Context = Pair.Key;
		const FMonitoredContext& MonitoredContext = Pair.Value;
		DebugBlock.ContextNames.Add(GetNameSafe(Context->GetOwner()));
		DebugBlock.CachedControlRotations.Add(MonitoredContext.CachedControlRotation);
		DebugBlock.MonitoringRequests.Add(MonitoredContext.MonitoringRequests);
	}
}

void FDrivenControlRotationHelperDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const int32 Num = FMath::Min(ContextNames.Num(), FMath::Min(CachedControlRotations.Num(), MonitoringRequests.Num()));
	Renderer.AddText(TEXT("%d monitored contexts"), Num);
	Renderer.AddIndent();
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Renderer.AddText(
					TEXT("{cam_notice}'%s'{cam_passive} (%d requests){cam_defualt} : %s\n"),
					*ContextNames[Index], MonitoringRequests[Index], *CachedControlRotations[Index].ToString());
		}
	}
	Renderer.RemoveIndent();
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

class FDrivenControlRotationCameraNodeEvaluator : public FInput2DCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(, FDrivenControlRotationCameraNodeEvaluator, FInput2DCameraNodeEvaluator)

public:

	FDrivenControlRotationCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnTeardown(const FCameraNodeEvaluatorTeardownParams& Params) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void LimitControlRotation(APlayerController* PlayerController, FRotator3d& InOutControlRotation);

private:

	TSharedPtr<FDrivenControlRotationHelperService> HelperService;

	FRotator3d LastControlRotation = FRotator3d::ZeroRotator;
	bool bLastWasActiveCameraRig = false;
	bool bIsLockedThisFrame = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FRotator3d LastDeltaControlRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FDrivenControlRotationCameraNodeEvaluator)

FDrivenControlRotationCameraNodeEvaluator::FDrivenControlRotationCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize | ECameraNodeEvaluatorFlags::SupportsOperations);
}

void FDrivenControlRotationCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	bLastWasActiveCameraRig = true;

	HelperService = Params.Evaluator->FindOrRegisterEvaluationService<FDrivenControlRotationHelperService>();
	HelperService->AddMonitoredContext(Params.EvaluationContext);
}

void FDrivenControlRotationCameraNodeEvaluator::OnTeardown(const FCameraNodeEvaluatorTeardownParams& Params)
{
	if (HelperService)
	{
		HelperService->RemoveMonitoredContext(Params.EvaluationContext);
	}
}

void FDrivenControlRotationCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController())
	{
		const FRotator3d ControlRotation = PlayerController->GetControlRotation();

		if (Params.bIsActiveCameraRig && !bIsLockedThisFrame)
		{
			// We are running in the active camera rig. Just move with the control rotation.
			InputValue = FVector2d(ControlRotation.Yaw, ControlRotation.Pitch);
		}
		else if (!Params.bIsActiveCameraRig && !bIsLockedThisFrame)
		{
			// We are not the active camera rig anymore. Only apply the delta between the last frame's
			// control rotation and this frame's control rotation. One important thing to note here
			// is that we do NOT get the control rotation from the player controller for this frame.
			// This is because any newly activated camera rig may have changed it for their own purpose,
			// such as a new camera rig that needs target preservation and, therefore, ran an IK 
			// correction that affect the control rotation (see OnExecuteOperation below). So here we
			// get the current control rotation from our helper service, which cached the control
			// rotation in OnPreUpdate, BEFORE the camera director runs and new camera rigs are
			// activated. This lets us get the "new" control rotation for this frame before it gets
			// messed up by new camera rigs' initializations.
			const FRotator3d CachedControlRotation = HelperService->GetCachedControlRotation(Params.EvaluationContext);
			FRotator3d DeltaRotation((CachedControlRotation - LastControlRotation).GetNormalized());
			FRotator3d LocalControlRotation(InputValue.Y + DeltaRotation.Pitch, InputValue.X + DeltaRotation.Yaw, 0.0);

			// If we have a player camera manager, apply its limits to the correct angles.
			LimitControlRotation(PlayerController, LocalControlRotation);

			InputValue.X = LocalControlRotation.Yaw;
			InputValue.Y = LocalControlRotation.Pitch;

#if UE_GAMEPLAY_CAMERAS_DEBUG
			LastDeltaControlRotation = DeltaRotation;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
		}

		LastControlRotation = ControlRotation;
	}

	bLastWasActiveCameraRig = Params.bIsActiveCameraRig;

	bIsLockedThisFrame = false;
}

void FDrivenControlRotationCameraNodeEvaluator::OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation)
{
	if (FYawPitchCameraOperation* YawPitchOp = Operation.CastOperation<FYawPitchCameraOperation>())
	{
		if (APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController())
		{
			// If we are running in the active camera rig, we have permission to affect the control rotation.
			// In that case, let's apply the yaw/pitch operation to it. Otherwise, let's only apply the correction
			// to our local cache.
			if (bLastWasActiveCameraRig)
			{
				FRotator3d ControlRotation = PlayerController->GetControlRotation();

				// Make sure the corrected yaw/pitch angles are in [0..360[ and ]-180..180] respectively.
				ControlRotation.Yaw = FRotator3d::ClampAxis(YawPitchOp->Yaw.Apply(ControlRotation.Yaw));
				ControlRotation.Pitch = FRotator3d::NormalizeAxis(YawPitchOp->Pitch.Apply(ControlRotation.Pitch));

				LimitControlRotation(PlayerController, ControlRotation);

				PlayerController->SetControlRotation(ControlRotation);

				InputValue = FVector2d(ControlRotation.Yaw, ControlRotation.Pitch);
			}
			else
			{
				InputValue.X = YawPitchOp->Yaw.Apply(InputValue.X);
				InputValue.Y = YawPitchOp->Pitch.Apply(InputValue.Y);
			}
		}
	}
	else if (FLockUserInputCameraOperation* LockOp = Operation.CastOperation<FLockUserInputCameraOperation>())
	{
		// We are told to lock input this frame, so let's keep our input value and overwrite the control rotation to 
		// keep it to what it was before.
		APlayerController* PlayerController = Params.EvaluationContext->GetPlayerController();
		if (LockOp->bLockThisFrame && bLastWasActiveCameraRig && PlayerController)
		{
			FRotator3d LockedControlRotation(InputValue.Y, InputValue.X, 0);
			LimitControlRotation(PlayerController, LockedControlRotation);
			PlayerController->SetControlRotation(LockedControlRotation);
		}

		bIsLockedThisFrame = LockOp->bLockThisFrame;
	}
}

void FDrivenControlRotationCameraNodeEvaluator::LimitControlRotation(APlayerController* PlayerController, FRotator3d& InOutControlRotation)
{
	// If we have a player camera manager, apply its limits to the correct angles.
	if (APlayerCameraManager* CameraManager = PlayerController->PlayerCameraManager)
	{
		CameraManager->LimitViewPitch(InOutControlRotation, CameraManager->ViewPitchMin, CameraManager->ViewPitchMax);
		CameraManager->LimitViewYaw(InOutControlRotation, CameraManager->ViewYawMin, CameraManager->ViewYawMax);
		CameraManager->LimitViewRoll(InOutControlRotation, CameraManager->ViewRollMin, CameraManager->ViewRollMax);
	}
}

void FDrivenControlRotationCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << LastControlRotation;
	Ar << bLastWasActiveCameraRig;
	Ar << bIsLockedThisFrame;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(, FDrivenControlRotationDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FVector2d, InputValue)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, LastDeltaControlRotation)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bLastWasActiveCameraRig)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FDrivenControlRotationDebugBlock)

void FDrivenControlRotationCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FDrivenControlRotationDebugBlock& DebugBlock = Builder.AttachDebugBlock<FDrivenControlRotationDebugBlock>();
	DebugBlock.InputValue = InputValue;
	DebugBlock.LastDeltaControlRotation = LastDeltaControlRotation;
	DebugBlock.bLastWasActiveCameraRig = bLastWasActiveCameraRig;
}

void FDrivenControlRotationDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (bLastWasActiveCameraRig)
	{
		Renderer.AddText(TEXT("[Active] yaw: %.3f pitch: %.3f"), InputValue.X, InputValue.Y);
	}
	else
	{
		Renderer.AddText(
				TEXT("[Driven] yaw: %.3f pitch: %.3f (delta yaw: %.3f pitch: %.3f)"), 
				InputValue.X, InputValue.Y, LastDeltaControlRotation.Yaw, LastDeltaControlRotation.Pitch);
	}
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UDrivenControlRotationCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FDrivenControlRotationCameraNodeEvaluator>();
}

