// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/Input1DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Input1DCameraNode)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FInput1DCameraNodeEvaluator)

void FInput1DCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << InputValue;
}

}  // namespace UE::Cameras

