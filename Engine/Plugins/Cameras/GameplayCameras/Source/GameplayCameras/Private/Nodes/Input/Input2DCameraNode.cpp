// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/Input2DCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Input2DCameraNode)

namespace UE::Cameras
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FInput2DCameraNodeEvaluator)

void FInput2DCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << InputValue;
}

}  // namespace UE::Cameras

