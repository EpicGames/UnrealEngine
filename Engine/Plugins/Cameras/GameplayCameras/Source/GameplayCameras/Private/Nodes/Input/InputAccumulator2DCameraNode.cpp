// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/InputAccumulator2DCameraNode.h"

#include "Nodes/Input/CameraRigInput2DSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputAccumulator2DCameraNode)

#define LOCTEXT_NAMESPACE "InputAccumulator2DCameraNode"

namespace UE::Cameras
{

class FInputAccumulator2DCameraNodeEvaluator : public FCameraRigInput2DSlotEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FInputAccumulator2DCameraNodeEvaluator, FCameraRigInput2DSlotEvaluator)

public:

	FInputAccumulator2DCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	FInput2DCameraNodeEvaluator* InputSlotEvaluator = nullptr;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FInputAccumulator2DCameraNodeEvaluator)

FInputAccumulator2DCameraNodeEvaluator::FInputAccumulator2DCameraNodeEvaluator()
{
	bIsAccumulated = true;
}

void FInputAccumulator2DCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	Super::OnBuild(Params);

	const UInputAccumulator2DCameraNode* AccumulatorNode = GetCameraNodeAs<UInputAccumulator2DCameraNode>();
	InputSlotEvaluator = Params.BuildEvaluatorAs<FInput2DCameraNodeEvaluator>(AccumulatorNode->InputSlot);
}

FCameraNodeEvaluatorChildrenView FInputAccumulator2DCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView{ InputSlotEvaluator };
}

void FInputAccumulator2DCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (InputSlotEvaluator)
	{
		InputSlotEvaluator->Run(Params, OutResult);
		DeltaInputValue = InputSlotEvaluator->GetInputValue();
	}

	Super::OnRun(Params, OutResult);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UInputAccumulator2DCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FInputAccumulator2DCameraNodeEvaluator>();
}

#undef LOCTEXT_NAMESPACE

