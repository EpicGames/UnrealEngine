// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/InputAxisBinding2DCameraNode.h"

#include "Components/InputComponent.h"
#include "Core/CameraParameterReader.h"
#include "EnhancedInputComponent.h"
#include "Nodes/Input/InputAxisBindingHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputAxisBinding2DCameraNode)

namespace UE::Cameras
{

class FInputAxisBinding2DCameraNodeEvaluator : public FCameraRigInput2DSlotEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FInputAxisBinding2DCameraNodeEvaluator, FCameraRigInput2DSlotEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<FVector2d> MultiplierReader;

	TObjectPtr<UEnhancedInputComponent> InputComponent;
	TArray<FEnhancedInputActionValueBinding*> AxisValueBindings;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FInputAxisBinding2DCameraNodeEvaluator)

void FInputAxisBinding2DCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UInputAxisBinding2DCameraNode* AxisBindingNode = GetCameraNodeAs<UInputAxisBinding2DCameraNode>();

	MultiplierReader.Initialize(AxisBindingNode->Multiplier);

	InputComponent = FInputAxisBindingHelpers::FindInputComponent(Params);
	FInputAxisBindingHelpers::BindActionValues(Params, AxisBindingNode, InputComponent, AxisBindingNode->AxisActions, AxisValueBindings);

	Super::OnInitialize(Params, OutResult);
}

void FInputAxisBinding2DCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const FVector2d Multiplier = MultiplierReader.Get(OutResult.VariableTable);

	const FVector2d HighestValue = FInputAxisBindingHelpers::GetHighestValue(AxisValueBindings);
	DeltaInputValue = FVector2d(HighestValue.X * Multiplier.X, HighestValue.Y * Multiplier.Y);

	Super::OnRun(Params, OutResult);
}

}  // namespace UE::Cameras

void UInputAxisBinding2DCameraNode::PostLoad()
{
	if (!InputSlotParameters_DEPRECATED.bIsAccumulated)
	{
		InputSlotParameters_DEPRECATED.bIsAccumulated = true;
		bIsAccumulated = false;
	}

	Super::PostLoad();
}

FCameraNodeEvaluatorPtr UInputAxisBinding2DCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FInputAxisBinding2DCameraNodeEvaluator>();
}

