// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/InputAxisBindingHelpers.h"

#include "Components/InputComponent.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "EnhancedInputComponent.h"
#include "GameFramework/Actor.h"
#include "UObject/Package.h"

namespace UE::Cameras
{

UEnhancedInputComponent* FInputAxisBindingHelpers::FindInputComponent(const UE::Cameras::FCameraNodeEvaluatorInitializeParams& Params)
{
	UObject* ContextOwner = Params.EvaluationContext->GetOwner();
	if (ContextOwner)
	{
		if (AActor* ContextOwnerActor = Cast<AActor>(ContextOwner))
		{
			return Cast<UEnhancedInputComponent>(ContextOwnerActor->InputComponent);
		}
		else if (AActor* OuterActor = ContextOwner->GetTypedOuter<AActor>())
		{
			return Cast<UEnhancedInputComponent>(OuterActor->InputComponent);
		}
	}
	return nullptr;
}

void FInputAxisBindingHelpers::BindActionValues(
		const UE::Cameras::FCameraNodeEvaluatorInitializeParams& Params,
		const UCameraNode* CameraNode,
		UEnhancedInputComponent* InputComponent,
		const TArray<TObjectPtr<UInputAction>>& AxisActions,
		TArray<FEnhancedInputActionValueBinding*>& OutAxisValueBindings)
{
	UObject* ContextOwner = Params.EvaluationContext->GetOwner();

	if (InputComponent)
	{
		for (TObjectPtr<UInputAction> AxisAction : AxisActions)
		{
			FEnhancedInputActionValueBinding* AxisValueBinding = &InputComponent->BindActionValue(AxisAction);
			OutAxisValueBindings.Add(AxisValueBinding);
		}
	}
	else if (Params.Evaluator->GetRole() == ECameraSystemEvaluatorRole::Game)
	{
		UE_LOGF(LogCameraSystem, Error, "No input component found on context owner '%ls' for node '%ls' in '%ls'.",
				*GetNameSafe(ContextOwner), 
				*GetNameSafe(CameraNode),
				*GetNameSafe(CameraNode ? CameraNode->GetOutermost() : nullptr));
	}
}

FVector2d FInputAxisBindingHelpers::GetHighestValue(const TArray<FEnhancedInputActionValueBinding*>& AxisValueBindings)
{
	FVector2d HighestValue(FVector2d::ZeroVector);
	double HighestSquaredLenth = 0.f;

	for (FEnhancedInputActionValueBinding* AxisValueBinding : AxisValueBindings)
	{
		if (!AxisValueBinding)
		{
			continue;
		}

		const FVector2d Value = AxisValueBinding->GetValue().Get<FVector2D>();
		const double ValueSquaredLength = Value.SquaredLength();
		if (ValueSquaredLength > HighestSquaredLenth)
		{
			HighestValue = Value;
		}
	}

	return HighestValue;
}

}  // namespace UE::Cameras

