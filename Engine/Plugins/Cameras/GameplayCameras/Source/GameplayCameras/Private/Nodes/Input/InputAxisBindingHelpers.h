// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraNode;
class UEnhancedInputComponent;
class UInputAction;
struct FEnhancedInputActionValueBinding;

namespace UE::Cameras { struct FCameraNodeEvaluatorInitializeParams; }

namespace UE::Cameras
{

class FInputAxisBindingHelpers
{
public:

	UE_API static UEnhancedInputComponent* FindInputComponent(const UE::Cameras::FCameraNodeEvaluatorInitializeParams& Params);

	UE_API static void BindActionValues(
			const UE::Cameras::FCameraNodeEvaluatorInitializeParams& Params,
			const UCameraNode* CameraNode,
			UEnhancedInputComponent* InputComponent,
			const TArray<TObjectPtr<UInputAction>>& AxisActions,
			TArray<FEnhancedInputActionValueBinding*>& OutAxisValueBindings);

	UE_API static FVector2d GetHighestValue(const TArray<FEnhancedInputActionValueBinding*>& AxisValueBindings);
};

}  // namespace UE::Cameras

#undef UE_API

