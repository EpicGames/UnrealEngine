// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Input/RawInputAxisBinding2DCameraNode.h"

#include "Components/InputComponent.h"
#include "Core/CameraParameterReader.h"
#include "EnhancedInputComponent.h"
#include "Nodes/Input/InputAxisBindingHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RawInputAxisBinding2DCameraNode)

namespace UE::Cameras
{

class FRawInputAxisBinding2DCameraNodeEvaluator : public FInput2DCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FRawInputAxisBinding2DCameraNodeEvaluator, FInput2DCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	TCameraParameterReader<FVector2d> MultiplierReader;

	TObjectPtr<UEnhancedInputComponent> InputComponent;
	TArray<FEnhancedInputActionValueBinding*> AxisValueBindings;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FRawInputAxisBinding2DCameraNodeEvaluator)

void FRawInputAxisBinding2DCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const URawInputAxisBinding2DCameraNode* AxisBindingNode = GetCameraNodeAs<URawInputAxisBinding2DCameraNode>();

	MultiplierReader.Initialize(AxisBindingNode->Multiplier);

	InputComponent = FInputAxisBindingHelpers::FindInputComponent(Params);
	FInputAxisBindingHelpers::BindActionValues(Params, AxisBindingNode, InputComponent, AxisBindingNode->AxisActions, AxisValueBindings);

	Super::OnInitialize(Params, OutResult);
}

void FRawInputAxisBinding2DCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const FVector2d Multiplier = MultiplierReader.Get(OutResult.VariableTable);

	const FVector2d HighestValue = FInputAxisBindingHelpers::GetHighestValue(AxisValueBindings);
	InputValue = FVector2d(HighestValue.X * Multiplier.X, HighestValue.Y * Multiplier.Y);

	Super::OnRun(Params, OutResult);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr URawInputAxisBinding2DCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FRawInputAxisBinding2DCameraNodeEvaluator>();
}

