// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Shakes/CameraShakeCameraNode.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/ShakeCameraNode.h"
#include "Services/CameraShakeService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeCameraNode)

#define LOCTEXT_NAMESPACE "CameraShakeCameraNode"

namespace UE::Cameras
{

class FCameraShakeCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCameraShakeCameraNodeEvaluator)

public:

	FCameraShakeCameraNodeEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	ECameraShakeEvaluationMode EvaluationMode = ECameraShakeEvaluationMode::VisualLayer;

	FShakeCameraNodeEvaluator* CameraShakeRootEvaluator = nullptr;

	TSharedPtr<FCameraShakeService> CameraShakeService;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraShakeCameraNodeEvaluator)

FCameraShakeCameraNodeEvaluator::FCameraShakeCameraNodeEvaluator()
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsParameterUpdate);
}

FCameraNodeEvaluatorChildrenView FCameraShakeCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView({ CameraShakeRootEvaluator });
}

void FCameraShakeCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UCameraShakeCameraNode* CameraShakeNode = GetCameraNodeAs<UCameraShakeCameraNode>();
	EvaluationMode = CameraShakeNode->EvaluationMode;

	// If evaluating the shake "inline", build our children evaluators.
	if (CameraShakeNode->EvaluationMode == ECameraShakeEvaluationMode::Inline)
	{
		const UCameraShakeAsset* CameraShake = CameraShakeNode->CameraShakeReference.GetCameraShake();
		if (CameraShake && CameraShake->RootNode)
		{
			CameraShakeRootEvaluator = Params.BuildEvaluatorAs<FShakeCameraNodeEvaluator>(CameraShake->RootNode);
		}
	}
}

void FCameraShakeCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UCameraShakeCameraNode* PrefabNode = GetCameraNodeAs<UCameraShakeCameraNode>();

	// In editor and other uncooked builds, we don't have the allocation info for the variables and data of the
	// reference shake because we try to keep assets standalone. So we need to make room for it.
	// With cooked data we would have all that info already.
	PrefabNode->CameraShakeReference.EnsureAllocationInfo(OutResult);

	// Apply overrides and defaults right away.
	PrefabNode->CameraShakeReference.ApplyParameterOverridesAndDefaults(OutResult);

	// If evaluating the shake later in the visual layer, acquire the shake service we will use to
	// keep that shake alive.
	if (EvaluationMode == ECameraShakeEvaluationMode::VisualLayer)
	{
		CameraShakeService = Params.Evaluator->FindEvaluationService<FCameraShakeService>();
		ensure(CameraShakeService);
	}
}

void FCameraShakeCameraNodeEvaluator::OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult)
{
	// Keep applying overrides in case they are driven by a variable.
	const UCameraShakeCameraNode* PrefabNode = GetCameraNodeAs<UCameraShakeCameraNode>();
	PrefabNode->CameraShakeReference.ApplyParameterOverrides(OutResult.VariableTable, false);
}

void FCameraShakeCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	// If evaluating the shake "inline", simply run it.
	if (CameraShakeRootEvaluator)
	{
		CameraShakeRootEvaluator->Run(Params, OutResult);

		FCameraNodeShakeParams ShakeParams(Params);
		FCameraNodeShakeResult ShakeResult(OutResult);
		CameraShakeRootEvaluator->ShakeResult(ShakeParams, ShakeResult);
		ShakeResult.ApplyDelta(ShakeParams);
	}
	// If evaluating the shake later in the visual layer, keep requesting that the shake service
	// maintains this shake alive. We already put the shake's parameters inside the variable table so
	// its values will blend with whoever else wants this shake, and it will eventually run with the
	// blended values.
	else if (CameraShakeService)
	{
		const UCameraShakeCameraNode* CameraShakeNode = GetCameraNodeAs<UCameraShakeCameraNode>();

		FStartCameraShakeParams StartParams;
		StartParams.CameraShake = CameraShakeNode->CameraShakeReference.GetCameraShake();
		CameraShakeService->RequestCameraShakeThisFrame(StartParams);
	}
}

}  // namespace UE::Cameras

void UCameraShakeCameraNode::OnPreBuild(FCameraBuildContext& BuildContext)
{
	// Build the inner camera shake. Silently skip it if it's not set... but we will
	// report an error in OnBuild about it.
	if (UCameraShakeAsset* CameraShake = CameraShakeReference.GetCameraShake())
	{
		const bool bNeedsBuild = CameraShake->BuildStatus == ECameraBuildStatus::Dirty;
		if (bNeedsBuild)
		{
			CameraShake->BuildCameraShake(BuildContext);
		}
	}

	// Make sure the property bag of the camera shake reference is up to date.
	CameraShakeReference.RebuildParametersIfNeeded();
}

void UCameraShakeCameraNode::OnBuild(FCameraObjectBuildContext& BuildContext)
{
	using namespace UE::Cameras;

	UCameraShakeAsset* CameraShake = CameraShakeReference.GetCameraShake();
	if (!CameraShake)
	{
		BuildContext.BuildLog.AddMessage(EMessageSeverity::Error, this, 
				LOCTEXT("MissingCameraShake", "No camera shake specified on camera shake node."));
		return;
	}

	const bool bDidModify = CameraShakeReference.PostBuild();
	if (bDidModify)
	{
		Modify();
	}

	if (BuildContext.IsCooking())
	{
		// Whatever allocations our inner camera shake needs for its evaluators and
		// their camera variables, we add that to our camera shake's allocation info.
		// If we're going to be running the shake in a deferred way, however, we need
		// to make the variables public so that they propagate to it.
		FCameraObjectAllocationInfo CameraShakeAllocationInfo(CameraShake->AllocationInfo);
		if (EvaluationMode == ECameraShakeEvaluationMode::VisualLayer)
		{
			for (FCameraVariableDefinition& VariableDefinition : CameraShakeAllocationInfo.VariableTableInfo.VariableDefinitions)
			{
				VariableDefinition.bIsPrivate = false;
			}
		}
		BuildContext.AllocationInfo.Append(CameraShakeAllocationInfo);
	}
}

void UCameraShakeCameraNode::GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos)
{
	CameraShakeReference.GetCustomCameraNodeParameters(OutParameterInfos);
}

FCameraNodeEvaluatorPtr UCameraShakeCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCameraShakeCameraNodeEvaluator>();
}

#if WITH_EDITOR

void UCameraShakeCameraNode::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UCameraShakeCameraNode, CameraShakeReference))
	{
		OnCustomCameraNodeParametersChanged(this);
	}
}

#endif

#undef LOCTEXT_NAMESPACE

