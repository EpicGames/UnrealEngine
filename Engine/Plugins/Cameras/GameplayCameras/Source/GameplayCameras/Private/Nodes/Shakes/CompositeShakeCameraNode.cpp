// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Shakes/CompositeShakeCameraNode.h"

#include "Core/CameraNodeEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeShakeCameraNode)

namespace UE::Cameras
{

class FCompositeShakeCameraNodeEvaluator : public FShakeCameraNodeEvaluator
{
	UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCompositeShakeCameraNodeEvaluator)

public:

	FCompositeShakeCameraNodeEvaluator()
	{
		SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);
	}

protected:

	// FShakeCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult) override;
	virtual void OnRestartShake(const FCameraNodeShakeRestartParams& Params) override;

private:

	TArray<FShakeCameraNodeEvaluator*> ShakeEvaluators;
};

UE_DEFINE_SHAKE_CAMERA_NODE_EVALUATOR(FCompositeShakeCameraNodeEvaluator)

void FCompositeShakeCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UCompositeShakeCameraNode* CompositeShake = GetCameraNodeAs<UCompositeShakeCameraNode>();
	for (const UShakeCameraNode* Shake : CompositeShake->Shakes)
	{
		FShakeCameraNodeEvaluator* ShakeEvaluator = Params.BuildEvaluatorAs<FShakeCameraNodeEvaluator>(Shake);
		if (ShakeEvaluator)
		{
			ShakeEvaluators.Add(ShakeEvaluator);
		}
	}
}

FCameraNodeEvaluatorChildrenView FCompositeShakeCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView{ TArrayView<FShakeCameraNodeEvaluator*>(ShakeEvaluators) };
}

void FCompositeShakeCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	for (FShakeCameraNodeEvaluator* ShakeEvaluator : ShakeEvaluators)
	{
		ShakeEvaluator->Run(Params, OutResult);
	}
}

void FCompositeShakeCameraNodeEvaluator::OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult)
{
	float MaxTimeLeft = 0.f;
	bool bAnyInfiniteShake = false;

	for (FShakeCameraNodeEvaluator* ShakeEvaluator : ShakeEvaluators)
	{
		ShakeEvaluator->ShakeResult(Params, OutResult);

		if (OutResult.ShakeTimeLeft >= 0.f)
		{
			MaxTimeLeft = FMath::Max(MaxTimeLeft, OutResult.ShakeTimeLeft);
		}
		else
		{
			bAnyInfiniteShake = true;
		}
	}

	OutResult.ShakeTimeLeft = (bAnyInfiniteShake ? -1.f : MaxTimeLeft);
}

void FCompositeShakeCameraNodeEvaluator::OnRestartShake(const FCameraNodeShakeRestartParams& Params)
{
	for (FShakeCameraNodeEvaluator* ShakeEvaluator : ShakeEvaluators)
	{
		ShakeEvaluator->RestartShake(Params);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UCompositeShakeCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCompositeShakeCameraNodeEvaluator>();
}

