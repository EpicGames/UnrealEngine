// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Shakes/EnvelopeShakeCameraNode.h"

#include "Core/CameraNodeEvaluator.h"
#include "Math/Interpolation.h"
#include "Math/PerlinNoise.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvelopeShakeCameraNode)

namespace UE::Cameras
{

class FEnvelopeShakeCameraNodeEvaluator : public FShakeCameraNodeEvaluator
{
	UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FEnvelopeShakeCameraNodeEvaluator)

protected:

	// FShakeCameraNodeEvaluator interface.
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult) override;
	virtual void OnRestartShake(const FCameraNodeShakeRestartParams& Params) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

private:

	FShakeCameraNodeEvaluator* ShakeEvaluator = nullptr;

	float EaseInTime;
	float EaseOutTime;
	float TotalTime;

	float CurrentTime = 0.f;
};

UE_DEFINE_SHAKE_CAMERA_NODE_EVALUATOR(FEnvelopeShakeCameraNodeEvaluator)

void FEnvelopeShakeCameraNodeEvaluator::OnBuild(const FCameraNodeEvaluatorBuildParams& Params)
{
	const UEnvelopeShakeCameraNode* EnvelopeNode = GetCameraNodeAs<UEnvelopeShakeCameraNode>();
	ShakeEvaluator = Params.BuildEvaluatorAs<FShakeCameraNodeEvaluator>(EnvelopeNode->Shake);
}

FCameraNodeEvaluatorChildrenView FEnvelopeShakeCameraNodeEvaluator::OnGetChildren()
{
	return FCameraNodeEvaluatorChildrenView{ ShakeEvaluator };
}

void FEnvelopeShakeCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UEnvelopeShakeCameraNode* EnvelopeNode = GetCameraNodeAs<UEnvelopeShakeCameraNode>();

	EaseInTime = EnvelopeNode->EaseInTime.GetValue(OutResult.VariableTable);
	EaseOutTime = EnvelopeNode->EaseOutTime.GetValue(OutResult.VariableTable);
	TotalTime = EnvelopeNode->TotalTime.GetValue(OutResult.VariableTable);

	EaseInTime = FMath::Max(0, EaseInTime);
	EaseOutTime = FMath::Max(0, EaseOutTime);
	TotalTime = FMath::Max(0, TotalTime);
	if (EaseInTime + EaseOutTime > TotalTime)
	{
		const float HalfTotalTime = TotalTime / 2.f;
		EaseInTime = EaseOutTime = HalfTotalTime;
	}

	CurrentTime = 0.f;
}

void FEnvelopeShakeCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (CurrentTime >= TotalTime)
	{
		return;
	}

	CurrentTime += Params.DeltaTime;

	if (ShakeEvaluator)
	{
		ShakeEvaluator->Run(Params, OutResult);
	}
}

void FEnvelopeShakeCameraNodeEvaluator::OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult)
{
	if (CurrentTime >= TotalTime || !ShakeEvaluator)
	{
		OutResult.ShakeTimeLeft = 0.f;
		return;
	}

	float Alpha = 1.f;
	if (EaseInTime > 0 && CurrentTime < EaseInTime)
	{
		Alpha = SmoothStep(CurrentTime / EaseInTime);
	}
	else if (EaseOutTime > 0 && CurrentTime > TotalTime - EaseOutTime)
	{
		Alpha = SmoothStep((TotalTime - CurrentTime) / EaseOutTime);
	}

	const float EnvelopeTimeLeft = (TotalTime - CurrentTime);
	{
		FCameraNodeShakeParams ChildParams(Params);
		ChildParams.ShakeScale *= Alpha;

		FCameraNodeShakeResult ChildResult(OutResult.ShakenResult);

		ShakeEvaluator->ShakeResult(ChildParams, ChildResult);

		OutResult.ShakeDelta.Combine(ChildResult.ShakeDelta, ChildParams.ShakeScale);
		OutResult.ShakeTimeLeft = ChildResult.ShakeTimeLeft;
	}

	if (OutResult.ShakeTimeLeft >= 0.f)
	{
		// The underlying shake has a finite duration, so we end when it ends, or when we
		// reach our total time, whichever happens first.
		OutResult.ShakeTimeLeft = FMath::Clamp(OutResult.ShakeTimeLeft, 0, EnvelopeTimeLeft);
	}
	else
	{
		// The underlying shake is infinite, so we drive the duration.
		OutResult.ShakeTimeLeft = EnvelopeTimeLeft;
	}
}

void FEnvelopeShakeCameraNodeEvaluator::OnRestartShake(const FCameraNodeShakeRestartParams& Params)
{
	if (EaseInTime > 0 && CurrentTime < EaseInTime)
	{
		// Keep easing in, but extend TotalTime by the time we already did.
		TotalTime += CurrentTime;
	}
	else if (EaseOutTime > 0 && CurrentTime > TotalTime - EaseOutTime)
	{
		// Stop easing out and restart easing in at an equivalent time.
		const float Alpha = (TotalTime - CurrentTime) / EaseOutTime;
		CurrentTime = Alpha * EaseInTime;
	}
	else
	{
		// We're fully shaking. Keep doing that for another whole duration.
		TotalTime += (TotalTime - CurrentTime);
	}

	if (ShakeEvaluator)
	{
		ShakeEvaluator->RestartShake(Params);
	}
}

void FEnvelopeShakeCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << EaseInTime;
	Ar << EaseOutTime;
	Ar << TotalTime;
	Ar << CurrentTime;
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UEnvelopeShakeCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FEnvelopeShakeCameraNodeEvaluator>();
}

