// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Shakes/PerlinNoiseLocationShakeCameraNode.h"

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugGraph.h"
#include "Debug/CameraDebugRenderer.h"
#include "Math/PerlinNoise.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerlinNoiseLocationShakeCameraNode)

#define LOCTEXT_NAMESPACE "PerlinNoiseLocationShakeCameraNode"

namespace UE::Cameras
{

class FPerlinNoiseLocationShakeCameraNodeEvaluator : public FShakeCameraNodeEvaluator
{
	UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FPerlinNoiseLocationShakeCameraNodeEvaluator)

public:

protected:

	// FShakeCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	TCameraParameterReader<float> AmplitudeMultiplier;
	TCameraParameterReader<float> FrequencyMultiplier;
	TCameraParameterReader<int32> Octaves;

	FPerlinNoise GeneratorX;
	FPerlinNoise GeneratorY;
	FPerlinNoise GeneratorZ;

	FVector3d NoiseValue;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	TCameraDebugGraph<3> NoiseValues;
#endif
};

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FPerlinNoiseLocationShakeDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TCameraDebugGraph<3>, NoiseValues)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FPerlinNoiseLocationShakeDebugBlock)

UE_DEFINE_SHAKE_CAMERA_NODE_EVALUATOR(FPerlinNoiseLocationShakeCameraNodeEvaluator)

void FPerlinNoiseLocationShakeCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UPerlinNoiseLocationShakeCameraNode* ShakeNode = GetCameraNodeAs<UPerlinNoiseLocationShakeCameraNode>();

	AmplitudeMultiplier.Initialize(ShakeNode->AmplitudeMultiplier);
	FrequencyMultiplier.Initialize(ShakeNode->FrequencyMultiplier);
	Octaves.Initialize(ShakeNode->Octaves);

	GeneratorX = FPerlinNoise(
			ShakeNode->X.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->X.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
	GeneratorY = FPerlinNoise(
			ShakeNode->Y.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->Y.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
	GeneratorY = FPerlinNoise(
			ShakeNode->Y.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->Y.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
}

void FPerlinNoiseLocationShakeCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UPerlinNoiseLocationShakeCameraNode* ShakeNode = GetCameraNodeAs<UPerlinNoiseLocationShakeCameraNode>();

	GeneratorX.SetAmplitude(ShakeNode->X.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorX.SetFrequency(ShakeNode->X.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorX.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	GeneratorY.SetAmplitude(ShakeNode->Y.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorY.SetFrequency(ShakeNode->Y.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorY.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	GeneratorZ.SetAmplitude(ShakeNode->Z.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorZ.SetFrequency(ShakeNode->Z.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorZ.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	const float ValueX = GeneratorX.GenerateValue(Params.DeltaTime);
	const float ValueY = GeneratorY.GenerateValue(Params.DeltaTime);
	const float ValueZ = GeneratorZ.GenerateValue(Params.DeltaTime);

	NoiseValue = FVector3d(ValueX, ValueY, ValueZ);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	NoiseValues.Add(Params.DeltaTime, ValueX, ValueY, ValueZ);
#endif
}

void FPerlinNoiseLocationShakeCameraNodeEvaluator::OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult)
{
	OutResult.ShakeDelta.Location += NoiseValue;
	OutResult.ShakeTimeLeft = -1.f;
}

void FPerlinNoiseLocationShakeCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << GeneratorX;
	Ar << GeneratorY;
	Ar << GeneratorZ;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FPerlinNoiseLocationShakeCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FPerlinNoiseLocationShakeDebugBlock& DebugBlock = Builder.AttachDebugBlock<FPerlinNoiseLocationShakeDebugBlock>();
	DebugBlock.NoiseValues = NoiseValues;
}

void FPerlinNoiseLocationShakeDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.DrawGraph(NoiseValues, LOCTEXT("DebugGraphName", "Location Noise"));
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UPerlinNoiseLocationShakeCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPerlinNoiseLocationShakeCameraNodeEvaluator>();
}

#undef LOCTEXT_NAMESPACE

