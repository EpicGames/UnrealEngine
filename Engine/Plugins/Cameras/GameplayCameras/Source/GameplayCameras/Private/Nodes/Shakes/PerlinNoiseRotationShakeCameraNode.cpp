// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Shakes/PerlinNoiseRotationShakeCameraNode.h"

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugGraph.h"
#include "Debug/CameraDebugRenderer.h"
#include "Math/PerlinNoise.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerlinNoiseRotationShakeCameraNode)

#define LOCTEXT_NAMESPACE "PerlinNoiseRotationShakeCameraNode"

namespace UE::Cameras
{

class FPerlinNoiseRotationShakeCameraNodeEvaluator : public FShakeCameraNodeEvaluator
{
	UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FPerlinNoiseRotationShakeCameraNodeEvaluator)

public:

protected:

	// FShakeCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	TCameraParameterReader<float> AmplitudeMultiplier;
	TCameraParameterReader<float> FrequencyMultiplier;
	TCameraParameterReader<int32> Octaves;

	FPerlinNoise GeneratorYaw;
	FPerlinNoise GeneratorPitch;
	FPerlinNoise GeneratorRoll;

	FRotator3d NoiseValue;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	TCameraDebugGraph<3> NoiseValues;
#endif
};

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FPerlinNoiseRotationShakeDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TCameraDebugGraph<3>, NoiseValues)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FPerlinNoiseRotationShakeDebugBlock)

UE_DEFINE_SHAKE_CAMERA_NODE_EVALUATOR(FPerlinNoiseRotationShakeCameraNodeEvaluator)

void FPerlinNoiseRotationShakeCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::NeedsSerialize);

	const UPerlinNoiseRotationShakeCameraNode* ShakeNode = GetCameraNodeAs<UPerlinNoiseRotationShakeCameraNode>();

	AmplitudeMultiplier.Initialize(ShakeNode->AmplitudeMultiplier);
	FrequencyMultiplier.Initialize(ShakeNode->FrequencyMultiplier);
	Octaves.Initialize(ShakeNode->Octaves);

	GeneratorYaw = FPerlinNoise(
			ShakeNode->Yaw.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->Yaw.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
	GeneratorPitch = FPerlinNoise(
			ShakeNode->Pitch.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->Pitch.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
	GeneratorRoll = FPerlinNoise(
			ShakeNode->Roll.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable),
			ShakeNode->Roll.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable),
			Octaves.Get(OutResult.VariableTable));
}

void FPerlinNoiseRotationShakeCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	const UPerlinNoiseRotationShakeCameraNode* ShakeNode = GetCameraNodeAs<UPerlinNoiseRotationShakeCameraNode>();

	GeneratorYaw.SetAmplitude(ShakeNode->Yaw.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorYaw.SetFrequency(ShakeNode->Yaw.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorYaw.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	GeneratorPitch.SetAmplitude(ShakeNode->Pitch.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorPitch.SetFrequency(ShakeNode->Pitch.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorPitch.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	GeneratorRoll.SetAmplitude(ShakeNode->Roll.Amplitude * AmplitudeMultiplier.Get(OutResult.VariableTable));
	GeneratorRoll.SetFrequency(ShakeNode->Roll.Frequency * FrequencyMultiplier.Get(OutResult.VariableTable));
	GeneratorRoll.SetNumOctaves(Octaves.Get(OutResult.VariableTable));

	const float ValueYaw = GeneratorYaw.GenerateValue(Params.DeltaTime);
	const float ValuePitch = GeneratorPitch.GenerateValue(Params.DeltaTime);
	const float ValueRoll = GeneratorRoll.GenerateValue(Params.DeltaTime);
	NoiseValue = FRotator3d(ValuePitch, ValueYaw, ValueRoll);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	NoiseValues.Add(Params.DeltaTime, ValueYaw, ValuePitch, ValueRoll);
#endif
}

void FPerlinNoiseRotationShakeCameraNodeEvaluator::OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult)
{
	OutResult.ShakeDelta.Rotation += NoiseValue;
	OutResult.ShakeTimeLeft = -1.f;
}

void FPerlinNoiseRotationShakeCameraNodeEvaluator::OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Super::OnSerialize(Params, Ar);

	Ar << GeneratorYaw;
	Ar << GeneratorPitch;
	Ar << GeneratorRoll;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FPerlinNoiseRotationShakeCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FPerlinNoiseRotationShakeDebugBlock& DebugBlock = Builder.AttachDebugBlock<FPerlinNoiseRotationShakeDebugBlock>();
	DebugBlock.NoiseValues = NoiseValues;
}

void FPerlinNoiseRotationShakeDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.DrawGraph(NoiseValues, LOCTEXT("DebugGraphName", "Rotation Noise"));
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UPerlinNoiseRotationShakeCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FPerlinNoiseRotationShakeCameraNodeEvaluator>();
}

#undef LOCTEXT_NAMESPACE

