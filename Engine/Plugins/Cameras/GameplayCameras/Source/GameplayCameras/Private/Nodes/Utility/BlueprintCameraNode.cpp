// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Utility/BlueprintCameraNode.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraObjectBuildContext.h"
#include "Components/ActorComponent.h"
#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableTable.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameplayCameras.h"
#include "Helpers/CameraObjectInterfaceParameterOverrideHelper.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintCameraNode)

#define LOCTEXT_NAMESPACE "BlueprintCameraNode"

namespace UE::Cameras
{

class FBlueprintCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBlueprintCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	 virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif

private:

	void ApplyParameterOverrides(const FCameraVariableTable& VariableTable, const FCameraContextDataTable& ContextDataTable);

private:

	TObjectPtr<UBlueprintCameraNodeEvaluator> EvaluatorBlueprint;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FBlueprintCameraNodeEvaluator)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FBlueprintCameraDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, BlueprintEvaluatorName);
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FBlueprintCameraDebugBlock)


void FBlueprintCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const UBlueprintCameraNode* BlueprintNode = GetCameraNodeAs<UBlueprintCameraNode>();
	if (!ensure(BlueprintNode))
	{
		return;
	}

#if WITH_EDITOR
	if (!BlueprintNode->bRunInEditor && Params.Evaluator->GetRole() == ECameraSystemEvaluatorRole::EditorPreview)
	{
		return;
	}
#endif  // WITH_EDITOR

	if (const UBlueprintCameraNodeEvaluator* EvaluatorTemplate = BlueprintNode->GetCameraNodeEvaluatorTemplate())
	{
		UObject* Outer = Params.EvaluationContext->GetOwner();
		EvaluatorBlueprint = CastChecked<UBlueprintCameraNodeEvaluator>(StaticDuplicateObject(EvaluatorTemplate, Outer, NAME_None));

		ApplyParameterOverrides(OutResult.VariableTable, OutResult.ContextDataTable);

		EvaluatorBlueprint->NativeInitializeCameraNode(BlueprintNode, Params, OutResult);
	}
	else
	{
		UE_LOGF(LogCameraSystem, Error, "No Blueprint class set on camera node '%ls'.", *GetNameSafe(BlueprintNode));
	}
}

void FBlueprintCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	if (EvaluatorBlueprint)
	{
		ApplyParameterOverrides(OutResult.VariableTable, OutResult.ContextDataTable);

		EvaluatorBlueprint->NativeRunCameraNode(Params, OutResult);
	}
}

void FBlueprintCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EvaluatorBlueprint);
}

void FBlueprintCameraNodeEvaluator::ApplyParameterOverrides(const FCameraVariableTable& VariableTable, const FCameraContextDataTable& ContextDataTable)
{
	using namespace Internal;

	const UClass* EvaluatorBlueprintClass = EvaluatorBlueprint->GetClass();
	const UBlueprintCameraNode* BlueprintNode = GetCameraNodeAs<UBlueprintCameraNode>();
	const FCustomCameraNodeParameters& Overrides = BlueprintNode->CameraNodeEvaluatorOverrides;

	for (const FCustomCameraNodeBlendableParameter& BlendableParameter : Overrides.BlendableParameters)
	{
		FProperty* Property = EvaluatorBlueprintClass->FindPropertyByName(BlendableParameter.ParameterName);
		if (ensure(Property))
		{
			if (BlendableParameter.OverrideVariableID)
			{
				const uint8* ValuePtr = VariableTable.TryGetValue(
						BlendableParameter.OverrideVariableID,
						BlendableParameter.VariableType,
						BlendableParameter.BlendableStructType);
				if (ValuePtr)
				{
					Property->SetValue_InContainer(EvaluatorBlueprint, ValuePtr);
				}
			}
		}
	}

	// Set the value of any properties driven by a data parameter.
	for (const FCustomCameraNodeDataParameter& DataParameter : Overrides.DataParameters)
	{
		FProperty* Property = EvaluatorBlueprintClass->FindPropertyByName(DataParameter.ParameterName);
		if (ensure(Property))
		{
			if (DataParameter.OverrideDataID)
			{
				const uint8* DataPtr = ContextDataTable.TryGetData(
						DataParameter.OverrideDataID,
						DataParameter.DataType,
						DataParameter.DataTypeObject);
				if (DataPtr)
				{
					Property->SetValue_InContainer(EvaluatorBlueprint, DataPtr);
				}
			}
		}
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FBlueprintCameraNodeEvaluator::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FBlueprintCameraDebugBlock& DebugBlock = Builder.AttachDebugBlock<FBlueprintCameraDebugBlock>();

	const UBlueprintCameraNode* BlueprintNode = GetCameraNodeAs<UBlueprintCameraNode>();
	if (BlueprintNode->CameraNodeEvaluatorTemplate)
	{
		const UClass* BlueprintEvaluatorClass = BlueprintNode->CameraNodeEvaluatorTemplate->GetClass();
		DebugBlock.BlueprintEvaluatorName = GetNameSafe(BlueprintEvaluatorClass);
	}
}

void FBlueprintCameraDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(BlueprintEvaluatorName);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG


}  // namespace UE::Cameras

void UBlueprintCameraNodeEvaluator::NativeInitializeCameraNode(const UBlueprintCameraNode* InBlueprintNode, const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	using namespace UE::Cameras;

	ensure(BlueprintNode == nullptr);
	BlueprintNode = InBlueprintNode;

	SetupExecution(Params.EvaluationContext, OutResult);
	{
		bIsFirstFrame = true;

		InitializeCameraNode();
	}
	TeardownExecution();
}

void UBlueprintCameraNodeEvaluator::NativeRunCameraNode(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	using namespace UE::Cameras;

	SetupExecution(Params.EvaluationContext, OutResult);
	{
		bIsFirstFrame = Params.bIsFirstFrame;
		bIsActiveCameraRig = Params.bIsActiveCameraRig;

		TickCameraNode(Params.DeltaTime);
	}
	TeardownExecution();
}

void UBlueprintCameraNodeEvaluator::SetupExecution(TSharedPtr<const FCameraEvaluationContext> EvaluationContext, FCameraNodeEvaluationResult& OutResult)
{
	EvaluationContextOwner = EvaluationContext->GetOwner();

	ensure(!CameraData.IsValid());
	CameraData = FBlueprintCameraEvaluationDataRef::MakeExternalRef(&OutResult);
	VariableTable = FBlueprintCameraEvaluationDataRef::MakeExternalRef(&OutResult);;

	ensure(!CurrentContext.IsValid());
	CurrentContext = EvaluationContext;
}

void UBlueprintCameraNodeEvaluator::TeardownExecution()
{
	VariableTable = FBlueprintCameraEvaluationDataRef();
	CameraData = FBlueprintCameraEvaluationDataRef();
	CurrentContext = nullptr;
}

AActor* UBlueprintCameraNodeEvaluator::FindEvaluationContextOwnerActor(TSubclassOf<AActor> ActorClass) const
{
	if (CurrentContext)
	{
		AActor* OwnerActor = nullptr;

		if (UActorComponent* ContextOwnerAsComponent = Cast<UActorComponent>(CurrentContext->GetOwner()))
		{
			OwnerActor = ContextOwnerAsComponent->GetOwner();
		}
		else if (AActor* ContextOwnerAsActor = Cast<AActor>(CurrentContext->GetOwner()))
		{
			OwnerActor = ContextOwnerAsActor;
		}

		if (OwnerActor && (!ActorClass || OwnerActor->IsA(ActorClass)))
		{
			return OwnerActor;
		}
		return nullptr;
	}
	else
	{
		FFrame::KismetExecutionMessage(
				TEXT("Can't access evaluation context outside of RunCameraDirector"), 
				ELogVerbosity::Error);
		return nullptr;
	}
}

FBlueprintCameraPose UBlueprintCameraNodeEvaluator::GetCurrentCameraPose() const
{
	return UBlueprintCameraEvaluationDataFunctionLibrary::GetCameraPose(CameraData);
}

void UBlueprintCameraNodeEvaluator::SetCurrentCameraPose(const FBlueprintCameraPose& InCameraPose)
{
	UBlueprintCameraEvaluationDataFunctionLibrary::SetCameraPose(CameraData, InCameraPose);
}

void UBlueprintCameraNodeEvaluator::SetDefaultOwningCameraRigParameters(FBlueprintCameraEvaluationDataRef TargetCameraData) const
{
	using namespace UE::Cameras;

	if (FCameraNodeEvaluationResult* Result = TargetCameraData.GetResult())
	{
		const UCameraRigAsset* OwningCameraRig = BlueprintNode->GetTypedOuter<UCameraRigAsset>();
		FCameraObjectInterfaceParameterOverrideHelper Helper(*Result);
		Helper.ApplyParameterDefaults(OwningCameraRig);
	}
}

UCameraRigAsset* UBlueprintCameraNodeEvaluator::GetCameraRig() const
{
	return BlueprintNode->GetTypedOuter<UCameraRigAsset>();
}

APlayerController* UBlueprintCameraNodeEvaluator::GetPlayerController() const
{
	if (CurrentContext)
	{
		return CurrentContext->GetPlayerController();
	}

	return nullptr;
}

UWorld* UBlueprintCameraNodeEvaluator::GetWorld() const
{
	if (UWorld* CachedWorld = WeakCachedWorld.Get())
	{
		return CachedWorld;
	}

	if (HasAllFlags(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	UObject* Outer = GetOuter();
	while (Outer)
	{
		UWorld* World = Outer->GetWorld();
		if (World)
		{
			WeakCachedWorld = World;
			return World;
		}

		Outer = Outer->GetOuter();
	}

	return nullptr;
}

UBlueprintCameraNode::UBlueprintCameraNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UBlueprintCameraNode::OnObjectsReplaced);
#endif
}

void UBlueprintCameraNode::PostLoad()
{
	Super::PostLoad();

	if (CameraNodeEvaluatorClass_DEPRECATED)
	{
		CameraNodeEvaluatorTemplate = NewObject<UBlueprintCameraNodeEvaluator>(
				this, CameraNodeEvaluatorClass_DEPRECATED, NAME_None, RF_Transactional);

		RebuildOverrides();

		CameraNodeEvaluatorClass_DEPRECATED = nullptr;
	}
}

void UBlueprintCameraNode::BeginDestroy()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif

	Super::BeginDestroy();
}

void UBlueprintCameraNode::RebuildOverrides()
{
	using namespace UE::Cameras::Internal;

	// If there is no evaluator set, clear all overrides.
	if (!CameraNodeEvaluatorTemplate)
	{
		if (CameraNodeEvaluatorOverrides.HasAnyParameters())
		{
			Modify();

			CameraNodeEvaluatorOverrides.Reset();
		}
		return;
	}

	// Remember the overrides already present on parameters.
	TMap<FName, UCameraVariableAsset*> OldOverrideVariableMap;
	TMap<FName, FCameraVariableID> OldOverrideVariableIDMap;
	for (const FCustomCameraNodeBlendableParameter& OldOverride : CameraNodeEvaluatorOverrides.BlendableParameters)
	{
		if (OldOverride.OverrideVariable)
		{
			OldOverrideVariableMap.Add(OldOverride.ParameterName, OldOverride.OverrideVariable);
		}
		if (OldOverride.OverrideVariableID)
		{
			OldOverrideVariableIDMap.Add(OldOverride.ParameterName, OldOverride.OverrideVariableID);
		}
	}
	TMap<FName, FCameraContextDataID> OldOverrideDataIDMap;
	for (const FCustomCameraNodeDataParameter& OldOverride : CameraNodeEvaluatorOverrides.DataParameters)
	{
		if (OldOverride.OverrideDataID)
		{
			OldOverrideDataIDMap.Add(OldOverride.ParameterName, OldOverride.OverrideDataID);
		}
	}

	// Build the new list of blendable and data parameters.
	// All exposed blendable properties on the Blueprint class show up as blendable parameters.
	// All other exposed properties on the Blueprint class show up as data parameters.
	FCustomCameraNodeParameters NewOverrides;

	for (TFieldIterator<FProperty> PropertyIt(CameraNodeEvaluatorTemplate->GetClass()); PropertyIt; ++PropertyIt)
	{
		FProperty* Property(*PropertyIt);

		if (!Property->GetOwnerClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			continue;
		}
		if (Property->HasAnyPropertyFlags(CPF_EditorOnly | CPF_Protected | CPF_DisableEditOnInstance))
		{
			continue;
		}
		if (!Property->HasAnyPropertyFlags(CPF_Edit))
		{
			continue;
		}

		bool bIsBlendableProperty = false;
		ECameraVariableType BlendablePropertyType = ECameraVariableType::Boolean;

		bool bIsDataProperty = false;
		ECameraContextDataType DataPropertyType = ECameraContextDataType::Name;
		ECameraContextDataContainerType DataPropertyContainerType = ECameraContextDataContainerType::None;
		const UObject* DataPropertyTypeObject = nullptr;

		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			bIsBlendableProperty = true;
			BlendablePropertyType = ECameraVariableType::Boolean;
		}
		else if (FIntProperty* Int32Property = CastField<FIntProperty>(Property))
		{
			bIsBlendableProperty = true;
			BlendablePropertyType = ECameraVariableType::Integer32;
		}
		else if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
		{
			bIsBlendableProperty = true;
			BlendablePropertyType = ECameraVariableType::Float;
		}
		else if (FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
		{
			bIsBlendableProperty = true;
			BlendablePropertyType = ECameraVariableType::Double;
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct == TVariantStructure<FVector2f>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector2f;
			}
			else if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector2d;
			}
			else if (StructProperty->Struct == TVariantStructure<FVector3f>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector3f;
			}
			else if (StructProperty->Struct == TBaseStructure<FVector>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector3d;
			}
			else if (StructProperty->Struct == TVariantStructure<FVector4f>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector4f;
			}
			else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Vector4d;
			}
			else if (StructProperty->Struct == TVariantStructure<FRotator3f>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Rotator3f;
			}
			else if (StructProperty->Struct == TBaseStructure<FRotator>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Rotator3d;
			}
			else if (StructProperty->Struct == TVariantStructure<FTransform3f>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Transform3f;
			}
			else if (StructProperty->Struct == TBaseStructure<FTransform>::Get())
			{
				bIsBlendableProperty = true;
				BlendablePropertyType = ECameraVariableType::Transform3d;
			}
			// TODO: make blendable property if the struct is registered as blendable
			else
			{
				bIsDataProperty = true;
				DataPropertyType = ECameraContextDataType::Struct;
				DataPropertyTypeObject = StructProperty->Struct;
			}
		}
		else if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
		{
			bIsDataProperty = true;
			DataPropertyType = ECameraContextDataType::Name;
		}
		else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property))
		{
			bIsDataProperty = true;
			DataPropertyType = ECameraContextDataType::String;
		}
		else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			bIsDataProperty = true;
			DataPropertyType = ECameraContextDataType::Enum;
			DataPropertyTypeObject = EnumProperty->GetEnum();
		}
		else if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
		{
			bIsDataProperty = true;
			DataPropertyType = ECameraContextDataType::Class;
			DataPropertyTypeObject = ClassProperty->MetaClass;
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			bIsDataProperty = true;
			DataPropertyType = ECameraContextDataType::Object;
			DataPropertyTypeObject = ObjectProperty->PropertyClass;
		}

		if (bIsBlendableProperty)
		{
			FCustomCameraNodeBlendableParameter NewOverride;
			NewOverride.ParameterName = Property->GetFName();
			NewOverride.VariableType = BlendablePropertyType;

			// If this blendable parameter existed before and had an overriding variable set,
			// preserve that override.
			UCameraVariableAsset* OldOverrideVariable = nullptr;
			OldOverrideVariableMap.RemoveAndCopyValue(NewOverride.ParameterName, OldOverrideVariable);
			if (OldOverrideVariable && OldOverrideVariable->GetVariableType() == BlendablePropertyType)
			{
				NewOverride.OverrideVariable = OldOverrideVariable;
			}
			FCameraVariableID OldOverrideVariableID;
			OldOverrideVariableIDMap.RemoveAndCopyValue(NewOverride.ParameterName, OldOverrideVariableID);
			if (OldOverrideVariableID)
			{
				NewOverride.OverrideVariableID = OldOverrideVariableID;
			}

			NewOverrides.BlendableParameters.Add(NewOverride);
		}
		else if (bIsDataProperty)
		{
			FCustomCameraNodeDataParameter NewOverride;
			NewOverride.ParameterName = Property->GetFName();
			NewOverride.DataType = DataPropertyType;
			NewOverride.DataTypeObject = DataPropertyTypeObject;

			// If this data parameter existed before and had an override data ID set,
			// preserve that override.
			FCameraContextDataID OldOverrideDataID;
			OldOverrideDataIDMap.RemoveAndCopyValue(NewOverride.ParameterName, OldOverrideDataID);
			if (OldOverrideDataID)
			{
				NewOverride.OverrideDataID = OldOverrideDataID;
			}

			NewOverrides.DataParameters.Add(NewOverride);
		}
		else
		{
			UE_LOGF(
					LogCameraSystem, Warning, 
					"Property '%ls' on Blueprint camera node evaluator class '%ls' cannot be exposed as "
						 "neither a blendable or data parameter. The property type is not (yet) supported.",
					*Property->GetName(),
					*CameraNodeEvaluatorTemplate->GetClass()->GetName());
		}
	}

	if (NewOverrides != CameraNodeEvaluatorOverrides)
	{
		Modify();

		CameraNodeEvaluatorOverrides = NewOverrides;
	}
}

void UBlueprintCameraNode::OnPreBuild(FCameraBuildContext& BuildContext)
{
	RebuildOverrides();
}

void UBlueprintCameraNode::OnBuild(FCameraObjectBuildContext& BuildContext)
{
	if (!CameraNodeEvaluatorTemplate)
	{
		BuildContext.BuildLog.AddMessage(
				EMessageSeverity::Error, this,
				LOCTEXT("MissingBlueprintEvaluatorTemplate", "No evaluator Blueprint is set."));
		return;
	}
}

void UBlueprintCameraNode::GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos)
{
	using namespace UE::Cameras::Internal;

	if (!CameraNodeEvaluatorTemplate)
	{
		return;
	}

	const UClass* CameraNodeEvaluatorClass = CameraNodeEvaluatorTemplate->GetClass();

	for (FCustomCameraNodeBlendableParameter& BlendableParameter : CameraNodeEvaluatorOverrides.BlendableParameters)
	{
		FProperty* BlendableProperty = CameraNodeEvaluatorClass->FindPropertyByName(BlendableParameter.ParameterName);
		if (!BlendableProperty)
		{
			continue;
		}

		const void* DefaultValuePtr = BlendableProperty->ContainerPtrToValuePtr<void>(CameraNodeEvaluatorTemplate);
		OutParameterInfos.AddBlendableParameter(BlendableParameter, (const uint8*)DefaultValuePtr);
	}

	for (FCustomCameraNodeDataParameter& DataParameter : CameraNodeEvaluatorOverrides.DataParameters)
	{
		FProperty* DataProperty = CameraNodeEvaluatorClass->FindPropertyByName(DataParameter.ParameterName);
		if (!DataProperty)
		{
			continue;
		}

		const void* DefaultValuePtr = DataProperty->ContainerPtrToValuePtr<void>(CameraNodeEvaluatorTemplate);
		OutParameterInfos.AddDataParameter(DataParameter, (const uint8*)DefaultValuePtr);
	}
}

FCameraNodeEvaluatorPtr UBlueprintCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FBlueprintCameraNodeEvaluator>();
}

#if WITH_EDITOR

void UBlueprintCameraNode::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	UObject* NewEvaluatorTemplate = ReplacementMap.FindRef(CameraNodeEvaluatorTemplate);
	if (NewEvaluatorTemplate)
	{
		CameraNodeEvaluatorTemplate = CastChecked<UBlueprintCameraNodeEvaluator>(NewEvaluatorTemplate);

		RebuildOverrides();

		OnCustomCameraNodeParametersChanged(this);
	}
}

void UBlueprintCameraNode::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UBlueprintCameraNode, CameraNodeEvaluatorTemplate))
	{
		RebuildOverrides();

		OnCustomCameraNodeParametersChanged(this);
	}
}

EObjectTreeGraphObjectSupportFlags UBlueprintCameraNode::GetSupportFlags(FName InGraphName) const
{
	return (Super::GetSupportFlags(InGraphName) | EObjectTreeGraphObjectSupportFlags::CustomTitle);
}

void UBlueprintCameraNode::GetGraphNodeName(FName InGraphName, FText& OutName) const
{
	const UClass* EvaluatorBlueprintClass = CameraNodeEvaluatorTemplate ? CameraNodeEvaluatorTemplate->GetClass() : nullptr;
	const FText EvaluatorBlueprintName = EvaluatorBlueprintClass ? EvaluatorBlueprintClass->GetDisplayNameText() : LOCTEXT("None", "None");

	OutName = FText::Format(LOCTEXT("GraphNodeNameFormat", "Blueprint ({0})"), EvaluatorBlueprintName);
}

#endif  // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

