// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraAction.h"

#include "Services/CameraActionEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAction)

FCameraActionEvaluatorPtr UCameraAction::BuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	FCameraActionEvaluator* NewEvaluator = OnBuildEvaluator(Builder);
	NewEvaluator->SetPrivateCameraAction(this);
	return NewEvaluator;
}

