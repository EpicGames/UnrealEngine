// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraActionEvaluator.h"

#include "Services/CameraAction.h"

namespace UE::Cameras
{

UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(FCameraActionEvaluator)

void FCameraActionEvaluator::Initialize(const FCameraActionEvaluatorInitializeParams& Params, FCameraActionEvaluationResult& OutResult)
{
	OnInitialize(Params, OutResult);
}

void FCameraActionEvaluator::Teardown(const FCameraActionEvaluatorTeardownParams& Params)
{
	OnTeardown(Params);
}

void FCameraActionEvaluator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PrivateCameraAction);
	OnAddReferencedObjects(Collector);
}

void FCameraActionEvaluator::PreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	OnPreScopeRun(Params, OutResult);
}

void FCameraActionEvaluator::PostScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	OnPostScopeRun(Params, OutResult);
}

void FCameraActionEvaluator::Serialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar)
{
	OnSerialize(Params, Ar);
}

void FCameraActionEvaluator::SetPrivateCameraAction(TObjectPtr<const UCameraAction> InCameraAction)
{
	PrivateCameraAction = InCameraAction;
}

void FCameraActionEvaluator::SetActionEvaluatorFlags(ECameraActionEvaluatorFlags InFlags)
{
	PrivateFlags = InFlags;
}

void FCameraActionEvaluator::AddActionEvaluatorFlags(ECameraActionEvaluatorFlags InFlags)
{
	PrivateFlags |= InFlags;
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FCameraActionEvaluator::BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	OnBuildDebugBlocks(Params, Builder);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

