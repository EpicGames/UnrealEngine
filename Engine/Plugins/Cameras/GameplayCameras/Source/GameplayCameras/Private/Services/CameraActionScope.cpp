// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraActionScope.h"

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigAsset.h"  // IWYU pragma: keep
#include "Core/CameraRigEvaluationInfo.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugRenderer.h"
#include "Services/CameraAction.h"
#include "Services/CameraActionEvaluator.h"
#include "Services/CameraActionInstanceID.h"
#include "Templates/SharedPointer.h"

namespace UE::Cameras
{

FCameraActionInstanceID FCameraActionScope::StartAction(const UCameraAction* InCameraAction)
{
	if (InCameraAction)
	{
		FCameraActionEvaluatorBuilder Builder;
		InCameraAction->BuildEvaluator(Builder);
		if (TSharedPtr<FCameraActionEvaluator> Evaluator = Builder.BuiltEvaluator)
		{
			FActionInfo& NewAction = Actions.Emplace_GetRef();
			NewAction.Evaluator = Evaluator;
			NewAction.InstanceID = FCameraActionInstanceID(NextActionID++);
			return NewAction.InstanceID;
		}
	}
	return FCameraActionInstanceID();
}

bool FCameraActionScope::IsActionRunning(const FCameraActionInstanceID InInstanceID) const
{
	return Actions.ContainsByPredicate([InInstanceID](const FActionInfo& Item)
			{
				return Item.InstanceID == InInstanceID;
			});
}

FCameraActionEvaluator* FCameraActionScope::GetAction(const FCameraActionInstanceID InInstanceID) const
{
	const FActionInfo* ActionInfo = Actions.FindByPredicate([InInstanceID](const FActionInfo& Item)
			{
				return Item.InstanceID == InInstanceID;
			});
	if (ActionInfo)
	{
		return ActionInfo->Evaluator.Get();
	}
	return nullptr;
}

bool FCameraActionScope::StopAction(const FCameraActionInstanceID InInstanceID)
{
	const int32 Index = Actions.IndexOfByPredicate([InInstanceID](const FActionInfo& Item)
			{
				return Item.InstanceID == InInstanceID;
			});
	if (Index != INDEX_NONE)
	{
		FActionInfo& ActionInfo = Actions[Index];
		StopAction(ActionInfo);
		Actions.RemoveAt(Index);
		return true;
	}
	return false;
}

void FCameraActionScope::StopAction(FActionInfo& ActionInfo)
{
	// Only teardown the evaluator if it was initialized in the first place.
	if (ActionInfo.State == EActionState::Running && ActionInfo.Evaluator.IsValid())
	{
		FCameraActionEvaluatorTeardownParams Params;
		Params.Scope = SharedThis(this);
		ActionInfo.Evaluator->Teardown(Params);
	}
	ActionInfo.Evaluator.Reset();
	ActionInfo.State = EActionState::Finished;
}

void FCameraActionScope::Initialize(const FCameraRigEvaluationInfo& CameraRigInfo)
{
	// Don't store CameraRigInfo directly because we want to keep the evaluation context
	// as a weak pointer.
	CameraRigInstanceID = CameraRigInfo.InstanceID;
	EvaluationContext = CameraRigInfo.EvaluationContext;
	CameraRig = CameraRigInfo.CameraRig;
	ChildResult = CameraRigInfo.LastResult;
	ChildEvaluator = CameraRigInfo.RootEvaluator;
}

FCameraRigEvaluationInfo FCameraActionScope::GetCameraRigEvaluationInfo() const
{
	FCameraRigEvaluationInfo CameraRigEvaluationInfo(
			CameraRigInstanceID,
			EvaluationContext.Pin(),
			CameraRig,
			ChildResult,
			ChildEvaluator);
	return CameraRigEvaluationInfo;
}

void FCameraActionScope::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraRig);

	for (FActionInfo& ActionInfo : Actions)
	{
		ActionInfo.Evaluator->AddReferencedObjects(Collector);
	}
}

void FCameraActionScope::PreScopeRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraActionEvaluationParams ActionParams(Params);
	ActionParams.Scope = SharedThis(this);

	for (auto It = Actions.CreateIterator(); It; ++It)
	{
		FActionInfo& ActionInfo(*It);
		FCameraActionEvaluationResult ActionResult(OutResult);
		if (!PrepareActionForRun(ActionInfo, ActionResult))
		{
			StopAction(ActionInfo);
			It.RemoveCurrent();
			continue;
		}

		// Only check for time-out on pre-scope since we don't want to update ElapsedTime twice in a frame.
		if (HasActionTimedOut(Params.DeltaTime, ActionInfo))
		{
			StopAction(ActionInfo);
			It.RemoveCurrent();
			continue;
		}

		ActionInfo.Evaluator->PreScopeRun(ActionParams, ActionResult);
		if (ActionResult.bIsActionFinished)
		{
			ActionInfo.State = EActionState::Finished;
			if (!Params.IsStatelessEvaluation())
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FCameraActionScope::PostScopeRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraActionEvaluationParams ActionParams(Params);

	for (auto It = Actions.CreateIterator(); It; ++It)
	{
		FActionInfo& ActionInfo(*It);
		FCameraActionEvaluationResult ActionResult(OutResult);
		if (!PrepareActionForRun(ActionInfo, ActionResult))
		{
			StopAction(ActionInfo);
			It.RemoveCurrent();
			continue;
		}

		ActionInfo.Evaluator->PostScopeRun(ActionParams, ActionResult);
		if (ActionResult.bIsActionFinished)
		{
			ActionInfo.State = EActionState::Finished;
			if (!Params.IsStatelessEvaluation())
			{
				It.RemoveCurrent();
			}
		}
	}
}

bool FCameraActionScope::PrepareActionForRun(FActionInfo& ActionInfo, FCameraActionEvaluationResult& OutResult)
{
	if (!ensure(ActionInfo.Evaluator))
	{
		return false;
	}

	if (ActionInfo.State == EActionState::Uninitialized)
	{
		FCameraActionEvaluatorInitializeParams InitParams;
		InitParams.Scope = SharedThis(this);
		ActionInfo.Evaluator->Initialize(InitParams, OutResult);
		ActionInfo.State = EActionState::Running;
	}

	return ActionInfo.State == EActionState::Running;
}

bool FCameraActionScope::HasActionTimedOut(float DeltaTime, FActionInfo& ActionInfo)
{
	ActionInfo.ElapsedTime += DeltaTime;
	const UCameraAction* ActionData = ActionInfo.Evaluator->GetCameraAction();
	if (ensure(ActionData))
	{
		return (ActionData->TimeOut > 0.f && ActionInfo.ElapsedTime >= ActionData->TimeOut);
	}
	return true;
}

void FCameraActionScope::Serialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar)
{
	int32 NumActionsToSerialize = Actions.Num();

	if (Ar.IsSaving())
	{
		int32 NumActions = Actions.Num();
		Ar << NumActions;
	}
	else if (Ar.IsLoading())
	{
		int32 LoadedNumActions = 0;
		Ar << LoadedNumActions;

		ensureMsgf(
				LoadedNumActions == Actions.Num(),
				TEXT("The number of actions changed since this action scope was serialized!"));
		NumActionsToSerialize = LoadedNumActions;
	}

	for (int32 Index = 0; Index < NumActionsToSerialize; ++Index)
	{
		FActionInfo& ActionInfo(Actions[Index]);
		ActionInfo.Evaluator->Serialize(Params, Ar);
		Ar << ActionInfo.InstanceID;

		uint8 RawState = (uint8)ActionInfo.State;
		Ar << RawState;
		ActionInfo.State = (EActionState)RawState;
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FCameraActionScopeDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(, FCameraActionScopeDebugBlock)

public:

	FCameraActionScopeDebugBlock() = default;
	FCameraActionScopeDebugBlock(const FString& InScopeCameraRigName);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FString ScopeCameraRigName;
};

class FCameraActionDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(, FCameraActionDebugBlock)

public:

	FCameraActionDebugBlock() = default;
	FCameraActionDebugBlock(const UCameraAction* InCameraAction);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FString ActionClassName;
};

void FCameraActionScope::BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	const FString CameraRigName = GetNameSafe(CameraRig);
	FCameraActionScopeDebugBlock& DebugBlock = Builder.StartChildDebugBlock<FCameraActionScopeDebugBlock>(CameraRigName);
	{
		for (const FActionInfo& ActionInfo : Actions)
		{
			if (ActionInfo.Evaluator)
			{
				const UCameraAction* ActionData = ActionInfo.Evaluator->GetCameraAction();
				Builder.StartChildDebugBlock<FCameraActionDebugBlock>(ActionData);
				{
					ActionInfo.Evaluator->BuildDebugBlocks(Params, Builder);
				}
				Builder.EndChildDebugBlock();
			}
		}
	}
	Builder.EndChildDebugBlock();
}

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraActionScopeDebugBlock)

FCameraActionScopeDebugBlock::FCameraActionScopeDebugBlock(const FString& InScopeCameraRigName)
	: ScopeCameraRigName(InScopeCameraRigName)
{
}

void FCameraActionScopeDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("%d action(s) for {cam_notice}%s{cam_default}"), GetChildren().Num(), *ScopeCameraRigName);
}

void FCameraActionScopeDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << ScopeCameraRigName;
}


UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraActionDebugBlock)

FCameraActionDebugBlock::FCameraActionDebugBlock(const UCameraAction* InCameraAction)
{
	ActionClassName = (InCameraAction ? InCameraAction->GetClass()->GetName() : TEXT("<no action>"));
}

void FCameraActionDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	Renderer.AddText(TEXT("{cam_passive}[%s]{cam_default} "), *ActionClassName);
}

void FCameraActionDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << ActionClassName;
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

