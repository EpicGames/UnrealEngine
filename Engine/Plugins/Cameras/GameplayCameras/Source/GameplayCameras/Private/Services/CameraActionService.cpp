// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraActionService.h"

#include "Containers/AllowShrinking.h"
#include "Core/CameraRigAsset.h"  // IWYU pragma: keep
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Core/RootCameraNodeCameraRigEvent.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "Misc/EngineVersionComparison.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Services/CameraAction.h"
#include "Services/CameraActionEvaluator.h"
#include "Services/CameraActionScope.h"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FCameraActionService)

FCameraActionService::~FCameraActionService()
{
	if (RootNodeEvaluator)
	{
		RootNodeEvaluator->OnCameraRigEvent().RemoveAll(this);
	}
}

void FCameraActionService::OnInitialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	SetEvaluationServiceFlags(
			ECameraEvaluationServiceFlags::NeedsPostCameraDirectorUpdate |
			ECameraEvaluationServiceFlags::NeedsPostUpdate);

	RootNodeEvaluator = Params.Evaluator->GetRootNodeEvaluator();
	if (ensure(RootNodeEvaluator))
	{
		RootNodeEvaluator->OnCameraRigEvent().AddRaw(this, &FCameraActionService::OnRootNodeCameraRigEvent);
	}
}

void FCameraActionService::OnPostCameraDirectorUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	for (FActionSetInfo& ActionSet : ActionSets)
	{
		if (!ensure(ActionSet.Data))
		{
			continue;
		}

		if (ActionSet.bIsPending)
		{
			// This action is pending and its first evaluator needs to be created on the active camera rig instance's
			// action scope.
			TSharedPtr<FCameraActionScope> ActionScope = RootNodeEvaluator->GetActiveCameraRigActionScope(true);
			if (ensure(ActionScope))
			{
				FCameraActionInstanceID NewInstanceID = ActionScope->StartAction(ActionSet.Data);
				if (ensure(NewInstanceID.IsValid()))
				{
					ActionSet.Actions.Add(FActionInfo{ ActionScope, NewInstanceID });
				}
			}
			ActionSet.bIsPending = false;
		}
		else if (bHasAnyNewActiveCameraRigs && ActionSet.Data->bPropagateToNewCameraRigs)
		{
			// This action is already running but a new camera rig instance has been activated, and the action wants
			// to propagate to it, so we clone its evaluator onto the new action scope.
			TSharedPtr<FCameraActionScope> ActionScope = RootNodeEvaluator->GetActiveCameraRigActionScope(true);
			if (ensure(ActionScope))
			{
				CloneAction(ActionSet, ActionScope.ToSharedRef());
			}
		}
	}

	bHasAnyNewActiveCameraRigs = false;
}

void FCameraActionService::OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	if (Params.EvaluationType == ECameraNodeEvaluationType::Standard)
	{
		CleanUpActions();
	}
}

void FCameraActionService::OnTeardown(const FCameraEvaluationServiceTeardownParams& Params)
{
	if (ensure(RootNodeEvaluator))
	{
		RootNodeEvaluator->OnCameraRigEvent().RemoveAll(this);
	}

	RootNodeEvaluator = nullptr;
	ActionSets.Reset();
}

void FCameraActionService::OnRootNodeCameraRigEvent(const FRootCameraNodeCameraRigEvent& InEvent)
{
	if (InEvent.EventType == ERootCameraNodeCameraRigEventType::Activated && InEvent.EventLayer == ECameraRigLayer::Main)
	{
		bHasAnyNewActiveCameraRigs = true;
	}
}

FCameraActionInstanceID FCameraActionService::StartAction(const UCameraAction* CameraAction)
{
	if (!CameraAction)
	{
		return FCameraActionInstanceID();
	}
	
	FActionSetInfo& NewActionSet = ActionSets.Emplace_GetRef();
	NewActionSet.Data = CameraAction;
	NewActionSet.InstanceID = FCameraActionInstanceID(NextActionSetID++);
	NewActionSet.bIsPending = true;

	return NewActionSet.InstanceID;
}

bool FCameraActionService::IsActionRunning(const FCameraActionInstanceID InInstanceID) const
{
	if (InInstanceID.IsValid())
	{
		return ActionSets.ContainsByPredicate([InInstanceID](const FActionSetInfo& Item)
				{
					return Item.InstanceID == InInstanceID;
				});
	}
	return false;
}

bool FCameraActionService::StopAction(const FCameraActionInstanceID InInstanceID)
{
	const int32 Index = ActionSets.IndexOfByPredicate([InInstanceID](const FActionSetInfo& Item)
			{
				return Item.InstanceID == InInstanceID;
			});
	if (Index != INDEX_NONE)
	{
		FActionSetInfo& ActionSet = ActionSets[Index];
		for (FActionInfo& Action : ActionSet.Actions)
		{
			TSharedPtr<FCameraActionScope> ActionScope = Action.ActionScope.Pin();
			if (ensure(ActionScope && Action.ActionID.IsValid()))
			{
				const bool bStopped = ActionScope->StopAction(Action.ActionID);
				ensure(bStopped);
			}
		}
		ActionSets.RemoveAt(Index);
		return true;
	}
	return false;
}

bool FCameraActionService::StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass)
{
	bool bAnyStopped = false;
	for (auto It = ActionSets.CreateIterator(); It; ++It)
	{
		FActionSetInfo& ActionSet(*It);
		if (ActionSet.Data && ActionSet.Data->IsA(InActionClass))
		{
			for (FActionInfo& Action : ActionSet.Actions)
			{
				if (TSharedPtr<FCameraActionScope> ActionScope = Action.ActionScope.Pin())
				{
					const bool bStopped = ActionScope->StopAction(Action.ActionID);
					ensure(bStopped);
					bAnyStopped = true;
				}
			}
			It.RemoveCurrent();
		}
	}
	return bAnyStopped;
}

int32 FCameraActionService::GetNumScopeActions(const FCameraActionInstanceID InInstanceID) const
{
	const int32 Index = ActionSets.IndexOfByPredicate([InInstanceID](const FActionSetInfo& Item)
			{
				return Item.InstanceID == InInstanceID;
			});
	if (Index != INDEX_NONE)
	{
		const FActionSetInfo& ActionSet = ActionSets[Index];
		return ActionSet.Actions.Num();
	}
	return 0;
}

void FCameraActionService::CloneAction(FActionSetInfo& ActionSet, TSharedRef<FCameraActionScope> NewActionScope)
{
	if (!ensure(ActionSet.Actions.Num() > 0))
	{
		return;
	}

	const FActionInfo& ActiveAction = ActionSet.Actions.Last();
	TSharedPtr<FCameraActionScope> ActiveActionScope = ActiveAction.ActionScope.Pin();
	if (!ensure(ActiveActionScope))
	{
		return;
	}

	FCameraActionEvaluator* ActiveEvaluator = ActiveActionScope->GetAction(ActiveAction.ActionID);
	if (!ensure(ActiveEvaluator))
	{
		return;
	}

	TArray<uint8> EvaluatorSnapshot;
	FCameraActionEvaluatorSerializeParams Params;
	{
		FMemoryWriter Writer(EvaluatorSnapshot);
		ActiveEvaluator->Serialize(Params, Writer);
	}

	const FCameraActionInstanceID NewInstanceID = NewActionScope->StartAction(ActionSet.Data);
	FCameraActionEvaluator* NewEvaluator = NewActionScope->GetAction(NewInstanceID);
	if (ensure(NewEvaluator))
	{
		FMemoryReader Reader(EvaluatorSnapshot);
		NewEvaluator->Serialize(Params, Reader);

		ActionSet.Actions.Add(FActionInfo{ NewActionScope, NewInstanceID });
	}
}

void FCameraActionService::CleanUpActions()
{
	for (auto SetIt = ActionSets.CreateIterator(); SetIt; ++SetIt)
	{
		FActionSetInfo& ActionSet(*SetIt);

		ensure(!ActionSet.bIsPending);

		for (auto ActionIt = ActionSet.Actions.CreateIterator(); ActionIt; ++ActionIt)
		{
			// Remove finished actions, whether they finished on their own, or were ended by their action scope 
			// being discarded.
			FActionInfo& ActionInfo(*ActionIt);
			if (TSharedPtr<FCameraActionScope> ActionScope = ActionInfo.ActionScope.Pin())
			{
				if (!ActionScope->IsActionRunning(ActionInfo.ActionID))
				{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
					ActionIt.RemoveCurrent(EAllowShrinking::No);
#else
					ActionIt.RemoveCurrent();
#endif
				}
			}
			else
			{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
				ActionIt.RemoveCurrent(EAllowShrinking::No);
#else
				ActionIt.RemoveCurrent();
#endif
			}
		}
		if (ActionSet.Actions.IsEmpty())
		{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,8,0)
			SetIt.RemoveCurrent(EAllowShrinking::No);
#else
			SetIt.RemoveCurrent();
#endif
		}
	}
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FCameraActionServiceDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(, FCameraActionServiceDebugBlock)

public:

	FCameraActionServiceDebugBlock() = default;
	FCameraActionServiceDebugBlock(const FCameraActionService& InService);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	struct FActionDebugInfo
	{
		FString ScopeCameraRigName;
		FCameraActionInstanceID ActionID;
	};

	struct FActionSetDebugInfo
	{
		FString ActionName;
		TArray<FActionDebugInfo> Actions;
		FCameraActionInstanceID InstanceID;
	};

	friend FArchive& operator<< (FArchive&, FActionDebugInfo&);
	friend FArchive& operator<< (FArchive&, FActionSetDebugInfo&);

	TArray<FActionSetDebugInfo> ActionSets;
};

FArchive& operator<< (FArchive& Ar, FCameraActionServiceDebugBlock::FActionDebugInfo& Item)
{
	Ar << Item.ScopeCameraRigName;
	Ar << Item.ActionID;
	return Ar;
}

FArchive& operator<< (FArchive& Ar, FCameraActionServiceDebugBlock::FActionSetDebugInfo& Item)
{
	Ar << Item.ActionName;
	Ar << Item.Actions;
	Ar << Item.InstanceID;
	return Ar;
}

UE_DEFINE_CAMERA_DEBUG_BLOCK(FCameraActionServiceDebugBlock)

FCameraActionServiceDebugBlock::FCameraActionServiceDebugBlock(const FCameraActionService& InService)
{
	for (const FCameraActionService::FActionSetInfo& ActionSet : InService.ActionSets)
	{
		FActionSetDebugInfo& ActionSetDebugInfo = ActionSets.Emplace_GetRef();
		ActionSetDebugInfo.ActionName = GetNameSafe(ActionSet.Data);
		ActionSetDebugInfo.InstanceID = ActionSet.InstanceID;

		for (const FCameraActionService::FActionInfo& Action : ActionSet.Actions)
		{
			FActionDebugInfo& ActionDebugInfo = ActionSetDebugInfo.Actions.Emplace_GetRef();
			ActionDebugInfo.ActionID = Action.ActionID;
			if (TSharedPtr<const FCameraActionScope> ActionScope = Action.ActionScope.Pin())
			{
				ActionDebugInfo.ScopeCameraRigName = GetNameSafe(ActionScope->GetCameraRig());
			}
			else
			{
				ActionDebugInfo.ScopeCameraRigName = TEXT("None");
			}
		}
	}
}

void FCameraActionServiceDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	for (const FActionSetDebugInfo& ActionSet : ActionSets)
	{
		Renderer.AddText(TEXT("Action Set: {cam_notice}%s{cam_default}\n"), *ActionSet.ActionName);
		Renderer.AddIndent();
		{
			for (const FActionDebugInfo& Action : ActionSet.Actions)
			{
				Renderer.AddText(TEXT("Action Scope {cam_notice2}%s{cam_default}\n"), *Action.ScopeCameraRigName);
			}
		}
		Renderer.RemoveIndent();
	}
}

void FCameraActionServiceDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << ActionSets;
}

void FCameraActionService::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	Builder.AttachDebugBlock<FCameraActionServiceDebugBlock>(*this);
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

} // namespace UE::Cameras

