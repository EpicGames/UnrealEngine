// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraModifierService.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "UObject/ObjectMacros.h"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FCameraModifierService)

void FCameraModifierService::OnInitialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::None);

	ensure(Evaluator == nullptr);
	Evaluator = Params.Evaluator;
}

void FCameraModifierService::OnTeardown(const FCameraEvaluationServiceTeardownParams& Params)
{
	ensure(Evaluator != nullptr);
	Evaluator = nullptr;
}

FCameraRigInstanceID FCameraModifierService::StartCameraModifierRig(const UCameraRigAsset* CameraRig, ECameraRigLayer Layer, int32 OrderKey)
{
	EnsureModifierContextCreated();

	return StartCameraModifierRig(CameraRig, ModifierContext.ToSharedRef(), Layer, OrderKey);
}

FCameraRigInstanceID FCameraModifierService::StartCameraModifierRig(const UCameraRigAsset* CameraRig, TSharedRef<FCameraEvaluationContext> EvaluationContext, ECameraRigLayer Layer, int32 OrderKey)
{
	if (ensure(Evaluator))
	{
		FActivateCameraRigParams ActivateParams;
		ActivateParams.EvaluationContext = EvaluationContext;
		ActivateParams.CameraRig = CameraRig;
		ActivateParams.Layer = Layer;
		ActivateParams.OrderKey = FirstBlendStackOrderKey + OrderKey;
		return Evaluator->GetRootNodeEvaluator()->ActivateCameraRig(ActivateParams);
	}
	return FCameraRigInstanceID();
}

void FCameraModifierService::StopCameraModifierRig(FCameraRigInstanceID CameraRigID, bool bImmediately)
{
	if (ensure(Evaluator))
	{
		FDeactivateCameraRigParams DeactivateParams;
		DeactivateParams.InstanceID = CameraRigID;
		DeactivateParams.bDeactiveImmediately = bImmediately;
		Evaluator->GetRootNodeEvaluator()->DeactivateCameraRig(DeactivateParams);
	}
}

void FCameraModifierService::EnsureModifierContextCreated()
{
	if (!ModifierContext)
	{
		ModifierContext = MakeShared<FCameraEvaluationContext>();
	}
}

}  // namespace UE::Cameras

