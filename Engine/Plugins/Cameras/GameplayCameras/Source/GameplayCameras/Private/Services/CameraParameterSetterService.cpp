// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraParameterSetterService.h"

namespace UE::Cameras
{

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FCameraParameterSetterService)

FCameraParameterSetterService::FCameraParameterSetterService()
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPreUpdate);
}

void FCameraParameterSetterService::OnPreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	UpdateCameraVariableSetters(Params.DeltaTime);
}

void FCameraParameterSetterService::StopCameraVariableSetter(const FCameraVariableSetterHandle& InHandle, bool bImmediately)
{
	if (VariableSetters.IsValidIndex(InHandle.Value))
	{
		FVariableSetterEntry& Entry(VariableSetters[InHandle.Value]);
		if (Entry.ThisHandle.SerialNumber == InHandle.SerialNumber)
		{
			Entry.Setter->Stop(bImmediately);
		}
	}
}

void FCameraParameterSetterService::UpdateCameraVariableSetters(float DeltaTime)
{
	for (auto It = VariableSetters.CreateIterator(); It; ++It)
	{
		FVariableSetterEntry& Entry(*It);
		if (!Entry.Setter.IsValid())
		{
			It.RemoveCurrent();
			continue;
		}

		Entry.Setter->Update(DeltaTime);

		if (!Entry.Setter->IsActive())
		{
			It.RemoveCurrent();
		}
	}
}

void FCameraParameterSetterService::ApplyCameraVariableSetters(FCameraVariableTable& OutVariableTable)
{
	for (FVariableSetterEntry& Entry : VariableSetters)
	{
		Entry.Setter->Apply(OutVariableTable);
	}
}

}  // namespace UE::Cameras

