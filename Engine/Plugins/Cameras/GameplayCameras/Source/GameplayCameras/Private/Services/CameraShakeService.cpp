// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/CameraShakeService.h"

#include "Core/BlendCameraNode.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraNodeEvaluatorStorage.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraShakeAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Core/ShakeCameraNode.h"
#include "Nodes/Blends/InterruptedBlendCameraNode.h"
#include "Nodes/Blends/ReverseBlendCameraNode.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeService)

namespace UE::Cameras
{

class FCameraShakeServiceCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCameraShakeServiceCameraNodeEvaluator)

public:

	FCameraShakeInstanceID StartCameraShake(const FStartCameraShakeParams& Params);
	bool IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const;
	bool StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately);
	void RequestCameraShakeThisFrame(const FStartCameraShakeParams& Params);

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	struct FShakeEntry;

	FShakeEntry* AddCameraShake(const FStartCameraShakeParams& Params);

	void InitializeEntry(
		FShakeEntry& NewEntry, 
		const UCameraShakeAsset* CameraShake,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	bool InitializeEntryBlendOut(FShakeEntry& Entry);

	void PopEntry(int32 EntryIndex);

private:

	enum class EBlendStatus
	{
		None,
		BlendIn,
		BlendOut
	};

	struct FShakeEntry
	{
		FCameraShakeInstanceID EntryID;
		TWeakPtr<const FCameraEvaluationContext> EvaluationContext;
		TObjectPtr<const UCameraShakeAsset> CameraShake;
		FCameraNodeEvaluatorStorage EvaluatorStorage;
		FBlendCameraNodeEvaluator* BlendEvaluator = nullptr;
		FShakeCameraNodeEvaluator* RootEvaluator = nullptr;
		FCameraNodeEvaluatorHierarchy EvaluatorHierarchy;
		FCameraNodeEvaluationResult Result;
		float CurrentTime = 0.f;
		float ShakeScale = 1.f;
		EBlendStatus BlendStatus = EBlendStatus::None;
		ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;
		FMatrix UserPlaySpaceMatrix = FMatrix::Identity;
		uint8 NumRequests = 0;
		bool bPersistentRequest = false;
		bool bIsFirstFrame = false;
		bool bIsBlendFull = false;
		bool bIsBlendFinished = false;
	};

	FCameraSystemEvaluator* OwningEvaluator = nullptr;
	TSharedPtr<const FCameraEvaluationContext> ShakeContext;
	const FCameraVariableTable* BlendedParameters = nullptr;;

	TArray<FShakeEntry> Entries;

	uint32 NextEntryID = 0;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FCameraShakeServiceCameraNodeEvaluator)

FCameraNodeEvaluatorChildrenView FCameraShakeServiceCameraNodeEvaluator::OnGetChildren()
{
	FCameraNodeEvaluatorChildrenView ChildrenView;
	for (FShakeEntry& Entry : Entries)
	{
		ChildrenView.Add(Entry.RootEvaluator);
	}
	return ChildrenView;
}

FCameraShakeInstanceID FCameraShakeServiceCameraNodeEvaluator::StartCameraShake(const FStartCameraShakeParams& Params)
{
	if (!Params.CameraShake)
	{
		return FCameraShakeInstanceID();
	}

	// If this shake wants to only have a single instance active at a time, look for a running
	// one and restart it.
	if (Params.CameraShake->bIsSingleInstance)
	{
		FShakeEntry* ExistingEntry = nullptr;
		for (FShakeEntry& Entry : Entries)
		{
			if (Entry.CameraShake == Params.CameraShake)
			{
				ExistingEntry = &Entry;
				break;
			}
		}
		if (ExistingEntry && ensure(ExistingEntry->RootEvaluator))
		{
			FCameraNodeShakeRestartParams RestartParams;
			ExistingEntry->RootEvaluator->RestartShake(RestartParams);
			return ExistingEntry->EntryID;
		}
	}

	FShakeEntry* NewEntry = AddCameraShake(Params);
	if (NewEntry)
	{
		NewEntry->bPersistentRequest = true;
		return NewEntry->EntryID;
	}
	return FCameraShakeInstanceID();
}

bool FCameraShakeServiceCameraNodeEvaluator::IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const
{
	return Entries.ContainsByPredicate(
			[InInstanceID](const FShakeEntry& Entry)
			{
				return Entry.EntryID == InInstanceID;
			});
}

bool FCameraShakeServiceCameraNodeEvaluator::StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately)
{
	const int32 EntryIndex = Entries.IndexOfByPredicate(
			[InInstanceID](const FShakeEntry& Entry)
			{
				return Entry.EntryID == InInstanceID;
			});
	if (EntryIndex != INDEX_NONE)
	{
		if (bImmediately)
		{
			PopEntry(EntryIndex);
		}
		else
		{
			const bool bHasBlendOut = InitializeEntryBlendOut(Entries[EntryIndex]);
			if (!bHasBlendOut)
			{
				PopEntry(EntryIndex);
			}
		}
		return true;
	}
	return false;
}

void FCameraShakeServiceCameraNodeEvaluator::RequestCameraShakeThisFrame(const FStartCameraShakeParams& Params)
{
	if (!Params.CameraShake)
	{
		return;
	}

	// Record this request on a running camera shake, if any.
	bool bFoundShake = false;
	for (FShakeEntry& Entry : Entries)
	{
		if (Entry.CameraShake == Params.CameraShake && !Entry.bPersistentRequest)
		{
			++Entry.NumRequests;
			bFoundShake = true;
			break;
		}
	}

	if (bFoundShake)
	{
		return;
	}

	// Create a new camera shake if there wasn't any, and record this first request.
	FShakeEntry* NewEntry = AddCameraShake(Params);
	if (NewEntry)
	{
		NewEntry->bPersistentRequest = false;
		NewEntry->NumRequests = 1;
	}
}

FCameraShakeServiceCameraNodeEvaluator::FShakeEntry* FCameraShakeServiceCameraNodeEvaluator::AddCameraShake(const FStartCameraShakeParams& Params)
{
	ensure(Params.CameraShake);

	FShakeEntry NewEntry;
	InitializeEntry(NewEntry, Params.CameraShake, ShakeContext);

	NewEntry.ShakeScale = Params.ShakeScale;
	NewEntry.PlaySpace = Params.PlaySpace;
	NewEntry.UserPlaySpaceMatrix = FMatrix::Identity;
	if (Params.PlaySpace == ECameraShakePlaySpace::UserDefined)
	{
		NewEntry.UserPlaySpaceMatrix = FRotationMatrix(Params.UserPlaySpaceRotation);
	}

	const int32 AddedIndex = Entries.Add(MoveTemp(NewEntry));
	return &Entries[AddedIndex];
}

void FCameraShakeServiceCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	OwningEvaluator = Params.Evaluator;
	ShakeContext = Params.EvaluationContext;
	BlendedParameters = OwningEvaluator->GetRootNodeEvaluator()->GetBlendedParameters();
}

void FCameraShakeServiceCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	TArray<int32> EntriesToRemove;

	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		FShakeEntry& Entry(Entries[Index]);

		// See if anybody still cares about this shake.
		if (!Entry.bPersistentRequest && Entry.NumRequests == 0)
		{
			EntriesToRemove.Add(Index);
			continue;
		}

		// Set us up for updating this shake.
		TSharedPtr<const FCameraEvaluationContext> CurContext = Entry.EvaluationContext.Pin();

		FCameraNodeEvaluationParams CurParams(Params);
		CurParams.EvaluationContext = CurContext;
		CurParams.bIsFirstFrame = Entry.bIsFirstFrame;

		FCameraNodeEvaluationResult& CurResult(Entry.Result);

		// Start with the input given to us.
		{
			CurResult.CameraPose = OutResult.CameraPose;
			CurResult.VariableTable.OverrideAll(OutResult.VariableTable);
			CurResult.ContextDataTable.OverrideAll(OutResult.ContextDataTable);
			CurResult.CameraRigJoints.OverrideAll(OutResult.CameraRigJoints);
			CurResult.PostProcessSettings.OverrideAll(OutResult.PostProcessSettings);

			const FCameraNodeEvaluationResult& ContextResult(CurContext->GetInitialResult());
			CurResult.bIsCameraCut = OutResult.bIsCameraCut || ContextResult.bIsCameraCut;
			CurResult.bIsValid = true;
		}

		// Add any parameters coming from the main blend stack.
		if (BlendedParameters)
		{
			CurResult.VariableTable.Override(*BlendedParameters, ECameraVariableTableFilter::KnownOnly);
		}

		// Update timing.
		Entry.CurrentTime += Params.DeltaTime;

		// Run the blend.
		if (Entry.BlendEvaluator)
		{
			Entry.BlendEvaluator->Run(CurParams, CurResult);
		}
		
		// Run the shake and apply it.
		float CurTimeLeft = 0.f;
		if (Entry.RootEvaluator)
		{
			Entry.RootEvaluator->Run(CurParams, CurResult);

			FCameraNodeShakeParams ShakeParams(CurParams);
			ShakeParams.ShakeScale = Entry.ShakeScale;
			ShakeParams.PlaySpace = Entry.PlaySpace;
			ShakeParams.UserPlaySpaceMatrix = Entry.UserPlaySpaceMatrix;

			FCameraNodeShakeResult ShakeResult(CurResult);

			Entry.RootEvaluator->ShakeResult(ShakeParams, ShakeResult);

			ShakeResult.ApplyDelta(ShakeParams);

			CurTimeLeft = ShakeResult.ShakeTimeLeft;
		}

		// We are done with this shake this frame, so clear requests.
		Entry.NumRequests = 0;

		// If it says it's finished, schedule it for removal.
		// (note that negative time left means the shake should play indefinitely)
		if (CurTimeLeft == 0)
		{
			EntriesToRemove.Add(Index);
			continue;
		}

		// Check if we need to start blending out.
		if (Entry.BlendStatus != EBlendStatus::BlendOut &&
				Entry.CameraShake->BlendOut &&
				CurTimeLeft >= 0.f && 
				CurTimeLeft < Entry.CameraShake->BlendOut->BlendTime.GetValue(CurResult.VariableTable))
		{
			const bool bHasBlendOut = InitializeEntryBlendOut(Entry);
			if (!bHasBlendOut)
			{
				EntriesToRemove.Add(Index);
				continue;
			}
		}

		// Apply blending.
		if (Entry.BlendEvaluator)
		{
			FCameraNodeBlendParams BlendParams(Params, CurResult);
			FCameraNodeBlendResult BlendResult(OutResult);
			Entry.BlendEvaluator->BlendResults(BlendParams, BlendResult);

			Entry.bIsBlendFull = BlendResult.bIsBlendFull;
			Entry.bIsBlendFinished = BlendResult.bIsBlendFinished;
		}
		else
		{
			OutResult.OverrideAll(CurResult);
		}

		// Update blend status.
		if (Entry.BlendStatus == EBlendStatus::BlendIn)
		{
			if (Entry.bIsBlendFull && Entry.bIsBlendFinished)
			{
				Entry.BlendStatus = EBlendStatus::None;
			}
		}
		else if (Entry.BlendStatus == EBlendStatus::BlendOut)
		{
			if (Entry.bIsBlendFull && Entry.bIsBlendFinished)
			{
				EntriesToRemove.Add(Index);
			}
		}
	}

	// Remove any finished shakes.
	for (int32 Index = EntriesToRemove.Num() - 1; Index >= 0; --Index)
	{
		PopEntry(EntriesToRemove[Index]);
	}
}

void FCameraShakeServiceCameraNodeEvaluator::InitializeEntry(
	FShakeEntry& NewEntry, 
	const UCameraShakeAsset* CameraShake,
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext)
{
	// Generate the hierarchy of node evaluators inside our storage buffer.
	FCameraNodeEvaluatorTreeBuildParams BuildParams;
	BuildParams.RootCameraNode = CameraShake->RootNode;
	BuildParams.AllocationInfo = &CameraShake->AllocationInfo.EvaluatorInfo;
	FCameraNodeEvaluator* RootEvaluator = NewEntry.EvaluatorStorage.BuildEvaluatorTree(BuildParams);

	// Generate the blend-in evaluator.
	FBlendCameraNodeEvaluator* BlendInEvaluator = nullptr;
	if (CameraShake->BlendIn)
	{
		FCameraNodeEvaluatorTreeBuildParams BlendBuildParams;
		BlendBuildParams.RootCameraNode = CameraShake->BlendIn;
		BlendInEvaluator = NewEntry.EvaluatorStorage.BuildEvaluatorTree(BlendBuildParams)->CastThis<FBlendCameraNodeEvaluator>();
	}

	// Allocate variable table and context data table.
	NewEntry.Result.VariableTable.Initialize(CameraShake->AllocationInfo.VariableTableInfo);
	NewEntry.Result.ContextDataTable.Initialize(CameraShake->AllocationInfo.ContextDataTableInfo);

	// Set all the data from the context.
	const FCameraNodeEvaluationResult& ContextResult = EvaluationContext->GetInitialResult();
	NewEntry.Result.VariableTable.OverrideAll(ContextResult.VariableTable, true);
	NewEntry.Result.ContextDataTable.OverrideAll(ContextResult.ContextDataTable);

	// Initialize the node evaluators.
	if (BlendInEvaluator)
	{
		FCameraNodeEvaluatorInitializeParams BlendInInitParams;
		BlendInInitParams.Evaluator = OwningEvaluator;
		BlendInInitParams.EvaluationContext = EvaluationContext;
		BlendInInitParams.Layer = ECameraRigLayer::Visual;
		BlendInEvaluator->Initialize(BlendInInitParams, NewEntry.Result);
	}
	if (RootEvaluator)
	{
		FCameraNodeEvaluatorInitializeParams InitParams(&NewEntry.EvaluatorHierarchy);
		InitParams.Evaluator = OwningEvaluator;
		InitParams.EvaluationContext = EvaluationContext;
		InitParams.Layer = ECameraRigLayer::Visual;
		RootEvaluator->Initialize(InitParams, NewEntry.Result);
	}

	// Wrap up!
	NewEntry.EntryID = FCameraShakeInstanceID(NextEntryID++);
	NewEntry.EvaluationContext = EvaluationContext;
	NewEntry.CameraShake = CameraShake;
	NewEntry.BlendEvaluator = BlendInEvaluator;
	NewEntry.BlendStatus = (BlendInEvaluator != nullptr ? EBlendStatus::BlendIn : EBlendStatus::None);
	NewEntry.bIsBlendFull = (BlendInEvaluator == nullptr);
	NewEntry.bIsBlendFinished = (BlendInEvaluator == nullptr);
	NewEntry.bIsFirstFrame = true;
	if (RootEvaluator)
	{
		NewEntry.RootEvaluator = RootEvaluator->CastThisChecked<FShakeCameraNodeEvaluator>();
	}
}

bool FCameraShakeServiceCameraNodeEvaluator::InitializeEntryBlendOut(FShakeEntry& Entry)
{
	if (const USimpleFixedTimeBlendCameraNode* BlendOut = Entry.CameraShake->BlendOut)
	{
		// Swap the blend-in evaluator on this entry with a blend-out one.
		if (Entry.BlendStatus != EBlendStatus::BlendOut)
		{
			FCameraNodeEvaluatorBuilder BlendOutBuilder(Entry.EvaluatorStorage);
			FCameraNodeEvaluatorBuildParams BlendOutBuildParams(BlendOutBuilder);
			FBlendCameraNodeEvaluator* BlendOutEvaluator = BlendOutBuildParams.BuildEvaluatorAs<FBlendCameraNodeEvaluator>(BlendOut);

			FCameraNodeEvaluatorInitializeParams BlendOutInitParams;
			BlendOutInitParams.Evaluator = OwningEvaluator;
			BlendOutInitParams.EvaluationContext = Entry.EvaluationContext.Pin();
			BlendOutInitParams.Layer = ECameraRigLayer::Visual;
			BlendOutEvaluator->Initialize(BlendOutInitParams, Entry.Result);

			// Reverse this blend so it plays as a blend-out. Also, see if we are going to 
			// interrupt an ongoing blend-in... if so, give a chance for the blend-out to
			// start at an "equivalent spot".
			if (!BlendOutEvaluator->SetReversed(true))
			{
				BlendOutEvaluator = Entry.EvaluatorStorage.BuildEvaluator<FReverseBlendCameraNodeEvaluator>(BlendOutEvaluator);
			}
			if (Entry.BlendStatus == EBlendStatus::BlendIn && ensure(Entry.BlendEvaluator))
			{
				FBlendCameraNodeEvaluator* OngoingBlend = Entry.BlendEvaluator;

				FCameraNodeBlendInterruptionParams InterruptionParams;
				InterruptionParams.InterruptedBlend = OngoingBlend;
				if (!BlendOutEvaluator->InitializeFromInterruption(InterruptionParams))
				{
					BlendOutEvaluator = Entry.EvaluatorStorage.BuildEvaluator<FInterruptedBlendCameraNodeEvaluator>(BlendOutEvaluator, OngoingBlend);
				}
			}
			// Note: neither the reverse or interrupted blends need initialization, but
			// technically we're missing calling it on them.
			Entry.BlendEvaluator = BlendOutEvaluator;

			Entry.BlendStatus = EBlendStatus::BlendOut;
			Entry.bIsBlendFinished = false;
			Entry.bIsBlendFull = false;
		}
		// else: we were already blending out, so let this continue.

		return true;
	}
	else
	{
		// No blend out, just stop.
		Entry.BlendEvaluator = nullptr;
		Entry.BlendStatus = EBlendStatus::BlendOut;
		Entry.bIsBlendFull = true;
		Entry.bIsBlendFinished = true;

		return false;
	}
}

void FCameraShakeServiceCameraNodeEvaluator::PopEntry(int32 EntryIndex)
{
	if (!ensure(Entries.IsValidIndex(EntryIndex)))
	{
		return;
	}

	Entries.RemoveAt(EntryIndex);
}

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FCameraShakeService)

void FCameraShakeService::OnInitialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::None);

	ensure(Evaluator == nullptr);
	Evaluator = Params.Evaluator;
}

void FCameraShakeService::OnTeardown(const FCameraEvaluationServiceTeardownParams& Params)
{
	ensure(Evaluator != nullptr);
	Evaluator = nullptr;
}

FCameraShakeInstanceID FCameraShakeService::StartCameraShake(const FStartCameraShakeParams& Params)
{
	EnsureShakeContextCreated();

	if (ensure(ShakeEvaluator))
	{
		return ShakeEvaluator->StartCameraShake(Params);
	}

	return FCameraShakeInstanceID();
}

bool FCameraShakeService::IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const
{
	if (ShakeEvaluator)
	{
		return ShakeEvaluator->IsCameraShakePlaying(InInstanceID);
	}
	return false;
}

bool FCameraShakeService::StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately)
{
	if (ShakeEvaluator)
	{
		return ShakeEvaluator->StopCameraShake(InInstanceID, bImmediately);
	}
	return false;
}

void FCameraShakeService::RequestCameraShakeThisFrame(const FStartCameraShakeParams& Params)
{
	EnsureShakeContextCreated();

	if (ensure(ShakeEvaluator))
	{
		ShakeEvaluator->RequestCameraShakeThisFrame(Params);
	}
}

void FCameraShakeService::EnsureShakeContextCreated()
{
	// Create the evaluation context, which is a "null" context with no particular logic.
	if (!ShakeContext)
	{
		ShakeContext = MakeShared<FCameraEvaluationContext>();
		ShakeContext->GetInitialResult().bIsValid = true;
	}

	// Create the camera rig that will contain and run all the camera shakes.
	if (!ShakeContainerRig)
	{
		ShakeContainerRig = NewObject<UCameraRigAsset>(GetTransientPackage(), TEXT("CameraShakeContainerRig"), RF_Transient);
		ShakeContainerRig->RootNode = NewObject<UCameraShakeServiceCameraNode>(ShakeContainerRig, NAME_None, RF_Transient);
		ShakeContainerRig->BuildCameraRig();
	}

	// Instantiate the "container" camera rig inside the visual layer.
	if (!ShakeEvaluator)
	{
		FRootCameraNodeEvaluator* RootEvaluator = Evaluator->GetRootNodeEvaluator();

		FActivateCameraRigParams ActivateParams;
		ActivateParams.EvaluationContext = ShakeContext;
		ActivateParams.CameraRig = ShakeContainerRig;
		ActivateParams.Layer = ECameraRigLayer::Visual;
		FCameraRigInstanceID InstanceID = RootEvaluator->ActivateCameraRig(ActivateParams);

		FCameraRigEvaluationInfo ShakeContainerRigInfo;
		RootEvaluator->GetCameraRigInfo(InstanceID, ShakeContainerRigInfo);

		if (ShakeContainerRigInfo.RootEvaluator)
		{
			ShakeEvaluator = ShakeContainerRigInfo.RootEvaluator->CastThis<FCameraShakeServiceCameraNodeEvaluator>();
		}
		ensure(ShakeEvaluator);
	}
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr UCameraShakeServiceCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FCameraShakeServiceCameraNodeEvaluator>();
}

