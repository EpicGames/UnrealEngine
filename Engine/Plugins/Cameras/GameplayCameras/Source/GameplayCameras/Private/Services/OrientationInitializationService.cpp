// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/OrientationInitializationService.h"

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraOperation.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigCombinationRegistry.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Core/RootCameraNodeCameraRigEvent.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "HAL/IConsoleManager.h"

namespace UE::Cameras
{

bool GGameplayCamerasDebugOrientationInitializationShowLastTargetPreservation = false;
static FAutoConsoleVariableRef CVarGameplayCamerasDebugOrientationInitializationShowLastTargetPreservation(
	TEXT("GameplayCameras.Debug.OrientationInitialization.ShowLastTargetPreservation"),
	GGameplayCamerasDebugOrientationInitializationShowLastTargetPreservation,
	TEXT(""));

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FOrientationInitializationService)

void FOrientationInitializationService::SetYawPitchPreservationOverride(const FRotator3d& InOrientation)
{
	YawPitchPreservationOverride = InOrientation;
}

void FOrientationInitializationService::SetTargetPreservationOverride(const FVector3d& InTarget)
{
	TargetPreservationOverride = InTarget;
}

void FOrientationInitializationService::OnInitialize(const FCameraEvaluationServiceInitializeParams& Params)
{
	SetEvaluationServiceFlags(
			ECameraEvaluationServiceFlags::NeedsRootCameraNodeEvents |
			ECameraEvaluationServiceFlags::NeedsPostUpdate);

	Evaluator = Params.Evaluator;
}

void FOrientationInitializationService::OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	TSharedPtr<FCameraEvaluationContext> ActiveContext = Params.Evaluator->GetEvaluationContextStack().GetActiveContext();

	bHasPreviousContextTransform = false;
	if (ActiveContext)
	{
		const FCameraNodeEvaluationResult& InitialResult = ActiveContext->GetInitialResult();
		PreviousContextLocation = InitialResult.CameraPose.GetLocation();
		PreviousContextRotation = InitialResult.CameraPose.GetRotation();
		bHasPreviousContextTransform = true;
	}

	PreviousEvaluationContext = ActiveContext;

	YawPitchPreservationOverride.Reset();
	TargetPreservationOverride.Reset();
}

void FOrientationInitializationService::OnRootCameraNodeEvent(const FRootCameraNodeCameraRigEvent& InEvent)
{
	if (InEvent.EventType == ERootCameraNodeCameraRigEventType::Activated && InEvent.EventLayer == ECameraRigLayer::Main)
	{
		ECameraRigInitialOrientation InitialOrientation = ECameraRigInitialOrientation::None;

		if (const UCameraRigAsset* NewCameraRig = InEvent.CameraRigInfo.CameraRig)
		{
			// If the new camera rig is a combination, find its initial orientation settings
			// on one of its combined rigs.
			TArray<const UCameraRigAsset*> NewCombinedCameraRigs;
			UCombinedCameraRigsCameraNode::GetAllCombinationCameraRigs(NewCameraRig, NewCombinedCameraRigs);
			for (const UCameraRigAsset* NewCombinedCameraRig : NewCombinedCameraRigs)
			{
				if (NewCombinedCameraRig->InitialOrientation != ECameraRigInitialOrientation::None)
				{
					InitialOrientation = NewCombinedCameraRig->InitialOrientation;
					break;
				}
			}
		}

		if (InEvent.Transition && InEvent.Transition->bOverrideInitialOrientation)
		{
			InitialOrientation = InEvent.Transition->InitialOrientation;
		}

		switch (InitialOrientation)
		{
			case ECameraRigInitialOrientation::ContextYawPitch:
				TryInitializeContextYawPitch(InEvent.CameraRigInfo);
				break;
			case ECameraRigInitialOrientation::PreviousYawPitch:
				TryPreserveYawPitch(InEvent.CameraRigInfo);
				break;
			case ECameraRigInitialOrientation::PreviousAbsoluteTarget:
				TryPreserveTarget(InEvent.CameraRigInfo, false);
				break;
			case ECameraRigInitialOrientation::PreviousRelativeTarget:
				TryPreserveTarget(InEvent.CameraRigInfo, true);
				break;
			default:
				break;
		}
	}
}

void FOrientationInitializationService::TryInitializeContextYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo)
{
	if (!CameraRigInfo.CameraRig)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize orientation on invalid camera rig.");
		return;
	}

	if (!CameraRigInfo.EvaluationContext)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize orientation on camera rig '%ls' with invalid evaluation context.",
				*CameraRigInfo.CameraRig->GetPathName());
		return;
	}

	const FCameraNodeEvaluationResult& InitialResult = CameraRigInfo.EvaluationContext->GetInitialResult();
	if (!InitialResult.bIsValid)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize orientation on camera rig '%ls' with invalid initial context result.",
				*CameraRigInfo.CameraRig->GetPathName());
		return;
	}

	const FRotator3d& InitialRotation = InitialResult.CameraPose.GetRotation();
	TryInitializeYawPitch(CameraRigInfo, InitialRotation.Yaw, InitialRotation.Pitch);
}

void FOrientationInitializationService::TryPreserveYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo)
{
	const FCameraSystemEvaluationResult& LastResult = Evaluator->GetEvaluatedResult();
	if (!LastResult.bIsValid)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize camera rig orientation when previous camera result is invalid.");
		return;
	}

	if (!CameraRigInfo.CameraRig)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize camera rig orientation with invalid camera rig.");
		return;
	}
	
	FRotator3d LastOrientation = LastResult.CameraPose.GetRotation();
	if (YawPitchPreservationOverride.IsSet())
	{
		LastOrientation = YawPitchPreservationOverride.GetValue();
	}
	TryInitializeYawPitch(CameraRigInfo, LastOrientation.Yaw, LastOrientation.Pitch);
}

void FOrientationInitializationService::TryInitializeYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo, TOptional<double> Yaw, TOptional<double> Pitch)
{
	if (!CameraRigInfo.RootEvaluator)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize orientation on camera rig '%ls' because it has no evaluator.",
				*CameraRigInfo.CameraRig->GetPathName());
		return;
	}

	FCameraOperationParams OperationParams;
	OperationParams.Evaluator = Evaluator;
	OperationParams.EvaluationContext = CameraRigInfo.EvaluationContext;

	FYawPitchCameraOperation Operation;
	if (Yaw.IsSet())
	{
		Operation.Yaw = FConsumableDouble::Absolute(Yaw.GetValue());
	}
	if (Pitch.IsSet())
	{
		Operation.Pitch = FConsumableDouble::Absolute(Pitch.GetValue());
	}

	FCameraNodeEvaluatorHierarchy CameraRigHierarchy(CameraRigInfo.RootEvaluator);
	CameraRigHierarchy.CallExecuteOperation(OperationParams, Operation);
}

void FOrientationInitializationService::TryPreserveTarget(const FCameraRigEvaluationInfo& CameraRigInfo, bool bUseRelativeTarget)
{
	const FCameraSystemEvaluationResult& LastResult = Evaluator->GetEvaluatedResult();
	if (!LastResult.bIsValid)
	{
		// The previous result might be invalid on the very first frame of the game, when the first
		// camera rig activates.
		return;
	}

	if (!CameraRigInfo.CameraRig)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize camera rig orientation with invalid camera rig.");
		return;
	}
	
	if (!CameraRigInfo.RootEvaluator)
	{
		UE_LOGF(LogCameraSystem, Error, "Can't initialize orientation on camera rig '%ls' because it has no evaluator.",
				*CameraRigInfo.CameraRig->GetPathName());
		return;
	}

	FVector3d TargetToPreserve = LastResult.CameraPose.GetTarget();

	if (TargetPreservationOverride.IsSet())
	{
		TargetToPreserve = TargetPreservationOverride.GetValue();
	}
	else
	{
		// Relative target preservation means that if the context turned since last frame, we want to preserve
		// a target that has "turned" along with it.
		if (bUseRelativeTarget && bHasPreviousContextTransform &&
				(PreviousEvaluationContext == nullptr || CameraRigInfo.EvaluationContext == PreviousEvaluationContext))
		{
			const FRotator3d PreviousInverseContextRotation = PreviousContextRotation.GetInverse();
			const FVector3d LastRelativeTarget = PreviousInverseContextRotation.RotateVector(TargetToPreserve - PreviousContextLocation);

			const FCameraNodeEvaluationResult& InitialResult = CameraRigInfo.EvaluationContext->GetInitialResult();
			const FVector3d& InitialLocation = InitialResult.CameraPose.GetLocation();
			const FRotator3d& InitialRotation = InitialResult.CameraPose.GetRotation();
			TargetToPreserve = InitialRotation.RotateVector(LastRelativeTarget) + InitialLocation;
		}
	}

	FCameraIKAimParams AimParams;
	AimParams.bIsFirstFrame = true;
	AimParams.bIsActiveCameraRig = true;
	AimParams.DeltaTime = 0.f;
	AimParams.Evaluator = Evaluator;
	AimParams.TargetLocation = TargetToPreserve;

	FCameraIKAim CameraAim;
	CameraAim.Run(AimParams, CameraRigInfo);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugLastEvaluatedTarget = LastResult.CameraPose.GetTarget();
	CameraAim.GetLastRunDebugInfo(LastAimDebugInfo);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FOrientationInitializationDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FOrientationInitializationDebugBlock)

public:

	FOrientationInitializationDebugBlock() = default;
	FOrientationInitializationDebugBlock(const FOrientationInitializationService& InService);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FVector3d LastEvaluatedTarget;
	FCameraIKAimDebugInfo AimDebugInfo;
};

UE_DEFINE_CAMERA_DEBUG_BLOCK(FOrientationInitializationDebugBlock)

void FOrientationInitializationService::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	Builder.AttachDebugBlock<FOrientationInitializationDebugBlock>(*this);
}

FOrientationInitializationDebugBlock::FOrientationInitializationDebugBlock(const FOrientationInitializationService& InService)
{
	LastEvaluatedTarget = InService.DebugLastEvaluatedTarget;
	AimDebugInfo = InService.LastAimDebugInfo;
}

void FOrientationInitializationDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	if (GGameplayCamerasDebugOrientationInitializationShowLastTargetPreservation)
	{
		AimDebugInfo.DebugDraw(Params, Renderer);

		const FVector2d TextOffset(20, -10);
		Renderer.DrawPoint(LastEvaluatedTarget, 10.f, FLinearColor::Green, 0.f);
		Renderer.DrawTextView(LastEvaluatedTarget, TextOffset, TEXT("Preserved Target"), FLinearColor::Green, nullptr);
	}
}

void FOrientationInitializationDebugBlock::OnSerialize(FArchive& Ar)
{
	Ar << LastEvaluatedTarget;
	Ar << AimDebugInfo;
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

