// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/PlayerControlRotationService.h"

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockBuilder.h"
#include "Debug/CameraDebugRenderer.h"
#include "EnhancedInputComponent.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "PlayerControlRotationService"

namespace UE::Cameras
{

float GGameplayCamerasControlRotationDebugArrowLength = 200.f;
static FAutoConsoleVariableRef CVarGameplayCamerasControlRotationDebugArrowLength(
	TEXT("GameplayControlRotation.DebugArrowLength"),
	GGameplayCamerasControlRotationDebugArrowLength,
	TEXT(""));

UE_DEFINE_CAMERA_EVALUATION_SERVICE(FPlayerControlRotationEvaluationService)

UE_DECLARE_CAMERA_DEBUG_BLOCK_START(GAMEPLAYCAMERAS_API, FPlayerControlRotationDebugBlock)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FTransform, PawnTransform)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, ControlRotation)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FRotator3d, CameraRotation)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bIsFrozen)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FString, FreezeReason)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(TCameraDebugGraph<1>, AxisActionAngularSpeedGraph)
	UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FCameraDebugClock, AxisActionValueClock)
UE_DECLARE_CAMERA_DEBUG_BLOCK_END()

UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FPlayerControlRotationDebugBlock)

FPlayerControlRotationEvaluationService::FPlayerControlRotationEvaluationService()
	: PreviousAxisBindingValue(EForceInit::ForceInit)
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPostUpdate);
}

FPlayerControlRotationEvaluationService::FPlayerControlRotationEvaluationService(const FPlayerControlRotationParams& InParams)
	: ServiceParams(InParams)
	, PreviousAxisBindingValue(EForceInit::ForceInit)
{
	SetEvaluationServiceFlags(ECameraEvaluationServiceFlags::NeedsPostUpdate);
}

void FPlayerControlRotationEvaluationService::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ServiceParams.AxisActions);
}

void FPlayerControlRotationEvaluationService::MonitorActiveContext(TSharedPtr<const FCameraEvaluationContext> ActiveContext)
{
	// Try to find the currently active input component. If we fail at any point, get rid of any
	// previous bindings we had.
	if (!ActiveContext)
	{
		UnbindActionValues();
		return;
	}

	APlayerController* PlayerController = ActiveContext->GetPlayerController();
	if (!PlayerController)
	{
		UnbindActionValues();
		return;
	}
	
	APawn* Pawn = PlayerController->GetPawn();
	if (!Pawn)
	{
		UnbindActionValues();
		return;
	}

	UEnhancedInputComponent* InputComponent = Cast<UEnhancedInputComponent>(Pawn->InputComponent);
	if (!InputComponent)
	{
		UnbindActionValues();
		return;
	}

	// If the input component has changed this we last checked, re-bind our input actions to this new one.
	if (InputComponent != WeakInputComponent)
	{
		BindActionValues(InputComponent);
	}
}

void FPlayerControlRotationEvaluationService::BindActionValues(UEnhancedInputComponent* InputComponent)
{
	UE_LOGF(LogCameraSystem, Verbose, "FPlayerControlRotationEvaluationService: binding to input actions on '%ls'", *GetNameSafe(InputComponent));

	WeakInputComponent = InputComponent;

	AxisBindings.Reset();

	for (TObjectPtr<UInputAction> AxisAction : ServiceParams.AxisActions)
	{
		FEnhancedInputActionValueBinding& AxisBinding = InputComponent->BindActionValue(AxisAction);
		AxisBindings.Add(&AxisBinding);
	}
}

void FPlayerControlRotationEvaluationService::UnbindActionValues()
{
	UE_LOGF(LogCameraSystem, Verbose, "FPlayerControlRotationEvaluationService: unbinding from input actions");
	WeakInputComponent.Reset();
	AxisBindings.Reset();
}

void FPlayerControlRotationEvaluationService::OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	// Grab the new camera orientation and set is as the control rotation by default.
	CameraRotation = OutResult.EvaluationResult.CameraPose.GetRotation();
	CurrentControlRotation = CameraRotation;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugDidApplyControlRotation = false;
#endif

	// Get the active context.
	FCameraEvaluationContextStack& EvaluationContextStack = Params.Evaluator->GetEvaluationContextStack();
	TSharedPtr<const FCameraEvaluationContext> ActiveContext = EvaluationContextStack.GetActiveContext();

	// Check if we need to bind to a new input component, or abandon the one we had.
	MonitorActiveContext(ActiveContext);

	// If there's no active context, we are done, control rotation isn't frozen.
	if (!ActiveContext.IsValid())
	{
		bIsFrozen = false;
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = TEXT("no active context");
#endif
		return;
	}

	// Find the player controller whose control rotation we need to manage.
	APlayerController* PlayerController = ActiveContext->GetPlayerController();
	if (!PlayerController)
	{
		bIsFrozen = false;
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = TEXT("no player controller on active context");
#endif
		return;
	}

	// Get the pawn rotation for the debug information.
#if UE_GAMEPLAY_CAMERAS_DEBUG
	{
		APawn* PlayerPawn = PlayerController->GetPawn();
		if (PlayerPawn)
		{
			DebugPawnTransform = PlayerPawn->GetActorTransform();
		}
	}
#endif

	// Update our current control rotation, and apply it if allowed.
	if (!AxisBindings.IsEmpty())
	{
		UpdateControlRotation(Params, OutResult);
	}
	else
	{
		bIsFrozen = false;
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = TEXT("no input bindings defined");
#endif
	}
	if (ServiceParams.bApplyControlRotation)
	{
		PlayerController->SetControlRotation(CurrentControlRotation);
	}
#if UE_GAMEPLAY_CAMERAS_DEBUG
	bDebugDidApplyControlRotation = ServiceParams.bApplyControlRotation;
#endif
}

void FPlayerControlRotationEvaluationService::UpdateControlRotation(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult)
{
	// If we were not already frozen, see if a camera rig is requesting it.
	const FBuiltInCameraVariables& BuiltInVariables = FBuiltInCameraVariables::Get();
	FCameraVariableTable& OutVariableTable = OutResult.EvaluationResult.VariableTable;
	if (!bIsFrozen)
	{
		const bool bFreezeControlRotation = OutVariableTable.GetValue<bool>(
				BuiltInVariables.FreezeControlRotationDefinition, false);
		const bool bHasCustomControlRotation = OutVariableTable.TryGetValue<FRotator3d>(
				BuiltInVariables.ControlRotationDefinition, FrozenControlRotation);

		bIsFrozen = (bFreezeControlRotation && bHasCustomControlRotation);
	}

	// See if the player is actively using the controls.
	FVector2d MaxAxisBindingValue(EForceInit::ForceInit);
	double MaxAxisBindingValueSquaredLength = 0.0;
	for (FEnhancedInputActionValueBinding* AxisBinding : AxisBindings)
	{
		const FVector2d Value = AxisBinding->GetValue().Get<FVector2D>();
		const double ValueSquaredLength = Value.SquaredLength();
		if (ValueSquaredLength > MaxAxisBindingValueSquaredLength)
		{
			MaxAxisBindingValue = Value;
			MaxAxisBindingValueSquaredLength = ValueSquaredLength;
		}
	}

	// Compute input direction speed change.
	const FVector2d PreviousAxisDir = PreviousAxisBindingValue.GetSafeNormal();
	const FVector2d CurrentAxisDir = MaxAxisBindingValue.GetSafeNormal();
	const double AxisActionAngleChange = (!PreviousAxisDir.IsZero() && !CurrentAxisDir.IsZero()) ?
		FMath::RadiansToDegrees(FMath::Acos(PreviousAxisDir.Dot(CurrentAxisDir))) :
		0.0;
	const double AxisActionAngularSpeed = AxisActionAngleChange / (Params.DeltaTime > 0 ? Params.DeltaTime : 1.f);
	PreviousAxisBindingValue = MaxAxisBindingValue;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	AxisActionAngularSpeedGraph.Add(Params.DeltaTime, (float)AxisActionAngularSpeed);
	AxisActionValueClock.Update(MaxAxisBindingValue);
#endif

	// If we are not feezing the control rotation, we are done.
	// (our CurrentControlRotation is already set to the camera rotation)
	if (!bIsFrozen)
	{
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = FString::Printf(
				TEXT("no freeze request (input speed %7.2fdeg/s)"),
				AxisActionAngularSpeed);
#endif
		return;
	}

	// If the player isn't using the controls, we can immediately unfreeze the control rotation,
	// and we are done. (our CurrentControlRotation is already set to the camera rotation)
	const double SquaredThreshold = FMath::Square(ServiceParams.AxisActionMagnitudeThreshold);
	if (MaxAxisBindingValueSquaredLength <= SquaredThreshold)
	{
		bIsFrozen = false;
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = FString::Printf(
				TEXT("no input (magnitude squared %7.2f < %7.f)"),
				MaxAxisBindingValueSquaredLength, SquaredThreshold);
#endif
		return;
	}

	// If the player is using the controls, but is turning more than our defined threshold, immediately
	// unfreeze the control rotation, and we are done.
	// (our CurrentControlRotation is already set to the camera rotation)
	if (AxisActionAngularSpeed >= ServiceParams.AxisActionAngularSpeedThreshold)
	{
		bIsFrozen = false;
#if UE_GAMEPLAY_CAMERAS_DEBUG
		DebugFreezeReason = FString::Printf(
				TEXT("changed input (%7.2fdeg/s > %7.2fdeg/s)"),
				AxisActionAngularSpeed, ServiceParams.AxisActionAngularSpeedThreshold);
#endif
		return;
	}

	// We keep the CurrentControlRotation frozen for one more frame.
	ensure(bIsFrozen);
	CurrentControlRotation = FrozenControlRotation;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	DebugFreezeReason = FString::Printf(
			TEXT("unchanged input (%7.2fdeg/s < %7.2fdeg/s)"), 
			AxisActionAngularSpeed, ServiceParams.AxisActionAngularSpeedThreshold);
#endif
}

#if UE_GAMEPLAY_CAMERAS_DEBUG

void FPlayerControlRotationEvaluationService::OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder)
{
	FPlayerControlRotationDebugBlock& DebugBlock = Builder.AttachDebugBlock<FPlayerControlRotationDebugBlock>();

	DebugBlock.PawnTransform = DebugPawnTransform;
	DebugBlock.ControlRotation = CurrentControlRotation;
	DebugBlock.CameraRotation = CameraRotation;
	DebugBlock.bIsFrozen = bIsFrozen;
	DebugBlock.FreezeReason = DebugFreezeReason;
	DebugBlock.AxisActionAngularSpeedGraph = AxisActionAngularSpeedGraph;
	DebugBlock.AxisActionValueClock = AxisActionValueClock;
}

void FPlayerControlRotationDebugBlock::OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer)
{
	const FVector3d PawnLocation = PawnTransform.GetLocation();

	const FRotator3d CameraYaw(0, CameraRotation.Yaw, 0);
	const FRotator3d ControlYaw(0, ControlRotation.Yaw, 0);

	const FVector3d ForwardArrowEnd = FVector3d::ForwardVector * GGameplayCamerasControlRotationDebugArrowLength;

	// Pawn orientation.
	Renderer.DrawDirectionalArrow(
			PawnLocation, PawnLocation + PawnTransform.TransformVectorNoScale(ForwardArrowEnd),
			5.f, FColorList::MandarianOrange, 1.f);

	// Camera rotation.
	Renderer.DrawDirectionalArrow(
			PawnLocation, PawnLocation + CameraYaw.RotateVector(ForwardArrowEnd),
			5.f, FColorList::PaleGreen, 1.f);

	// Control rotation.
	FVector3d ControlYawArrowEnd = PawnLocation + ControlYaw.RotateVector(ForwardArrowEnd);
	Renderer.DrawDirectionalArrow(
			PawnLocation, ControlYawArrowEnd,
			5.f, FColorList::Green, 2.f);

	const FColor TextColor(255, 255, 255, 192);
	const FVector2d TextOffset(5.f, 5.f);
	if (bIsFrozen)
	{
		FString DebugText = FString::Printf(
				TEXT("camera: %+7.2f, frozen: %+7.2f\n%s"), 
				CameraRotation.Yaw, ControlRotation.Yaw, *FreezeReason);
		Renderer.DrawText(ControlYawArrowEnd, TextOffset, DebugText, TextColor);
	}
	else
	{
		FString DebugText = FString::Printf(
				TEXT("camera: %+7.2f\n%s"), 
				CameraRotation.Yaw, *FreezeReason);
		Renderer.DrawText(ControlYawArrowEnd, TextOffset, DebugText, TextColor);
	}

	// Value clock and angular speed graph.
	Renderer.DrawClock(AxisActionValueClock, LOCTEXT("AxisBindingValue", "AxisBindingValue"));
	Renderer.DrawGraph(AxisActionAngularSpeedGraph, LOCTEXT("AxisBindingAngularSpeed", "AxisBindingAngularSpeed"));
}

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

