// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/OverridablePropertyBag.h"

#include "Misc/EngineVersionComparison.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OverridablePropertyBag)

FCustomVersionRegistration GOverridablePropertyBagCustomVersion(
		FOverridablePropertyBagCustomVersion::GUID,
		FOverridablePropertyBagCustomVersion::LatestVersion,
		TEXT("OverridablePropertyBagCustomVersion"));

bool FInstancedOverridablePropertyBag::IsPropertyOverriden(const FGuid& InPropertyID) const
{
	return OverridenPropertyIDs.Contains(InPropertyID);
}

void FInstancedOverridablePropertyBag::SetPropertyOverriden(const FGuid& InPropertyID, bool bIsOverriden)
{
	if (bIsOverriden)
	{
		OverridenPropertyIDs.AddUnique(InPropertyID);
	}
	else
	{
		OverridenPropertyIDs.Remove(InPropertyID);
	}
}

bool FInstancedOverridablePropertyBag::IsPropertyOverriden(const FName InPropertyName) const
{
	if (const FPropertyBagPropertyDesc* PropertyDesc = FindPropertyDescByName(InPropertyName))
	{
		return IsPropertyOverriden(PropertyDesc->ID);
	}
	return false;
}

void FInstancedOverridablePropertyBag::SetPropertyOverriden(const FName InPropertyName, bool bIsOverriden)
{
	if (const FPropertyBagPropertyDesc* PropertyDesc = FindPropertyDescByName(InPropertyName))
	{
		return SetPropertyOverriden(PropertyDesc->ID, bIsOverriden);
	}
}

void FInstancedOverridablePropertyBag::MigrateToNewBagInstanceWithOverrides(const FInstancedPropertyBag& NewBagInstance)
{
	FInstancedPropertyBag::MigrateToNewBagInstanceWithOverrides(NewBagInstance, OverridenPropertyIDs);

	// Remove overrides for propeties that don't exist anymore.
	if (const UPropertyBag* ParametersType = GetPropertyBagStruct())
	{
		for (TArray<FGuid>::TIterator It = OverridenPropertyIDs.CreateIterator(); It; ++It)
		{
			if (!ParametersType->FindPropertyDescByID(*It))
			{
				It.RemoveCurrentSwap();
			}
		}
	}
}

bool FInstancedOverridablePropertyBag::SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName NAME_InstancedPropertyBag = FInstancedPropertyBag::StaticStruct()->GetFName();
	if (Tag.GetType().IsStruct(NAME_InstancedPropertyBag))
	{
		// Read the structured data as an FInstancedPropertyBag. Our list of overriden property IDs
		// will stay empty.
		FInstancedPropertyBag::Serialize(Slot.GetUnderlyingArchive());
		return true;
	}
	return false;
}

bool FInstancedOverridablePropertyBag::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FOverridablePropertyBagCustomVersion::GUID);

	if (Ar.IsLoading() && 
			Ar.CustomVer(FOverridablePropertyBagCustomVersion::GUID) < FOverridablePropertyBagCustomVersion::FixSerializer)
	{
		// There was a short time during which developer data was saved with only default tagged
		// serialization. This causes data corruption so we avoid that with this use-case here.
		static const FInstancedOverridablePropertyBag ThisDefaults;

		UScriptStruct* ThisStruct = FInstancedOverridablePropertyBag::StaticStruct();
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
		ThisStruct->SerializeTaggedProperties(Ar, (uint8*)this, ThisStruct, (const uint8*)&ThisDefaults);
#else
		ThisStruct->SerializeTaggedProperties(Ar, (uint8*)this, ThisStruct, (uint8*)&ThisDefaults);
#endif  // UE >= 5.7.0

		return true;
	}

	FInstancedPropertyBag::Serialize(Ar);
	Ar << OverridenPropertyIDs;
	return true;
}

