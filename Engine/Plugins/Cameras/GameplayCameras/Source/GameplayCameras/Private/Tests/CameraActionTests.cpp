// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Misc/AutomationTest.h"
#include "Nodes/Blends/LinearBlendCameraNode.h"
#include "Nodes/Common/OffsetCameraNode.h"
#include "Services/CameraActionService.h"
#include "Build/CameraAssetAssembleUtils.h"
#include "Tests/GameplayCamerasTestObjects.h"

namespace UE::Cameras::Test
{
	struct FCameraActionTestData
	{
		TSharedPtr<FCameraSystemEvaluator> Evaluator;
		TSharedPtr<FCameraActionService> ActionService;
		FRootCameraNodeEvaluator* RootEvaluator = nullptr;

		TSharedPtr<FCameraEvaluationContext> TestEvaluationContext;
		UCameraRigAsset* TestCameraRig = nullptr;

		void UpdateEvaluator(float DeltaTime = 0.3f);
		void ForceReactivateTestCameraRig(const UCameraRigTransition* TransitionOverride = nullptr);

		TSharedPtr<FCameraActionScope> GetActiveActionScope();
		FCameraActionInstanceID StartAction(const UCameraAction* Action);
	};

	FCameraActionTestData CreateCameraActionTestData()
	{
		FCameraActionTestData TestData;

		// Create the evaluator, and a test evaluation context.
		TestData.Evaluator = MakeShared<FCameraSystemEvaluator>();
		TestData.Evaluator->Initialize();

		TestData.TestEvaluationContext = FCameraEvaluationContextAssembler()
			.CreateCameraRig(TEXT("TestRig"))
				.MakeRootNode<UOffsetCameraNode>()
					.Done()
				.Pin(TestData.TestCameraRig)
				.Done()
			.MakeSingleDirector()
				.Setup([](USingleCameraDirector* Director, FNamedObjectRegistry* NamedObjectRegistry)
					{
						Director->CameraRig = NamedObjectRegistry->Get<UCameraRigAsset>(TEXT("TestRig"));
					})
				.Done()
			.BuildCameraAsset()
			.Get();
		ensure(TestData.TestCameraRig);

		// Activate the test evaluation context.
		TestData.Evaluator->PushEvaluationContext(TestData.TestEvaluationContext.ToSharedRef());

		// Grab the action service and root evaluator.
		TestData.ActionService = TestData.Evaluator->FindEvaluationService<FCameraActionService>();
		ensure(TestData.ActionService.IsValid());

		TestData.RootEvaluator = TestData.Evaluator->GetRootNodeEvaluator();
		ensure(TestData.RootEvaluator);

		// Update the evaluator once so we get usable setup.
		TestData.UpdateEvaluator();

		return TestData;
	}

	void FCameraActionTestData::UpdateEvaluator(float DeltaTime)
	{
		FCameraSystemEvaluationParams Params;
		Params.DeltaTime = DeltaTime;
		Evaluator->Update(Params);
	}

	void FCameraActionTestData::ForceReactivateTestCameraRig(const UCameraRigTransition* TransitionOverride)
	{
		FActivateCameraRigParams NewRigParams;
		NewRigParams.EvaluationContext = TestEvaluationContext;
		NewRigParams.CameraRig = TestCameraRig;
		NewRigParams.TransitionOverride = TransitionOverride;
		NewRigParams.bForceActivate = true;
		RootEvaluator->ActivateCameraRig(NewRigParams);
	}

	TSharedPtr<FCameraActionScope> FCameraActionTestData::GetActiveActionScope()
	{
		return RootEvaluator->GetActiveCameraRigActionScope(false);
	}

	FCameraActionInstanceID FCameraActionTestData::StartAction(const UCameraAction* Action)
	{
		return ActionService->StartAction(Action);
	}

}  // namespace UE::Cameras::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraActionSimpleTest, "System.Engine.GameplayCameras.CameraAction.Simple", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraActionSimpleTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	FCameraActionTestData TestData = CreateCameraActionTestData();

	TSharedPtr<FCameraActionScope> ActiveActionScope = TestData.GetActiveActionScope();
	UTEST_NULL("No initial action scope", ActiveActionScope.Get());

	// Push our test action.
	UUpdateTrackerCameraAction* NewAction = NewObject<UUpdateTrackerCameraAction>();
	NewAction->NumUpdates = 2;
	FCameraActionInstanceID NewInstanceID = TestData.StartAction(NewAction);

	// Update twice... the action should be gone after the second time.
	TestData.UpdateEvaluator();
	UTEST_TRUE("Action is running", TestData.ActionService->IsActionRunning(NewInstanceID));

	TestData.UpdateEvaluator();
	UTEST_FALSE("Action has finished", TestData.ActionService->IsActionRunning(NewInstanceID));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraActionCloningTest, "System.Engine.GameplayCameras.CameraAction.Cloning", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraActionCloningTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	FCameraActionTestData TestData = CreateCameraActionTestData();

	// Push our test action as clonable.
	UUpdateTrackerCameraAction* NewAction = NewObject<UUpdateTrackerCameraAction>();
	NewAction->bPropagateToNewCameraRigs = true;
	NewAction->NumUpdates = 3;
	FCameraActionInstanceID NewInstanceID = TestData.StartAction(NewAction);

	// Update once, the camera action should be running.
	TestData.UpdateEvaluator();
	UTEST_TRUE("Action is running", TestData.ActionService->IsActionRunning(NewInstanceID));
	UTEST_EQUAL("One action active", TestData.ActionService->GetNumScopeActions(NewInstanceID), 1);

	// Force activate a new rig, with a transition blend to make sure both rigs will run for a few more frames.
	UCameraRigTransition* NewTransition = NewObject<UCameraRigTransition>();
	NewTransition->Blend = NewObject<ULinearBlendCameraNode>();
	TestData.ForceReactivateTestCameraRig(NewTransition);
	TestData.UpdateEvaluator();

	// The action should have been cloned over, with both instances left with 2 updates.
	UTEST_TRUE("Action is running", TestData.ActionService->IsActionRunning(NewInstanceID));
	UTEST_EQUAL("Two actions active", TestData.ActionService->GetNumScopeActions(NewInstanceID), 2);

	TestData.UpdateEvaluator();
	UTEST_FALSE("Action has finished", TestData.ActionService->IsActionRunning(NewInstanceID));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraActionPendingTest, "System.Engine.GameplayCameras.CameraAction.StartAndActivateOnSameFrame", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraActionPendingTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	FCameraActionTestData TestData = CreateCameraActionTestData();

	// In the same frame, start an action and push a new rig instance.
	UUpdateTrackerCameraAction* NewAction = NewObject<UUpdateTrackerCameraAction>();
	NewAction->NumUpdates = 2;
	FCameraActionInstanceID NewInstanceID = TestData.StartAction(NewAction);

	TestData.ForceReactivateTestCameraRig();
	TestData.UpdateEvaluator();

	// The camera action should be running, but only in the newest camera rig's scope.
	UTEST_TRUE("Action is running", TestData.ActionService->IsActionRunning(NewInstanceID));
	UTEST_EQUAL("One action active", TestData.ActionService->GetNumScopeActions(NewInstanceID), 1);

	// Run again, it should be done.
	TestData.UpdateEvaluator();
	UTEST_FALSE("Action has finished", TestData.ActionService->IsActionRunning(NewInstanceID));
	UTEST_EQUAL("No more action active", TestData.ActionService->GetNumScopeActions(NewInstanceID), 0);

	return true;
}

