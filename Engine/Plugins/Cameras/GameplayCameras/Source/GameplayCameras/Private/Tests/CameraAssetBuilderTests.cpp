// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraAsset.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraParameters.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Nodes/Common/LensParametersCameraNode.h"
#include "Nodes/Common/OffsetCameraNode.h"
#include "StructUtils/PropertyBag.h"
#include "Build/CameraAssetAssembleUtils.h"
#include "Tests/GameplayCamerasTestObjects.h"

#define LOCTEXT_NAMESPACE "CameraAssetBuilderTests"

namespace UE::Cameras::Test
{

template<typename ParameterType>
bool CheckDefaultParameter(const FInstancedPropertyBag& PropertyBag, const FName ParameterName, typename TCallTraits<typename ParameterType::ValueType>::ParamType ExpectedValue)
{
	TValueOrError<ParameterType*, EPropertyBagResult> ValueOrError = PropertyBag.GetValueStruct<ParameterType>(ParameterName);
	if (!ValueOrError.HasError() || ValueOrError.HasValue())
	{
		const ParameterType* Value = ValueOrError.GetValue();
		return Value->Value == ExpectedValue;
	}
	return false;
}

}  // namespace UE::Cameras::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraAssetBuilderDefaultParametersTest, "System.Engine.GameplayCameras.CameraAssetBuilder.DefaultParameters", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraAssetBuilderDefaultParametersTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	UCameraRigAsset* CameraRigOne = FCameraRigAssetAssembler(TEXT("One"))
		.MakeRootNode<UOffsetCameraNode>().Named(TEXT("Offset"))
			.Done()
		.AddBlendableParameter(TEXT("TranslationOffset"), ECameraVariableType::Vector3d, TEXT("Offset"), GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, TranslationOffset))
		.BuildCameraRig()
		.SetDefaultParameterValue<FVector3d>(TEXT("TranslationOffset"), FVector3d(10.0, 20.0, 0.0))
		.Get();

	UCameraRigAsset* CameraRigTwo = FCameraRigAssetAssembler(TEXT("Two"))
		.MakeRootNode<ULensParametersCameraNode>().Named(TEXT("Lens"))
			.Done()
		.AddBlendableParameter(TEXT("FocalLength"), ECameraVariableType::Float, TEXT("Lens"), GET_MEMBER_NAME_CHECKED(ULensParametersCameraNode, FocalLength))
		.BuildCameraRig()
		.SetDefaultParameterValue<float>(TEXT("FocalLength"), 55.0f)
		.Get();

	UFixedTestCameraDirector* CameraDirector = nullptr;
	UCameraAsset* Camera = FCameraAssetAssembler()
		.MakeDirector<UFixedTestCameraDirector>()
			.Pin(CameraDirector)
			.Done()
		.AddInterfaceParameter(CameraRigOne, TEXT("TranslationOffset"))
		.AddInterfaceParameter(CameraRigTwo, TEXT("FocalLength"))
		.Get();
	CameraDirector->AddCameraRig(CameraRigOne, TEXT("One"));
	CameraDirector->AddCameraRig(CameraRigTwo, TEXT("Two"));

	Camera->BuildCamera();

	TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions = Camera->GetParameterDefinitions();
	UTEST_EQUAL("NumParameters", ParameterDefinitions.Num(), 2);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[0].ParameterName, FName("TranslationOffset"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[0].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[0].VariableType, ECameraVariableType::Vector3d);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[1].ParameterName, FName("FocalLength"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[1].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[1].VariableType, ECameraVariableType::Float);

	const FInstancedPropertyBag& DefaultParameters = Camera->GetDefaultParameters();
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FVector3dCameraParameter>(DefaultParameters, TEXT("TranslationOffset"), FVector3d(10.0, 20.0, 0.0)));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FFloatCameraParameter>(DefaultParameters, TEXT("FocalLength"), 55.f));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraAssetBuilderDefaultParametersCollisionTest, "System.Engine.GameplayCameras.CameraAssetBuilder.DefaultParametersCollision", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraAssetBuilderDefaultParametersCollisionTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	UCameraRigAsset* CameraRigOne = FCameraRigAssetAssembler(TEXT("One"))
		.MakeRootNode<UOffsetCameraNode>().Named(TEXT("Offset"))
			.Done()
		.AddBlendableParameter(TEXT("TranslationOffset"), ECameraVariableType::Vector3d, TEXT("Offset"), GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, TranslationOffset))
		.AddBlendableParameter(TEXT("RotationOffset"), ECameraVariableType::Rotator3d, TEXT("Offset"), GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, RotationOffset))
		.BuildCameraRig()
		.SetDefaultParameterValue<FVector3d>(TEXT("TranslationOffset"), FVector3d(10.0, 20.0, 0.0))
		.Get();

	UCameraRigAsset* CameraRigTwo = FCameraRigAssetAssembler(TEXT("Two"))
		.MakeRootNode<ULensParametersCameraNode>().Named(TEXT("Lens"))
			.Done()
		.AddBlendableParameter(TEXT("FocalLength"), ECameraVariableType::Float, TEXT("Lens"), GET_MEMBER_NAME_CHECKED(ULensParametersCameraNode, FocalLength))
		.AddBlendableParameter(TEXT("Aperture"), ECameraVariableType::Float, TEXT("Lens"), GET_MEMBER_NAME_CHECKED(ULensParametersCameraNode, Aperture))
		.BuildCameraRig()
		.SetDefaultParameterValue<float>(TEXT("FocalLength"), 55.0f)
		.Get();

	UCameraRigAsset* CameraRigThree = FCameraRigAssetAssembler(TEXT("Three"))
		.MakeArrayRootNode()
			.AddChild<UOffsetCameraNode>(&UArrayCameraNode::Children).Named(TEXT("Offset2"))
				.Done()
			.AddChild<ULensParametersCameraNode>(&UArrayCameraNode::Children).Named(TEXT("Lens2"))
				.Done()
			.Done()
		.AddBlendableParameter(TEXT("TranslationOffset"), ECameraVariableType::Vector3d, TEXT("Offset2"), GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, TranslationOffset))
		.AddBlendableParameter(TEXT("FocalLength"), ECameraVariableType::Float, TEXT("Lens2"), GET_MEMBER_NAME_CHECKED(ULensParametersCameraNode, FocalLength))
		.BuildCameraRig()
		.SetDefaultParameterValue<FVector3d>(TEXT("TranslationOffset"), FVector3d(0.0, 10.0, 20.0))
		.SetDefaultParameterValue<float>(TEXT("FocalLength"), 35.0f)
		.Get();

	UFixedTestCameraDirector* CameraDirector = nullptr;
	UCameraAsset* Camera = FCameraAssetAssembler()
		.MakeDirector<UFixedTestCameraDirector>()
			.Pin(CameraDirector)
			.Done()
		.AddInterfaceParameter(CameraRigOne, TEXT("TranslationOffset"), TEXT("One_TranslationOffset"))
		.AddInterfaceParameter(CameraRigOne, TEXT("RotationOffset"))
		.AddInterfaceParameter(CameraRigTwo, TEXT("FocalLength"), TEXT("Two_FocalLength"))
		.AddInterfaceParameter(CameraRigTwo, TEXT("Aperture"))
		.AddInterfaceParameter(CameraRigThree, TEXT("TranslationOffset"), TEXT("Three_TranslationOffset"))
		.AddInterfaceParameter(CameraRigThree, TEXT("FocalLength"), TEXT("Three_FocalLength"))
		.Get();
	CameraDirector->AddCameraRig(CameraRigOne, TEXT("One"));
	CameraDirector->AddCameraRig(CameraRigTwo, TEXT("Two"));
	CameraDirector->AddCameraRig(CameraRigThree, TEXT("Three"));

	Camera->BuildCamera();

	TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions = Camera->GetParameterDefinitions();
	UTEST_EQUAL("NumParameters", ParameterDefinitions.Num(), 6);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[0].ParameterName, FName("One_TranslationOffset"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[0].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[0].VariableType, ECameraVariableType::Vector3d);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[1].ParameterName, FName("RotationOffset"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[1].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[1].VariableType, ECameraVariableType::Rotator3d);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[2].ParameterName, FName("Two_FocalLength"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[2].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[2].VariableType, ECameraVariableType::Float);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[3].ParameterName, FName("Aperture"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[3].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[3].VariableType, ECameraVariableType::Float);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[4].ParameterName, FName("Three_TranslationOffset"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[4].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[4].VariableType, ECameraVariableType::Vector3d);
	UTEST_EQUAL("Parameter_Name", ParameterDefinitions[5].ParameterName, FName("Three_FocalLength"));
	UTEST_EQUAL("Parameter_Type", ParameterDefinitions[5].ParameterType, ECameraObjectInterfaceParameterType::Blendable);
	UTEST_EQUAL("Parameter_VariableType", ParameterDefinitions[5].VariableType, ECameraVariableType::Float);

	const FInstancedPropertyBag& DefaultParameters = Camera->GetDefaultParameters();
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FVector3dCameraParameter>(DefaultParameters, TEXT("One_TranslationOffset"), FVector3d(10.0, 20.0, 0.0)));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FRotator3dCameraParameter>(DefaultParameters, TEXT("RotationOffset"), FRotator(0.0, 0.0, 0.0)));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FFloatCameraParameter>(DefaultParameters, TEXT("Two_FocalLength"), 55.f));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FFloatCameraParameter>(DefaultParameters, TEXT("Aperture"), 0.f));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FVector3dCameraParameter>(DefaultParameters, TEXT("Three_TranslationOffset"), FVector3d(0.0, 10.0, 20.0)));
	UTEST_TRUE("Parameter_Default", CheckDefaultParameter<FFloatCameraParameter>(DefaultParameters, TEXT("Three_FocalLength"), 35.f));

	return true;
}

#undef LOCTEXT_NAMESPACE

