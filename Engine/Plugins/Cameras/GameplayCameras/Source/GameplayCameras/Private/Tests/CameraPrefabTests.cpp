// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraAsset.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Misc/AutomationTest.h"
#include "Nodes/Common/CameraRigCameraNode.h"
#include "Nodes/Common/LensParametersCameraNode.h"
#include "Build/CameraAssetAssembleUtils.h"
#include "Tests/GameplayCamerasTestObjects.h"

#define LOCTEXT_NAMESPACE "CameraPrefabTests"

namespace UE::Cameras::Test
{

TSharedRef<UE::Cameras::FNamedObjectRegistry> CreatePrefabTestCameraRigs()
{
	TSharedRef<FNamedObjectRegistry> NamedObjectRegistry = MakeShared<FNamedObjectRegistry>();

	// A camera rig with a lens parameter node.
	// That node has FocalLength, Aperture, and FocusDistance exposed as rig parameters.
	// Default values are 22, 3, and 500 respectively.
	UCameraRigAsset* InnerRig = FCameraRigAssetAssembler(NamedObjectRegistry, TEXT("InnerRig"))
		.MakeRootNode<ULensParametersCameraNode>()
			.Named(TEXT("InnerLensNode"))
			.Setup([](ULensParametersCameraNode* Node)
			{
				// Note: Only bEnableFocusDistance defaults to false, but set all of them to be explicit about the functionality we need.
				Node->bEnableAperture = true;
				Node->bEnableFocalLength = true;
				Node->bEnableFocusDistance = true;
			})
			.Done()
		.AddBlendableParameter(
				TEXT("InnerFocalLength"), ECameraVariableType::Float, 
				TEXT("InnerLensNode"), TEXT("FocalLength"))
		.AddBlendableParameter(
				TEXT("InnerAperture"), ECameraVariableType::Float,
				TEXT("InnerLensNode"), TEXT("Aperture"))
		.AddBlendableParameter(
				TEXT("InnerFocusDistance"), ECameraVariableType::Float,
				TEXT("InnerLensNode"), TEXT("FocusDistance"))
		.BuildCameraRig()
		.SetDefaultParameterValue<float>(TEXT("InnerFocalLength"), 22.f)
		.SetDefaultParameterValue<float>(TEXT("InnerAperture"), 3.f)
		.SetDefaultParameterValue<float>(TEXT("InnerFocusDistance"), 500.f)
		.Named(TEXT("InnerRig"))
		.Get();

	// A rig that uses the first one as a prefab.
	// It forwards FocalLength and Aperture as rig parameters.
	// It changes their default values to 18 and 4.5 respectively.
	// It overrides FocusDistance to 600.
	UCameraRigAsset* FirstLevelRig = FCameraRigAssetAssembler(NamedObjectRegistry, TEXT("FirstLevelRig"))
		.MakeRootNode<UCameraRigCameraNode>()
			.Named(TEXT("FirstLevelPrefabNode"))
			.Setup([InnerRig](UCameraRigCameraNode* Node)
				{
					Node->CameraRigReference.SetCameraRig(InnerRig);
				})
			.Done()
		.AddBlendableParameter(
				TEXT("FirstLevelFocalLength"), ECameraVariableType::Float,
				TEXT("FirstLevelPrefabNode"), TEXT("InnerFocalLength"))
		.AddBlendableParameter(
				TEXT("FirstLevelAperture"), ECameraVariableType::Float,
				TEXT("FirstLevelPrefabNode"), TEXT("InnerAperture"))
		.BuildCameraRig()
		.SetDefaultParameterValue<float>(TEXT("FirstLevelFocalLength"), 18.f)
		.SetDefaultParameterValue<float>(TEXT("FirstLevelAperture"), 4.5f)
		.Setup([](UCameraRigAsset* CameraRig, FNamedObjectRegistry* NamedObjectRegistry)
				{
					UCameraRigCameraNode* FirstLevelPrefabNode = NamedObjectRegistry->Get<UCameraRigCameraNode>(TEXT("FirstLevelPrefabNode"));
					FInstancedOverridablePropertyBag& Parameters = FirstLevelPrefabNode->CameraRigReference.GetParameters();
					Parameters.SetPropertyOverriden(TEXT("InnerFocusDistance"), true);
					Parameters.GetValueStruct<FFloatCameraParameter>(TEXT("InnerFocusDistance")).GetValue()->Value = 600.f;
				})
		.Named(TEXT("FirstLevelRig"))
		.Get();

	// A rig that uses the previous rig as a prefab.
	// It overrides FocalLenth to 16.
	FCameraRigAssetAssembler(NamedObjectRegistry, TEXT("SecondLevelRig"))
		.MakeRootNode<UCameraRigCameraNode>()
			.Named(TEXT("SecondLevelPrefabNode"))
			.Setup([FirstLevelRig](UCameraRigCameraNode* Node)
				{
					Node->CameraRigReference.SetCameraRig(FirstLevelRig);
				})
			.Done()
		.BuildCameraRig()
		.Setup([](UCameraRigAsset* CameraRig, FNamedObjectRegistry* NamedObjectRegistry)
				{
					UCameraRigCameraNode* SecondLevelPrefabNode = NamedObjectRegistry->Get<UCameraRigCameraNode>(TEXT("SecondLevelPrefabNode"));
					FInstancedOverridablePropertyBag& Parameters = SecondLevelPrefabNode->CameraRigReference.GetParameters();
					Parameters.SetPropertyOverriden(TEXT("FirstLevelFocalLength"), true);
					Parameters.GetValueStruct<FFloatCameraParameter>(TEXT("FirstLevelFocalLength")).GetValue()->Value = 16.f;
				})
		.Named(TEXT("SecondLevelRig"))
		.Get();

	return NamedObjectRegistry;
}

void RunCameraRig(UCameraRigAsset* InCameraRig, FCameraPose& OutCameraPose)
{
	TSharedRef<FCameraEvaluationContext> EvaluationContext = FCameraEvaluationContextAssembler()
		.MakeSingleDirector(InCameraRig)
			.Done()
		.BuildCameraAsset()
		.Get();

	TSharedRef<FCameraSystemEvaluator> Evaluator = FCameraSystemEvaluatorAssembler::Build();
	Evaluator->PushEvaluationContext(EvaluationContext);

	EvaluationContext->GetInitialResult().bIsValid = true;

	FCameraSystemEvaluationParams Params;
	Params.DeltaTime = 0.3f;
	Evaluator->Update(Params);

	const FCameraSystemEvaluationResult& EvaluatedResult = Evaluator->GetEvaluatedResult();
	OutCameraPose = EvaluatedResult.CameraPose;
}

}  // namespace UE::Cameras::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraRigPrefabDefaultsTest, "System.Engine.GameplayCameras.Prefab.Defaults", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraRigPrefabDefaultsTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	TSharedRef<FNamedObjectRegistry> NamedObjectRegistry = CreatePrefabTestCameraRigs();

	{
		FCameraPose CameraPose;
		RunCameraRig(NamedObjectRegistry->Get<UCameraRigAsset>("InnerRig"), CameraPose);
		UTEST_EQUAL(TEXT("FocalLength"), CameraPose.GetFocalLength(), 22.f);
		UTEST_EQUAL(TEXT("Aperture"), CameraPose.GetAperture(), 3.f);
		UTEST_EQUAL(TEXT("FocusDistance"), CameraPose.GetFocusDistance(), 500.f);
	}

	{
		FCameraPose CameraPose;
		RunCameraRig(NamedObjectRegistry->Get<UCameraRigAsset>("FirstLevelRig"), CameraPose);
		UTEST_EQUAL(TEXT("FocalLength"), CameraPose.GetFocalLength(), 18.f);
		UTEST_EQUAL(TEXT("Aperture"), CameraPose.GetAperture(), 4.5f);
		UTEST_EQUAL(TEXT("FocusDistance"), CameraPose.GetFocusDistance(), 600.f);
	}

	{
		FCameraPose CameraPose;
		RunCameraRig(NamedObjectRegistry->Get<UCameraRigAsset>("SecondLevelRig"), CameraPose);
		UTEST_EQUAL(TEXT("FocalLength"), CameraPose.GetFocalLength(), 16.f);
		UTEST_EQUAL(TEXT("Aperture"), CameraPose.GetAperture(), 4.5f);
		UTEST_EQUAL(TEXT("FocusDistance"), CameraPose.GetFocusDistance(), 600.f);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

