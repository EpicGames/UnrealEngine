// Copyright Epic Games, Inc. All Rights Reserved.

#include "Build/CameraBuildContext.h"
#include "Core/CameraRigAsset.h"
#include "Misc/AutomationTest.h"
#include "Nodes/Blends/SmoothBlendCameraNode.h"
#include "Nodes/Common/CameraRigCameraNode.h"
#include "Nodes/Common/LensParametersCameraNode.h"
#include "Nodes/Common/OffsetCameraNode.h"
#include "Build/CameraAssetAssembleUtils.h"

#define LOCTEXT_NAMESPACE "CameraRigAssetBuilderTests"

namespace UE::Cameras::Test
{

void BuildCameraRigAsIfCooking(UCameraRigAsset* CameraRig)
{
	FCameraBuildLog BuildLog;
	BuildLog.SetForwardMessagesToLogging(true);
	FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::Cooking);
	CameraRig->BuildCameraRig(BuildContext);
}

}  // namespace UE::Cameras::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraRigAssetBuilderNullTest, "System.Engine.GameplayCameras.CameraRigAssetBuilder.Null", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraRigAssetBuilderNullTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	UCameraRigAsset* CameraRig = FCameraRigAssetAssembler(TEXT("EmptyTest")).Get();
	UTEST_EQUAL("Dirty status", CameraRig->BuildStatus, ECameraBuildStatus::Dirty);

	UE::Cameras::Test::BuildCameraRigAsIfCooking(CameraRig);
	UTEST_EQUAL("Clean status", CameraRig->BuildStatus, ECameraBuildStatus::Clean);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraRigAssetBuilderSimpleAllocationTest, "System.Engine.GameplayCameras.CameraRigAssetBuilder.SimpleAllocation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraRigAssetBuilderSimpleAllocationTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	UCameraRigAsset* CameraRig = FCameraRigAssetAssembler()
		.MakeRootNode<UArrayCameraNode>()
			.AddChild<UOffsetCameraNode>(&UArrayCameraNode::Children).Done()
			.Done()
		.Get();

	UTEST_EQUAL("No evaluator allocation info", CameraRig->AllocationInfo.EvaluatorInfo.TotalSizeof, 0);
	UE::Cameras::Test::BuildCameraRigAsIfCooking(CameraRig);

	int32 TotalSizeof = UArrayCameraNode::GetEvaluatorAllocationInfo().Key;
	const TTuple<int32, int32> OffsetEvaluatorInfo = UOffsetCameraNode::GetEvaluatorAllocationInfo();
	TotalSizeof = Align(TotalSizeof, OffsetEvaluatorInfo.Value) + OffsetEvaluatorInfo.Key;
	UTEST_EQUAL("Evaluator allocation info", CameraRig->AllocationInfo.EvaluatorInfo.TotalSizeof, TotalSizeof);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraRigAssetBuilderSimpleParameterTest, "System.Engine.GameplayCameras.CameraRigAssetBuilder.SimpleParameter", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraRigAssetBuilderSimpleParameterTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	UOffsetCameraNode* OffsetNode = nullptr;
	UCameraRigAsset* CameraRig = FCameraRigAssetAssembler(TEXT("SimpleTest"))
		.MakeRootNode<UArrayCameraNode>()
			.AddChild<UOffsetCameraNode>(&UArrayCameraNode::Children)
				.Pin(OffsetNode)
				.Done()
			.Done()
		.AddBlendableParameter(TEXT("Test"), ECameraVariableType::Vector3d, OffsetNode, GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, TranslationOffset))
		.Get();

	UE::Cameras::Test::BuildCameraRigAsIfCooking(CameraRig);

	UCameraObjectInterfaceBlendableParameter* Parameter = CameraRig->Interface.BlendableParameters[0];
	UTEST_EQUAL("Test parameter", Parameter->InterfaceParameterName, TEXT("Test"));
	UTEST_TRUE("Test parameter variable ID", Parameter->PrivateVariableID.IsValid());
	UTEST_EQUAL("Test node parameter", OffsetNode->TranslationOffset.VariableID, Parameter->PrivateVariableID);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraRigAssetBuilderDrivenOverridesTest, "System.Engine.GameplayCameras.CameraRigAssetBuilder.DrivenOverrides", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraRigAssetBuilderDrivenOverridesTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	TSharedRef<FNamedObjectRegistry> Registry = MakeShared<FNamedObjectRegistry>();

	// Make a camera rig with an offset node (10, 20, 30) and a focal length node (20). Expose both parameters
	// as interface parameters.
	UCameraRigAsset* InnerCameraRig = FCameraRigAssetAssembler(Registry, TEXT("InnerCameraRig"))
		.MakeArrayRootNode()
			.AddArrayChild<UOffsetCameraNode>().Named(TEXT("Offset"))
				.SetParameter(&UOffsetCameraNode::TranslationOffset, FVector3d(10, 20, 30))
				.Done()
			.AddArrayChild<ULensParametersCameraNode>().Named(TEXT("Lens"))
				.SetParameter(&ULensParametersCameraNode::FocalLength, 20.f)
				.Done()
			.Done()
		.AddBlendableParameter(TEXT("OffsetParam"), ECameraVariableType::Vector3d, TEXT("Offset"), GET_MEMBER_NAME_CHECKED(UOffsetCameraNode, TranslationOffset))
		.AddBlendableParameter(TEXT("FocalLengthParam"), ECameraVariableType::Float, TEXT("Lens"), GET_MEMBER_NAME_CHECKED(ULensParametersCameraNode, FocalLength))
		.Get();

	UE::Cameras::Test::BuildCameraRigAsIfCooking(InnerCameraRig);

	// Make a camera rig that uses the previous one, with overrides on both the offset (now 15, 25, 35)
	// and the focal length (now 25). Expose the offset further up as an interface parameter.
	UCameraRigCameraNode* MiddlePrefabNode = nullptr;

	UCameraRigAsset* MiddleCameraRig = FCameraRigAssetAssembler(Registry, TEXT("MiddleCameraRig"))
		.MakeRootNode<UCameraRigCameraNode>()
			.Pin(MiddlePrefabNode)
			.Setup([InnerCameraRig](UCameraRigCameraNode* Node, FNamedObjectRegistry* Registry)
					{
						Node->CameraRigReference.SetCameraRig(InnerCameraRig);

						FInstancedPropertyBag& ParameterOverrides = Node->CameraRigReference.GetParameters();

						auto OffsetParamOverride = ParameterOverrides.GetValueStruct<FVector3dCameraParameter>("OffsetParam");
						if (ensure(OffsetParamOverride.HasValue()))
						{
							OffsetParamOverride.GetValue()->Value = FVector3d(15, 25, 35);
						}

						auto FocalLengthParamOverride = ParameterOverrides.GetValueStruct<FFloatCameraParameter>("FocalLengthParam");
						if (ensure(FocalLengthParamOverride.HasValue()))
						{
							FocalLengthParamOverride.GetValue()->Value = 25.f;
						}
					})
			.Done()
			.AddBlendableParameter(TEXT("MiddleOffsetParam"), ECameraVariableType::Vector3d, MiddlePrefabNode, TEXT("OffsetParam"))
		.Get();

	UE::Cameras::Test::BuildCameraRigAsIfCooking(MiddleCameraRig);

	// Make another camera rig that uses the previous one, which makes a total of 3 nesting levels of camera rigs.
	// This level overrides the offset parameter some more (now 20, 50, 70).
	UCameraRigCameraNode* OuterPrefabNode = nullptr;

	UCameraRigAsset* OuterCameraRig = FCameraRigAssetAssembler(Registry, TEXT("OuterCameraRig"))
		.MakeRootNode<UCameraRigCameraNode>()
			.Pin(OuterPrefabNode)
			.Setup([MiddleCameraRig](UCameraRigCameraNode* Node, FNamedObjectRegistry* Registry)
					{
						Node->CameraRigReference.SetCameraRig(MiddleCameraRig);

						FInstancedPropertyBag& ParameterOverrides = Node->CameraRigReference.GetParameters();

						auto MiddleOffsetParamOverride = ParameterOverrides.GetValueStruct<FVector3dCameraParameter>("MiddleOffsetParam");
						if (ensure(MiddleOffsetParamOverride.HasValue()))
						{
							MiddleOffsetParamOverride.GetValue()->Value = FVector3d(20, 50, 70);
						}
					})
			.Done()
		.Get();

	UE::Cameras::Test::BuildCameraRigAsIfCooking(OuterCameraRig);

	UCameraObjectInterfaceBlendableParameter* OffsetParam = InnerCameraRig->Interface.BlendableParameters[0];
	UCameraObjectInterfaceBlendableParameter* FocalLengthParam = InnerCameraRig->Interface.BlendableParameters[1];

	// Test that the inner nodes are driven by the interface parameters.
	{
		UOffsetCameraNode* OffsetNode = Registry->Get<UOffsetCameraNode>("Offset");
		UTEST_EQUAL_EXPR(OffsetNode->TranslationOffset.VariableID, OffsetParam->PrivateVariableID);

		ULensParametersCameraNode* LensNode = Registry->Get<ULensParametersCameraNode>("Lens");
		UTEST_EQUAL_EXPR(LensNode->FocalLength.VariableID, FocalLengthParam->PrivateVariableID);
	}

	// Test that the middle prefab node is driving the inner interface parameters, and that one of those
	// overrides is in turn driven by the middle camera rig's interface parameter.
	{
		FInstancedPropertyBag& ParameterOverrides = MiddlePrefabNode->CameraRigReference.GetParameters();

		const FPropertyBagPropertyDesc* OffsetParamDesc = ParameterOverrides.FindPropertyDescByName("OffsetParam");
		FVector3dCameraParameter* OffsetParamOverride = ParameterOverrides.GetValueStruct<FVector3dCameraParameter>("OffsetParam").GetValue();
		UTEST_NOT_NULL("OffsetParamDesc", OffsetParamDesc);
		UTEST_NOT_NULL("OffsetParamOverride", OffsetParamOverride);

		UTEST_EQUAL_EXPR(OffsetParamDesc->ID, OffsetParam->GetGuid());
		UTEST_EQUAL_EXPR(OffsetParamOverride->Value, FVector3d(15, 25, 35));

		const FPropertyBagPropertyDesc* FocalLengthParamDesc = ParameterOverrides.FindPropertyDescByName("FocalLengthParam");
		FFloatCameraParameter* FocalLengthParamOverride = ParameterOverrides.GetValueStruct<FFloatCameraParameter>("FocalLengthParam").GetValue();
		UTEST_NOT_NULL("FocalLengthParamDesc", FocalLengthParamDesc);
		UTEST_NOT_NULL("FocalLengthParamOverride", FocalLengthParamOverride);

		UTEST_EQUAL_EXPR(FocalLengthParamDesc->ID, FocalLengthParam->GetGuid());
		UTEST_EQUAL_EXPR(FocalLengthParamOverride->Value, 25.f);
	}

	UCameraObjectInterfaceBlendableParameter* MiddleOffsetParam = MiddleCameraRig->Interface.BlendableParameters[0];
	{
		FInstancedPropertyBag& ParameterOverrides = MiddlePrefabNode->CameraRigReference.GetParameters();

		const FPropertyBagPropertyDesc* OffsetParamDesc = ParameterOverrides.FindPropertyDescByName("OffsetParam");
		FVector3dCameraParameter* OffsetParamOverride = ParameterOverrides.GetValueStruct<FVector3dCameraParameter>("OffsetParam").GetValue();
		UTEST_EQUAL_EXPR(OffsetParamOverride->VariableID, MiddleOffsetParam->PrivateVariableID);
	}

	// Test that the outer prefab node is driving the middle interface parameters.
	{
		FInstancedPropertyBag& ParameterOverrides = OuterPrefabNode->CameraRigReference.GetParameters();

		const FPropertyBagPropertyDesc* OffsetParamDesc = ParameterOverrides.FindPropertyDescByName("MiddleOffsetParam");
		FVector3dCameraParameter* OffsetParamOverride = ParameterOverrides.GetValueStruct<FVector3dCameraParameter>("MiddleOffsetParam").GetValue();
		UTEST_NOT_NULL("OffsetParamDesc", OffsetParamDesc);
		UTEST_NOT_NULL("OffsetParamOverride", OffsetParamOverride);

		UTEST_EQUAL_EXPR(OffsetParamDesc->ID, MiddleOffsetParam->GetGuid());
		UTEST_EQUAL_EXPR(OffsetParamOverride->Value, FVector3d(20, 50, 70));
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

