// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CameraRigAsset.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/RootCameraNode.h"
#include "Misc/AutomationTest.h"
#include "Build/CameraAssetAssembleUtils.h"
#include "Tests/GameplayCamerasTestObjects.h"

#define LOCTEXT_NAMESPACE "CameraSystemUpdateTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraSystemFrameFlagsTest, "System.Engine.GameplayCameras.CameraSystem.FrameFlags", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraSystemFrameFlagsTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;
	using namespace UE::Cameras::Test;

	TSharedRef<FCameraEvaluationContext> EvaluationContext = FCameraEvaluationContextAssembler()
		.CreateCameraRig(TEXT("TestRig"))
			.MakeRootNode<UUpdateTrackerCameraNode>()
				.Done()
			.Done()
		.MakeSingleDirector()
			.Setup([](USingleCameraDirector* Director, FNamedObjectRegistry* NamedObjectRegistry)
				{
					Director->CameraRig = NamedObjectRegistry->Get<UCameraRigAsset>(TEXT("TestRig"));
				})
			.Done()
		.BuildCameraAsset()
		.Get();

	TSharedRef<FCameraSystemEvaluator> Evaluator = FCameraSystemEvaluatorAssembler::Build();
	Evaluator->PushEvaluationContext(EvaluationContext);

	EvaluationContext->GetInitialResult().bIsValid = true;

	FCameraSystemEvaluationParams Params;
	Params.DeltaTime = 0.3f;
	Evaluator->Update(Params);

	FCameraRigEvaluationInfo CameraRigInfo;
	FRootCameraNodeEvaluator* RootEvaluator = Evaluator->GetRootNodeEvaluator();
	RootEvaluator->GetActiveCameraRigInfo(CameraRigInfo);
	UTEST_NOT_NULL("RootEvaluator", CameraRigInfo.RootEvaluator);

	FUpdateTrackerCameraNodeEvaluator* UpdateTracker = CameraRigInfo.RootEvaluator->CastThis<FUpdateTrackerCameraNodeEvaluator>();
	UTEST_NOT_NULL("UpdateTracker", UpdateTracker);

	{
		UTEST_EQUAL("NumReceivedUpdates", UpdateTracker->ReceivedUpdates.Num(), 1);
		FTrackedUpdateInfo Update = UpdateTracker->ReceivedUpdates[0];
		UTEST_EQUAL("DeltaTime", Update.DeltaTime, 0.3f);
		UTEST_FALSE("IsCameraCut", Update.bIsCameraCut);
		UTEST_TRUE("IsFirstFrame", Update.bIsFirstFrame);
	}

	Evaluator->Update(Params);

	{
		UTEST_EQUAL("NumReceivedUpdates", UpdateTracker->ReceivedUpdates.Num(), 2);
		FTrackedUpdateInfo Update = UpdateTracker->ReceivedUpdates[1];
		UTEST_EQUAL("DeltaTime", Update.DeltaTime, 0.3f);
		UTEST_FALSE("IsCameraCut", Update.bIsCameraCut);
		UTEST_FALSE("IsFirstFrame", Update.bIsFirstFrame);
	}

	EvaluationContext->GetInitialResult().bIsCameraCut = true;

	Evaluator->Update(Params);

	{
		UTEST_EQUAL("NumReceivedUpdates", UpdateTracker->ReceivedUpdates.Num(), 3);
		FTrackedUpdateInfo Update = UpdateTracker->ReceivedUpdates[2];
		UTEST_EQUAL("DeltaTime", Update.DeltaTime, 0.3f);
		UTEST_TRUE("IsCameraCut", Update.bIsCameraCut);
		UTEST_FALSE("IsFirstFrame", Update.bIsFirstFrame);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

