// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/CriticalDamper.h"
#include "Misc/AutomationTest.h"

#define LOCTEXT_NAMESPACE "CriticalDamperTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayCamerasCriticalDamperTest, "System.Engine.GameplayCameras.CriticalDamper", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FGameplayCamerasCriticalDamperTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cameras;

	{
		FCriticalDamper Damper(10.f);
		float Test = Damper.Update(0, 1.f);
		UTEST_EQUAL("Dampen 0", Test, 0.f);
	}

	{
		FCriticalDamper Damper(10.f);
		Damper.Reset(1, 0);
		for (int32 Frame = 0; Frame < 30; ++Frame)
		{
			Damper.Update(0.34f);
		}
		UTEST_EQUAL("Dampen 1", Damper.GetX0(), 0.f);
	}

	{
		FCriticalDamper Damper(10.f);
		float Value = 1.f;
		for (int32 Frame = 0; Frame < 30; ++Frame)
		{
			Value = Damper.Update(Value, 0.34f);
		}
		UTEST_EQUAL("Dampen 1 with value passing", Damper.GetX0(), 0.f);
	}

	{
		FCriticalDamper Damper(10.f);
		float Value = 1.f;
		for (int32 Frame = 0; Frame < 30; ++Frame)
		{
			Value = Damper.Update(Value, 0, 0.34f);
		}
		UTEST_EQUAL("Dampen 1 with state passing", Damper.GetX0(), 0.f);
	}

	{
		FCriticalDamper Damper(10.f);
		float Value = -1.f;
		for (int32 Frame = 0; Frame < 30; ++Frame)
		{
			Value = Damper.Update(Value, 0.34f);
		}
		UTEST_EQUAL("Dampen -1", Damper.GetX0(), 0.f);
	}

	{
		FCriticalDamper Damper(10.f);
		float Value = 5.f;
		for (int32 Frame = 0; Frame < 40; ++Frame)
		{
			Value = Damper.Update(Value, 2.f, 0.34f);
		}
		UTEST_EQUAL("Dampen 5->2 (X0)", Damper.GetX0(), 0.f);
		UTEST_EQUAL_TOLERANCE("Dampen 5->2", Value, 2.f, 0.00001f);
	}

	{
		FCriticalDamper Damper1(10.f);
		Damper1.Reset(1, 0);
		Damper1.Update(0.34f);
		Damper1.Update(0.34f);

		FCriticalDamper Damper2(10.f);
		Damper2.Reset(Damper1.GetX0(), Damper1.GetX0Derivative());

		UTEST_EQUAL("Framerate equivalence", Damper2.GetX0(), Damper1.GetX0());
		UTEST_EQUAL("Framerate equivalence", Damper2.GetX0Derivative(), Damper1.GetX0Derivative());
		UTEST_EQUAL("Framerate equivalence", Damper2.GetW0(), Damper1.GetW0());
		Damper1.Update(0.34f);
		Damper1.Update(0.34f);
		Damper2.Update(0.68f);
		UTEST_EQUAL_TOLERANCE("Framerate equivalence", Damper2.GetX0(), Damper1.GetX0(), 0.001f);
		UTEST_EQUAL_TOLERANCE("Framerate equivalence", Damper2.GetX0Derivative(), Damper1.GetX0Derivative(), 0.001f);
	}

	{
		FCriticalDamper Damper1(10.f);
		Damper1.Reset(0, 0);
		float X1 = 5.f;
		X1 = Damper1.Update(X1, 6, 0.34f);
		X1 = Damper1.Update(X1, 7, 0.34f);
		X1 = Damper1.Update(X1, 8, 0.34f);
		X1 = Damper1.Update(X1, 9, 0.34f);
		X1 = Damper1.Update(X1, 10, 0.34f);
		X1 = Damper1.Update(X1, 11, 0.34f);

		FCriticalDamper Damper2(10.f);
		Damper2.Reset(0, 0);
		float X2 = 5.f;
		X2 = Damper2.Update(X2, 7, 0.68f);
		X2 = Damper2.Update(X2, 9, 0.68f);
		X2 = Damper2.Update(X2, 11, 0.68f);

		UTEST_EQUAL_TOLERANCE("Framerate equivalence", X2, X1, 0.001f);
		UTEST_EQUAL_TOLERANCE("Framerate equivalence", Damper2.GetX0(), Damper1.GetX0(), 0.001f);
		UTEST_EQUAL_TOLERANCE("Framerate equivalence", Damper2.GetX0Derivative(), Damper1.GetX0Derivative(), 0.001f);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

