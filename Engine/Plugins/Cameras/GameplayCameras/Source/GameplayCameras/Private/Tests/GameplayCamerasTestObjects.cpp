// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasTestObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCamerasTestObjects)

namespace UE::Cameras::Test
{

UE_DEFINE_CAMERA_NODE_EVALUATOR(FUpdateTrackerCameraNodeEvaluator)

void FUpdateTrackerCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	ReceivedUpdates.Add(FTrackedUpdateInfo
			{
				Params.DeltaTime,
				Params.bIsFirstFrame,
				OutResult.bIsCameraCut
			});
}

UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(FFixedTextCameraDirectorEvaluator)

void FFixedTextCameraDirectorEvaluator::SetActiveCameraRig(const FName InCameraRigName)
{
	const UFixedTestCameraDirector* FixedTestDirector = GetCameraDirectorAs<UFixedTestCameraDirector>();
	ActiveIndex = FixedTestDirector->GetCameraRigIndex(InCameraRigName);
}

void FFixedTextCameraDirectorEvaluator::OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult)
{
	const UFixedTestCameraDirector* FixedTestDirector = GetCameraDirectorAs<UFixedTestCameraDirector>();
	UCameraRigAsset* ChosenCameraRig = FixedTestDirector->GetCameraRig(ActiveIndex);
	if (ChosenCameraRig)
	{
		OutResult.Add(GetEvaluationContext(), ChosenCameraRig);
	}
}

UE_DEFINE_CAMERA_ACTION_EVALUATOR(FUpdateTrackerCameraActionEvaluator)

void FUpdateTrackerCameraActionEvaluator::OnPreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult)
{
	const UUpdateTrackerCameraAction* Action = GetCameraActionAs<UUpdateTrackerCameraAction>();
	if (ensure(Action))
	{
		++NumUpdatesReceived;
		if (NumUpdatesReceived >= Action->NumUpdates)
		{
			OutResult.bIsActionFinished = true;
		}
	}
}

void FUpdateTrackerCameraActionEvaluator::OnSerialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar)
{
	Ar << NumUpdatesReceived;
}

}  // namespace UE::Cameras::Test

FCameraNodeEvaluatorPtr UUpdateTrackerCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras::Test;
	return Builder.BuildEvaluator<FUpdateTrackerCameraNodeEvaluator>();
}

void UFixedTestCameraDirector::AddCameraRig(UCameraRigAsset* InCameraRig, const FName InCameraRigName)
{
	CameraRigs.Add(InCameraRig);
	CameraRigNames.Add(InCameraRigName);
}

int32 UFixedTestCameraDirector::GetCameraRigIndex(const FName InCameraRigName) const
{
	const int32 FoundIndex = CameraRigNames.Find(InCameraRigName);
	if (CameraRigs.IsValidIndex(FoundIndex))
	{
		return FoundIndex;
	}
	return INDEX_NONE;
}

UCameraRigAsset* UFixedTestCameraDirector::GetCameraRig(int32 Index) const
{
	if (CameraRigs.IsValidIndex(Index))
	{
		return CameraRigs[Index];
	}
	return nullptr;
}

FCameraDirectorEvaluatorPtr UFixedTestCameraDirector::OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras::Test;
	return Builder.BuildEvaluator<FFixedTextCameraDirectorEvaluator>();
}

void UFixedTestCameraDirector::OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const
{
	UsageInfo.CameraRigs.Append(CameraRigs);
}

FCameraActionEvaluatorPtr UUpdateTrackerCameraAction::OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras::Test;
	return Builder.BuildEvaluator<FUpdateTrackerCameraActionEvaluator>();
}

