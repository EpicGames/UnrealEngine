// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"
#include "Services/CameraAction.h"
#include "Services/CameraActionEvaluator.h"

#include "GameplayCamerasTestObjects.generated.h"

namespace UE::Cameras::Test
{

struct FTrackedUpdateInfo
{
	float DeltaTime = 0.f;
	bool bIsFirstFrame = false;
	bool bIsCameraCut = false;
};

class FUpdateTrackerCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FUpdateTrackerCameraNodeEvaluator)

public:

	TArray<FTrackedUpdateInfo> ReceivedUpdates;

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
};

class FFixedTextCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FFixedTextCameraDirectorEvaluator)

public:

	void SetActiveCameraRig(const FName InCameraRigName);

protected:

	// FCameraDirectorEvaluator interface.
	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override;

private:

	int32 ActiveIndex = INDEX_NONE;
};

class FUpdateTrackerCameraActionEvaluator : public FCameraActionEvaluator
{
	UE_DECLARE_CAMERA_ACTION_EVALUATOR(GAMEPLAYCAMERAS_API, FUpdateTrackerCameraActionEvaluator)

protected:

	// FCameraActionEvaluator interface.
	virtual void OnPreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar) override;

private:

	int32 NumUpdatesReceived = 0;
};

}  // namespace UE::Cameras::Test

UCLASS(MinimalAPI, Hidden)
class UUpdateTrackerCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	UPROPERTY(EditAnywhere, Category=Common)
	FDoubleCameraParameter DoubleParameter;

	UPROPERTY(EditAnywhere, Category=Common)
	FVector3dCameraParameter VectorParameter;
};

UCLASS(MinimalAPI, Hidden)
class UFixedTestCameraDirector : public UCameraDirector
{
	GENERATED_BODY()

public:

	void AddCameraRig(UCameraRigAsset* InCameraRig, const FName InCameraRigName);
	int32 GetCameraRigIndex(const FName InCameraRigName) const;
	UCameraRigAsset* GetCameraRig(int32 Index) const;

protected:

	// UCameraDirector interface.
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const override;
	virtual void OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const override;

private:

	UPROPERTY()
	TArray<TObjectPtr<UCameraRigAsset>> CameraRigs;

	UPROPERTY()
	TArray<FName> CameraRigNames;
};

UCLASS(MinimalAPI, Hidden)
class UUpdateTrackerCameraAction : public UCameraAction
{
	GENERATED_BODY()

public:

	UPROPERTY()
	int32 NumUpdates = 1;

protected:

	// UCameraAction interface.
	virtual FCameraActionEvaluatorPtr OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const;
};

