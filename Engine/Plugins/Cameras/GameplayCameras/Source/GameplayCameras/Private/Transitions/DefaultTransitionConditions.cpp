// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transitions/DefaultTransitionConditions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultTransitionConditions)

bool UIsCameraRigTransitionCondition::OnTransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const
{
	const bool bPreviousMatches = (!PreviousCameraRig || PreviousCameraRig == Params.FromCameraRig);
	const bool bNextMatches = (!NextCameraRig || NextCameraRig == Params.ToCameraRig);
	return bPreviousMatches && bNextMatches;
}

