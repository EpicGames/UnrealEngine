// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transitions/GameplayTagTransitionConditions.h"

#include "Core/CameraRigAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagTransitionConditions)

bool UGameplayTagTransitionCondition::OnTransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const
{
	bool bPreviousMatches = true;

	if (!PreviousGameplayTagQuery.IsEmpty())
	{
		bPreviousMatches = false;

		if (Params.FromCameraRig)
		{
			FGameplayTagContainer TagContainer;
			Params.FromCameraRig->GetOwnedGameplayTags(TagContainer);
			if (TagContainer.MatchesQuery(PreviousGameplayTagQuery))
			{
				bPreviousMatches = true;
			}
		}
	}

	bool bNextMatches = true;

	if (!NextGameplayTagQuery.IsEmpty())
	{
		bNextMatches = false;

		if (Params.ToCameraRig)
		{
			FGameplayTagContainer TagContainer;
			Params.ToCameraRig->GetOwnedGameplayTags(TagContainer);
			if (TagContainer.MatchesQuery(NextGameplayTagQuery))
			{
				bNextMatches = true;
			}
		}
	}

	return bPreviousMatches && bNextMatches;
}

