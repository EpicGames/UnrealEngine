// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValueInterpolators/AccelerationDecelerationValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AccelerationDecelerationValueInterpolator)

namespace UE::Cameras
{

template<typename ValueType>
class TAccelerationDecelerationValueInterpolator : public TCameraValueInterpolator<ValueType>
{
public:

	TAccelerationDecelerationValueInterpolator(const UAccelerationDecelerationValueInterpolator* InInterpolator)
		: TCameraValueInterpolator<ValueType>(InInterpolator)
	{}

protected:

	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) override;
	virtual void OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar) override;

private:

	float CurrentSpeed = 0.f;
};

template<typename ValueType>
void TAccelerationDecelerationValueInterpolator<ValueType>::OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult)
{
	// We need to start decelerating when it would take us to the target value.
	//
	// The equation is:
	//
	//    v1 = a*t + v0
	//
	// Where:
	//    v1 is the next speed
	//    v0 is the current speed
	//    a is the acceleration (negative in the case of deceleration)
	// 
	// If t0 is the time at which we stop (v1 = 0), then:
	//
	//    0 = a*t0 + v0
	//    t0 = -v0/a
	//
	// Distance travelled over that time is:
	//
	//    d = a/2*t^2 + v*t
	//
	// Let's call d0 the distance travelled before we stop:
	//
	//    d0 = a/2*t0^2 + v0*t0
	//    d0 = a/2*(-v0/a)^2 + v0*(-v0)/a
	//    d0 = v0^2/(2*a) - v0^2/a
	//    d0 = (v0^2 - (2*v0^2))/(2*a)
	//    d0 = -v0^2/(2*a)
	//
	// So the speed at which we should go when it's time to decelerate is:
	//
	//    (2*a)*d0 = -v0^2
	//    sqrt(-2*a*d0) = v0
	//
	const double DistanceToTarget = TCameraValueInterpolationTraits<ValueType>::Distance(this->CurrentValue, this->TargetValue);
	if (DistanceToTarget > 0)
	{
		const UAccelerationDecelerationValueInterpolator* SpeedInterpolator = this->template GetParametersAs<UAccelerationDecelerationValueInterpolator>();
		const double MaxSpeed = FMath::Min(
				SpeedInterpolator->MaxSpeed, 
				FMath::Sqrt(2.0 * SpeedInterpolator->Deceleration * DistanceToTarget));
		CurrentSpeed = FMath::Min(CurrentSpeed + SpeedInterpolator->Acceleration * Params.DeltaTime, MaxSpeed);

		const double DistanceThisFrame = CurrentSpeed * Params.DeltaTime;
		if (DistanceThisFrame >= DistanceToTarget)
		{
			CurrentSpeed = 0;
			this->CurrentValue = this->TargetValue;
			this->bIsFinished = true;
		}
		else
		{
			const ValueType Direction = TCameraValueInterpolationTraits<ValueType>::Direction(this->CurrentValue, this->TargetValue);
			this->CurrentValue = this->CurrentValue + Direction * DistanceThisFrame;
		}
	}
	else
	{
		this->CurrentSpeed = 0;
		this->CurrentValue = this->TargetValue;
		this->bIsFinished = true;
	}
}

template<typename ValueType>
void TAccelerationDecelerationValueInterpolator<ValueType>::OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar)
{
	Ar << CurrentSpeed;
}

}  // namespace UE::Cameras

UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(UAccelerationDecelerationValueInterpolator, UE::Cameras::TAccelerationDecelerationValueInterpolator)

