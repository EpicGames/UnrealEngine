// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValueInterpolators/CriticalDamperValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CriticalDamperValueInterpolator)

namespace UE::Cameras
{

template<typename ValueType>
class TCriticalDamperValueInterpolator : public TCameraValueInterpolator<ValueType>
{
public:

	using ValueTypeParam = typename TCameraValueInterpolator<ValueType>::ValueTypeParam;

	TCriticalDamperValueInterpolator(const UCriticalDamperValueInterpolator* InInterpolator)
		: TCameraValueInterpolator<ValueType>(InInterpolator)
	{}

protected:

	virtual void OnReset(ValueTypeParam OldCurrentValue, ValueTypeParam OldTargetValue) override;
	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) override;
	virtual void OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar) override;

private:

	UE::Cameras::FCriticalDamper Damper;
	bool bIsFirstFrame = true;
};

template<typename ValueType>
void TCriticalDamperValueInterpolator<ValueType>::OnReset(ValueTypeParam OldCurrentValue, ValueTypeParam OldTargetValue)
{
	const double DistanceToTarget = TCameraValueInterpolationTraits<ValueType>::Distance(this->CurrentValue, this->TargetValue);
	Damper.Reset(DistanceToTarget, Damper.GetX0Derivative());
}

template<typename ValueType>
void TCriticalDamperValueInterpolator<ValueType>::OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult)
{
	const double DistanceToTarget = TCameraValueInterpolationTraits<ValueType>::Distance(this->CurrentValue, this->TargetValue);
	if (bIsFirstFrame)
	{
		const UCriticalDamperValueInterpolator* CriticalDamperInterpolator = this->template GetParametersAs<UCriticalDamperValueInterpolator>();
		Damper.SetW0(CriticalDamperInterpolator->DampingFactor);
		Damper.Reset(DistanceToTarget, 0);
		bIsFirstFrame = false;
	}

	const double NextDistanceToTarget = Damper.Update(DistanceToTarget, Params.DeltaTime);
	const double ClosingDistance = DistanceToTarget - NextDistanceToTarget;
	const ValueType Direction = TCameraValueInterpolationTraits<ValueType>::Direction(this->CurrentValue, this->TargetValue);
	this->CurrentValue = this->CurrentValue + Direction * ClosingDistance;
	this->bIsFinished = ClosingDistance <= UE_DOUBLE_SMALL_NUMBER;
}

template<typename ValueType>
void TCriticalDamperValueInterpolator<ValueType>::OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar)
{
	Ar << Damper;
	Ar << bIsFirstFrame;
}

}  // namespace UE::Cameras

UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(UCriticalDamperValueInterpolator, UE::Cameras::TCriticalDamperValueInterpolator)

