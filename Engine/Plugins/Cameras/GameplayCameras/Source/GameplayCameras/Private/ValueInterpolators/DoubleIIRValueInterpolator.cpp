// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValueInterpolators/DoubleIIRValueInterpolator.h"

#include "ValueInterpolators/IIRValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DoubleIIRValueInterpolator)

namespace UE::Cameras
{

template<typename ValueType>
class TDoubleIIRValueInterpolator : public TCameraValueInterpolator<ValueType>
{
public:

	TDoubleIIRValueInterpolator(const UDoubleIIRValueInterpolator* InParameters)
		: TCameraValueInterpolator<ValueType>(InParameters)
		, IntermediateInterpolator(InParameters->IntermediateSpeed, InParameters->bUseFixedStep)
		, PrimaryInterpolator(InParameters->PrimarySpeed, InParameters->bUseFixedStep)
		, bUseFixedStep(InParameters->bUseFixedStep)
	{}

protected:

	using ValueTypeParam = typename TCameraValueInterpolator<ValueType>::ValueTypeParam;

	virtual void OnReset(ValueTypeParam OldCurrentValue, ValueTypeParam OldTargetValue) override
	{
		IntermediateInterpolator.Reset(this->CurrentValue, this->TargetValue);
		PrimaryInterpolator.Reset(this->CurrentValue, this->TargetValue);
	}

	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) override
	{
		if (bUseFixedStep)
		{
			float RemainingTime = Params.DeltaTime;

			// Move the substep target value linearly toward the real target value while we evaluate the substeps.
			ValueType LastToTargetValue(this->TargetValue - LastTargetValue);
			if constexpr (std::is_same_v<FRotator3d, ValueType>)
			{
				LastToTargetValue = LastToTargetValue.GetNormalized();
			}
			const ValueType EquilibriumStepRate = LastToTargetValue * (1.f / RemainingTime);

			ValueType LerpedTargetValue = LastTargetValue;

			while (RemainingTime > KINDA_SMALL_NUMBER)
			{
				const float StepTime = FMath::Min(MaxSubstepTime, RemainingTime);

				LerpedTargetValue += EquilibriumStepRate * StepTime;
				RemainingTime -= StepTime;

				RunSubstep(LerpedTargetValue, StepTime, OutResult);
			}

			this->CurrentValue = PrimaryInterpolator.GetCurrentValue();
			LastTargetValue = this->TargetValue;
		}
		else
		{
			this->CurrentValue = RunSubstep(this->TargetValue, Params.DeltaTime, OutResult);
		}
	}

	virtual void OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar) override
	{
		IntermediateInterpolator.Serialize(Params, Ar);
		PrimaryInterpolator.Serialize(Params, Ar);

		Ar << LastTargetValue;
		Ar << bUseFixedStep;
	}

private:

	ValueType RunSubstep(ValueType SubstepTargetValue, float SubstepDeltaTime, FCameraValueInterpolationResult& OutResult)
	{
		FCameraValueInterpolationParams SubParams;
		SubParams.DeltaTime = SubstepDeltaTime;

		IntermediateInterpolator.Reset(IntermediateInterpolator.GetCurrentValue(), SubstepTargetValue);
		ValueType IntermediateValue = IntermediateInterpolator.Run(SubParams, OutResult);

		PrimaryInterpolator.Reset(PrimaryInterpolator.GetCurrentValue(), IntermediateValue);
		return PrimaryInterpolator.Run(SubParams, OutResult);
	}
	
private:

	static constexpr float MaxSubstepTime = 1.f / 120.f;

	TIIRValueInterpolator<ValueType> IntermediateInterpolator;
	TIIRValueInterpolator<ValueType> PrimaryInterpolator;

	ValueType LastTargetValue;
	bool bUseFixedStep = false;
};

}  // namespace UE::Cameras

UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(UDoubleIIRValueInterpolator, UE::Cameras::TDoubleIIRValueInterpolator)

