// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValueInterpolators/IIRValueInterpolator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IIRValueInterpolator)

UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(UIIRValueInterpolator, UE::Cameras::TIIRValueInterpolator)

