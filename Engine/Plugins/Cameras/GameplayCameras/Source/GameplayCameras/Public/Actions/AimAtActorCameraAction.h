// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actions/BaseAimAtCameraAction.h"

#include "AimAtActorCameraAction.generated.h"

/**
 * A camera action that aims at a given target, using a given interpolation.
 */
UCLASS()
class UAimAtActorCameraAction : public UBaseAimAtCameraAction
{
	GENERATED_BODY()

public:

	/** The actor to attach to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Aiming", meta=(ExposeOnSpawn=true))
	TObjectPtr<AActor> TargetActor;

	/** An optional socket to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Aiming", meta=(ExposeOnSpawn=true))
	FName TargetSocketName;

	/** An optional bone to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Aiming", meta=(ExposeOnSpawn=true))
	FName TargetBoneName;

protected:

	// UCameraAction interface.
	virtual FCameraActionEvaluatorPtr OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const override;
};

