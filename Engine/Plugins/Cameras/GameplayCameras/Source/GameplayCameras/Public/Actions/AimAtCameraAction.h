// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actions/BaseAimAtCameraAction.h"

#include "AimAtCameraAction.generated.h"

/**
 * A camera action that aims at a given target, using a given interpolation.
 */
UCLASS()
class UAimAtCameraAction : public UBaseAimAtCameraAction
{
	GENERATED_BODY()

public:

	/** The target to aim at. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	FVector TargetLocation;

protected:

	// UCameraAction interface.
	virtual FCameraActionEvaluatorPtr OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const override;
};

