// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraValueInterpolator.h"
#include "Services/CameraAction.h"
#include "Services/CameraActionEvaluator.h"

#include "BaseAimAtCameraAction.generated.h"

/** Lock-on policy for the aim-at camera action. */
UENUM()
enum class EAimAtCameraActionLockOnPolicy : uint8
{
	/** End the action once the aiming is accomplished. */
	Disengage,
	/** Continuously aim at the target until the action is explicitly stopped. */
	KeepLock
};

/**
 * The base class for all aiming camera actions.
 */
UCLASS()
class UBaseAimAtCameraAction : public UCameraAction
{
	GENERATED_BODY()

public:

	/** The interpolation to use for aiming. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	TObjectPtr<UCameraValueInterpolator> Interpolator;

	/** The tolerance within which we can consider the aiming to be locked on target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	float LockOnAngleTolerance = 0.05f;

	/** The lock-on policy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	EAimAtCameraActionLockOnPolicy LockOnPolicy = EAimAtCameraActionLockOnPolicy::Disengage;

	/** Where to aim the target in screen-space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Aiming, meta=(ExposeOnSpawn=true))
	FVector2D TargetFraming = { 0.5, 0.5 };
};

namespace UE::Cameras
{

/**
 * Base class for any aiming camera action evaluator.
 */
class FBaseAimAtCameraActionEvaluator : public FCameraActionEvaluator
{
	UE_DECLARE_CAMERA_ACTION_EVALUATOR(, FBaseAimAtCameraActionEvaluator)

protected:

	// FCameraActionEvaluator
	virtual void OnInitialize(const FCameraActionEvaluatorInitializeParams& Params, FCameraActionEvaluationResult& OutResult) override;
	virtual void OnPreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

protected:

	// FBaseAimAtCameraActionEvaluator
	virtual FVector3d UpdateTargetLocation(const FCameraActionEvaluationParams& Params, const FCameraActionEvaluationResult& Result);

private:

	void LockUserInputThisFrame(const FCameraActionEvaluationParams& Params);
	void RunPreviewEvaluation(const FCameraActionEvaluationParams& Params, const FCameraNodeEvaluationResult& Result);
	bool ComputeDesiredCorrection(const FCameraActionEvaluationParams& Params, FRotator3d& OutCorrection);
	bool ExecuteYawPitchCorrection(const FCameraActionEvaluationParams& Params, const FRotator3d& Correction);

	bool ContinueAimAction(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult);
	bool KeepLockOn(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult);

private:

	TUniquePtr<TCameraValueInterpolator<FVector2d>> Interpolator;

	FCameraNodeEvaluatorHierarchy ChildHierarchy;

	FCameraNodeEvaluationResult ScratchResult;
	TArray<uint8> EvaluatorSnapshot;

	FVector3d TargetLocation;
	FVector2d TargetFraming = { 0.5, 0.5 };

	bool bIsLockedOn = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	float DebugElapsedTime = 0.f;
	FRotator3d DebugCorrectionLeft;
	FRotator3d DebugCurrentCorrection;
	FVector3d DebugTargetFramingAim;
	FVector3d DebugCameraLocation;
	FVector3d DebugCameraAim;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

}  // namespace UE::Cameras

