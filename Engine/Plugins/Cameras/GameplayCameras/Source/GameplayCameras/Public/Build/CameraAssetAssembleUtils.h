// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigTransition.h"
#include "Core/CameraSystemEvaluator.h"
#include "Core/CameraVariableReferences.h"
#include "Core/CameraVariableTableFwd.h"
#include "Directors/SingleCameraDirector.h"
#include "Nodes/Common/ArrayCameraNode.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Package.h"

#include <type_traits>

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

class FCameraEvaluationContextAssembler;

/**
 * Template mix-in for adding "go back to parent" support to an assembler class.
 */
template<typename ParentType>
struct TScopedConstruction
{
	TScopedConstruction(ParentType& InParent)
		: Parent(InParent)
	{}

	/** Return the parent assembler instance. */
	ParentType& Done() { return Parent; }

protected:

	ParentType& Parent;
};

/**
 * A generic utility class that defines a fluent interface for setting properties and adding items to
 * array properties on a given object.
 */
template<typename ObjectType>
struct TCameraObjectInitializer
{
	/** Sets a value on the given public property (via its member field). */
	template<typename PropertyType>
	TCameraObjectInitializer<ObjectType>& Set(PropertyType ObjectType::*Field, typename TCallTraits<PropertyType>::ParamType Value)
	{
		PropertyType& FieldPtr = (Object->*Field);
		FieldPtr = Value;
		return *this;
	}
	
	/** Adds an item to a given public array property (via its member field). */
	template<typename ItemType>
	TCameraObjectInitializer<ObjectType>& Add(TArray<ItemType> ObjectType::*Field, typename TCallTraits<ItemType>::ParamType NewItem)
	{
		TArray<ItemType>& ArrayPtr = (Object->*Field);
		ArrayPtr.Add(NewItem);
		return *this;
	}

protected:

	void SetObject(ObjectType* InObject)
	{
		Object = InObject;
	}

private:

	ObjectType* Object = nullptr;
};

/**
 * A simple repository matching UObject instances to names.
 */
class FNamedObjectRegistry : public TSharedFromThis<FNamedObjectRegistry>
{
public:

	/** Adds an object to the repository. */
	void Register(UObject* InObject, const FString& InName)
	{
		if (ensure(InObject && !InName.IsEmpty()))
		{
			NamedObjects.Add(InName, InObject);
		}		
	}

	/** Gets an object from the repository. */
	UObject* Get(const FString& InName) const
	{
		if (UObject* const* Found = NamedObjects.Find(InName))
		{
			return *Found;
		}
		return nullptr;
	}

	/** Gets an object from the repository with a call to CastChecked. */
	template<typename ObjectClass>
	ObjectClass* Get(const FString& InName) const
	{
		return CastChecked<ObjectClass>(Get(InName));
	}

private:

	TMap<FString, UObject*> NamedObjects;
};

/**
 * Interface for something that has access to a named object repository.
 */
struct IHasNamedObjectRegistry
{
	virtual ~IHasNamedObjectRegistry() = default;

	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() = 0;
};

/**
 * An assembler class for camera nodes.
 */
template<
	typename ParentType,
	typename NodeType,
	typename V = std::enable_if_t<TPointerIsConvertibleFromTo<NodeType, UCameraNode>::Value>
	>
class TCameraNodeAssembler 
	: public TScopedConstruction<ParentType>
	, public TCameraObjectInitializer<NodeType>
	, public IHasNamedObjectRegistry
{
public:

	using ThisType = TCameraNodeAssembler<ParentType, NodeType, V>;

	/** Creates a new instance of this assembler class. */
	TCameraNodeAssembler(ParentType& InParent, UObject* Outer = nullptr)
		: TScopedConstruction<ParentType>(InParent)
	{
		if (Outer == nullptr)
		{
			Outer = GetTransientPackage();
		}
		CameraNode = NewObject<NodeType>(Outer);
		TCameraObjectInitializer<NodeType>::SetObject(CameraNode);
	}

	/** Gets the built camera node. */
	NodeType* Get() const { return CameraNode; }

	/** Pins the built camera node to a given pointer, for being able to later refer to it. */
	ThisType& Pin(NodeType*& OutPtr) { OutPtr = CameraNode; return *this; }

	/** Give a name to the built camera node, to be recalled later. */
	template<typename VV = std::enable_if_t<TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value>>
	ThisType& Named(const TCHAR* InName)
	{
		TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry = GetNamedObjectRegistry();
		if (ensure(NamedObjectRegistry))
		{
			NamedObjectRegistry->Register(CameraNode, InName);
		}
		return *this;
	}

	/** Sets the value of a camera parameter field on the camera node. */
	template<typename ParameterType>
	ThisType& SetParameter(
			ParameterType NodeType::*ParameterField,
			typename TCallTraits<typename ParameterType::ValueType>::ParamType Value)
	{
		ParameterType& ParameterRef = (CameraNode->*ParameterField);
		ParameterRef.Value = Value;
		return *this;
	}

	/**
	 * Runs a custom setup callback on the camera node.
	 */
	ThisType& Setup(TFunction<void(NodeType*)> SetupCallback)
	{
		SetupCallback(CameraNode);
		return *this;
	}

	/**
	 * Runs a custom setup callback on the camera node with the named object registry provided.
	 */
	ThisType& Setup(TFunction<void(NodeType*, FNamedObjectRegistry*)> SetupCallback)
	{
		TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry = GetNamedObjectRegistry();
		SetupCallback(CameraNode, NamedObjectRegistry.Get());
		return *this;
	}

	/**
	 * Adds a child camera node via a public array member field on the camera node.
	 * Returns an assembler for the child. You can go back to the current assembler by
	 * calling Done() on the child assembler.
	 */
	template<
		typename ChildNodeType, 
		typename ArrayItemType,
		typename = std::enable_if_t<TPointerIsConvertibleFromTo<ChildNodeType, ArrayItemType>::Value>
		>
	TCameraNodeAssembler<ThisType, ChildNodeType>
	AddChild(TArray<TObjectPtr<ArrayItemType>> NodeType::*ArrayField)
	{
		TCameraNodeAssembler<ThisType, ChildNodeType> ChildAssembler(*this, CameraNode->GetOuter());
		TArray<TObjectPtr<ArrayItemType>>& ArrayRef = (CameraNode->*ArrayField);
		ArrayRef.Add(ChildAssembler.Get());
		return ChildAssembler;
	}

	/**
	 * Convenience implementation of AddChild specifically for array nodes.
	 */
	template<
		typename ChildNodeType,
		typename = std::enable_if_t<
			TPointerIsConvertibleFromTo<NodeType, UArrayCameraNode>::Value &&
			TPointerIsConvertibleFromTo<ChildNodeType, UCameraNode>::Value>
		>
	TCameraNodeAssembler<ThisType, ChildNodeType>
	AddArrayChild()
	{
		TCameraNodeAssembler<ThisType, ChildNodeType> ChildAssembler(*this, CameraNode->GetOuter());
		CastChecked<UArrayCameraNode>(CameraNode)->Children.Add(ChildAssembler.Get());
		return ChildAssembler;
	}

	/** 
	 * Casting operator that returns an assembler for the same camera node, but typed
	 * around a parent class of the camera node's class. Mostly useful for implicit casting
	 * when using AddChild().
	 */
	template<
		typename OtherNodeType,
		typename = std::enable_if_t<TPointerIsConvertibleFromTo<NodeType, OtherNodeType>::Value>
		>
	operator TCameraNodeAssembler<ParentType, OtherNodeType>() const
	{
		return TCameraNodeAssembler<ParentType, OtherNodeType>(
				EForceReuseCameraNode::Yes, 
				TScopedConstruction<ParentType>::Parent, 
				CameraNode);
	}

	/** Gets the named object registry from the parent. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		if constexpr (TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value)
		{
			return TScopedConstruction<ParentType>::Parent.GetNamedObjectRegistry();
		}
		else
		{
			return nullptr;
		}
	}

private:

	enum class EForceReuseCameraNode { Yes };

	TCameraNodeAssembler(EForceReuseCameraNode ForceReuse, ParentType& InParent, NodeType* ExistingCameraNode)
		: TScopedConstruction<ParentType>(InParent)
		, CameraNode(ExistingCameraNode)
	{
		TCameraObjectInitializer<NodeType>::SetObject(CameraNode);
	}

private:

	NodeType* CameraNode;
};

/**
 * Assembler class for camera rig transitions.
 */
template<typename ParentType>
class TCameraRigTransitionAssembler 
	: public TScopedConstruction<ParentType>
	, public TCameraObjectInitializer<UCameraRigTransition>
	, public IHasNamedObjectRegistry
{
public:

	using ThisType = TCameraRigTransitionAssembler<ParentType>;

	/** Creates a new instance of this assembler class. */
	TCameraRigTransitionAssembler(ParentType& InParent, UObject* Outer = nullptr)
		: TScopedConstruction<ParentType>(InParent)
	{
		if (Outer == nullptr)
		{
			Outer = GetTransientPackage();
		}

		Transition = NewObject<UCameraRigTransition>(Outer);
		TCameraObjectInitializer<UCameraRigTransition>::SetObject(Transition);
	}

	/** Gets the built transition object. */
	UCameraRigTransition* Get() const { return Transition; }

	/** Pins the built transition to a given pointer, for being able to later refer to it. */
	ThisType& Pin(UCameraRigTransition*& OutPtr) { OutPtr = Transition; return *this; }

	/** Give a name to the built transition, to be recalled later. */
	template<typename VV = std::enable_if_t<TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value>>
	ThisType& Named(const TCHAR* InName)
	{
		TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry = GetNamedObjectRegistry();
		if (ensure(NamedObjectRegistry))
		{
			NamedObjectRegistry->Register(Transition, InName);
		}
		return *this;
	}

	/** 
	 * Creates a blend node of the given type, and returns an assembler for it.
	 * You can go back to this transition assembler by calling Done() on the blend assembler.
	 */
	template<
		typename BlendType,
		typename = std::enable_if_t<TPointerIsConvertibleFromTo<BlendType, UBlendCameraNode>::Value>
		>
	TCameraNodeAssembler<ThisType, BlendType> MakeBlend()
	{
		TCameraNodeAssembler<ThisType, BlendType> BlendAssembler(*this, Transition->GetOuter());
		Transition->Blend = BlendAssembler.Get();
		return BlendAssembler;
	}

	/** Adds a transition condition. */
	template<
		typename ConditionType,
		typename = std::enable_if_t<TPointerIsConvertibleFromTo<ConditionType, UCameraRigTransitionCondition>::Value>
		>
	ThisType& AddCondition()
	{
		ConditionType* NewCondition = NewObject<ConditionType>(Transition->GetOuter());
		Transition->Conditions.Add(NewCondition);
		return *this;
	}

	/** Adds a transition condition. */
	template<
		typename ConditionType,
		typename = std::enable_if_t<TPointerIsConvertibleFromTo<ConditionType, UCameraRigTransitionCondition>::Value>
		>
	ThisType& AddCondition(TFunction<void(ConditionType*)> SetupCallback)
	{
		ConditionType* NewCondition = NewObject<ConditionType>(Transition->GetOuter());
		SetupCallback(NewCondition);
		Transition->Conditions.Add(NewCondition);
		return *this;
	}

	/** Gets the named object registry from the parent. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		if constexpr (TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value)
		{
			return TScopedConstruction<ParentType>::Parent.GetNamedObjectRegistry();
		}
		else
		{
			return nullptr;
		}
	}

private:

	UCameraRigTransition* Transition;
};

/**
 * The root assembler class for building a camera rig. Follow the fluent interface to construct the
 * hierarchy of camera nodes, add transitions, etc.
 *
 * For instance:
 *
 *		UCameraRigAsset* CameraRig = FCameraRigAssetAssembler(TEXT("SimpleTest"))
 *			.MakeRootNode<UArrayCameraNode>()
 *				.AddChild<UOffsetCameraNode>(&UArrayCameraNode::Children)
 *					.SetParameter(&UOffsetCameraNode::TranslationOffset, FVector3d{ 1, 0, 0 })
 *					.Done()
 *				.AddChild<ULensParametersCameraNode>(&UArrayCameraNode::Children)
 *					.SetParameter(&ULensParametersCameraNode::FocalLength, 18.f)
 *					.Done()
 *				.Done()
 *			.AddEnterTransition()
 *				.MakeBlend<USmoothBlendCameraNode>()
 *				.Done()
 *			.Get();
 */
template<typename ThisType>
class TCameraRigAssetAssemblerBase 
	: public TCameraObjectInitializer<UCameraRigAsset>
	, public IHasNamedObjectRegistry
{
public:

	/** Gets the built camera rig. */
	UCameraRigAsset* Get() { return CameraRig; }

	/** Pins the built camera rig to a given pointer, for being able to later refer to it. */
	ThisType& Pin(UCameraRigAsset*& OutPtr)
	{
		OutPtr = CameraRig; 
		return *static_cast<ThisType*>(this);
	}

	/** Give a name to the built camera rig, to be recalled later. */
	ThisType& Named(const TCHAR* InName)
	{
		if (ensure(NamedObjectRegistry))
		{
			NamedObjectRegistry->Register(CameraRig, InName);
		}
		return *static_cast<ThisType*>(this);
	}

	/**
	 * Creates a new camera node and sets it as the root node of the rig.
	 * Returns the assembler for the root camera node. You can come back to the rig assembler
	 * by calling Done() on the node assembler.
	 */
	template<typename NodeType>
	TCameraNodeAssembler<ThisType, NodeType> MakeRootNode()
	{
		ThisType* ActualThis = static_cast<ThisType*>(this);
		TCameraNodeAssembler<ThisType, NodeType> NodeAssembler(*ActualThis, CameraRig);
		CameraRig->RootNode = NodeAssembler.Get();
		return NodeAssembler;
	}

	/**
	 * A convenience method that calls MakeRootNode with a UArrayCameraNode.
	 */
	TCameraNodeAssembler<ThisType, UArrayCameraNode> MakeArrayRootNode()
	{
		return MakeRootNode<UArrayCameraNode>();
	}

	/**
	 * Adds a new enter transition and returns an assembler for it. You can come back to the
	 * rig assembler by calling Done() on the transition assembler.
	 */
	TCameraRigTransitionAssembler<ThisType> AddEnterTransition()
	{
		TCameraRigTransitionAssembler<ThisType> TransitionAssembler(*this, CameraRig);
		CameraRig->EnterTransitions.Add(TransitionAssembler.Get());
		return TransitionAssembler;
	}

	/**
	 * Adds a new exit transition and returns an assembler for it. You can come back to the
	 * rig assembler by calling Done() on the transition assembler.
	 */
	TCameraRigTransitionAssembler<ThisType> AddExitTransition()
	{
		TCameraRigTransitionAssembler<ThisType> TransitionAssembler(*this, CameraRig);
		CameraRig->ExitTransitions.Add(TransitionAssembler.Get());
		return TransitionAssembler;
	}

	/**
	 * Creates a new exposed rig parameter and hooks it up to the given camera node's property.
	 * When building the node hierarchy, you can use the Pin() method on the node assemblers to
	 * save a pointer to nodes you need for ExposeParameter().
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddBlendableParameter(const FString& ParameterName, ECameraVariableType ParameterType, UCameraNode* Target, FName TargetPropertyName)
	{
		UCameraObjectInterfaceBlendableParameter* BlendableParameter = NewObject<UCameraObjectInterfaceBlendableParameter>(CameraRig);
		BlendableParameter->InterfaceParameterName = ParameterName;
		BlendableParameter->VariableType = ParameterType;

		NamedObjectRegistry->Register(BlendableParameter, ParameterName);
		CameraRig->Interface.BlendableParameters.Add(BlendableParameter);

		UCameraObjectInterfaceParameterGetter* Getter = NewObject<UCameraObjectInterfaceParameterGetter>(CameraRig);
		Getter->ParameterGuid = BlendableParameter->GetGuid();
		ensure(Getter->ParameterGuid.IsValid());
		CameraRig->Connections.Add(Getter, NAME_None, Target, TargetPropertyName);

		return *static_cast<ThisType*>(this);
	}

	/**
	 * A variant of ExposeParameter that retrieves the target node from the named registry.
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddBlendableParameter(const FString& ParameterName, ECameraVariableType ParameterType, const FString& TargetName, FName TargetPropertyName)
	{
		UCameraNode* Target = NamedObjectRegistry->Get<UCameraNode>(TargetName);
		ensure(Target);
		return AddBlendableParameter(ParameterName, ParameterType, Target, TargetPropertyName);
	}

	/**
	 * Creates a new exposed rig parameter and hooks it up to the given camera node's property.
	 * When building the node hierarchy, you can use the Pin() method on the node assemblers to
	 * save a pointer to nodes you need for ExposeParameter().
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddDataParameter(const FString& ParameterName, TSubclassOf<UObject> ObjectClass, UCameraNode* Target, FName TargetPropertyName)
	{
		UCameraObjectInterfaceDataParameter* DataParameter = NewObject<UCameraObjectInterfaceDataParameter>(CameraRig);
		DataParameter->DataType = ECameraContextDataType::Object;
		DataParameter->DataContainerType = ECameraContextDataContainerType::None;
		DataParameter->DataTypeObject = ObjectClass.Get();
		DataParameter->InterfaceParameterName = ParameterName;

		NamedObjectRegistry->Register(DataParameter, ParameterName);
		CameraRig->Interface.DataParameters.Add(DataParameter);

		UCameraObjectInterfaceParameterGetter* Getter = NewObject<UCameraObjectInterfaceParameterGetter>(CameraRig);
		Getter->ParameterGuid = DataParameter->GetGuid();
		ensure(Getter->ParameterGuid.IsValid());
		CameraRig->Connections.Add(Getter, NAME_None, Target, TargetPropertyName);

		return *static_cast<ThisType*>(this);
	}

	/**
	 * Creates a new exposed rig parameter and hooks it up to the given camera node's property.
	 * When building the node hierarchy, you can use the Pin() method on the node assemblers to
	 * save a pointer to nodes you need for ExposeParameter().
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddDataParameter(const FString& ParameterName, const UScriptStruct* InScriptStruct, UCameraNode* Target, FName TargetPropertyName)
	{
		UCameraObjectInterfaceDataParameter* DataParameter = NewObject<UCameraObjectInterfaceDataParameter>(CameraRig);
		DataParameter->DataType = ECameraContextDataType::Struct;
		DataParameter->DataContainerType = ECameraContextDataContainerType::None;
		DataParameter->DataTypeObject = InScriptStruct;
		DataParameter->InterfaceParameterName = ParameterName;

		NamedObjectRegistry->Register(DataParameter, ParameterName);
		CameraRig->Interface.DataParameters.Add(DataParameter);

		UCameraObjectInterfaceParameterGetter* Getter = NewObject<UCameraObjectInterfaceParameterGetter>(CameraRig);
		Getter->ParameterGuid = DataParameter->GetGuid();
		ensure(Getter->ParameterGuid.IsValid());
		CameraRig->Connections.Add(Getter, NAME_None, Target, TargetPropertyName);

		return *static_cast<ThisType*>(this);
	}

	/**
	 * A variant of AddDataParameter that retrieves the target node from the named registry.
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddDataParameter(const FString& ParameterName, TSubclassOf<UObject> ObjectClass, const FString& TargetName, FName TargetPropertyName)
	{
		UCameraNode* Target = NamedObjectRegistry->Get<UCameraNode>(TargetName);
		ensure(Target);
		return AddDataParameter(ParameterName, ObjectClass, Target, TargetPropertyName);
	}

	/**
	 * A variant of AddDataParameter that retrieves the target node from the named registry.
	 *
	 * The created parameter is automatically stored in the named object registry under its name.
	 */
	ThisType& AddDataParameter(const FString& ParameterName, const UScriptStruct* InScriptStruct, const FString& TargetName, FName TargetPropertyName)
	{
		UCameraNode* Target = NamedObjectRegistry->Get<UCameraNode>(TargetName);
		ensure(Target);
		return AddDataParameter(ParameterName, InScriptStruct, Target, TargetPropertyName);
	}

	/** Runs arbitrary setup logic on the camera rig. */
	ThisType& Setup(TFunction<void(UCameraRigAsset*)> SetupCallback)
	{
		SetupCallback(CameraRig);
		return *static_cast<ThisType*>(this);;
	}

	/** Runs arbitrary setup logic on the camera rig. */
	ThisType& Setup(TFunction<void(UCameraRigAsset*, FNamedObjectRegistry*)> SetupCallback)
	{
		SetupCallback(CameraRig, NamedObjectRegistry.Get());
		return *static_cast<ThisType*>(this);;
	}

	/**
	 * Builds the camera rig.
	 */
	ThisType& BuildCameraRig()
	{
		CameraRig->BuildCameraRig();
		return *static_cast<ThisType*>(this);
	}

	/**
	 * Sets the default value for a camera rig parameter.
	 * Requires that the camera rig has been built first.
	 */
	template<typename ValueType>
	ThisType& SetDefaultParameterValue(const FName ParameterName, TCallTraits<ValueType>::ParamType ParameterValue)
	{
		const FCameraObjectInterfaceParameterDefinition* ParameterDefinition = CameraRig->GetParameterDefinitions().FindByPredicate(
				[ParameterName](const FCameraObjectInterfaceParameterDefinition& Item)
				{
					return Item.ParameterName == ParameterName;
				});
		if (!ensureMsgf(ParameterDefinition, TEXT("You must build the camera rig before setting default parameter values")))
		{
			return *static_cast<ThisType*>(this);
		}

		FInstancedPropertyBag& DefaultParameters = CameraRig->GetDefaultParameters();
		const UPropertyBag* PropertyBag = DefaultParameters.GetPropertyBagStruct();
		if (!ensureMsgf(PropertyBag, TEXT("You must build the camera rig before setting default parameter values")))
		{
			return *static_cast<ThisType*>(this);
		}

		const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag->FindPropertyDescByName(ParameterName);
		if (!ensureMsgf(PropertyDesc, TEXT("No such camera rig parameter")))
		{
			return *static_cast<ThisType*>(this);
		}

		uint8* RawParameters = DefaultParameters.GetMutableValue().GetMemory();
		void* TargetAddress = RawParameters + PropertyDesc->CachedProperty->GetOffset_ForInternal();
		const void* SourceAddress = &ParameterValue;

		if constexpr (IsSupportedBlendableParameterType<ValueType>())
		{
			if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Blendable)
			{
				if (ParameterDefinition->VariableType == ECameraVariableType::BlendableStruct)
				{
					PropertyDesc->CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
				}
				else
				{
					SetDefaultBlendableParameterValue(*ParameterDefinition, TargetAddress, ParameterValue);
				}
			}
			else if (ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Data)
			{
				PropertyDesc->CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
			}
		}
		// If the data type is not a supported blendable parameter type
		// then we need a version of this function that doesn't use SetDefaultBlendableParameterValue at all
		// otherwise we will have a compile failure due to SetDefaultBlendableParameterValue not being implemented
		// for that data type
		else
		{
			if (ensure(ParameterDefinition->ParameterType == ECameraObjectInterfaceParameterType::Data))
			{
				PropertyDesc->CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
			}
		}		

		return *static_cast<ThisType*>(this);
	}

	/** Gets the named object registry. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		return NamedObjectRegistry;
	}

protected:

	TCameraRigAssetAssemblerBase(TSharedPtr<FNamedObjectRegistry> InNamedObjectRegistry, FName Name = NAME_None, UObject* Outer = nullptr)
	{
		Initialize(InNamedObjectRegistry, Name, Outer);
	}

private:

	void Initialize(TSharedPtr<FNamedObjectRegistry> InNamedObjectRegistry, FName Name, UObject* Outer)
	{
		if (Outer == nullptr)
		{
			Outer = GetTransientPackage();
		}

		CameraRig = NewObject<UCameraRigAsset>(Outer, Name);
		TCameraObjectInitializer<UCameraRigAsset>::SetObject(CameraRig);

		NamedObjectRegistry = InNamedObjectRegistry;
		if (!NamedObjectRegistry)
		{
			NamedObjectRegistry = MakeShared<FNamedObjectRegistry>();
		}

		NamedObjectRegistry->Register(CameraRig, Name.ToString());
	}

	template<typename ValueType>
	void SetDefaultBlendableParameterValue(const FCameraObjectInterfaceParameterDefinition& ParameterDefinition, void* TargetAddress, ValueType ParameterValue);

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	template<>\
	void SetDefaultBlendableParameterValue<ValueType>(const FCameraObjectInterfaceParameterDefinition& ParameterDefinition, void* TargetAddress, ValueType ParameterValue)\
	{\
		using ParameterType = F##ValueName##CameraParameter;\
		if (ensure(ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable && ParameterDefinition.VariableType == ECameraVariableType::ValueName))\
		{\
			ParameterType* TargetParameter = static_cast<ParameterType*>(TargetAddress);\
			TargetParameter->Value = ParameterValue;\
		}\
	}
	UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

private:

	template <typename ValueType>
	static constexpr bool IsSupportedBlendableParameterType() { return false; }

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	template<>\
	constexpr bool IsSupportedBlendableParameterType<ValueType>()\
	{\
		return true;\
	}
	UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

	UCameraRigAsset* CameraRig;

	TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry;
};

/**
 * Default version of the camera rig asset assembler.
 */
class FCameraRigAssetAssembler 
	: public TCameraRigAssetAssemblerBase<FCameraRigAssetAssembler>
{
public:

	UE_API FCameraRigAssetAssembler(FName Name = NAME_None, UObject* Outer = nullptr);
	UE_API FCameraRigAssetAssembler(TSharedPtr<FNamedObjectRegistry> InNamedObjectRegistry, FName Name = NAME_None, UObject* Outer = nullptr);
};

/**
 * Version of the camera rig asset assembler that has a scoped parent, with a Done() method exposed
 * to go back to it.
 */
template<typename ParentType>
class TScopedCameraRigAssetAssembler
	: public TScopedConstruction<ParentType>
	, public TCameraRigAssetAssemblerBase<TScopedCameraRigAssetAssembler<ParentType>>
{
public:

	TScopedCameraRigAssetAssembler(ParentType& InParent, FName Name = NAME_None, UObject* Outer = nullptr)
		: TScopedConstruction<ParentType>(InParent)
		, TCameraRigAssetAssemblerBase<TScopedCameraRigAssetAssembler<ParentType>>(Name, Outer)
	{
	}

	TScopedCameraRigAssetAssembler(ParentType& InParent, TSharedPtr<FNamedObjectRegistry> InNamedObjectRegistry, FName Name = NAME_None, UObject* Outer = nullptr)
		: TScopedConstruction<ParentType>(InParent)
		, TCameraRigAssetAssemblerBase<TScopedCameraRigAssetAssembler<ParentType>>(InNamedObjectRegistry, Name, Outer)
	{
	}
};

/**
 * Assembler class for a camera director.
 */
template<
	typename ParentType,
	typename DirectorType,
	typename V = std::enable_if_t<TPointerIsConvertibleFromTo<DirectorType, UCameraDirector>::Value>
	>
class TCameraDirectorAssembler
	: public TScopedConstruction<ParentType>
	, public TCameraObjectInitializer<DirectorType>
	, public IHasNamedObjectRegistry
{
public:

	using ThisType = TCameraDirectorAssembler<ParentType, DirectorType, V>;

	/** Creates a new instance of this assembler class. */
	TCameraDirectorAssembler(ParentType& InParent, UObject* Outer = nullptr)
		: TScopedConstruction<ParentType>(InParent)
	{
		if (Outer == nullptr)
		{
			Outer = GetTransientPackage();
		}
		CameraDirector = NewObject<DirectorType>(Outer);
		TCameraObjectInitializer<DirectorType>::SetObject(CameraDirector);
	}

	/** Gets the build camera director. */
	UCameraDirector* Get() const { return CameraDirector; }

	/** Pins the built camera director to a given pointer, for being able to later refer to it. */
	ThisType& Pin(DirectorType*& OutPtr) { OutPtr = CameraDirector; return *this; }

	/** Give a name to the built camera drector, to be recalled later. */
	template<typename VV = std::enable_if_t<TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value>>
	ThisType& Named(const TCHAR* InName)
	{
		TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry = GetNamedObjectRegistry();
		if (ensure(NamedObjectRegistry))
		{
			NamedObjectRegistry->Register(CameraDirector, InName);
		}
		return *this;
	}

	/** Set a parameter on the camera director. */
	template<typename ParameterType>
	ThisType& SetParameter(
			ParameterType DirectorType::*ParameterField,
			typename TCallTraits<typename ParameterType::ValueType>::ParamType Value)
	{
		ParameterType& ParameterRef = (CameraDirector->*ParameterField);
		ParameterRef.Value = Value;
		return *this;
	}

	/** Runs arbitrary setup logic on the camera director. */
	ThisType& Setup(TFunction<void(DirectorType*)> SetupCallback)
	{
		SetupCallback(CameraDirector);
		return *this;
	}

	/** Runs arbitrary setup logic on the camera director. */
	ThisType& Setup(TFunction<void(DirectorType*, FNamedObjectRegistry*)> SetupCallback)
	{
		TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry = GetNamedObjectRegistry();
		SetupCallback(CameraDirector, NamedObjectRegistry.Get());
		return *this;
	}

	/** Gets the named object registry from the parent. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		if constexpr (TPointerIsConvertibleFromTo<ParentType, IHasNamedObjectRegistry>::Value)
		{
			return TScopedConstruction<ParentType>::Parent.GetNamedObjectRegistry();
		}
		else
		{
			return nullptr;
		}
	}

private:

	DirectorType* CameraDirector;
};

/**
 * Assembler class for a camera asset.
 */
class FCameraAssetAssembler
	: public TCameraObjectInitializer<UCameraAsset>
	, public IHasNamedObjectRegistry
{
public:

	using ThisType = FCameraAssetAssembler;

	/** Create a new instance of this assembler class. */
	UE_API FCameraAssetAssembler(UObject* Owner = nullptr);

	/** Gets the created camera asset. */
	UCameraAsset* Get() const { return CameraAsset; }

	/** Builds a new camera director of the given type and returns an assembler object for its. */
	template<typename DirectorType>
	TCameraDirectorAssembler<ThisType, DirectorType> MakeDirector()
	{
		TCameraDirectorAssembler<ThisType, DirectorType> DirectorAssembler(*this, CameraAsset);
		CameraAsset->SetCameraDirector(DirectorAssembler.Get());
		return DirectorAssembler;
	}

	/**
	 * Creates a new interface parameter that exposes the given camera rig parameter, optionally
	 * under a different name.
	 */
	ThisType& AddInterfaceParameter(UCameraRigAsset* SourceCameraRig, const FName SourceParameterName, const FString& InterfaceParameterName)
	{
		if (ensure(SourceCameraRig))
		{
			UCameraAssetInterfaceParameter* InterfaceParameter = NewObject<UCameraAssetInterfaceParameter>(CameraAsset);
			InterfaceParameter->SourceCameraRig = SourceCameraRig;
			InterfaceParameter->SourceParameterName = SourceParameterName;
			InterfaceParameter->InterfaceParameterName = (
					InterfaceParameterName.IsEmpty() ?
					SourceParameterName.ToString() : InterfaceParameterName);

			CameraAsset->Interface.Parameters.Add(InterfaceParameter);
		}
		return *static_cast<ThisType*>(this);
	}

	ThisType& AddInterfaceParameter(UCameraRigAsset* SourceCameraRig, const FName SourceParameterName)
	{
		return AddInterfaceParameter(SourceCameraRig, SourceParameterName, SourceParameterName.ToString());
	}

	ThisType& AddInterfaceParameter(const FString& SourceCameraRigName, const FName& SourceParameterName, const FString& InterfaceParameterName)
	{
		return AddInterfaceParameter(
				NamedObjectRegistry->Get<UCameraRigAsset>(SourceCameraRigName),
				SourceParameterName,
				InterfaceParameterName);
	}

	ThisType& AddInterfaceParameter(const FString& SourceCameraRigName, const FName SourceParameterName)
	{
		return AddInterfaceParameter(SourceCameraRigName, SourceParameterName, SourceParameterName.ToString());
	}

	/** Gets the named object registry. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		return NamedObjectRegistry;
	}

private:

	UCameraAsset* CameraAsset;

	TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry;
};

/**
 * Assembler class for a camera evaluation context and its camera asset.
 */
class FCameraEvaluationContextAssembler
	: public TCameraObjectInitializer<FCameraEvaluationContext>
	, public IHasNamedObjectRegistry
{
public:

	using ThisType = FCameraEvaluationContextAssembler;

	/** Creates a new instance of this assembler class. */
	UE_API FCameraEvaluationContextAssembler(UObject* Owner = nullptr);

	/** Gets the created evaluation context. */
	TSharedRef<FCameraEvaluationContext> Get() const { return EvaluationContext.ToSharedRef(); }

	/** Pins the created camera asset. */
	ThisType& PinCameraAsset(UCameraAsset*& OutPtr) { OutPtr = CameraAsset; return *this; }

	/** Builds the camera asset. */
	ThisType& BuildCameraAsset() { CameraAsset->BuildCamera(); return *this; }

	/** Builds a new camera director of the given type and returns an assembler object for its. */
	template<typename DirectorType>
	TCameraDirectorAssembler<ThisType, DirectorType> MakeDirector()
	{
		TCameraDirectorAssembler<ThisType, DirectorType> DirectorAssembler(*this, EvaluationContext->GetOwner());
		CameraAsset->SetCameraDirector(DirectorAssembler.Get());
		return DirectorAssembler;
	}

	/** Builds a new single camera director and returns an assembler object for its. */
	TCameraDirectorAssembler<ThisType, USingleCameraDirector> MakeSingleDirector()
	{
		return MakeDirector<USingleCameraDirector>();
	}

	/** Builds a new single camera director, set its camera rig, and returns an assembler object for its. */
	TCameraDirectorAssembler<ThisType, USingleCameraDirector> MakeSingleDirector(UCameraRigAsset* InCameraRig)
	{
		auto DirectorAssembler = MakeDirector<USingleCameraDirector>();
		DirectorAssembler.Setup([InCameraRig](USingleCameraDirector* Director)
				{
					Director->CameraRig = InCameraRig;
				});
		return DirectorAssembler;
	}

	/** Builds a new single camera director, set its camera rig to the named object, and returns an assembler object for its. */
	TCameraDirectorAssembler<ThisType, USingleCameraDirector> MakeSingleDirector(const FString& InCameraRigName)
	{
		UCameraRigAsset* CameraRig = NamedObjectRegistry->Get<UCameraRigAsset>(InCameraRigName);

		auto DirectorAssembler = MakeDirector<USingleCameraDirector>();
		DirectorAssembler.Setup([CameraRig](USingleCameraDirector* Director)
				{
					Director->CameraRig = CameraRig;
				});
		return DirectorAssembler;
	}

	/** Creates a new camera rig asset assembler and gives it a name to be recalled later.*/
	TScopedCameraRigAssetAssembler<ThisType> CreateCameraRig(FName Name = NAME_None)
	{
		TScopedCameraRigAssetAssembler<ThisType> CameraRigAssembler(*this, GetNamedObjectRegistry(), Name, CameraAsset);
		return CameraRigAssembler;
	}

	/** Runs arbitrary setup logic on the evaluation context. */
	ThisType& Setup(TFunction<void(TSharedRef<FCameraEvaluationContext>)> SetupCallback)
	{
		SetupCallback(EvaluationContext.ToSharedRef());
		return *this;
	}

	/** Runs arbitrary setup logic on the evaluation context. */
	ThisType& Setup(TFunction<void(TSharedRef<FCameraEvaluationContext>, FNamedObjectRegistry*)> SetupCallback)
	{
		SetupCallback(EvaluationContext.ToSharedRef(), NamedObjectRegistry.Get());
		return *this;
	}


	/** Gets the named object registry. */
	virtual TSharedPtr<FNamedObjectRegistry> GetNamedObjectRegistry() override
	{
		return NamedObjectRegistry;
	}

private:

	UCameraAsset* CameraAsset;

	TSharedPtr<FCameraEvaluationContext> EvaluationContext;

	TSharedPtr<FNamedObjectRegistry> NamedObjectRegistry;
};

/**
 * Assembler class for a camera system evaluator.
 */
class FCameraSystemEvaluatorAssembler
{
public:

	/** Makes a new camera system evaluator. */
	static TSharedRef<FCameraSystemEvaluator> Build(UObject* OwnerObject = nullptr)
	{
		TSharedRef<FCameraSystemEvaluator> NewEvaluator = MakeShared<FCameraSystemEvaluator>();
		NewEvaluator->Initialize(OwnerObject);
		return NewEvaluator;
	}
};

}  // namespace UE::Cameras::Build

#undef UE_API