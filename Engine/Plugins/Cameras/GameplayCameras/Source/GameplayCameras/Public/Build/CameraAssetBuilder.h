// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "CoreTypes.h"
#include "GameplayCameras.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraAsset;
class UCameraRigAsset;
struct FInstancedPropertyBag;

namespace UE::Cameras
{

/**
 * A class that can prepare a camera asset for runtime use.
 */
class FCameraAssetBuilder
{
public:

	/** Creates a new camera builder. */
	UE_API FCameraAssetBuilder(FCameraBuildContext& InBuildContext);

	/** Builds the given camera. */
	UE_API void BuildCamera(UCameraAsset* InCameraAsset, bool bBuildReferencedAssets = true);

	/** Gets the last built referenced assets. */
	TConstArrayView<UObject*> GetBuiltReferencedAssets() const { return BuiltReferencedAssets; }

public:

	// Internal API.

	UE_API static bool BuildDefaultInterfaceIfNeeded(UCameraAsset* InCameraAsset);

private:

	void BuildCameraImpl(bool bBuildReferencedAssets);

	void BuildParameters();
	void CopyDefaultParameterValue(const UCameraRigAsset* InCameraRig, const FGuid& PropertyID, FInstancedPropertyBag& DefaultParameters);

	void UpdateBuildStatus();

private:

	FCameraBuildContext BuildContext;

	UCameraAsset* CameraAsset = nullptr;

	TArray<UObject*> BuiltReferencedAssets;
};

}  // namespace UE::Cameras

#undef UE_API
