// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildLog.h"

namespace UE::Cameras
{

enum ECameraBuildReason
{
	/** The reason for this build is undefined. */
	Undefined,
	/** A user clicked on the Build button, or similar. */
	UserAction,
	/** The build is happening during cooking. */
	Cooking,
	/** The build is happening for starting PIE. */
	StartingPIE
};

/**
 * Camera build context.
 */
struct FCameraBuildContext
{
	FCameraBuildContext(FCameraBuildLog& InBuildLog, ECameraBuildReason InBuildReason = ECameraBuildReason::Undefined)
		: BuildLog(InBuildLog)
		, BuildReason(InBuildReason)
	{}

	/** The build log for emitting messages. */
	FCameraBuildLog& BuildLog;

	/** The reason for this build. */
	ECameraBuildReason GetBuildReason() const { return BuildReason; }

public:

	/** Whether the build reason is for cooking. */
	bool IsCooking() const { return BuildReason == ECameraBuildReason::Cooking; }

private:

	ECameraBuildReason BuildReason = ECameraBuildReason::Undefined;
};

}  // namespace UE::Cameras

