// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Logging/TokenizedMessage.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

/**
 * A message emitted by a camera build process.
 */
struct FCameraBuildLogMessage
{
	/** Severity of the message. */
	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	/** An optional object that the message relates to. */
	const UObject* Object = nullptr;
	/** The actual message. */
	FText Text;

	/** Generates a plain string representation of this message. */
	UE_API FString ToString() const;
	/** Sends a string version of this message to the LogCameraSystem console log. */
	UE_API void SendToLogging(const FString& InLoggingPrefix) const;
};

/**
 * Build log, populated when building a camera rig, or other camera asset.
 */
class FCameraBuildLog
{
public:

	/**
	 * Sets a string that will be prefixed to all messages sent to the console.
	 * Only useful when IsForwardingMessagesToLogging is true.
	 * This is generally set to the name of the camera asset being built.
	 */
	UE_API void SetLoggingPrefix(const FString& InPrefix);

	/** Returns whether build messages are sent to the console. */
	bool IsForwardingMessagesToLogging() const { return bForwardToLogging; }
	/** Sets whether build messages are sent to the console. */
	UE_API void SetForwardMessagesToLogging(bool bInForwardToLogging);

	/** Adds a new message. */
	UE_API void AddMessage(EMessageSeverity::Type InSeverity, FText&& InText);
	/** Adds a new message. */
	UE_API void AddMessage(EMessageSeverity::Type InSeverity, const UObject* InObject, FText&& InText);

	/** Gets the list of received messages so far. */
	TArrayView<const FCameraBuildLogMessage> GetMessages() const { return Messages; }

	/** Clears all messages. */
	UE_API void ResetMessages();

	/** Returns whether any warning has been logged. */
	bool HasWarnings() const { return bHasWarnings; }
	/** Returns whether any error has been logged. */
	bool HasErrors() const { return bHasErrors; }

private:

	TArray<FCameraBuildLogMessage> Messages;

	FString LoggingPrefix;
	bool bForwardToLogging = true;

	bool bHasWarnings = false;
	bool bHasErrors = false;
};

}  // namespace UE::Cameras

#undef UE_API
