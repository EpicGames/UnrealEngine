// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"

#include "CameraBuildStatus.generated.h"

/**
 * Enumeration that describes if a camera asset needs to be rebuilt.
 */
UENUM()
enum class ECameraBuildStatus : uint8
{
	Clean,
	CleanWithWarnings,
	WithErrors,
	Dirty
};

UINTERFACE(MinimalAPI)
class UHasCameraBuildStatus : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for an object that can be built and has a build status,
 * such as a camera asset or camera rig.
 */
class IHasCameraBuildStatus
{
	GENERATED_BODY()

public:

	/** Gets the build status of the object. */
	virtual ECameraBuildStatus GetBuildStatus() const PURE_VIRTUAL(IHasCameraBuildStatus::GetBuildStatus, return ECameraBuildStatus::Dirty;);

	/** Dirties the build status of the object. */
	virtual void DirtyBuildStatus() PURE_VIRTUAL();
};

