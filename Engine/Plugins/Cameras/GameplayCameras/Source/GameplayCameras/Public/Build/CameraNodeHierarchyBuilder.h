// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "Core/CameraNodeHierarchy.h"

#define UE_API GAMEPLAYCAMERAS_API

class UBaseCameraObject;

namespace UE::Cameras
{

struct FCameraObjectBuildContext;

/**
 * A helper class that can build a hierarchy of camera nodes.
 */
class FCameraNodeHierarchyBuilder
{
public:

	/** Creates a new camera node hierarchy builder. */
	UE_API FCameraNodeHierarchyBuilder(FCameraBuildContext& InBuildContext, UBaseCameraObject* InCameraObject);

	/** Gets the camera node hierarchy. */
	const FCameraNodeHierarchy& GetHierarchy() const { return CameraNodeHierarchy; }

	/** Calls PreBuild on all the camera nodes. */
	UE_API void PreBuild();

	/** Calls Build on all the camera nodes and computes the allocation info. */
	UE_API void Build();

private:

	UE_API void CallBuild(FCameraObjectBuildContext& BuildContext, UCameraNode* CameraNode);
	UE_API void BuildParametersAllocationInfo(FCameraObjectBuildContext& BuildContext);

private:

	FCameraBuildContext BuildContext;
	UBaseCameraObject* CameraObject = nullptr;
	FCameraNodeHierarchy CameraNodeHierarchy;
};

}  // namespace UE::Cameras

#undef UE_API
