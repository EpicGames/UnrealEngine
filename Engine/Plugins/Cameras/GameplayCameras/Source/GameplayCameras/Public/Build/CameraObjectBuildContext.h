// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "Core/BaseCameraObject.h"

namespace UE::Cameras
{

/**
 * Camera object build context.
 */
struct FCameraObjectBuildContext : public FCameraBuildContext
{
	FCameraObjectBuildContext(const FCameraBuildContext& InParentContext)
		: FCameraBuildContext(InParentContext)
	{}

	/** The allocation information for the camera rig. */
	FCameraObjectAllocationInfo AllocationInfo;
};

}  // namespace UE::Cameras

