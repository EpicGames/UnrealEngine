// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "Containers/ArrayView.h"
#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraVariableTableFwd.h"

#define UE_API GAMEPLAYCAMERAS_API

class UBaseCameraObject;
class UCameraObjectInterfaceBlendableParameter;
class UCameraObjectInterfaceDataParameter;
class UCameraNode;
struct FCameraObjectInterface;

namespace UE::Cameras
{

class FCameraNodeHierarchy;

namespace Internal { struct FInterfaceParameterBindingBuilder; }

class FCameraObjectInterfaceBuilder
{
public:

	UE_API FCameraObjectInterfaceBuilder(FCameraBuildContext& InBuildContext);

	UE_API void BuildInterface(UBaseCameraObject* InCameraObject, const FCameraNodeHierarchy& InHierarchy, bool bCollectStrayNodes);
	UE_API void BuildInterface(UBaseCameraObject* InCameraObject, TArrayView<UCameraNode*> InCameraObjectNodes);

private:

	UE_API void BuildInterfaceImpl();

	UE_API void GatherOldDrivenParameters();
	UE_API void BuildInterfaceParameters();
	UE_API void BuildInterfaceParameterBindings();
	UE_API void DiscardUnusedParameters();

private:

	UE_API bool SetupCameraParameterOrVariableReferenceOverride(const UCameraObjectInterfaceBlendableParameter* BlendableParameter, UCameraNode* Target, FName TargetPropertyName);
	UE_API bool SetupCustomBlendableParameterOverride(const UCameraObjectInterfaceBlendableParameter* BlendableParameter, UCameraNode* Target, FName TargetPropertyName);

	UE_API bool SetupDataContextPropertyOverride(const UCameraObjectInterfaceDataParameter* DataParameter, UCameraNode* Target, FName TargetPropertyName);
	UE_API bool SetupCustomDataParameterOverride(const UCameraObjectInterfaceDataParameter* DataParameter, UCameraNode* Target, FName TargetPropertyName);

private:

	FCameraBuildContext BuildContext;

	UBaseCameraObject* CameraObject = nullptr;
	TArray<UCameraNode*> CameraObjectNodes;

	using FDrivenParameterKey = TTuple<UObject*, FName>;
	TMap<FDrivenParameterKey, FCameraVariableID> OldDrivenBlendableParameters;
	TMap<FDrivenParameterKey, FCameraContextDataID> OldDrivenDataParameters;

	friend struct Internal::FInterfaceParameterBindingBuilder;
};

}  // namespace UE::Cameras

#undef UE_API
