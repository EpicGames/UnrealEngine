// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#define UE_API GAMEPLAYCAMERAS_API

class UBaseCameraObject;
class UCameraNode;
class UCameraObjectInterfaceDataParameter;
struct FCameraObjectInterfaceParameterDefinition;
struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;

namespace UE::Cameras
{

/**
 * Builds the property bag that contains a property for each exposed parameter on the given camera object.
 * Each property's value is set to the default value of the corresponding parameter.
 */
class FCameraObjectInterfaceParameterBuilder
{
public:

	UE_API FCameraObjectInterfaceParameterBuilder();

	UE_API void BuildParameters(UBaseCameraObject* InCameraObject);

public:

	static UE_API void AppendDefaultParameterProperties(const UBaseCameraObject* CameraObject, TArray<FPropertyBagPropertyDesc>& OutProperties);
	static UE_API void AppendDefaultParameterProperties(TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions, TArray<FPropertyBagPropertyDesc>& OutProperties);

	static UE_API bool SetDefaultParameterValue(UBaseCameraObject* CameraObject, const FCameraObjectInterfaceParameterDefinition& ParameterDefinition, UCameraNode* TargetNode, FName TargetPropertyName, bool bAddParameterIfMissing = true);

	static void FixUpDefaultParameterProperties(TConstArrayView<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions, FInstancedPropertyBag& InOutPropertyBag);

private:

	void BuildParametersImpl();

	void BuildParameterDefinitions();
	void BuildDefaultParameters();

private:

	UBaseCameraObject* CameraObject = nullptr;
};

}  // namespace UE::Cameras

#undef UE_API
