// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "Core/CameraNodeHierarchy.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraRigAsset;

namespace UE::Cameras
{

/**
 * A class that can prepare a camera rig for runtime use.
 *
 * This builder class sets up internal camera variables that handle exposed camera
 * rig parameters, computes the allocation information of the camera rig, and
 * does various kinds of validation.
 *
 * Once the build process is done, the BuildStatus property is set on the camera rig.
 */
class FCameraRigAssetBuilder
{
public:

	/** Creates a new camera rig builder. */
	UE_API FCameraRigAssetBuilder(FCameraBuildContext& InBuildContext);

	/** Builds the given camera rig. */
	UE_API void BuildCameraRig(UCameraRigAsset* InCameraRig);

private:

	void BuildCameraRigImpl();

	void UpdateBuildStatus();

private:

	FCameraBuildContext BuildContext;

	UCameraRigAsset* CameraRig = nullptr;

	FCameraNodeHierarchy CameraNodeHierarchy;
};

}  // namespace UE::Cameras

#undef UE_API
