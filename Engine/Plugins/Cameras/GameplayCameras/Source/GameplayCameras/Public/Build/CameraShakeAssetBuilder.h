// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildContext.h"
#include "Core/CameraNodeHierarchy.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraShakeAsset;

namespace UE::Cameras
{

class FCameraShakeAssetBuilder
{
public:

	UE_API FCameraShakeAssetBuilder(FCameraBuildContext& InBuildContext);

	UE_API void BuildCameraShake(UCameraShakeAsset* InCameraShake);

private:

	void BuildCameraShakeImpl();

	void UpdateBuildStatus();

private:

	FCameraBuildContext BuildContext;

	UCameraShakeAsset* CameraShake = nullptr;

	FCameraNodeHierarchy CameraNodeHierarchy;
};

}  // namespace UE::Cameras

#undef UE_API
