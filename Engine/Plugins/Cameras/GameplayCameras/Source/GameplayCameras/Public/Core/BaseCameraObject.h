// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraContextDataTableAllocationInfo.h"
#include "Core/CameraEventHandler.h"
#include "Core/CameraNodeEvaluatorFwd.h"
#include "Core/CameraObjectInterface.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraVariableTableAllocationInfo.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "BaseCameraObject.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class IObjectTreeGraphRootObject;
class UCameraNode;

namespace UE::Cameras
{
	class FCameraBuildLog;
	class FCameraObjectInterfaceParameterBuilder;

	/**
	 * Interface for listening to changes on a camera rig asset.
	 */
	class ICameraObjectEventHandler
	{
	public:
		virtual ~ICameraObjectEventHandler() = default;

		/** Called when the camera object's interface has changed. */
		virtual void OnCameraObjectInterfaceChanged() {}
	};
}

/**
 * Structure describing various allocations needed by a camera rig.
 */
USTRUCT()
struct FCameraObjectAllocationInfo
{
	GENERATED_BODY()

	/** Allocation info for node evaluators. */
	UPROPERTY()
	FCameraNodeEvaluatorAllocationInfo EvaluatorInfo;

	/** Allocation info for the variable table. */
	UPROPERTY()
	FCameraVariableTableAllocationInfo VariableTableInfo;

	/** Allocation info for the context data table. */
	UPROPERTY()
	FCameraContextDataTableAllocationInfo ContextDataTableInfo;

public:

	GAMEPLAYCAMERAS_API void Append(const FCameraObjectAllocationInfo& OtherAllocationInfo);

	bool operator==(const FCameraObjectAllocationInfo& Other) const = default;
};

template<>
struct TStructOpsTypeTraits<FCameraObjectAllocationInfo> : public TStructOpsTypeTraitsBase2<FCameraObjectAllocationInfo>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * A connection inside a camera object graph.
 */
USTRUCT()
struct FCameraObjectConnection
{
	GENERATED_BODY()
	
public:

	/** The source object of the connection. */
	UPROPERTY()
	TObjectPtr<UObject> Source;

	/** The source object property of the connection. */
	UPROPERTY()
	FName SourcePropertyName;

	/** The target object of the connection. */
	UPROPERTY()
	TObjectPtr<UObject> Target;

	/** The target object property of the connection. */
	UPROPERTY()
	FName TargetPropertyName;
};

/**
 * A collection of connections inside a camera object graph.
 */
USTRUCT()
struct FCameraObjectConnections
{
	GENERATED_BODY()

public:

	/** The list of connections. */
	UPROPERTY()
	TArray<FCameraObjectConnection> Connections;

public:

	UE_API void Add(UObject* InSource, FName InSourcePropertyName, UObject* InTarget, FName InTargetPropertyName);

	UE_API FCameraObjectConnection* FindBySource(UObject* InSource);
	UE_API FCameraObjectConnection* FindBySource(UObject* InSource, FName InSourcePropertyName);
	UE_API FCameraObjectConnection* FindByTarget(UObject* InTarget);
	UE_API FCameraObjectConnection* FindByTarget(UObject* InTarget, FName InTargetPropertyName);
};

/**
 * A base class for a camera object that has a graph of camera nodes, connections between them,
 * and some exposed parameters.
 */
UCLASS(Abstract, MinimalAPI)
class UBaseCameraObject : public UObject
{
	GENERATED_BODY()

public:

	/** The public data interface of this camera object. */
	UPROPERTY()
	FCameraObjectInterface Interface;

	UPROPERTY()
	FCameraObjectConnections Connections;

	/** Event handlers to be notified of data changes. */
	UE::Cameras::TCameraEventHandlerContainer<UE::Cameras::ICameraObjectEventHandler> EventHandlers;

	/** Allocation information for all the nodes and variables in this camera object. */
	UPROPERTY()
	FCameraObjectAllocationInfo AllocationInfo;

public:

	/** Gets the object's unique ID. */
	const FGuid& GetGuid() const { return Guid; }

	/** Gets the default values for the parameters exposed on this camera rig. */
	const FInstancedPropertyBag& GetDefaultParameters() const { return DefaultParameters; }

	/** Gets the default values for the parameters exposed on this camera rig. */
	FInstancedPropertyBag& GetDefaultParameters() { return DefaultParameters; }

	/** Gets the definitions of parameters exposed on this camera rig. */
	TConstArrayView<FCameraObjectInterfaceParameterDefinition> GetParameterDefinitions() const { return ParameterDefinitions; }

public:

	/** Finds a parameter definition by name. */
	UE_API bool FindParameterDefinitionByName(const FName ParameterName, FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const;

	/** Finds a parameter definition by ID. */
	UE_API bool FindParameterDefinitionByGuid(const FGuid& ParameterGuid, FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const;

public:

	/** Get the root node of this camera object. */
	virtual UCameraNode* GetRootNode() { return nullptr; }

public:

	// UObject interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

protected:

#if WITH_EDITORONLY_DATA
	// Utility method to be called on PostLoad by subclasses that need upgrading old data.
	void UpgradeInterfaceConnections(IObjectTreeGraphRootObject* RootObject, FName DefaultGraphName);
#endif

private:

	/** The camera object's unique ID. */
	UPROPERTY()
	FGuid Guid;

	/** The default interface parameter values, generated during build. */
	UPROPERTY()
	FInstancedPropertyBag DefaultParameters;

	/** The definitions of parameters exposed on this camera rig. */
	UPROPERTY()
	TArray<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions;

	// Flag for DefaultParameters maybe having missing property flags.
	bool bDefaultParametersMayHaveMissingPropertyFlags = false;

	friend class UE::Cameras::FCameraObjectInterfaceParameterBuilder;
};

#undef UE_API

