// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraVariableTableFwd.h"
#include "StructUtils/OverridablePropertyBag.h"

#include "BaseCameraObjectReference.generated.h"

class UBaseCameraObject;

namespace UE::Cameras
{
	struct FCameraNodeEvaluationResult;
	struct FCameraNodeParameterInfos;
}

/**
 * Metadata for a referenced camera object's interface parameters.
 */
USTRUCT()
struct FCameraObjectInterfaceParameterMetaData
{
	GENERATED_BODY()

	/** The GUID of the parameter. */
	UPROPERTY()
	FGuid ParameterGuid;

	/** 
	 * The ID to use for overriding a blendable parameter.
	 * NOTE: only used for blendable structs, since other types are parameter structs with their
	 * own variable ID inside.
	 */
	UPROPERTY()
	FCameraVariableID OverrideVariableID;

	/** The ID to use for overriding a data parameter. */
	UPROPERTY()
	FCameraContextDataID OverrideDataID;

	
	// Deprecated

	UPROPERTY()
	bool bIsOverridden_DEPRECATED = false;
};

USTRUCT(BlueprintType)
struct FBaseCameraObjectReference
{
	GENERATED_BODY()

public:

	virtual ~FBaseCameraObjectReference() = default;

	/** Gets the parameters for this camera rig, some of which containing overrides. */
	const FInstancedOverridablePropertyBag& GetParameters() const
	{
		return Parameters;
	}

	/** Gets the parameters for this camera rig, some of which containing overrides. */
	FInstancedOverridablePropertyBag& GetParameters()
	{
		return Parameters;
	}

	/** Gets the IDs of the parameters with override values. */
	TConstArrayView<FGuid> GetOverriddenParameterGuids() const
	{
		return Parameters.GetOverridenPropertyIDs();
	}

public:

	/** Gets the camera object that this reference points to. */
	virtual const UBaseCameraObject* GetCameraObject() const { return nullptr; }

	/** Returns whether the override parameters structure needs to be rebuilt. */
	GAMEPLAYCAMERAS_API bool NeedsRebuildParameters() const;
	/** Rebuilds the override parameters structure, if needed. */
	GAMEPLAYCAMERAS_API bool RebuildParametersIfNeeded();
	/** Rebuilds the override parameters structure. */
	GAMEPLAYCAMERAS_API void RebuildParameters();

public:

	// Internal API.

	void PostSerialize(const FArchive& Ar);

	GAMEPLAYCAMERAS_API void GetCustomCameraNodeParameters(UE::Cameras::FCameraNodeParameterInfos& OutParameterInfos);

	bool PostBuild();
	
	GAMEPLAYCAMERAS_API bool IsParameterOverridden(const FGuid& PropertyID) const;
	GAMEPLAYCAMERAS_API void SetParameterOverridden(const FGuid& PropertyID, bool bIsOverridden);

	template<typename ContainerType>
	void GetOverriddenParameterGuids(ContainerType& OutOverriddenIDs) const;

protected:

	const FCameraObjectInterfaceParameterMetaData* FindMetaData(const FGuid& PropertyID) const;

private:

	FCameraObjectInterfaceParameterMetaData& GetMetaData(const FGuid& PropertyID);

protected:

	/** The camera rig's parameters. */
	UPROPERTY(EditAnywhere, Category="", meta=(FixedLayout, InterpBagProperties))
	FInstancedOverridablePropertyBag Parameters;

	/** Metadata for the parameters. */
	UPROPERTY()
	TArray<FCameraObjectInterfaceParameterMetaData> ParameterMetaData;
};

template<typename ContainerType>
void FBaseCameraObjectReference::GetOverriddenParameterGuids(ContainerType& OutOverriddenIDs) const
{
	Parameters.GetOverridenPropertyIDs(OutOverriddenIDs);
}

template<>
struct TStructOpsTypeTraits<FBaseCameraObjectReference> : public TStructOpsTypeTraitsBase2<FBaseCameraObjectReference>
{
	enum
	{
		WithPostSerialize = true
	};
};

