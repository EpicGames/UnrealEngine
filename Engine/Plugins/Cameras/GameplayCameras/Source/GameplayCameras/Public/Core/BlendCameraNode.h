// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"

#include "BlendCameraNode.generated.h"

/**
 * Base class for blend camera nodes.
 */
UCLASS(MinimalAPI, Abstract, meta=(CameraNodeCategories="Blends"))
class UBlendCameraNode : public UCameraNode
{
	GENERATED_BODY()
};

namespace UE::Cameras
{

class FBlendCameraNodeEvaluator;

/**
 * Parameter struct for blending camera node parameters.
 */
struct FCameraNodePreBlendParams
{
	FCameraNodePreBlendParams(
			const FCameraNodeEvaluationParams& InEvaluationParams,
			const FCameraPose& InLastCameraPose,
			const FCameraVariableTable& InChildVariableTable)
		: EvaluationParams(InEvaluationParams)
		, LastCameraPose(InLastCameraPose)
		, ChildVariableTable(InChildVariableTable)
	{}

	/** The parameters for the evaluation that will happen afterwards. */
	const FCameraNodeEvaluationParams& EvaluationParams;
	/** Last frame's camera pose. */
	const FCameraPose& LastCameraPose;
	/** The variable table of the node tree being blended. */
	const FCameraVariableTable& ChildVariableTable;
	/** The filter to use for variable table blending. */
	ECameraVariableTableFilter VariableTableFilter = ECameraVariableTableFilter::None;
};

/**
 * Result struct for blending camera node parameters.
 */
struct FCameraNodePreBlendResult
{
	FCameraNodePreBlendResult(FCameraVariableTable& InVariableTable)
		: VariableTable(InVariableTable)
	{}

	/** The variable table to received blended parameters. */
	FCameraVariableTable& VariableTable;

	/** Whether the blend has reached 100%. */
	bool bIsBlendFull = false;

	/** Whether the blend is finished. */
	bool bIsBlendFinished = false;
};

/**
 * Parameter struct for blending camera node tree results.
 */
struct FCameraNodeBlendParams
{
	FCameraNodeBlendParams(
			const FCameraNodeEvaluationParams& InChildParams,
			const FCameraNodeEvaluationResult& InChildResult)
		: ChildParams(InChildParams)
		, ChildResult(InChildResult)
	{}

	/** The parameters that the blend received during the evaluation. */
	const FCameraNodeEvaluationParams& ChildParams;
	/** The result that the blend should apply over another result. */
	const FCameraNodeEvaluationResult& ChildResult;
};

/**
 * Result struct for blending camera node tree results.
 */
struct FCameraNodeBlendResult
{
	FCameraNodeBlendResult(FCameraNodeEvaluationResult& InBlendedResult)
		: BlendedResult(InBlendedResult)
	{}

	/** The result upon which another result should be blended. */
	FCameraNodeEvaluationResult& BlendedResult;

	/** Whether the blend has reached 100%. */
	bool bIsBlendFull = false;

	/** Whether the blend is finished. */
	bool bIsBlendFinished = false;
};

/**
 * Parameter struct for initializing a blend from an interrupted blend.
 */
struct FCameraNodeBlendInterruptionParams
{
	/** The existing blend that was interrupted. */
	const FBlendCameraNodeEvaluator* InterruptedBlend = nullptr;
};

/**
 * Base evaluator class for blend camera nodes.
 */
class FBlendCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBlendCameraNodeEvaluator)

public:

	/** Blend the parameters produced by a camera node tree over another set of values. */
	GAMEPLAYCAMERAS_API void BlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult);

	/** Blend the result of a camera node tree over another result. */
	GAMEPLAYCAMERAS_API void BlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult);

public:

	/**
	 * Initialize this blend from an interrupted blend.
	 *
	 * @return If true, the blend sub-class fully supports seamless transition from the interrupted blend.
	 *		   If false, the blend will be wrapped in an Interrupted Blend node that will freeze the
	 *		   interrupted blend to ensure a seamless transition.
	 */
	bool InitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params);

	/**
	 * Reverse the direction of this blend.
	 *
	 * @return If true, the blend sub-class fully supports reverse blending.
	 *         If false, the blend will be wrapped in a Reverse Blend node that will cache and swap the
	 *         "from" and "to" results given to the BlendParameters and BlendResults methods.
	 */
	bool SetReversed(bool bInReverse);

	/**
	 * Freezes this blend. A frozen blend can't access its evaluation context or underlying camera node
	 * anymore, and must operate entirely standalone.
	 */
	void Freeze();

protected:

	/** Blend the parameters produced by a camera node tree over another set of values. */
	virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) {}

	/** Blend the result of a camera node tree over another result. */
	virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) {}

	/** Initialize this blend from an interrupted blend. See comments from InitializeFromInterruption. */
	virtual bool OnInitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params) { return false; }

	/** Reverse the direction of this blend. See comments from SetReversed. */
	virtual bool OnSetReversed(bool bInReverse) { return false; }

	/** Freezes this blend. See commands from Freeze. */
	virtual void OnFreeze() {}
};

}  // namespace UE::Cameras

// Macros for declaring and defining new blend node evaluators. They are the same
// as the base ones for generic node evaluators, but the first one prevents you
// from having to specify FBlendCameraNodeEvaluator as the base class, which saves
// a little bit of typing.
//
#define UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(ApiDeclSpec, ClassName)\
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, ::UE::Cameras::FBlendCameraNodeEvaluator)

#define UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_BLEND_CAMERA_NODE_EVALUATOR(ClassName)\
	UE_DEFINE_CAMERA_NODE_EVALUATOR(ClassName)

