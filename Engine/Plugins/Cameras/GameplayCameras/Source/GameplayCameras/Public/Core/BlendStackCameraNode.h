// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendStackEntryID.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraNodeEvaluatorStorage.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigEvaluationInfo.h"
#include "Core/CameraRigInstanceID.h"
#include "Debug/CameraDebugBlock.h"
#include "IGameplayCamerasLiveEditListener.h"

#include "BlendStackCameraNode.generated.h"

class UBlendStackRootCameraNode;
class UCameraAsset;
class UCameraRigAsset;
class UCameraRigTransition;

namespace UE::Cameras
{

class FBlendStackRootCameraNodeEvaluator;
class FCameraEvaluationContext;
class FCameraSystemEvaluator;
enum class EBlendStackCameraRigEventType;
struct FBlendStackCameraRigEvent;

#if UE_GAMEPLAY_CAMERAS_DEBUG
class FBlendStackCameraDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if WITH_EDITOR
class IGameplayCamerasLiveEditManager;
#endif

}  // namespace UE::Cameras

/**
 * Describes a type of blend stack.
 */
UENUM()
enum class ECameraBlendStackType
{
	/**
	 * Camera rigs are evaluated in isolation before being blended together, and get 
	 * automatically popped out of the stack when another rig has reached 100% blend above 
	 * them.
	 */
	IsolatedTransient,
	/**
	 * Camera rigs are evaluated in an additive way, i.e. the result of a lower camera rig
	 * becomes the input of the next ones. Also, camera rigs stay in the stack until explicitly 
	 * removed.
	 */
	AdditivePersistent
};

/**
 * A blend stack implemented as a camera node.
 */
UCLASS(MinimalAPI, Hidden)
class UBlendStackCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The type of blend stack this should run as. */
	UPROPERTY()
	ECameraBlendStackType BlendStackType = ECameraBlendStackType::IsolatedTransient;

	/** The layer that this blend stack represents, if any. */
	UPROPERTY()
	ECameraRigLayer Layer = ECameraRigLayer::None;
};

namespace UE::Cameras
{

DECLARE_MULTICAST_DELEGATE_OneParam(FOnBlendStackCameraRigEvent, const FBlendStackCameraRigEvent&);

/**
 * Evaluator for a blend stack camera node.
 */
class FBlendStackCameraNodeEvaluator 
	: public TCameraNodeEvaluator<UBlendStackCameraNode>
#if WITH_EDITOR
	, public IGameplayCamerasLiveEditListener
#endif
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBlendStackCameraNodeEvaluator)

public:

	FBlendStackCameraNodeEvaluator();
	~FBlendStackCameraNodeEvaluator();

	/** Returns information about the top (active) camera rig, if any. */
	FCameraRigEvaluationInfo GetActiveCameraRigEvaluationInfo() const;

	/** Returns information about a given camera rig, if any. */
	FCameraRigEvaluationInfo GetCameraRigEvaluationInfo(FBlendStackEntryID EntryID) const;

	/** Returns whether the stack contains any running camera rig with the given context. */
	bool HasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FBlendStackCameraDebugBlock* BuildDetailedDebugBlock(
			const FCameraDebugBlockBuildParams& Params,
			const FLinearColor& StartColor, const FLinearColor& EndColor,
			FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

public:

	/** Gets the delegate for blend stack events. */
	FOnBlendStackCameraRigEvent& OnCameraRigEvent() { return OnCameraRigEventDelegate; }

protected:

	// FCameraNodeEvaluator interface
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if WITH_EDITOR
	// IGameplayCamerasLiveEditListener interface
	virtual void OnPostBuildAsset(const FGameplayCameraAssetBuildEvent& BuildEvent) override;
#endif

protected:

	struct FCameraRigEntry;

	void InitializeEntry(
		FCameraRigEntry& NewEntry, 
		const UCameraRigAsset* CameraRig,
		TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
		UBlendStackRootCameraNode* EntryRootNode,
		bool bIsActiveEntry);

	int32 IndexOfEntry(const FBlendStackEntryID EntryID) const;

	void FreezeEntry(FCameraRigEntry& Entry);

	void PopEntry(int32 EntryIndex);
	void PopEntries(int32 FirstIndexToKeep);

	void BroadcastCameraRigEvent(EBlendStackCameraRigEventType EventType, const FCameraRigEntry& Entry, const UCameraRigTransition* Transition = nullptr) const;

#if WITH_EDITOR
	void AddPackageListeners(FCameraRigEntry& Entry);
	void RemoveListenedPackages(FCameraRigEntry& Entry);
	void RemoveListenedPackages(TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager, FCameraRigEntry& Entry);
#endif

protected:

#if WITH_EDITOR
	virtual void OnEntryReinitialized(int32 EntryIndex) {}
#endif

protected:

	struct FResolvedEntry
	{
		FResolvedEntry(FCameraRigEntry& InEntry, TSharedPtr<const FCameraEvaluationContext> InContext)
			: Entry(InEntry), Context(InContext)
		{}

		FCameraRigEntry& Entry;
		TSharedPtr<const FCameraEvaluationContext> Context;
		int32 EntryIndex = INDEX_NONE;
		bool bIsActiveEntry = false;
	};

	void ResolveEntries(const FCameraNodeEvaluationParams& Params, TArray<FResolvedEntry>& OutResolvedEntries);
	void OnRunFinished(FCameraNodeEvaluationResult& OutResult);

protected:

	struct FCameraRigEntry
	{
		/** The ID for this entry. */
		FBlendStackEntryID EntryID;
		/** Evaluation context in which this entry runs. */
		TWeakPtr<const FCameraEvaluationContext> EvaluationContext;
		/** The camera rig asset that this entry runs. */
		TObjectPtr<const UCameraRigAsset> CameraRig;
		/** The root node. */
		TObjectPtr<UBlendStackRootCameraNode> RootNode;
		/** Storage buffer for all evaluators in this node tree. */
		FCameraNodeEvaluatorStorage EvaluatorStorage;
		/** Root evaluator. */
		FBlendStackRootCameraNodeEvaluator* RootEvaluator = nullptr;
		/** The evaluator tree. */
		FCameraNodeEvaluatorHierarchy EvaluatorHierarchy;
		/** Context result for this node tree. */
		FCameraNodeEvaluationResult ContextResult;
		/** Evaluation result for this node tree. */
		FCameraNodeEvaluationResult Result;

		struct
		{
			/** Whether this is the first frame this entry runs. */
			bool bIsFirstFrame:1 = false;
			/** Whether the context's initial result was valid last frame. */
			bool bWasContextInitialResultValid:1 = false;
			/** Whether to force a camera cut on this entry this frame. */
			bool bForceCameraCut:1 = false;
			/** Whether this entry is frozen. */
			bool bIsFrozen:1 = false;

#if UE_GAMEPLAY_CAMERAS_TRACE
			bool bLogWarnings:1 = true;
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
		} Flags;

#if WITH_EDITOR
		TArray<TWeakObjectPtr<const UPackage>, TInlineAllocator<4>> ListenedPackages;
#endif  // WITH_EDITOR
	};

	/** The camera system evaluator running this node. */
	FCameraSystemEvaluator* OwningEvaluator = nullptr;

	/** Entries in the blend stack. */
	TArray<FCameraRigEntry> Entries;

	/** Next entry ID to use. */
	uint32 NextEntryID = 0;

	/** The layer that this blend stack represents, if any. */
	ECameraRigLayer Layer = ECameraRigLayer::None;

	/** The delegate to invoke when an event occurs in this blend stack. */
	FOnBlendStackCameraRigEvent OnCameraRigEventDelegate;

#if WITH_EDITOR
	TMap<TWeakObjectPtr<const UPackage>, int32> AllListenedPackages;
	TSet<TWeakObjectPtr<const UCameraRigAsset>> BuiltCameraRigs;
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	friend class FBlendStackSummaryCameraDebugBlock;
	friend class FBlendStackCameraDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FBlendStackSummaryCameraDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FBlendStackSummaryCameraDebugBlock)

public:

	FBlendStackSummaryCameraDebugBlock();
	FBlendStackSummaryCameraDebugBlock(const FBlendStackCameraNodeEvaluator& InEvaluator);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	int32 NumEntries;
	ECameraBlendStackType BlendStackType;
	ECameraRigLayer BlendStackLayer;
};

class FBlendStackCameraDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FBlendStackCameraDebugBlock)

protected:

	virtual void OnSerialize(FArchive& Ar) override;

private:

	using FBlendStackStartEndColors = TTuple<FLinearColor, FLinearColor>;
	FBlendStackStartEndColors StartEndColors;
};

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

