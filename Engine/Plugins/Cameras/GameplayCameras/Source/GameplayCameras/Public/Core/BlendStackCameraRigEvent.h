// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigEvaluationInfo.h"
#include "UObject/ObjectPtr.h"

class UCameraRigTransition;

namespace UE::Cameras
{

class FBlendStackCameraNodeEvaluator;

/** A type of blend stack event. */
enum class EBlendStackCameraRigEventType
{
	/** A camera rig was pushed on the blend stack. */
	Pushed,
	/** A camera rig was frozen. */
	Frozen,
	/** A camera rig was popped out of the blend stack. */
	Popped
};

/**
 * A structure describing an event happening on a blend stack.
 */
struct FBlendStackCameraRigEvent
{
	/** The event type. */
	EBlendStackCameraRigEventType EventType;
	/** The blend stack evaluator in which the event is happening. */
	const FBlendStackCameraNodeEvaluator* BlendStackEvaluator = nullptr;
	/** The evaluation information for the camera rig associated with the event. */
	FCameraRigEvaluationInfo CameraRigInfo;
	/** If the event is Pushed, the transition used for the new camera rig. */
	TObjectPtr<const UCameraRigTransition> Transition;
};

}  // namespace UE::Cameras

