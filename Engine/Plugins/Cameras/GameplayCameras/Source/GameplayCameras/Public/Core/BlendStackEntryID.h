// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Serialization/Archive.h"

struct FCameraRigInstanceID;

namespace UE::Cameras
{

class FBlendStackCameraNodeEvaluator;

/**
 * An ID for an entry in a camera blend stack.
 *
 * Note about overflowing: the max value for a uint32 is 4294967295. We use it for an INVALID
 * blend stack entry, so we have at most 4294967294 instances to go through before overflowing.
 * If somehow we wanted to push, on average, one new camera rig on a blend stack every second,
 * we would only overflow after ~136 years.
 */
struct FBlendStackEntryID
{
public:

	FBlendStackEntryID() : Value(INVALID) {}

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

public:

	friend bool operator==(FBlendStackEntryID A, FBlendStackEntryID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FBlendStackEntryID A, FBlendStackEntryID B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FBlendStackEntryID In)
	{
		return In.Value;
	}

	friend FArchive& operator<< (FArchive& Ar, FBlendStackEntryID& In)
	{
		Ar << In.Value;
		return Ar;
	}

	friend FString LexToString(FBlendStackEntryID In)
	{
		return LexToString(In.Value);
	}

private:

	FBlendStackEntryID(uint32 InValue) 
		: Value(InValue)
	{}

	static const uint32 INVALID = uint32(-1);

	uint32 Value;

	friend struct ::FCameraRigInstanceID;
	friend class FBlendStackCameraNodeEvaluator;
};

}  // namespace UE::Cameras

