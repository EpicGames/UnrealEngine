// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableAllocationInfo.h"
#include "CoreTypes.h"

#include "BuiltInCameraVariables.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UDoubleCameraVariable;
class UVector2dCameraVariable;

/**
 * Built-in floating point camera variables.
 */
UENUM()
enum class EBuiltInDoubleCameraVariable
{
	None,
	Yaw,
	Pitch,
	Roll,
	Zoom
};

/**
 * Built-in 2-dimensional camera variables.
 */
UENUM()
enum class EBuiltInVector2dCameraVariable
{
	None,
	YawPitch
};

UENUM()
enum class EBuiltInRotator3dCameraVariable
{
	None,
	ControlRotation
};

namespace UE::Cameras
{

/**
 * A utility class that provides definitions for "well-known" camera variables.
 */
class FBuiltInCameraVariables
{
public:

	/** Singleton access. */
	UE_API static const FBuiltInCameraVariables& Get();

	/** Get the definition of a built-in camera variable. */
	UE_API const FCameraVariableDefinition& GetDefinition(EBuiltInDoubleCameraVariable BuiltInVariable) const;
	/** Get the definition of a built-in camera variable. */
	UE_API const FCameraVariableDefinition& GetDefinition(EBuiltInVector2dCameraVariable BuiltInVariable) const;
	/** Get the definition of a built-in camera variable. */
	UE_API const FCameraVariableDefinition& GetDefinition(EBuiltInRotator3dCameraVariable BuiltInVariable) const;

public:

	FCameraVariableDefinition YawDefinition;
	FCameraVariableDefinition PitchDefinition;
	FCameraVariableDefinition RollDefinition;
	FCameraVariableDefinition ZoomDefinition;

	FCameraVariableDefinition YawPitchDefinition;

	FCameraVariableDefinition FreezeControlRotationDefinition;
	FCameraVariableDefinition ControlRotationDefinition;

protected:

	FBuiltInCameraVariables();
};

}  // namespace UE::Cameras

#undef UE_API

