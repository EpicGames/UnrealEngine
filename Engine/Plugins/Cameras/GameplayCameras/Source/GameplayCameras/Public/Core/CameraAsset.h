// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildStatus.h"
#include "Core/CameraContextDataTableAllocationInfo.h"
#include "Core/CameraEventHandler.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraRigTransition.h"
#include "Core/CameraVariableTableAllocationInfo.h"
#include "Core/ObjectTreeGraphObject.h"
#include "Core/ObjectTreeGraphRootObject.h"
#include "CoreTypes.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/ObjectPtr.h"

#include "CameraAsset.generated.h"

class UCameraAsset;
class UCameraDirector;
class UCameraRigAsset;

namespace UE::Cameras
{
	class FCameraAssetBuilder;
	struct FCameraBuildContext;

	/**
	 * Interface for listening to changes on a camera asset.
	 */
	class ICameraAssetEventHandler
	{
	public:
		virtual ~ICameraAssetEventHandler() = default;

		/** Called when the camera director has been changed. */
		virtual void OnCameraDirectorChanged(UCameraAsset* InCameraAsset, const TCameraPropertyChangedEvent<UCameraDirector*>& Event) {}
		/* Changed when the enter transitions have been changed. */
		virtual void OnEnterTransitionsChanged(UCameraAsset* InCameraAsset, const TCameraArrayChangedEvent<UCameraRigTransition*>& Event) {}
		/* Changed when the exit transitions have been changed. */
		virtual void OnExitTransitionsChanged(UCameraAsset* InCameraAsset, const TCameraArrayChangedEvent<UCameraRigTransition*>& Event) {}
		/** Called when the camera asset's interface has changed. */
		virtual void OnCameraAssetInterfaceChanged() {}
	};
}

/**
 * An interface parameter on a camera asset, which exposes one of its camera rigs' own interface parameters.
 */
UCLASS()
class UCameraAssetInterfaceParameter : public UObject
{
	GENERATED_BODY()

public:

	/** The exposed name for this parameter. */
	UPROPERTY(EditAnywhere, Category="Camera")
	FString InterfaceParameterName;

	/** The camera rig that owns the source parameter. */
	UPROPERTY(EditAnywhere, Category="Camera")
	TSoftObjectPtr<UCameraRigAsset> SourceCameraRig;

	/** The name of the parameter on the source camera rig. */
	UPROPERTY(EditAnywhere, Category="Camera")
	FName SourceParameterName;

public:

	/** Gets this parameter's unique ID. */
	const FGuid& GetGuid() const { return Guid; }

	/** Gets the parameter definition. */
	bool GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const;

protected:

	/** The Guid of this parameter. */
	UPROPERTY()
	FGuid Guid;

protected:

	// UObject interface.
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
};

/**
 * A structure defining the public interface of a camera asset.
 */
USTRUCT()
struct FCameraAssetInterface
{
	GENERATED_BODY()

public:

	/** The list of exposed parameters on the camera asset. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraAssetInterfaceParameter>> Parameters;
};

/**
 * Structure describing various allocations needed by a camera asset.
 */
USTRUCT()
struct FCameraAssetAllocationInfo
{
	GENERATED_BODY()

	/** Combined variable table allocation info for all the camera rigs. */
	UPROPERTY()
	FCameraVariableTableAllocationInfo VariableTableInfo;

	/** Combined context data table allocation info for all the camera rigs. */
	UPROPERTY()
	FCameraContextDataTableAllocationInfo ContextDataTableInfo;

	GAMEPLAYCAMERAS_API friend bool operator==(const FCameraAssetAllocationInfo& A, const FCameraAssetAllocationInfo& B);
};

/**
 * A complete camera asset.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCameraAsset 
	: public UObject
	, public IHasCameraBuildStatus
	, public IObjectTreeGraphObject
	, public IObjectTreeGraphRootObject
{
	GENERATED_BODY()

public:

	/** Gets the camera director. */
	UCameraDirector* GetCameraDirector() const { return CameraDirector; }
	/** Sets the camera director. */
	GAMEPLAYCAMERAS_API void SetCameraDirector(UCameraDirector* InCameraDirector);

	/** Gets the enter transitions. */
	TArrayView<const TObjectPtr<UCameraRigTransition>> GetEnterTransitions() const { return EnterTransitions; }
	/** Adds an enter transition. */
	GAMEPLAYCAMERAS_API void AddEnterTransition(UCameraRigTransition* InTransition);
	/** Removes an enter transition. */
	GAMEPLAYCAMERAS_API int32 RemoveEnterTransition(UCameraRigTransition* InTransition);

	/** Gets the exit transitions. */
	TArrayView<const TObjectPtr<UCameraRigTransition>> GetExitTransitions() const { return ExitTransitions; }
	/** Adds an exit transition. */
	GAMEPLAYCAMERAS_API void AddExitTransition(UCameraRigTransition* InTransition);
	/** Removes an exit transition. */
	GAMEPLAYCAMERAS_API int32 RemoveExitTransition(UCameraRigTransition* InTransition);

public:

	/** Gets the default parameter values for all camera rigs. */
	const FInstancedPropertyBag& GetDefaultParameters() const { return DefaultParameters; }
	/** Gets the default parameter values for all camera rigs. */
	FInstancedPropertyBag& GetDefaultParameters() { return DefaultParameters; }

	/** Gets the definitions of parameters exposed on this camera asset. */
	TConstArrayView<FCameraObjectInterfaceParameterDefinition> GetParameterDefinitions() const { return ParameterDefinitions; }

public:

	/**
	 * Builds and validates this camera, including all its camera rigs.
	 * Errors and warnings will go to the console.
	 */
	GAMEPLAYCAMERAS_API void BuildCamera();

	/**
	 * Builds and validates this camera, including all its camera rigs.
	 * Errors and warnings will go to the provided build context.
	 */
	GAMEPLAYCAMERAS_API void BuildCamera(UE::Cameras::FCameraBuildContext& InBuildContext);

public:

	// Graph names for ObjectTreeGraph API.
	GAMEPLAYCAMERAS_API static const FName SharedTransitionsGraphName;

	// IHasCameraBuildStatus interface.
	virtual ECameraBuildStatus GetBuildStatus() const override { return BuildStatus; }
	virtual void DirtyBuildStatus() override;

	/** Sets the build status. */
	void SetBuildStatus(ECameraBuildStatus InBuildStatus) { BuildStatus = InBuildStatus; }

	/** Gets the allocation info for this camera asset. */
	const FCameraAssetAllocationInfo& GetAllocationInfo() const { return AllocationInfo; }

protected:

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override { return EObjectTreeGraphObjectSupportFlags::CommentText; }
	virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
#endif

	// IObjectTreeGraphRootObject interface.
#if WITH_EDITOR
	virtual void GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const override;
	virtual void AddConnectableObject(FName InGraphName, UObject* InObject) override;
	virtual void RemoveConnectableObject(FName InGraphName, UObject* InObject) override;
#endif

	// UObject interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	/** Event handlers to be notified of data changes. */
	UE::Cameras::TCameraEventHandlerContainer<UE::Cameras::ICameraAssetEventHandler> EventHandlers;

	/** The public interface of this camera asset. */
	UPROPERTY()
	FCameraAssetInterface Interface;

private:

	/** The camera director to use in this camera. */
	UPROPERTY(Instanced)
	TObjectPtr<UCameraDirector> CameraDirector;

	/** A list of default enter transitions for all the camera rigs in this asset. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraRigTransition>> EnterTransitions;

	/** A list of default exit transitions for all the camera rigs in this asset. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraRigTransition>> ExitTransitions;

	/** The current build state of this camera asset. */
	UPROPERTY(Transient)
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Dirty;

	/** Generated values for all camera rig parameters. */
	UPROPERTY()
	FInstancedPropertyBag DefaultParameters;

	/** Generated list of all the camera rigs' parameters. */
	UPROPERTY()
	TArray<FCameraObjectInterfaceParameterDefinition> ParameterDefinitions;

	/** Allocation info for the camera asset. */
	UPROPERTY()
	FCameraAssetAllocationInfo AllocationInfo;

#if WITH_EDITORONLY_DATA

	/** Position of the camera node in the shared transitions graph editor. */
	UPROPERTY()
	FIntVector2 TransitionGraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the transition graph editor. */
	UPROPERTY()
	FString TransitionGraphNodeComment;

	/** All nodes used in the shared transitions graph editor. */
	UPROPERTY(Instanced, meta=(ObjectTreeGraphHidden=true))
	TArray<TObjectPtr<UObject>> AllSharedTransitionsObjects;

	// Only specified here so that the schema can use GET_MEMBER_NAME_CHECKED...
	friend class UCameraSharedTransitionGraphSchema;

#endif  // WITH_EDITORONLY_DATA

	// Flag for DefaultParameters maybe having missing property flags.
	bool bDefaultParametersMayHaveMissingPropertyFlags = false;
	// Flag for whether we need to upgrade the interface on the next build.
	bool bNeedsDefaultInterface = false;

	friend class UE::Cameras::FCameraAssetBuilder;
};

