// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/OverridablePropertyBag.h"
#include "UObject/ObjectPtr.h"

#include "CameraAssetReference.generated.h"

class UCameraAsset;
struct FPropertyTag;

namespace UE::Cameras
{
	class FCameraAssetReferenceDetailsCustomization;
	struct FCameraNodeEvaluationResult;
}

/**
 * A structure holding a reference to a camera asset, along with the interface parameter
 * override values for any of its camera rigs.
 */
USTRUCT(BlueprintType)
struct FCameraAssetReference
{
	GENERATED_BODY()

public:

	FCameraAssetReference();
	FCameraAssetReference(UCameraAsset* InCameraAsset);

	/** Returns whether this reference points to a valid camera asset. */
	bool IsValid() const
	{
		return CameraAsset != nullptr;
	}

	/** Gets the referenced camera asset. */
	UCameraAsset* GetCameraAsset()
	{
		return CameraAsset;
	}

	/** Gets the referenced camera asset. */
	const UCameraAsset* GetCameraAsset() const
	{
		return CameraAsset;
	}

	/** Sets the referenced camera asset. */
	void SetCameraAsset(UCameraAsset* InCameraAsset)
	{
		if (CameraAsset != InCameraAsset)
		{
			CameraAsset = InCameraAsset;
			RebuildParameters();
		}
	}

	/** Gets the parameters for this camera, some of which containing overrides. */
	const FInstancedOverridablePropertyBag& GetParameters() const
	{
		return Parameters;
	}

	/** Gets the parameters for this camera, some of which containing overrides. */
	FInstancedOverridablePropertyBag& GetParameters()
	{
		return Parameters;
	}

	/** Gets the IDs of the parameters with override values. */
	TConstArrayView<FGuid> GetOverriddenParameterGuids() const
	{
		return Parameters.GetOverridenPropertyIDs();
	}

public:

	/** Applies the parameter override values to the given evaluation result. */
	GAMEPLAYCAMERAS_API void ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const;

public:

	/** Returns whether the override parameters structure needs to be rebuilt. */
	GAMEPLAYCAMERAS_API bool NeedsRebuildParameters() const;
	/** Rebuilds the override parameters structure, if needed. */
	GAMEPLAYCAMERAS_API bool RebuildParametersIfNeeded();
	/** Rebuilds the override parameters structure. */
	GAMEPLAYCAMERAS_API void RebuildParameters();

public:

	// Internal API.

	bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	void PostSerialize(const FArchive& Ar);

	GAMEPLAYCAMERAS_API bool IsParameterOverridden(const FGuid PropertyID) const;
	GAMEPLAYCAMERAS_API void SetParameterOverridden(const FGuid PropertyID, bool bIsOverridden);

private:

	/** The referenced camera asset. */
	UPROPERTY(EditAnywhere, Category=Camera)
	TObjectPtr<UCameraAsset> CameraAsset;

	/** The camera asset's parameters. */
	UPROPERTY(EditAnywhere, Category="", meta=(FixedLayout, InterpBagProperties))
	FInstancedOverridablePropertyBag Parameters;


	// Deprecated

	UPROPERTY()
	TArray<FGuid> ParameterOverrideGuids_DEPRECATED;

	friend class UE::Cameras::FCameraAssetReferenceDetailsCustomization;
};

template<>
struct TStructOpsTypeTraits<FCameraAssetReference> : public TStructOpsTypeTraitsBase2<FCameraAssetReference>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithPostSerialize = true
	};
};

