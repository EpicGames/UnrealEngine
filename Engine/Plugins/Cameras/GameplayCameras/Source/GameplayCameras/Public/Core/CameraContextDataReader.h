// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTable.h"
#include "CoreTypes.h"
#include <type_traits>

namespace UE::Cameras
{

template<typename DataType>
struct TCameraContextDataReader
{
	TCameraContextDataReader() {}

	template<typename ParameterType>
	TCameraContextDataReader(const ParameterType& Parameter)
	{
		Initialize(Parameter);
	}

	TCameraContextDataReader(const DataType* InDefaultValuePtr, FCameraContextDataID InDataID)
	{
		Initialize(InDefaultValuePtr, InDataID);
	}

	template<typename ParameterType>
	void Initialize(const ParameterType& Parameter)
	{
		static_assert(
				std::is_same<DataType, typename ParameterType::DataType>(),
				"The given parameter is of the wrong type for this reader! Data types must be the same.");

		DefaultValuePtr = &Parameter.Value;
		DataID = Parameter.DataID;

		ensureMsgf(DefaultValuePtr, TEXT("The given parameter doesn't have a value!"));
	}

	void Initialize(const DataType* InDefaultValuePtr, FCameraContextDataID InDataID)
	{
		DefaultValuePtr = InDefaultValuePtr;
		DataID = InDataID;

		ensureMsgf(DefaultValuePtr, TEXT("No default value was provided!"));
	}

	DataType Get(const FCameraContextDataTable& ContextDataTable) const
	{
		if (!DataID.IsValid())
		{
			return (DefaultValuePtr ? *DefaultValuePtr : DataType());
		}

		if constexpr(std::is_enum_v<DataType>)
		{
			// Enums need to be converted: they are always stored as uint32 in the table,
			// but they may actually have a different backing type. If that type is shorter
			// or longer, we can't reinterpret_cast the memory, we need to convert the
			// uint32 value instead.
			if (const uint8* RawValue = ContextDataTable.TryGetData(
						DataID, ECameraContextDataType::Enum, StaticEnum<DataType>()))
			{
				return DataType(*reinterpret_cast<const uint32*>(RawValue));
			}
		}
		else
		{
			if (const DataType* ActualValue = ContextDataTable.TryGetData<DataType>(DataID))
			{
				return *ActualValue;
			}
		}
		return (DefaultValuePtr ? *DefaultValuePtr : DataType());
	}

	const DataType& GetRef(const FCameraContextDataTable& ContextDataTable) const
	{
		static const DataType StaticDefaultValue = DataType();

		static_assert(!std::is_enum_v<DataType>, "Can't use GetRef with enum types, please use Get instead");

		if (!DataID.IsValid())
		{
			return (DefaultValuePtr ? *DefaultValuePtr : StaticDefaultValue);
		}

		if (const DataType* ActualValue = ContextDataTable.TryGetData<DataType>(DataID))
		{
			return *ActualValue;
		}
		return (DefaultValuePtr ? *DefaultValuePtr : StaticDefaultValue);
	}

private:

	const DataType* DefaultValuePtr = nullptr;
	FCameraContextDataID DataID;
};

}  // namespace UE::Cameras

