// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/ScriptArray.h"
#include "GameplayCameras.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include <type_traits>

#define UE_API GAMEPLAYCAMERAS_API

struct FCameraContextDataDefinition;
struct FCameraContextDataTableAllocationInfo;

namespace UE::Cameras
{

class FCameraContextDataTable;

#if UE_GAMEPLAY_CAMERAS_DEBUG
class FContextDataTableDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

template<typename DataType>
struct TCameraContextDataTraits
{
	static ECameraContextDataType GetDataType();
	static const UObject* GetDataTypeObject();
};

/**
 * Filter for context data table operations.
 */
enum class ECameraContextDataTableFilter
{
	None = 0,
	/** Only include data that is common to both tables. */
	KnownOnly = 1 << 0,
	/** Only include data that was written this frame. */
	ChangedOnly = 1 << 1,
};
ENUM_CLASS_FLAGS(ECameraContextDataTableFilter)

/**
 * The camera context data table is a container for a collection of arbitrary values
 * of various types. It is the companion of the camera variable table (see FCameraVariableTable)
 * but for non-blendable values.
 */
class FCameraContextDataTable
{
public:

	FCameraContextDataTable();
	~FCameraContextDataTable();

	/** Initializes the context data table so that it fits the provided allocation info. */
	void Initialize(const FCameraContextDataTableAllocationInfo& AllocationInfo);

	/**
	 * Ensure that all the data referenced in the given allocation info exist in the table.
	 * Returns false if some data had to be added.
	 */
	bool EnsureData(const FCameraContextDataTableAllocationInfo& AllocationInfo);

	/** Adds a data entry to the table. */
	void AddData(const FCameraContextDataDefinition& DataDefinition);

public:

	// Getter methods.

	const FName& GetNameData(FCameraContextDataID InID) const;
	const FString& GetStringData(FCameraContextDataID InID) const;
	uint32 GetEnumData(FCameraContextDataID InID, const UEnum* EnumType) const;
	FConstStructView GetStructViewData(FCameraContextDataID InID, const UScriptStruct* StructType) const;
	FInstancedStruct GetInstancedStructData(FCameraContextDataID InID, const UScriptStruct* StructType) const;
	UObject* GetObjectData(FCameraContextDataID InID) const;
	UClass* GetClassData(FCameraContextDataID InID) const;
	
	template<typename EnumType>
	EnumType GetEnumData(FCameraContextDataID InID) const;

	template<typename StructType>
	const StructType& GetStructData(FCameraContextDataID InID) const;

	template<typename ObjectClass>
	ObjectClass* GetObjectData(FCameraContextDataID InID) const;

	template<typename BaseClass>
	TSubclassOf<BaseClass> GetClassData(FCameraContextDataID InID) const;

	template<typename ValueType>
	const ValueType* TryGetData(FCameraContextDataID InID) const;

	template<typename ValueType>
	TConstArrayView<ValueType> TryGetArrayData(FCameraContextDataID InID) const;

	template<typename ValueType>
	bool TryGetArrayData(FCameraContextDataID InID, TConstArrayView<ValueType>& OutValues) const;

	template<typename ValueType>
	const ValueType* TryGetArrayData(FCameraContextDataID InID, int32 ArrayIndex) const;

	// Setter methods.

	void SetNameData(FCameraContextDataID InID, const FName& InData);
	void SetStringData(FCameraContextDataID InID, const FString& InData);
	void SetEnumData(FCameraContextDataID InID, const UEnum* EnumType, uint32 InData);
	void SetObjectData(FCameraContextDataID InID, UObject* InData);
	void SetClassData(FCameraContextDataID InID, UClass* InData);

	template<typename EnumType>
	void SetEnumData(FCameraContextDataID InID, EnumType InData);

	template<typename StructType>
	void SetStructData(FCameraContextDataID InID, const StructType& InData);

	void SetStructViewData(FCameraContextDataID InID, const FStructView& InData);
	void SetInstancedStructData(FCameraContextDataID InID, const FInstancedStruct& InData);

	void SetNameArrayData(FCameraContextDataID InID, TConstArrayView<FName> InData);
	void SetStringArrayData(FCameraContextDataID InID, TConstArrayView<FString> InData);
	void SetEnumArrayData(FCameraContextDataID InID, const UEnum* EnumType, TConstArrayView<uint32> InData);
	void SetObjectArrayData(FCameraContextDataID InID, TConstArrayView<UObject*> InData);
	void SetClassArrayData(FCameraContextDataID InID, TConstArrayView<UClass*> InData);

	template<typename EnumType>
	void SetEnumArrayData(FCameraContextDataID InID, TConstArrayView<EnumType> InData);

	template<typename StructType>
	void SetStructArrayData(FCameraContextDataID InID, TConstArrayView<StructType> InData);

	void SetStructViewArrayData(FCameraContextDataID InID, TConstArrayView<FStructView> InData);
	void SetInstancedStructArrayData(FCameraContextDataID InID, TConstArrayView<FInstancedStruct> InData);

public:

	// Overriding.

	void OverrideAll(const FCameraContextDataTable& OtherTable);
	void OverrideKnown(const FCameraContextDataTable& OtherTable);
	void Override(const FCameraContextDataTable& OtherTable, ECameraContextDataTableFilter Filter);

public:

	/** Collects referenced objects. */
	void AddReferencedObjects(FReferenceCollector& ReferenceCollector);

public:

	/** Type of data for array entries. */
	using FEntryScriptArray = FScriptArray;

	// Low-level API.
	UE_API const uint8* GetData(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			bool bOnlyIfWritten = true) const;

	UE_API const uint8* TryGetData(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			bool bOnlyIfWritten = true) const;

	UE_API const FEntryScriptArray* TryGetArrayData(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			bool bOnlyIfWritten = true) const;

	UE_API const uint8* TryGetRawDataPtr(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			bool bOnlyIfWritten = true) const;

	UE_API void SetData(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			const uint8* InRawDataPtr,
			bool bMarkAsWrittenThisFrame = true);

	UE_API bool TrySetData(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			const uint8* InRawDataPtr,
			bool bMarkAsWrittenThisFrame = true);

	UE_API bool TrySetArrayDataNum(
			FCameraContextDataID DataID, 
			int32 Count,
			bool bMarkAsWrittenThisFrame = true);

	UE_API bool TrySetArrayData(
			FCameraContextDataID DataID, 
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			int32 Index,
			const uint8* InRawDataPtr,
			bool bMarkAsWrittenThisFrame = true);

	UE_API uint8* TryGetMutableRawDataPtr(
			FCameraContextDataID DataID,
			ECameraContextDataType ExpectedDataType,
			const UObject* ExpectedDataTypeObject,
			bool bMarkAsWrittenThisFrame = true);

	bool IsValueWritten(FCameraContextDataID InID) const;
	void UnsetValue(FCameraContextDataID InID);
	void UnsetAllValues();

	bool IsValueWrittenThisFrame(FCameraContextDataID InID) const;
	void ClearAllWrittenThisFrameFlags();

	void AutoResetValues();

private:

	enum class EEntryFlags : uint8
	{
		None = 0,
		AutoReset = 1 << 1,
		Written = 1 << 2,
		WrittenThisFrame = 1 << 3
	};
	FRIEND_ENUM_CLASS_FLAGS(EEntryFlags)

	struct FEntry
	{
		FCameraContextDataID ID;
		ECameraContextDataType Type;
		ECameraContextDataContainerType ContainerType;
		TObjectPtr<const UObject> TypeObject;
		uint32 Offset;
		EEntryFlags Flags;
#if WITH_EDITORONLY_DATA
		FString DebugName;
#endif
	};

	struct FArrayEntryHelper
	{
		FArrayEntryHelper(const FEntry& Entry, uint8* TableMemory);
		FArrayEntryHelper(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* RawPtr);

		bool IsValidIndex(int32 Index) const;
		int32 Num() const;
		uint8* GetRawPtr(int32 Index);
		void Resize(int32 Count);

		ECameraContextDataType ElementType;
		TObjectPtr<const UObject> ElementTypeObject;

		uint32 ElementSizeOf;
		uint32 ElementAlignOf;

		FEntryScriptArray* ScriptArray;
	};

	static bool GetDataTypeAllocationInfo(ECameraContextDataType DataType, const UObject* DataTypeObject, uint32& OutSizeOf, uint32& OutAlignOf);
	static bool GetDataTypeAllocationInfo(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint32& OutSizeOf, uint32& OutAlignOf);

	static bool ConstructDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DataPtr);
	static bool ConstructDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DataPtr);
	static bool DestroyDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DataPtr);
	static bool DestroyDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DataPtr);

	static bool SetDataValue(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject, uint8* DestDataPtr, const uint8* SrcDataPtr);
	static bool SetDataValue(ECameraContextDataType DataType, const UObject* DataTypeObject, uint8* DestDataPtr, const uint8* SrcDataPtr);

	static void CreateEntry(const FCameraContextDataDefinition& DataDefinition, FEntry& OutEntry);

	const FEntry* FindEntry(FCameraContextDataID InID) const;
	FEntry* FindEntry(FCameraContextDataID InID);

	void InternalOverride(const FCameraContextDataTable& OtherTable, ECameraContextDataTableFilter Filter);

	void ReallocateBuffer(uint32 MinRequired = 0);
	void DestroyBuffer();

	template<typename StorageType>
	const StorageType* GetDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject) const;

	template<typename StorageType>
	bool SetDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject, const StorageType& InData);

	template<typename StorageType>
	TConstArrayView<StorageType> GetArrayDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject) const;

	template<typename StorageType>
	bool SetArrayDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject, TConstArrayView<StorageType> InData);

private:

	static const uint32 GDefaultCapacity;
	static const uint32 GDefaultAlignment;

	TArray<FEntry> Entries;
	TMap<FCameraContextDataID, int32> EntryLookup;

	uint8* Memory = nullptr;
	uint32 Capacity = 0;
	uint32 Alignment = 0;
	uint32 Used = 0;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	friend class FContextDataTableDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

ENUM_CLASS_FLAGS(FCameraContextDataTable::EEntryFlags)

template<typename EnumType>
EnumType FCameraContextDataTable::GetEnumData(FCameraContextDataID InID) const
{
	if (const uint32* Value = GetDataImpl<uint32>(InID, ECameraContextDataType::Enum, StaticEnum<EnumType>()))
	{
		return EnumType(*Value);
	}
	return EnumType();
}

template<typename StructType>
const StructType& FCameraContextDataTable::GetStructData(FCameraContextDataID InID) const
{
	if (const StructType* Value = GetDataImpl<StructType>(InID, ECameraContextDataType::Struct, StructType::StaticStruct()))
	{
		return *Value;
	}

	static const StructType DefaultValue;
	return DefaultValue;
}

template<typename ObjectClass>
ObjectClass* FCameraContextDataTable::GetObjectData(FCameraContextDataID InID) const
{
	if (const TObjectPtr<UObject>* Value = GetDataImpl<TObjectPtr<UObject>>(InID, ECameraContextDataType::Object, nullptr))
	{
		return Cast<ObjectClass>(Value->Get());
	}
	return nullptr;
}

template<typename BaseClass>
TSubclassOf<BaseClass> FCameraContextDataTable::GetClassData(FCameraContextDataID InID) const
{
	if (const TObjectPtr<UClass>* Value = GetDataImpl<TObjectPtr<UClass>>(InID, ECameraContextDataType::Class, nullptr))
	{
		return TSubclassOf<BaseClass>(Value->Get());
	}
	return nullptr;
}

template<typename ValueType>
const ValueType* FCameraContextDataTable::TryGetData(FCameraContextDataID InID) const
{
	// Can't return a direct pointer to an enum value because we coerce them to uint32 but ValueType may be smaller or larger.
	static_assert(!std::is_enum_v<ValueType>, "Enum types aren't supported with this method.");

	ECameraContextDataType DataType = TCameraContextDataTraits<ValueType>::GetDataType();
	const UObject* DataTypeObject = TCameraContextDataTraits<ValueType>::GetDataTypeObject();
	if (const uint8* RawValue = TryGetData(InID, DataType, DataTypeObject))
	{
		return reinterpret_cast<const ValueType*>(RawValue);
	}
	return nullptr;
}

template<typename ValueType>
TConstArrayView<ValueType> FCameraContextDataTable::TryGetArrayData(FCameraContextDataID InID) const
{
	TConstArrayView<ValueType> Values;
	if (TryGetArrayData(InID, Values))
	{
		return Values;
	}
	return TConstArrayView<ValueType>();
}

template<typename ValueType>
bool FCameraContextDataTable::TryGetArrayData(FCameraContextDataID InID, TConstArrayView<ValueType>& OutValues) const
{
	// Can't return a direct pointer to an array of enums because we coerce them to uint32 but ValueType may be smaller or larger.
	static_assert(!std::is_enum_v<ValueType>, "Enum types aren't supported with this method.");

	ECameraContextDataType DataType = TCameraContextDataTraits<ValueType>::GetDataType();
	const UObject* DataTypeObject = TCameraContextDataTraits<ValueType>::GetDataTypeObject();
	if (const FEntryScriptArray* Array = TryGetArrayData(InID, DataType, DataTypeObject))
	{
		const ValueType* ArrayData = reinterpret_cast<const ValueType*>(Array->GetData());
		OutValues = TConstArrayView<ValueType>(ArrayData, Array->Num());
		return true;
	}
	return false;
}

template<typename ValueType>
const ValueType* FCameraContextDataTable::TryGetArrayData(FCameraContextDataID InID, int32 ArrayIndex) const
{
	TConstArrayView<ValueType> TypedArrayView = TryGetArrayData<ValueType>(InID);
	if (TypedArrayView.IsValidIndex(ArrayIndex))
	{
		return &TypedArrayView[ArrayIndex];
	}
	return nullptr;
}

template<typename EnumType>
void FCameraContextDataTable::SetEnumData(FCameraContextDataID InID, EnumType InData)
{
	SetDataImpl(InID, ECameraContextDataType::Enum, StaticEnum<EnumType>(), (uint32)InData);
}

template<typename StructType>
void FCameraContextDataTable::SetStructData(FCameraContextDataID InID, const StructType& InData)
{
	SetDataImpl(InID, ECameraContextDataType::Struct, StructType::StaticStruct(), InData);
}

template<typename EnumType>
void FCameraContextDataTable::SetEnumArrayData(FCameraContextDataID InID, TConstArrayView<EnumType> InData)
{
	TArray<uint32> CoercedData;
	for (EnumType Item : InData)
	{
		CoercedData.Add((uint32)Item);
	}
	SetArrayDataImpl(InID, ECameraContextDataType::Enum, StaticEnum<EnumType>(), CoercedData);
}

template<typename StructType>
void FCameraContextDataTable::SetStructArrayData(FCameraContextDataID InID, TConstArrayView<StructType> InData)
{
	SetArrayDataImpl(InID, ECameraContextDataType::Struct, StructType::StaticStruct(), InData);
}

template<typename StorageType>
const StorageType* FCameraContextDataTable::GetDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject) const
{
	const FEntry* Entry = FindEntry(InID);
	if (Entry && 
			Entry->Type == DataType && 
			Entry->ContainerType == ECameraContextDataContainerType::None && 
			Entry->TypeObject == DataTypeObject &&
			EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written))
	{
		const uint8* RawData = Memory + Entry->Offset;
		return reinterpret_cast<const StorageType*>(RawData);
	}
	return nullptr;
}

template<typename StorageType>
bool FCameraContextDataTable::SetDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject, const StorageType& InData)
{
	FEntry* Entry = FindEntry(InID);
	if (Entry && 
			Entry->Type == DataType && 
			Entry->ContainerType == ECameraContextDataContainerType::None && 
			Entry->TypeObject == DataTypeObject)
	{
		uint8* RawData = Memory + Entry->Offset;
		*reinterpret_cast<StorageType*>(RawData) = InData;
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
		return true;
	}
	return false;
}

template<typename StorageType>
TConstArrayView<StorageType> FCameraContextDataTable::GetArrayDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject) const
{
	const FEntry* Entry = FindEntry(InID);
	if (Entry && 
			Entry->Type == DataType && 
			Entry->ContainerType == ECameraContextDataContainerType::Array && 
			Entry->TypeObject == DataTypeObject &&
			EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written))
	{
		FEntryScriptArray* Array = (FEntryScriptArray*)(Memory + Entry->Offset);
		const StorageType* ArrayData = reinterpret_cast<const StorageType*>(Array->GetData());
		return TConstArrayView<StorageType>(ArrayData, Array->Num());
	}
	return TConstArrayView<StorageType>();
}

template<typename StorageType>
bool FCameraContextDataTable::SetArrayDataImpl(FCameraContextDataID InID, ECameraContextDataType DataType, const UObject* DataTypeObject, TConstArrayView<StorageType> InData)
{
	FEntry* Entry = FindEntry(InID);
	if (Entry && 
			Entry->Type == DataType && 
			Entry->ContainerType == ECameraContextDataContainerType::Array && 
			Entry->TypeObject == DataTypeObject)
	{
		FArrayEntryHelper Helper(*Entry, Memory);
		Helper.Resize(InData.Num());
		for (int32 Index = 0; Index < InData.Num(); ++Index)
		{
			uint8* RawData = Helper.GetRawPtr(Index);
			*reinterpret_cast<StorageType*>(RawData) = InData[Index];
		}
		EnumAddFlags(Entry->Flags, EEntryFlags::Written | EEntryFlags::WrittenThisFrame);
		return true;
	}
	return false;
}

template<typename DataType>
ECameraContextDataType TCameraContextDataTraits<DataType>::GetDataType()
{
	if constexpr(std::is_same_v<DataType, FName>)
	{
		return ECameraContextDataType::Name;
	}
	else if constexpr(std::is_same_v<DataType, FString>)
	{
		return ECameraContextDataType::String;
	}
	else if constexpr(std::is_enum_v<DataType>)
	{
		return ECameraContextDataType::Enum;
	}
	else if constexpr(TPointerIsConvertibleFromTo<DataType, UClass>::Value)
	{
		return ECameraContextDataType::Class;
	}
	else if constexpr(TPointerIsConvertibleFromTo<DataType, UObject>::Value)
	{
		return ECameraContextDataType::Object;
	}
	else if constexpr(std::is_same_v<decltype(DataType::StaticStruct()), UScriptStruct*>)
	{
		return ECameraContextDataType::Struct;
	}
	else
	{
		// Before C++23 we have to make the static assert dependent on the template argument,
		// so we can't just do static_assert(false).
		// The trick here (stolen from Raymond Chen's blog) is to rely on the fact that sizeof()
		// is never zero, and pointers don't need complete types, so !sizeof(DataType*) will
		// always be false and will satisfy the compiler.
		static_assert(!sizeof(DataType*), "DataType must be FName, FString, an enum, or a UClass, UObject, or UStruct type.");
	}
};

template<typename DataType>
const UObject* TCameraContextDataTraits<DataType>::GetDataTypeObject()
{
	if constexpr(std::is_enum_v<DataType>)
	{
		return StaticEnum<DataType>();
	}
	else if constexpr(std::is_same_v<decltype(DataType::StaticStruct()), UScriptStruct*>)
	{
		return DataType::StaticStruct();
	}
	else
	{
		return nullptr;
	}
}

}  // namespace UE::Cameras

#undef UE_API 