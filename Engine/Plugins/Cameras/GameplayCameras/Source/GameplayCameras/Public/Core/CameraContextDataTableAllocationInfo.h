// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#include "CameraContextDataTableAllocationInfo.generated.h"

class UCameraNode;

/**
 * Definition for one entry in a camera rig's context data registry.
 */
USTRUCT()
struct FCameraContextDataDefinition
{
	GENERATED_BODY()

	/** The ID of data. */
	UPROPERTY()
	FCameraContextDataID DataID;

	/** The type of the data. */
	UPROPERTY()
	ECameraContextDataType DataType = ECameraContextDataType::Name;

	/** The type of container for the data. */
	UPROPERTY()
	ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;

	/** An extra type object for the data. */
	UPROPERTY()
	TObjectPtr<const UObject> DataTypeObject;

	/** Whether the data should auto-reset to an "unset" state after every evaluation. */
	UPROPERTY()
	bool bAutoReset = false;

#if WITH_EDITORONLY_DATA
	/** The name of the data, for debugging purposes. */
	UPROPERTY()
	FString DataName;
#endif

	bool operator==(const FCameraContextDataDefinition& Other) const = default;
};

/**
 * Collection of context data entries for a camera rig.
 */
USTRUCT()
struct FCameraContextDataTableAllocationInfo
{
	GENERATED_BODY()

	/** The list of context data definitions. */
	UPROPERTY()
	TArray<FCameraContextDataDefinition> DataDefinitions;

	/**Combines the given allocation info with this one. */
	GAMEPLAYCAMERAS_API void Combine(const FCameraContextDataTableAllocationInfo& OtherInfo);

	/** Returns whether the given data definitions already exist in our list. */
	GAMEPLAYCAMERAS_API bool Contains(const FCameraContextDataTableAllocationInfo& OtherInfo) const;

	bool operator==(const FCameraContextDataTableAllocationInfo& Other) const = default;
};

