// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayCameras.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#include "CameraContextDataTableFwd.generated.h"

namespace UE::Cameras
{ 
	class FCameraContextDataTable;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	class FContextDataTableDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
}

/**
 * Supported types for a camera node's context data.
 * 
 * NOTE: simple types (bool, integer, float, etc.) and vector types (vector, rotator, transform)
 * are not supported as context data because they are supported as blendable parameters.
 */
UENUM()
enum class ECameraContextDataType
{
	Name,
	String,
	Enum UMETA(Hidden),
	Struct UMETA(Hidden),
	Object UMETA(Hidden),
	Class UMETA(Hidden),

	Count UMETA(Hidden)
};

/**
 * Supported container types for a camera node's context data.
 */
UENUM()
enum class ECameraContextDataContainerType
{
	None,
	Array
};

/**
 * The ID of a context data, used to refer to it in a camera context data table.
 */
USTRUCT(BlueprintType)
struct FCameraContextDataID
{
	GENERATED_BODY()

public:

	FCameraContextDataID() : Value(INVALID) {}

	uint32 GetValue() const { return Value; }

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

	static FCameraContextDataID FromHashValue(uint32 InValue)
	{
		return FCameraContextDataID(InValue);
	}

public:

	friend bool operator<(FCameraContextDataID A, FCameraContextDataID B)
	{
		return A.Value < B.Value;
	}

	friend bool operator==(FCameraContextDataID A, FCameraContextDataID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCameraContextDataID A, FCameraContextDataID B)
	{
		return A.Value != B.Value;
	}

	friend uint32 GetTypeHash(FCameraContextDataID In)
	{
		return In.Value;
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraContextDataID& In)
	{
		Ar << In.Value;
		return Ar;
	}

private:

	FCameraContextDataID(uint32 InValue) : Value(InValue) {}

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;
};

