// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraRigProxyRedirectTable.h"
#include "UObject/AssetRegistryTagsContext.h"

#include "CameraDirector.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraRigAsset;
class UCameraRigProxy;

namespace UE::Cameras { struct FCameraBuildContext; }

#if WITH_EDITOR

/**
 * Parameter struct passed by an asset factory when a new camera asset is created.
 * This lets a camera director setup data before the editor opens.
 */
struct FCameraDirectorFactoryCreateParams
{
};

#endif  // WITH_EDITOR

/**
 * Parameter struct for gathering camera rigs used by a director.
 */
struct FCameraDirectorRigUsageInfo
{
	/** Camera rigs used by the camera director. */
	TArray<UCameraRigAsset*> CameraRigs;
	/** Camera rig proxies used by the camera director. */
	TArray<UCameraRigProxyAsset*> CameraRigProxies;
};

/**
 * Base class for a camera director.
 */
UCLASS(MinimalAPI, Abstract, DefaultToInstanced)
class UCameraDirector : public UObject
{
	GENERATED_BODY()

public:

	using FCameraDirectorEvaluatorBuilder = UE::Cameras::FCameraDirectorEvaluatorBuilder;

	/** Build the evaluator for this director. */
	UE_API FCameraDirectorEvaluatorPtr BuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const;

	/** Builds and validates this camera director. */
	UE_API void BuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext);

	/** Gets the list of camera rigs used by this camera director. */
	UE_API void GatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const;

	/** Extend the owning camera asset's tags. */
	UE_API void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const;

#if WITH_EDITOR
	/** Called by the asset factories to setup new data before the editor opens. */
	UE_API void FactoryCreateAsset(const FCameraDirectorFactoryCreateParams& InParams);
#endif

public:

	// UObject interface.
	UE_API virtual void PostLoad() override;

protected:

	/** Build the evaluator for this director. */
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const { return nullptr; }

	/** Builds and validates this camera director. */
	virtual void OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext) {}

	/** Gets the list of camera rigs used by this camera director. */
	virtual void OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const {}

	/** Extend the owning camera asset's tags. */
	virtual void OnExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const {}

#if WITH_EDITOR
	/** Called by the asset factories to setup new data before the editor opens. */
	virtual void OnFactoryCreateAsset(const FCameraDirectorFactoryCreateParams& InParams) {}
#endif

public:

	/** 
	 * The table that maps camera rig proxies (used in the evaluator Blueprint graph)
	 * to actual camera rigs.
	 */
	UPROPERTY(EditAnywhere, Category="Evaluation")
	FCameraRigProxyRedirectTable CameraRigProxyRedirectTable;

	/**
	 * Whether this camera director should run in the editor. 
	 */
	UPROPERTY(EditAnywhere, Category="Evaluation")
	bool bRunInEditor = false;

private:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TObjectPtr<UCameraRigProxyTable> CameraRigProxyTable_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

#undef UE_API
