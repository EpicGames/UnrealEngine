// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigInstanceID.h"
#include "Core/CameraObjectRtti.h"
#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"

class UCameraDirector;
class UCameraRigAsset;
class UCameraRigProxyAsset;
class UCameraRigTransition;

namespace UE::Cameras
{

class FCameraDirectorEvaluator;
class FCameraDirectorEvaluatorStorage;
class FCameraEvaluationContext;
class FCameraSystemEvaluator;

/**
 * Parameter structure for initializing a newly created camera director evaluator.
 */
struct FCameraDirectorInitializeParams
{
	/** The evaluation context that owns the camera director. */
	TSharedPtr<FCameraEvaluationContext> OwnerContext;
};

/**
 * Parameter structure for activating a camera director evaluator.
 * A camera director evaluator can be activated and deactivated multiple times.
 * This means that OnActivated/OnDeactivated can be called multiple times in pairs,
 * unlike OnInitialize which is only called once.
 */
struct FCameraDirectorActivateParams
{
	/** The camera system that will run the camera director. */
	FCameraSystemEvaluator* Evaluator = nullptr;
};

/**
 * Parameter structure for deactivating a camera director evaluator.
 */
struct FCameraDirectorDeactivateParams
{
};

/**
 * Parameter structure for running a camera director.
 */
struct FCameraDirectorEvaluationParams
{
	/** Time interval for the update. */
	float DeltaTime = 0.f;
};

/** The type of request for activating or deactivating a camera rig. */
enum class ECameraRigActivationDeactivationRequestType
{
	Activate,
	Deactivate
};

/**
 * Structure for requesting that a camera rig be activated or deactivated.
 */
struct FCameraRigActivationDeactivationRequest
{
	/** The evaluation context to run the specified camera rig. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;
	/** The camera rig that should be running. */
	TObjectPtr<const UCameraRigAsset> CameraRig;
	/** The camera proxy that determines the rig that should be running (if CameraRig is null). */
	TObjectPtr<const UCameraRigProxyAsset> CameraRigProxy;
	/** The type of the request. */
	ECameraRigActivationDeactivationRequestType RequestType = ECameraRigActivationDeactivationRequestType::Activate;
	/** The layer on which to activate or deactivate this camera rig. */
	ECameraRigLayer Layer = ECameraRigLayer::Main;
	/** A transition to use for this activation, if possible, instead of looking up one according to usual rules. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;
	/** An order key for activating camera rigs on ordered layers. */
	int32 OrderKey = 0;
	/** Whether to force creating a new instance, even if an identical one is already active. */
	bool bForceActivateDeactivate = false;

public:

	FCameraRigActivationDeactivationRequest()
	{}
	FCameraRigActivationDeactivationRequest(
			TSharedPtr<const FCameraEvaluationContext> InContext, 
			TObjectPtr<const UCameraRigAsset> InCameraRig)
		: EvaluationContext(InContext)
		, CameraRig(InCameraRig)
	{}
	FCameraRigActivationDeactivationRequest(
			TSharedPtr<const FCameraEvaluationContext> InContext, 
			TObjectPtr<const UCameraRigProxyAsset> InCameraRigProxy)
		: EvaluationContext(InContext)
		, CameraRigProxy(InCameraRigProxy)
	{}

	bool IsValid() const
	{
		return EvaluationContext != nullptr && CameraRig != nullptr;
	}

public:

	bool ResolveCameraRigProxyIfNeeded(const FCameraDirectorEvaluator* InDirectorEvaluator);
	bool ResolveCameraRigProxyIfNeeded(const UCameraDirector* InDirector);
};

/**
 * Result structure for running a camera director.
 */
struct FCameraDirectorEvaluationResult
{
	using FRequests = TArray<FCameraRigActivationDeactivationRequest, TInlineAllocator<4>>;

	/** Requests for activating/deactivating camera rigs this frame. */
	FRequests Requests;

	/** Adds a simple activation request for the main layer. */
	void Add(
			TSharedPtr<const FCameraEvaluationContext> InEvaluationContext,
			TObjectPtr<const UCameraRigAsset> InCameraRig)
	{
		Requests.Add({ InEvaluationContext, InCameraRig });
	}

	/** Adds a simple activation request for the main layer. */
	void Add(
			TSharedPtr<const FCameraEvaluationContext> InEvaluationContext,
			TObjectPtr<const UCameraRigProxyAsset> InCameraRigProxy)
	{
		Requests.Add({ InEvaluationContext, InCameraRigProxy });
	}

	/** Adds an activation request for the specified layer */
	void Add(
			TSharedPtr<const FCameraEvaluationContext> InEvaluationContext,
			TObjectPtr<const UCameraRigAsset> InCameraRig,
			ECameraRigLayer Layer, 
			int32 OrderKey)
	{
		FCameraRigActivationDeactivationRequest& Request = Requests.Add_GetRef({ InEvaluationContext, InCameraRig });
		Request.Layer = Layer;
		Request.OrderKey = OrderKey;
	}

	/** Adds a deactivation request for the specified layer */
	void Remove(
				TSharedPtr<const FCameraEvaluationContext> InEvaluationContext, 
				TObjectPtr<const UCameraRigAsset> InCameraRig,
				ECameraRigLayer Layer)
	{
		FCameraRigActivationDeactivationRequest& Request = Requests.Add_GetRef({ InEvaluationContext, InCameraRig });
		Request.Layer = Layer;
		Request.RequestType = ECameraRigActivationDeactivationRequestType::Deactivate;
	}

	/** Reset this result. */
	void Reset()
	{
		Requests.Reset();
	}
};

/**
 * Structure for building director evaluators.
 */
struct FCameraDirectorEvaluatorBuilder
{
	FCameraDirectorEvaluatorBuilder(FCameraDirectorEvaluatorStorage& InStorage)
		: Storage(InStorage)
	{}

	/** Builds a director evaluator of the given type. */
	template<typename EvaluatorType, typename ...ArgTypes>
	EvaluatorType* BuildEvaluator(ArgTypes&&... InArgs);

private:

	FCameraDirectorEvaluatorStorage& Storage;
};

/**
 * Storage for a director evaluator.
 */
class FCameraDirectorEvaluatorStorage
{
public:

	/** Gets the stored evaluator, if any. */
	FCameraDirectorEvaluator* GetEvaluator() const { return Evaluator.Get(); }

	template<typename EvaluatorType, typename ...ArgTypes>
	EvaluatorType* BuildEvaluator(ArgTypes&&... InArgs);

	void DestroyEvaluator();

private:

	TSharedPtr<FCameraDirectorEvaluator> Evaluator;

	friend struct FCameraDirectorEvaluatorBuilder;
};

/**
 * Base class for camera director evaluators.
 */
class FCameraDirectorEvaluator
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraDirectorEvaluator)

public:

	GAMEPLAYCAMERAS_API FCameraDirectorEvaluator();
	virtual ~FCameraDirectorEvaluator() = default;

	/** Initializes a camera director evalutor. */
	GAMEPLAYCAMERAS_API void Initialize(const FCameraDirectorInitializeParams& Params);

	/** Activates the camera director evaluator. */
	GAMEPLAYCAMERAS_API void Activate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult);

	/** Deactivates the camera director evaluator. */
	GAMEPLAYCAMERAS_API void Deactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult);

	/** Gets the camera director. */
	const UCameraDirector* GetCameraDirector() const { return PrivateCameraDirector; }

	/** Gets the camera director. */
	template<typename CameraDirectorType>
	const CameraDirectorType* GetCameraDirectorAs() const
	{
		return Cast<CameraDirectorType>(PrivateCameraDirector);
	}

	/** Gets the owning evaluation context. */
	TSharedPtr<FCameraEvaluationContext> GetEvaluationContext() const { return WeakOwnerContext.Pin(); }

public:

	/** Runs the camera director to determine what camera rig(s) should be active this frame. */
	GAMEPLAYCAMERAS_API void Run(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult);

	/** Request that the next camera rig activation use the provided transition. */
	GAMEPLAYCAMERAS_API void OverrideNextActivationTransition(const UCameraRigTransition* TransitionOverride);

	/** Request that the next camera rig activation be forced. */
	GAMEPLAYCAMERAS_API void ForceNextActivation();

public:

	/** Add a child context to this camera director. */
	GAMEPLAYCAMERAS_API bool AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext);

	/** Remove the given child context from this camera director. */
	GAMEPLAYCAMERAS_API bool RemoveChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext);

	/** Garbage collection pass. */
	GAMEPLAYCAMERAS_API void AddReferencedObjects(FReferenceCollector& Collector);

public:

	// Internal API.

	void SetPrivateCameraDirector(const UCameraDirector* InCameraDirector);

	const UCameraRigAsset* FindCameraRigByProxy(const UCameraRigProxyAsset* InProxy) const;

	void OnEndCameraSystemUpdate();

protected:

	/** Result for adding/removing children contexts. */
	enum class EChildContextManipulationResult
	{
		/** The add or removal failed. */
		Failure,
		/** The add or removal was successful. */
		Success,
		/** The add or removal was successfully handled by a child director. */
		ChildContextSuccess
	};

	/** Parameter struct for adding/removing children contexts. */
	struct FChildContextManulationParams
	{
		TSharedPtr<FCameraEvaluationContext> ParentContext;
		TSharedPtr<FCameraEvaluationContext> ChildContext;
	};

	/** Result struct for adding/removing children contexts. */
	struct FChildContextManulationResult
	{
		EChildContextManipulationResult Result = EChildContextManipulationResult::Failure;
#if UE_GAMEPLAY_CAMERAS_DEBUG
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
	};

	/** Initializes a camera director evalutor. Only called once after construction. */
	virtual void OnInitialize(const FCameraDirectorInitializeParams& Params) {}

	/** Activates the camera director evaluator. May be called multiple times, in pair with OnDeactivate. */
	virtual void OnActivate(const FCameraDirectorActivateParams& Params, FCameraDirectorEvaluationResult& OutResult) {}

	/** Deactivates the camera director evaluator. May be called multiple times, in pair with OnActivate. */
	virtual void OnDeactivate(const FCameraDirectorDeactivateParams& Params, FCameraDirectorEvaluationResult& OutResult) {}

	/** Runs the camera director to determine what camera rig(s) should be active this frame. */
	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) {}

	/** Add a child context to this camera director. */
	virtual void OnAddChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) {}

	/** Remove the given child context from this camera director. */
	virtual void OnRemoveChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) {}

	/** Garbage collection pass. */
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) {}

private:

	/** The camera system this evaluator is running inside of. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** The evaluation context that owns this evaluator. */
	TWeakPtr<FCameraEvaluationContext> WeakOwnerContext;

	/** The camera director this evaluator is running. */
	TObjectPtr<const UCameraDirector> PrivateCameraDirector;

	/** A forced transition to use on the next update. Cleared after every system update. */
	TObjectPtr<const UCameraRigTransition> NextActivationTransitionOverride;

	/** Whether to force camera rig activation on the next update. Cleared after every system update. */
	bool bNextActivationForce = false;
};

template<typename EvaluatorType, typename ...ArgTypes>
EvaluatorType* FCameraDirectorEvaluatorBuilder::BuildEvaluator(ArgTypes&&... InArgs)
{
	return Storage.BuildEvaluator<EvaluatorType>(Forward<ArgTypes>(InArgs)...);
}

template<typename EvaluatorType, typename ...ArgTypes>
EvaluatorType* FCameraDirectorEvaluatorStorage::BuildEvaluator(ArgTypes&&... InArgs)
{
	// We should only build one evaluator.
	ensure(Evaluator == nullptr);
	Evaluator = MakeShared<EvaluatorType>(Forward<ArgTypes>(InArgs)...);
	return Evaluator->CastThisChecked<EvaluatorType>();
}

}  // namespace UE::Cameras

// Typedef to avoid having to deal with namespaces in UCameraNode subclasses.
using FCameraDirectorEvaluatorPtr = UE::Cameras::FCameraDirectorEvaluator*;

// Utility macros for declaring and defining camera director evaluators.
//
#define UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, ::UE::Cameras::FCameraDirectorEvaluator)

#define UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_DIRECTOR_EVALUATOR(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

