// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraObjectRtti.h"
#include "Templates/SharedPointer.h"

#include "CameraEvaluationContext.generated.h"

class APlayerController;
class UCameraAsset;
class UCameraDirector;

UENUM(BlueprintType)
enum class ECameraEvaluationDataCondition : uint8
{
	ActiveCameraRig
};

namespace UE::Cameras
{

class FCameraSystemEvaluator;
struct FCameraDirectorEvaluationResult;

/**
 * Parameter struct for initializing an evaluation context.
 */
struct FCameraEvaluationContextInitializeParams
{
	/** The owner of the context, if any. */
	TObjectPtr<UObject> Owner;
	/** The camera asset to run inside the context, if any. */
	TObjectPtr<const UCameraAsset> CameraAsset;
	/** The player controller driving this context, if any. */
	TObjectPtr<APlayerController> PlayerController;
};

/**
 * Parameter struct for activating an evaluation context, which happens when it is added
 * to the camera system's context stack.
 */
struct FCameraEvaluationContextActivateParams
{
	/** The system the context will evaluate inside of. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** The parent context, if any. */
	TSharedPtr<FCameraEvaluationContext> ParentContext;
};

/**
 * Parameter struct for deactivating an evaluation context, which happens when it is removed
 * from the camera system's context stack.
 */
struct FCameraEvaluationContextDeactivateParams
{
};

/**
 * Base class for providing a context to running camera rigs.
 */
class FCameraEvaluationContext : public TSharedFromThis<FCameraEvaluationContext>
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraEvaluationContext)

public:

	/** Constructs an evaluation context. */
	GAMEPLAYCAMERAS_API FCameraEvaluationContext();
	/** Constructs an evaluation context. */
	GAMEPLAYCAMERAS_API FCameraEvaluationContext(const FCameraEvaluationContextInitializeParams& Params);

	/** Initializes the evaluation context, if it was created with the default constructor. */
	GAMEPLAYCAMERAS_API void Initialize(const FCameraEvaluationContextInitializeParams& Params);

	/** Destroys this evaluation context. */
	GAMEPLAYCAMERAS_API virtual ~FCameraEvaluationContext();

	/** Gets the owner of this evaluation context, if any, and if still valid. */
	UObject* GetOwner() const { return WeakOwner.Get(); }

	/** 
	 * Gets the world in which this evaluation context runs. 
	 * This is the owner's world.
	 */
	UWorld* GetWorld() const;

	/**
	 * Gets the player controller (if any) in control of the cameras running inside
	 * of this evaluation context.
	 */
	APlayerController* GetPlayerController() const { return WeakPlayerController.Get(); }

	/** Gets the camera asset that is hosted in this context. */
	const UCameraAsset* GetCameraAsset() const { return CameraAsset; }

	/** Gets the initial evaluation result for all camera rigs in this context. */
	const FCameraNodeEvaluationResult& GetInitialResult() const { return InitialResult; }

	/** Gets the initial evaluation result for all camera rigs in this context. */
	FCameraNodeEvaluationResult& GetInitialResult() { return InitialResult; }

	/** Gets the camera director evaluator that runs the camera rigs of this context. */
	FCameraDirectorEvaluator* GetDirectorEvaluator() const { return DirectorEvaluator; }

	/** Gets the parent evaluation context, if any. */
	TSharedPtr<const FCameraEvaluationContext> GetParentContext() const { return WeakParent.Pin(); }

	/** Gets the parent evaluation context, if any. */
	TSharedPtr<FCameraEvaluationContext> GetParentContext() { return WeakParent.Pin(); }

	/** Gets the children evaluation contexts running inside this context. */
	TArrayView<const TSharedPtr<FCameraEvaluationContext>> GetChildrenContexts() const { return ChildrenContexts; }

	/** Gets the size of the viewport that cameras inside the context are being viewed through. */
	FIntPoint GetViewportSize() const;

public:

	/** Gets an evaluation result that will be conditionally overlaid on some camera rigs in this context. */
	const FCameraNodeEvaluationResult* GetConditionalResult(ECameraEvaluationDataCondition Condition) const { return ConditionalResults.Find(Condition); }

	/** Gets an evaluation result that will be conditionally overlaid on some camera rigs in this context. */
	FCameraNodeEvaluationResult* GetConditionalResult(ECameraEvaluationDataCondition Condition) { return ConditionalResults.Find(Condition); }

	/** Gets or adds an evaluation result that will be conditionally overlaid on some camera rigs in this context. */
	GAMEPLAYCAMERAS_API FCameraNodeEvaluationResult& GetOrAddConditionalResult(ECameraEvaluationDataCondition Condition);

public:

	/**
	 * Activates this evaluation context.
	 * This will create the camera director evaluator if necessary.
	 */
	void Activate(const FCameraEvaluationContextActivateParams& Params);

	/**
	 * Deactivates this evaluation context.
	 */
	void Deactivate(const FCameraEvaluationContextDeactivateParams& Params);

	/** Whether this context is active. */
	bool IsActive() const { return bActivated; }

	/**
	 * Returns the camera system for which this context was activated, or null if this
	 * context is currently not active.
	 */
	TSharedPtr<FCameraSystemEvaluator> GetCameraSystemEvaluator() const;

public:

	/** Adds a child context. */
	bool AddChildContext(TSharedRef<FCameraEvaluationContext> ChildContext);

	/** Removes a child context. */
	bool RemoveChildContext(TSharedRef<FCameraEvaluationContext> ChildContext);

public:

	// Internal API.
	void AddReferencedObjects(FReferenceCollector& Collector);

	void OnEndCameraSystemUpdate();

	bool RegisterChildContext(TSharedRef<FCameraEvaluationContext> ChildContext);
	bool UnregisterChildContext(TSharedRef<FCameraEvaluationContext> ChildContext);

#if WITH_EDITOR
	void SetEditorPreviewCameraRigIndex(int32 Index);
#endif  // WITH_EDITOR

protected:

	virtual void OnActivate(const FCameraEvaluationContextActivateParams& Params) {}
	virtual void OnDeactivate(const FCameraEvaluationContextDeactivateParams& Params) {}
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) {}

	FCameraSystemEvaluator* GetPrivateCameraSystemEvaluator() const { return CameraSystemEvaluator; }

	void AutoCreateDirectorEvaluator();

#if WITH_EDITOR
	void AutoCreateEditorPreviewDirectorEvaluator(const FCameraEvaluationContextActivateParams& Params);
#endif  // WITH_EDITOR

private:

	void ExecuteSetupAndTeardownRequests(FCameraDirectorEvaluationResult& DirectorResult);

protected:

	/** The owner of this context, if any. */
	TWeakObjectPtr<> WeakOwner;

	/**
	 * The player controller (if any) in control of the cameras running inside
	 * of this evaluation context.
	 */
	TWeakObjectPtr<APlayerController> WeakPlayerController;

	/** The camera asset hosted in this context. */
	TObjectPtr<const UCameraAsset> CameraAsset;

	/** The initial result for all camera rigs in this context. */
	FCameraNodeEvaluationResult InitialResult;

	/** An optional viewport size, if there is no PlayerController, or its viewport should not be used. */
	TOptional<FIntPoint> OverrideViewportSize;

private:

	/** The camera director evaluator, usually only valid if this context has been activated. */
	FCameraDirectorEvaluatorStorage DirectorEvaluatorStorage;
	FCameraDirectorEvaluator* DirectorEvaluator = nullptr;

	/** The camera system that this context has been activated for, or null if the context isn't currently active. */
	FCameraSystemEvaluator* CameraSystemEvaluator = nullptr;

	/** The parent context, if any. */
	TWeakPtr<FCameraEvaluationContext> WeakParent;

	/** The children contexts, if any. */
	using FChildrenContexts = TArray<TSharedPtr<FCameraEvaluationContext>>;
	FChildrenContexts ChildrenContexts;

	/** Conditional camera data for camera rigs running inside this context. */
	using FConditionalResults = TMap<ECameraEvaluationDataCondition, FCameraNodeEvaluationResult>;
	FConditionalResults ConditionalResults;

	/** Whether this context was initialized. */
	bool bInitialized = false;

	/** Whether this context is active. */
	bool bActivated = false;
};

}  // namespace UE::Cameras

// Utility macros for declaring and defining camera evaluation contexts.
//
#define UE_DECLARE_CAMERA_EVALUATION_CONTEXT(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, ::UE::Cameras::FCameraEvaluationContext)

#define UE_DECLARE_CAMERA_EVALUATION_CONTEXT_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_EVALUATION_CONTEXT(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

