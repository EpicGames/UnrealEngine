// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "GameplayCameras.h"
#include "Templates/Invoke.h"
#include "UObject/WeakObjectPtr.h"

class UCameraAsset;
class UCameraDirector;

namespace UE::Cameras
{

class FCameraDirectorEvaluator;
class FCameraEvaluationContext;
class FCameraSystemEvaluator;

/**
 * A simple stack of evaluation contexts. The top one is the active one.
 */
struct FCameraEvaluationContextStack
{
public:

	~FCameraEvaluationContextStack();

public:

	/** Gets the active (top) context. */
	TSharedPtr<FCameraEvaluationContext> GetActiveContext() const;

	/** Returns whether the given context exists in the stack. */
	bool HasContext(TSharedRef<FCameraEvaluationContext> Context) const;

	/** Push a new context on the stack and instantiate its director. */
	void PushContext(TSharedRef<FCameraEvaluationContext> Context);

	/**
	 * Tries to add a context inside the given context. This will query the active context's
	 * director in order to find an "available spot" for the child context.
	 * 
	 * @param Context  The new context to activate.
	 * @param ParentContext  The parent context to activate inside of, or the active context (if any) if null.
	 * @return Whether the child context was acccepted.
	 */
	bool AddChildContext(TSharedRef<FCameraEvaluationContext> Context, TSharedPtr<FCameraEvaluationContext> ParentContext = nullptr);

	/** Remove an existing context from the stack. */
	bool RemoveContext(TSharedRef<FCameraEvaluationContext> Context);

	/** Remove contexts owned by the given object. */
	bool RemoveContextsOwnedBy(UObject* ContextOwner, bool bAlsoRemoveInnerOwners = false);

	/** Pop the active (top) context. */
	void PopContext();

	/** The number of contexts on the stack. */
	int32 NumContexts() const { return Entries.Num(); }

	/** Gets all the contexts in the stack, from bottom to top. */
	void GetAllContexts(TArray<TSharedPtr<FCameraEvaluationContext>>& OutContexts) const;

	/** Finds a context owned by the given object, if any. */
	TSharedPtr<FCameraEvaluationContext> FindContextByOwner(const UObject* InOwner) const;

	/**
	 * Finds a context that fulfills the given predicate.
	 *
	 * The predicate function must have the signature: (TSharedRef<FCameraEvaluationContext>) -> bool
	 */
	template<typename Predicate>
	TSharedPtr<FCameraEvaluationContext> FindContextByPredicate(Predicate Pred) const;

	/** Empties the stack of all contexts. */
	void Reset();

	/** An event raised when the stack changes. */ 
	FSimpleMulticastDelegate& OnStackChanged() { return OnStackChangedEvent; }

public:

	// Internal API
	void Initialize(FCameraSystemEvaluator& InEvaluator);
	void AddReferencedObjects(FReferenceCollector& Collector);
	void OnEndCameraSystemUpdate();

private:

	struct FContextEntry
	{
		TWeakPtr<FCameraEvaluationContext> WeakContext;
	};

	/** The entries in the stack. */
	TArray<FContextEntry> Entries;

	/** The owner evaluator. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** An event raised when the stack changes. */ 
	FSimpleMulticastDelegate OnStackChangedEvent;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	friend class FCameraDirectorTreeDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

template<typename Predicate>
TSharedPtr<FCameraEvaluationContext> FCameraEvaluationContextStack::FindContextByPredicate(Predicate Pred) const
{
	for (const FContextEntry& Entry : Entries)
	{
		if (TSharedPtr<FCameraEvaluationContext> Context = Entry.WeakContext.Pin())
		{
			if (::Invoke(Pred, Context.ToSharedRef()))
			{
				return Context;
			}
		}
	}
	return TSharedPtr<FCameraEvaluationContext>();
}

}  // namespace UE::Cameras

