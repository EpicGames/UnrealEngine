// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraObjectRtti.h"
#include "Debug/CameraDebugBlockFwd.h"
#include "GameplayCameras.h"
#include "Templates/SharedPointer.h"

class FReferenceCollector;

namespace UE::Cameras
{

class FCameraSystemEvaluator;
struct FCameraNodeEvaluationResult;
struct FRootCameraNodeCameraRigEvent;

/** Flags for what callbacks an evaluation service wants to opt-into. */
enum class ECameraEvaluationServiceFlags
{
	None = 0,
	NeedsPreUpdate = 1 << 0,
	NeedsPostCameraDirectorUpdate = 1 << 1,
	NeedsPostUpdate = 1 << 2,
	NeedsRootCameraNodeEvents = 1 << 3,

	Default = NeedsPreUpdate | NeedsPostCameraDirectorUpdate | NeedsPostUpdate | NeedsRootCameraNodeEvents
};
ENUM_CLASS_FLAGS(ECameraEvaluationServiceFlags);

/** Parameter structure for initializing an evaluation service. */
struct FCameraEvaluationServiceInitializeParams
{
	FCameraSystemEvaluator* Evaluator = nullptr;
};

/** Parameter structure for tearing down an evaluation service. */
struct FCameraEvaluationServiceTeardownParams
{
	FCameraSystemEvaluator* Evaluator = nullptr;
};

/** Parameter structure for updating an evaluation service. */
struct FCameraEvaluationServiceUpdateParams
{
	FCameraSystemEvaluator* Evaluator = nullptr;
	float DeltaTime = 0.f;
	ECameraNodeEvaluationType EvaluationType = ECameraNodeEvaluationType::Standard;
};

/** Result structure for updating an evaluation service. */
struct FCameraEvaluationServiceUpdateResult
{
	FCameraEvaluationServiceUpdateResult(FCameraNodeEvaluationResult& InEvaluationResult)
		: EvaluationResult(InEvaluationResult)
	{}

	FCameraNodeEvaluationResult& EvaluationResult;
};

/**
 * An evaluation service running on a camera system.
 *
 * Evaluation services can run arbitrary logic before or after the root camera node update,
 * and respond to events in the node tree such as when camera rigs are activated or deactivated.
 */
class FCameraEvaluationService : public TSharedFromThis<FCameraEvaluationService>
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraEvaluationService)

public:

	GAMEPLAYCAMERAS_API FCameraEvaluationService();

	GAMEPLAYCAMERAS_API virtual ~FCameraEvaluationService();

public:

	/** Initializes the evaluation service. */
	void Initialize(const FCameraEvaluationServiceInitializeParams& Params);
	/** Runs at the start of the camera system update. */
	void PreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult);
	/** Runs after the camera director update. */
	void PostCameraDirectorUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult);
	/** Runs at the end of the camera system update. */
	void PostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult);
	/** Tears down the evaluation service. */
	void Teardown(const FCameraEvaluationServiceTeardownParams& Params);

	/** Collect referenced UObjects. */
	void AddReferencedObjects(FReferenceCollector& Collector);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this node evaluator. */
	void BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

public:

	// Internal API.

	ECameraEvaluationServiceFlags GetEvaluationServiceFlags() const { return PrivateFlags; }
	bool HasAllEvaluationServiceFlags(ECameraEvaluationServiceFlags InFlags) const;

protected:

	/** Initializes the evaluation service. */
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) {}
	/** Runs at the start of the camera system update. */
	virtual void OnPreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) {}
	/** Runs after the camera director update. */
	virtual void OnPostCameraDirectorUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) {}
	/** Runs at the end of the camera system update. */
	virtual void OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) {}
	/** Tears down the evaluation service. */
	virtual void OnTeardown(const FCameraEvaluationServiceTeardownParams& Params) {}

	/** Collect referenced UObjects. */
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) {}

	/** Called when the root camera node experiences an event. */
	virtual void OnRootCameraNodeEvent(const FRootCameraNodeCameraRigEvent& InEvent) {}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this node evaluator. */
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) {}
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void RootCameraNodeEventHandler(const FRootCameraNodeCameraRigEvent& InEvent);

protected:

	/**
	 * Sets the flags on this service.
	 * Can be called from the constructor, or during OnInitialize().
	 */
	void SetEvaluationServiceFlags(ECameraEvaluationServiceFlags InFlags);

private:

	/** The camera system that this service lives inside of. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** Evaluation service flags. */
	ECameraEvaluationServiceFlags PrivateFlags = ECameraEvaluationServiceFlags::None;
};

}  // namespace UE::Cameras

// Utility macros for declaring and defining camera evaluation services.
//
#define UE_DECLARE_CAMERA_EVALUATION_SERVICE(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, FCameraEvaluationService)

#define UE_DECLARE_CAMERA_EVALUATION_SERVICE_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_EVALUATION_SERVICE(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

