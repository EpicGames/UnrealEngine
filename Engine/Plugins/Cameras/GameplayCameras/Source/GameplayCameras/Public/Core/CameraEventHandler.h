// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/List.h"
#include "GameplayCameras.h"
#include "UObject/UnrealType.h"

namespace UE::Cameras
{

/** The type of change on an array property.  */
enum class ECameraArrayChangedEventType
{
	Add,
	Remove,
	Change
};

/** Parameter structure for a value property change event. */
template<typename T>
struct TCameraPropertyChangedEvent
{
	T NewValue;
};

/** Parameter structure for an array property change event. */
template<typename T>
struct TCameraArrayChangedEvent
{
	ECameraArrayChangedEventType EventType;

	TCameraArrayChangedEvent() {}
	TCameraArrayChangedEvent(EPropertyChangeType::Type InPropertyChangeType)
	{
		switch (InPropertyChangeType)
		{
			case EPropertyChangeType::ArrayAdd:
				EventType = ECameraArrayChangedEventType::Add;
				break;
			case EPropertyChangeType::ArrayRemove:
				EventType = ECameraArrayChangedEventType::Remove;
				break;
			default:
				EventType = ECameraArrayChangedEventType::Change;
				break;
		}
	}
};

/**
 * Wrapper struct, or handle, for a listener of data change events.
 * The listener class should own one of these for each object they're listening to.
 * The listener class can start listening by calling Register(HandlerWrapper, this) on the object
 * to listen to.
 */
template<typename HandlerInterface>
struct TCameraEventHandler : public TLinkedList<HandlerInterface*>
{
	TCameraEventHandler()
	{}

	TCameraEventHandler(HandlerInterface* InInterface)
		: TLinkedList<HandlerInterface*>(InInterface)
	{}

	~TCameraEventHandler()
	{
		this->Unlink();
	}
};

/**
 * A wrapper struct for a list of listeners waiting to be notified of data changes.
 * A class that can be listened-to should own one of these, and expose it so that listeners
 * can register themselves.
 */
template<typename HandlerInterface>
struct TCameraEventHandlerContainer
{
#if UE_GAMEPLAY_CAMERAS_EVENT_HANDLERS

	/** Invokes the given callback, with the given parameters, on any registered listener. */
	template<typename FuncType, typename... ArgTypes>
	void Notify(FuncType&& Func, ArgTypes&&... Args) const
	{
		for (typename TCameraEventHandler<HandlerInterface>::TIterator It(EventHandlers); It; ++It)
		{
			Invoke(Func, *It, Forward<ArgTypes>(Args)...);
		}
	}

	/** Registers a new listener by linking the given wrapper/handle to the list of listeners. */
	void Register(TCameraEventHandler<HandlerInterface>& InOutEventHandler, HandlerInterface* InInterface) const
	{
		ensure(!InOutEventHandler.IsLinked());
		InOutEventHandler = TCameraEventHandler<HandlerInterface>(InInterface);
		InOutEventHandler.LinkHead(EventHandlers);
	}

private:

	mutable TLinkedList<HandlerInterface*>* EventHandlers = nullptr;

#else

	// Stub for when events are disabled.
	template<typename ...ArgTypes>
	void Notify(ArgTypes&&...) const
	{}

#endif  // UE_GAMEPLAY_CAMERAS_EVENT_HANDLERS
};

}  // namespace UE::Cameras

