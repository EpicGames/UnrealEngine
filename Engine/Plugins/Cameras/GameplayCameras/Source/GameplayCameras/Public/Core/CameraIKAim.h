// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorHierarchy.h"
#include "Core/CameraNodeEvaluatorStorage.h"
#include "Math/MathFwd.h"
#include "GameplayCameras.h"

namespace UE::Cameras
{

struct FCameraRigEvaluationInfo;

/**
 * Parameter structure for aiming a camera rig at a target.
 */
struct FCameraIKAimParams
{
	/** The time interval to use when updating the camera rig. */
	float DeltaTime = 0.f;
	/** Whether this is the first update of the camera rig. */
	bool bIsFirstFrame = false;
	/** Whether this camera rig is the active one in its layer (e.g. top of the blend stack). */
	bool bIsActiveCameraRig = false;
	
	/** The desired target that the camera rig should be aiming at. */
	FVector3d TargetLocation = FVector3d::ZeroVector;

	/** The camera system inside which the evaluation takes place. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** The distance below which aiming should not take place. */
	double MinDistance = -1.0;
	/** The angle between desired and actual target below which we consider aiming is complete. */
	double AngleTolerance = -1.0;
	/** The distance between desired target and line of sight below which we consider aiming is complete. */
	double DistanceTolerance = -1.0;
	/** The maximum number of iterations to run. */
	uint8 MaxIterations = 0;
};

#if UE_GAMEPLAY_CAMERAS_DEBUG

struct FCameraIKAimIterationDebugInfo
{
	FVector3d CameraPoseLocation;
	FRotator3d CameraPoseRotation;
	double ErrorAngle = 0;
	double ErrorDistance = 0;
 
	FVector3d PivotJointLocation = FVector3d::ZeroVector;
	FVector2d YawPitchCorrection = FVector2d::ZeroVector;

	bool bNeededSolver = false;
	bool bFoundSolver = false;
	bool bSolvingSuccess = false;
};

struct FCameraIKAimDebugInfo
{
	TArray<FCameraIKAimIterationDebugInfo, TInlineAllocator<4>> Iterations;

	FVector3d DesiredTarget = FVector3d::ZeroVector;
	bool bSucceeded = false;

	void DebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) const;
};

FArchive& operator<< (FArchive& Ar, FCameraIKAimIterationDebugInfo& IterationDebugInfo);
FArchive& operator<< (FArchive& Ar, FCameraIKAimDebugInfo& DebugInfo);

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

/**
 * A class that can manipulate a camera rig in order to aim it at a desired target.
 */
class FCameraIKAim
{
public:

	/** Executes the aiming. */
	bool Run(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	void GetLastRunDebugInfo(FCameraIKAimDebugInfo& OutDebugInfo) const;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	enum class EAimResult
	{
		Failed,
		Aborted,
		Corrected,
		Completed
	};

	struct FAimIterationInfo
	{
		uint8 IterationIndex = 0;
		double ErrorAngle = 0.0;
		double ErrorDistance = 0.0;
		EAimResult Result = EAimResult::Failed;
	};

	bool DoRun(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo);
	void DoRunIteration(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, FAimIterationInfo& IterationInfo);

	void AimTwoBonesCameraRig(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, const FTransform3d& PivotTransform, FAimIterationInfo& IterationInfo);

	void RunRootCameraNode(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo);
	bool CheckTolerance(const FCameraIKAimParams& Params, const FCameraRigEvaluationInfo& CameraRigInfo, FAimIterationInfo& IterationInfo);

private:

	FCameraNodeEvaluationResult ScratchResult;
	
	FCameraNodeEvaluatorHierarchy CameraSystemHierarchy;
	TArray<uint8> EvaluatorSnapshot;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FCameraIKAimDebugInfo LastRunDebugInfo;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

}  // namespace UE::Cameras

