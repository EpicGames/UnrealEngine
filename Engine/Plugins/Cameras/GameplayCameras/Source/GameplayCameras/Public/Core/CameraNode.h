// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluatorBuilder.h"
#include "Core/ObjectChildrenView.h"
#include "Core/ObjectTreeGraphObject.h"
#include "CoreTypes.h"
#include "UObject/Object.h"

#include "CameraNode.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{
	class FCameraBuildLog;
	struct FCameraBuildContext;
	struct FCameraObjectBuildContext;
}

/** View on a camera node's children. */
using FCameraNodeChildrenView = UE::Cameras::TObjectChildrenView<TObjectPtr<UCameraNode>>;

/**
 * Flags describing the needs of a camera node.
 */
enum class ECameraNodeFlags
{
	None = 0,
	CustomGetChildren = 1 << 0
};
ENUM_CLASS_FLAGS(ECameraNodeFlags)

/**
 * The base class for a camera node.
 */
UCLASS(MinimalAPI, Abstract, DefaultToInstanced, EditInlineNew, meta=(CameraNodeCategories="Miscellaneous"))
class UCameraNode 
	: public UObject
	, public IObjectTreeGraphObject
{
	GENERATED_BODY()

public:
	
	using FCameraBuildLog = UE::Cameras::FCameraBuildLog;
	using FCameraBuildContext = UE::Cameras::FCameraBuildContext;
	using FCameraObjectBuildContext = UE::Cameras::FCameraObjectBuildContext;
	using FCameraNodeEvaluatorBuilder = UE::Cameras::FCameraNodeEvaluatorBuilder;

	/** Get the list of children under this node. */
	UE_API FCameraNodeChildrenView GetChildren();

	/** Optional build step executed at the beginning of the build process. */
	UE_API void PreBuild(FCameraBuildContext& BuildContext);

	/** Gets optional info about this node's required allocations at runtime. */
	UE_API void Build(FCameraObjectBuildContext& BuildContext);

	/** Builds the evaluator for this node. */
	UE_API FCameraNodeEvaluatorPtr BuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const;

protected:

	/** Adds the given flags to this node. */
	void AddNodeFlags(ECameraNodeFlags InFlags) { PrivateFlags |= InFlags; }

	/** Sets the flags for this node. */
	void SetNodeFlags(ECameraNodeFlags InFlags) { PrivateFlags = InFlags; }

protected:

	/** Get the list of children under this node. */
	virtual FCameraNodeChildrenView OnGetChildren() { return FCameraNodeChildrenView(); }

	/** Optional build step executed at the beginning of the build process. */
	virtual void OnPreBuild(FCameraBuildContext& BuildContext) {}

	/** Gets optional info about this node's required allocations at runtime. */
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) {}

	/** Builds the evaluator for this node. */
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const { return nullptr; }

protected:

	// UObject interface.
	UE_API virtual void PostLoad() override;

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	UE_API virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	UE_API virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override { return EObjectTreeGraphObjectSupportFlags::CommentText; }
	UE_API virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	UE_API virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
#endif

public:

	/** Specifies whether this node is enabled. */
	UPROPERTY(EditAnywhere, Category=Common)
	bool bIsEnabled = true;

#if WITH_EDITORONLY_DATA

public:

	/** Position of the camera node in the node graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the node graph editor. */
	UPROPERTY()
	FString GraphNodeComment;

private:

	// Deprecated properties.

	UPROPERTY()
	int32 GraphNodePosX_DEPRECATED = 0;
	UPROPERTY()
	int32 GraphNodePosY_DEPRECATED = 0;

#endif  // WITH_EDITORONLY_DATA

private:

	/** Flags on this camera node. */
	ECameraNodeFlags PrivateFlags = ECameraNodeFlags::None;
};

#undef UE_API
