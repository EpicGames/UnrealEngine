// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTable.h"
#include "Core/CameraObjectRtti.h"
#include "Core/CameraPose.h"
#include "Core/CameraRigEvaluationInfo.h"
#include "Core/CameraRigJoints.h"
#include "Core/CameraVariableTable.h"
#include "Core/ObjectChildrenView.h"
#include "Core/PostProcessSettingsCollection.h"
#include "Debug/CameraDebugBlockFwd.h"
#include "Debug/RootCameraDebugBlock.h"
#include "GameplayCameras.h"
#include "UObject/ObjectPtr.h"

#define UE_API GAMEPLAYCAMERAS_API

class FReferenceCollector;
class UCameraNode;
class UCameraRigAsset;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FCameraNodeEvaluator;
class FCameraNodeEvaluatorHierarchy;
class FCameraSystemEvaluator;
struct FCameraOperation;
struct FCameraNodeEvaluationParams;
struct FCameraNodeEvaluatorBuilder;
struct FCameraRigEvaluationInfo;

/**
 * Flags describing the needs of a camera node evaluator.
 */
enum class ECameraNodeEvaluatorFlags
{
	None = 0,
	NeedsParameterUpdate = 1 << 0,
	NeedsSerialize = 1 << 1,
	SupportsOperations = 1 << 2,

	Default = NeedsParameterUpdate | NeedsSerialize | SupportsOperations
};
ENUM_CLASS_FLAGS(ECameraNodeEvaluatorFlags)

/** View on a camera node evaluator's children. */
using FCameraNodeEvaluatorChildrenView = TObjectChildrenView<FCameraNodeEvaluator*>;

/**
 * Structure for building the tree of camera node evaluators.
 */
struct FCameraNodeEvaluatorBuildParams
{
	FCameraNodeEvaluatorBuildParams(FCameraNodeEvaluatorBuilder& InBuilder)
		: Builder(InBuilder)
	{}

	/** Builds an evaluator for the given camera node. */
	GAMEPLAYCAMERAS_API FCameraNodeEvaluator* BuildEvaluator(const UCameraNode* InNode) const;

	/** Builds an evaluator for the given camera node, and down-cast it to the given type. */
	template<typename EvaluatorType>
	EvaluatorType* BuildEvaluatorAs(const UCameraNode* InNode) const;

private:

	/** Builder object for building children evaluators. */
	FCameraNodeEvaluatorBuilder& Builder;
};

/**
 * Structure for initializing a camera node evaluator.
 */
struct FCameraNodeEvaluatorInitializeParams
{
	/** The evaluation running this evaluation. */
	FCameraSystemEvaluator* Evaluator = nullptr;
	/** The evaluation context (if any) responsible for this branch of the evaluation. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/**
	 * Information about the last active camera rig if the node tree being initialized
	 * is being pushed on top of a non-empty blend stack.
	 */
	FCameraRigEvaluationInfo LastActiveCameraRigInfo;

	/** The layer on which the node evaluator is being initialized. */
	ECameraRigLayer Layer = ECameraRigLayer::Main;

public:

	FCameraNodeEvaluatorInitializeParams() = default;
	FCameraNodeEvaluatorInitializeParams(FCameraNodeEvaluatorHierarchy* InHierarchy);

private:

	/** An optional hierarchy to populate while initialize the evaluator tree. */
	FCameraNodeEvaluatorHierarchy* Hierarchy = nullptr;

	friend class FCameraNodeEvaluator;
};

/**
 * Structure for tearing down a camera node evaluator.
 */
struct FCameraNodeEvaluatorTeardownParams
{
	/** The evaluation running this evaluation. */
	FCameraSystemEvaluator* Evaluator = nullptr;
	/** The evaluation context (if any) responsible for this branch of the evaluation. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The layer on which the node evaluator was running. */
	ECameraRigLayer Layer = ECameraRigLayer::Main;
};

/**
 * Parameter structure for updating the pre-blended parameters of a camera node.
 */
struct FCameraBlendedParameterUpdateParams
{
	FCameraBlendedParameterUpdateParams(const FCameraNodeEvaluationParams& InEvaluationParams, const FCameraPose& InLastCameraPose)
		: EvaluationParams(InEvaluationParams)
		, LastCameraPose(InLastCameraPose)
	{}

	/** Information about the evaluation pass that will happen afterwards. */
	const FCameraNodeEvaluationParams& EvaluationParams;
	/** Last frame's camera pose. */
	const FCameraPose& LastCameraPose;
};

/**
 * Result of updating the pre-blended parameters of a camera node.
 */
struct FCameraBlendedParameterUpdateResult
{
	FCameraBlendedParameterUpdateResult(FCameraVariableTable& InVariableTable)
		: VariableTable(InVariableTable)
	{}

	/** Variable table in which parameters should be stored or obtained. */
	FCameraVariableTable& VariableTable;
};

/** The type of evaluation being run. */
enum class ECameraNodeEvaluationType
{
	/** Normal evaluation. */
	Standard,
	/** View rotation evaluation. */
	ViewRotationPreview,
	/** Evaluation for IK aiming. */
	IK,

#if WITH_EDITOR
	/** Evaluation for editor preview. */
	EditorPreview
#endif  // WITH_EDITOR
};

/**
 * Parameter structure for running a camera node evaluator.
 */
struct FCameraNodeEvaluationParams
{
	/** The evaluator running this evaluation. */
	FCameraSystemEvaluator* Evaluator = nullptr;
	/** The evaluation context (if any) responsible for this branch of the evaluation. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;
	/** The time interval for the evaluation. */
	float DeltaTime = 0.f;
	/** The type of evaluation being run. */
	ECameraNodeEvaluationType EvaluationType = ECameraNodeEvaluationType::Standard;
	/** Whether this is the first evaluation of this camera node hierarchy. */
	bool bIsFirstFrame = false;
	/** Whether this camera node is running inside the active camera rig in this layer. */
	bool bIsActiveCameraRig = false;

	bool IsStatelessEvaluation() const
	{
		return EvaluationType == ECameraNodeEvaluationType::IK ||
			EvaluationType == ECameraNodeEvaluationType::ViewRotationPreview;
	}
};

/**
 * Input/output result structure for running a camera node evaluator.
 */
struct FCameraNodeEvaluationResult
{
	/** The camera pose. */
	FCameraPose CameraPose;

	/** The variable table. */
	FCameraVariableTable VariableTable;

	/** The context data table. */
	FCameraContextDataTable ContextDataTable;

	/** The list of joints in the current camera rig. */
	FCameraRigJoints CameraRigJoints;

	/** Post-process settings for the camera. */
	FPostProcessSettingsCollection PostProcessSettings;

	/** Whether the current frame is a camera cut. */
	bool bIsCameraCut = false;

	/** Whether this result is valid. */
	bool bIsValid = false;

public:

	/** Reset this result to its default (non-valid) state.  */
	UE_API void Reset();

	/** Reset all written-this-frame flags on the camera pose and tables. */
	UE_API void ResetFrameFlags();

	/** Override this result with the given other result. */
	UE_API void OverrideAll(const FCameraNodeEvaluationResult& OtherResult, bool bIncludePrivateValues = false);

	/** Interpolate this result towards the other given result. */
	UE_API void LerpAll(const FCameraNodeEvaluationResult& ToResult, float BlendFactor, bool bIncludePrivateValues = false);

public:

	/** Export this result into the given view info. */
	UE_API void GetViewInfo(FMinimalViewInfo& OutViewInfo) const;

	/** Serializes this result to the given archive. */
	UE_API void Serialize(FArchive& Ar);

	/** Collects objects from the context data table. */
	UE_API void AddReferencedObjects(FReferenceCollector& Collector);

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

public:

	// Internal API.

	UE_API void AddCameraPoseTrailPointIfNeeded();
	UE_API void AddCameraPoseTrailPointIfNeeded(const FVector3d& Point);
	UE_API void AppendCameraPoseLocationTrail(const FCameraNodeEvaluationResult& InResult);
	UE_API TConstArrayView<FVector3d> GetCameraPoseLocationTrail() const;

private:

	TArray<FVector3d> CameraPoseLocationTrail;

#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG
};

/**
 * Parameter structure for executing camera operations.
 */
struct FCameraOperationParams
{
	/** The evaluator running this operation. */
	FCameraSystemEvaluator* Evaluator = nullptr;
	/** The evaluation context (if any) responsible for this branch of the evaluation. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;
};

/**
 * Parameter structure for serializing the state of a camera node evaluator.
 */
struct FCameraNodeEvaluatorSerializeParams
{
};

#if WITH_EDITOR

/**
 * Parameter structure for drawing editor preview.
 */
struct FCameraEditorPreviewDrawParams
{
};

#endif  // WITH_EDITOR

/**
 * Base class for objects responsible for running a camera node.
 */
class FCameraNodeEvaluator
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraNodeEvaluator)

public:

	FCameraNodeEvaluator() = default;
	virtual ~FCameraNodeEvaluator() = default;

	/** Called to build any children evaluators. */
	void Build(const FCameraNodeEvaluatorBuildParams& Params);

	/** Initialize this evaluator and all its descendants. */
	void Initialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Tear down this evaluator and all its descendants. */
	void Teardown(const FCameraNodeEvaluatorTeardownParams& Params);

	/** Collect referenced UObjects for this node and all its descendants. */
	void AddReferencedObjects(FReferenceCollector& Collector);

	/** Get the list of children under this evaluator. */
	FCameraNodeEvaluatorChildrenView GetChildren();

	/** Called to update and store the blended parameters for this node. */
	void UpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult);

	/** Run this evaluator. */
	GAMEPLAYCAMERAS_API void Run(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Execute an IK operation. */
	void ExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation);

	/** Serializes the state of this evaluator. */
	void Serialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar);

	/** Gets the flags for this evaluator. */
	ECameraNodeEvaluatorFlags  GetNodeEvaluatorFlags() const { return PrivateFlags; }

	/** Get the camera node. */
	const UCameraNode* GetCameraNode() const { return PrivateCameraNode; }

	/** Get the camera node. */
	template<typename CameraNodeType>
	const CameraNodeType* GetCameraNodeAs() const
	{
		return Cast<CameraNodeType>(PrivateCameraNode);
	}

#if WITH_EDITOR
	void DrawEditorPreview(const FCameraEditorPreviewDrawParams& Params, FCameraDebugRenderer& Renderer);
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this node evaluator. */
	void BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

public:

	// Internal API.
	void SetPrivateCameraNode(TObjectPtr<const UCameraNode> InCameraNode);

protected:

	/**
	 * Sets the flags for this evaluator.
	 * Can be called from the constructor, or during OnInitialize().
	 */
	GAMEPLAYCAMERAS_API void SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags InFlags);

	/**
	 * Adds flags for this evaluator.
	 * Evaluators default to having all flags enabled, so this is mostly only for re-adding a
	 * flag that was removed by a base class.
	 */
	GAMEPLAYCAMERAS_API void AddNodeEvaluatorFlags(ECameraNodeEvaluatorFlags InFlags);

protected:

	/** Called to build any children evaluators. */
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) {}

	/** Initialize this evaluator. Children and descendants will be automatically initialized too. */
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) {}

	/** Tear down this evaluator and all its descendants. */
	virtual void OnTeardown(const FCameraNodeEvaluatorTeardownParams& Params) {}

	/** Collect referenced UObjects for this node. */
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) {}

	/** Get the list of children under this evaluator. */
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() { return FCameraNodeEvaluatorChildrenView(); }

	/**
	 * Called to update and store the blended parameters for this node.
	 * Requires setting the ECameraNodeEvaluatorFlags::NeedsParameterUpdate flag.
	 */
	virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) {}

	/**
	 * Run this evaluator. This node evaluator is responsible for calling Run() on its children as appropriate.
	 */
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) {}

	/** 
	 * Execute an IK operation.
	 * Requires setting the ECameraNodeEvaluatorFlags::SupportsOperations flag.
	 */
	virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) {}

	/**
	 * Serializes the state of this evaluator.
	 * Requires setting the ECameraNodeEvaluatorFlags::NeedsSerialize flag, which is set by default.
	 */
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) {}

#if WITH_EDITOR
	virtual void OnDrawEditorPreview(const FCameraEditorPreviewDrawParams& Params, FCameraDebugRenderer& Renderer) {}
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this node evaluator. */
	GAMEPLAYCAMERAS_API virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

protected:

	/** Whether to automatically add points to the debug trail after this evaluation has run. */
	bool bAutoCameraPoseMovementTrail = true;

#endif  // WITH_EDITOR || UE_GAMEPLAY_CAMERAS_DEBUG

private:

	/** The camera node to run. */
	TObjectPtr<const UCameraNode> PrivateCameraNode;

	/** The flags for this evaluator. */
	ECameraNodeEvaluatorFlags PrivateFlags = ECameraNodeEvaluatorFlags::Default;
};

/** Utility base class for camera node evaluators of a specific camera node type. */
template<typename CameraNodeType>
class TCameraNodeEvaluator : public FCameraNodeEvaluator
{
public:

	/** Gets the camera node. */
	const CameraNodeType* GetCameraNode() const
	{
		return GetCameraNodeAs<CameraNodeType>();
	}

	friend CameraNodeType;
};

template<typename EvaluatorType>
EvaluatorType* FCameraNodeEvaluatorBuildParams::BuildEvaluatorAs(const UCameraNode* InNode) const
{
	if (FCameraNodeEvaluator* NewEvaluator = BuildEvaluator(InNode))
	{
		return NewEvaluator->CastThisChecked<EvaluatorType>();
	}
	return nullptr;
}

}  // namespace UE::Cameras

// Utility macros for declaring and defining camera node evaluators.
//
#define UE_DECLARE_CAMERA_NODE_EVALUATOR(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, FCameraNodeEvaluator)

#define UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_NODE_EVALUATOR(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

#undef UE_API
