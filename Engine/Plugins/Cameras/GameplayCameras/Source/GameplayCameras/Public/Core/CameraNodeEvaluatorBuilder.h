// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluatorStorage.h"

namespace UE::Cameras
{

/** Structure for building camera node evaluators. */
struct FCameraNodeEvaluatorBuilder
{
	FCameraNodeEvaluatorBuilder(FCameraNodeEvaluatorStorage& InStorage)
		: Storage(InStorage)
	{}

	/** Builds a camera node evaluator of a given type. */
	template<typename EvaluatorType, typename ...ArgTypes>
	EvaluatorType* BuildEvaluator(ArgTypes&&... InArgs);

private:

	FCameraNodeEvaluatorStorage& Storage;
};

template<typename EvaluatorType, typename ...ArgTypes>
EvaluatorType* FCameraNodeEvaluatorBuilder::BuildEvaluator(ArgTypes&&... InArgs)
{
	return Storage.BuildEvaluator<EvaluatorType>(Forward<ArgTypes>(InArgs)...);
}

}  // namespace UE::Cameras

