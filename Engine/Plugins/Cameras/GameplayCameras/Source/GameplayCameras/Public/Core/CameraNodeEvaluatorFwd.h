// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "CameraNodeEvaluatorFwd.generated.h"

namespace UE::Cameras
{

class FCameraNodeEvaluator;
class FCameraNodeEvaluatorStorage;
struct FCameraNodeEvaluatorBuilder;

}  // namespace UE::Cameras

// Typedef to avoid having to deal with namespaces in UCameraNode subclasses.
using FCameraNodeEvaluatorPtr = UE::Cameras::FCameraNodeEvaluator*;

/** Allocation information for an entire tree of node evaluators. */
USTRUCT()
struct FCameraNodeEvaluatorAllocationInfo
{
	GENERATED_BODY()

	/** Total required size for the node evaluators. */
	UPROPERTY()
	int16 TotalSizeof = 0;

	/** Maximum required alignment for the node evaluators. */
	UPROPERTY()
	int16 MaxAlignof = 0;

public:

	friend bool operator==(const FCameraNodeEvaluatorAllocationInfo& A, const FCameraNodeEvaluatorAllocationInfo& B)
	{
		return A.TotalSizeof == B.TotalSizeof &&
			A.MaxAlignof == B.MaxAlignof;
	}
};

template<>
struct TStructOpsTypeTraits<FCameraNodeEvaluatorAllocationInfo> : public TStructOpsTypeTraitsBase2<FCameraNodeEvaluatorAllocationInfo>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

/** Allocation information for a node evaluator, auto-setup for a given type. */
template<typename EvaluatorType>
struct TCameraNodeEvaluatorAllocationInfo : FCameraNodeEvaluatorAllocationInfo
{
	TCameraNodeEvaluatorAllocationInfo() 
	{
		TotalSizeof = sizeof(EvaluatorType);
		MaxAlignof = alignof(EvaluatorType);
	}
};

