// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraNodeEvaluator.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

/**
 * A utility class that stores a flattened hierarchy of camera node evaluators.
 */
class FCameraNodeEvaluatorHierarchy
{
public:

	/** Builds an empty hierarchy. */
	UE_API FCameraNodeEvaluatorHierarchy();
	/** Builds a hierarchy starting from the given root evaluator. */
	UE_API FCameraNodeEvaluatorHierarchy(FCameraNodeEvaluator* InRootEvaluator);

	/** Get the list of evaluators in depth-first order. */
	UE_API TArrayView<FCameraNodeEvaluator* const> GetFlattenedHierarchy() const;

	/** Get the list of evaluators matching the given evaluator flags, in depth-first order. */
	UE_API void GetFlattenedHierarchy(ECameraNodeEvaluatorFlags FilterFlags, TArray<FCameraNodeEvaluator*> OutEvaluators) const;

public:

	/** Builds a hierarchy starting from the given root evaluator. */
	UE_API void Build(FCameraNodeEvaluator* InRootEvaluator);
	/** Append another hierarchy to the existing hierarchy, starting from the given root evaluator. */
	UE_API void Append(FCameraNodeEvaluator* InRootEvaluator);
	/** Append and tag another hierarchy to the existing hierarchy, starting from the given root evaluator. */
	UE_API void AppendTagged(const FName TaggedRangeName, FCameraNodeEvaluator* InRootEvaluator);
	/** Add an evaluator to the existing hierarchy. */
	UE_API void AddEvaluator(FCameraNodeEvaluator* Evaluator);
	/** Resets this object to an emtpy hierarchy. */
	UE_API void Reset();

public:

	/** Executes the given predicate on each evaluator in depth-first order. */
	template<typename PredicateClass>
	void ForEachEvaluator(PredicateClass&& Predicate) const
	{
		for (FCameraNodeEvaluator* Evaluator : FlattenedHierarchy)
		{
			Predicate(Evaluator);
		}
	}

	/** Executes the given predicate on each evaluator in the specified range in depth-first order. */
	template<typename PredicateClass>
	void ForEachEvaluator(const FName TaggedRangeName, PredicateClass&& Predicate) const
	{
		const FTaggedRange& TaggedRange = TaggedRanges.FindChecked(TaggedRangeName);
		for (int32 Index = TaggedRange.StartIndex; Index < TaggedRange.EndIndex; ++Index)
		{
			FCameraNodeEvaluator* Evaluator(FlattenedHierarchy[Index]);
			Predicate(Evaluator);
		}
	}

	/**
	 * Executes the given predicate on each evaluator matching the specified evaluator flags in 
	 * depth-first order.
	 */
	template<typename PredicateClass>
	void ForEachEvaluator(ECameraNodeEvaluatorFlags FilterFlags, PredicateClass&& Predicate) const
	{
		for (FCameraNodeEvaluator* Evaluator : FlattenedHierarchy)
		{
			if (EnumHasAllFlags(Evaluator->GetNodeEvaluatorFlags(), FilterFlags))
			{
				Predicate(Evaluator);
			}
		}
	}

	/** 
	 * Executes the given predicate on each evaluator in the specified range matching the specified 
	 * evaluator flags in depth-first order.
	 */
	template<typename PredicateClass>
	void ForEachEvaluator(const FName TaggedRangeName, ECameraNodeEvaluatorFlags FilterFlags, PredicateClass&& Predicate) const
	{
		const FTaggedRange& TaggedRange = TaggedRanges.FindChecked(TaggedRangeName);
		for (int32 Index = TaggedRange.StartIndex; Index < TaggedRange.EndIndex; ++Index)
		{
			FCameraNodeEvaluator* Evaluator(FlattenedHierarchy[Index]);
			if (EnumHasAllFlags(Evaluator->GetNodeEvaluatorFlags(), FilterFlags))
			{
				Predicate(Evaluator);
			}
		}
	}

public:

	/** Helper method to call UpdateParameters on the appropriate nodes in the hierarchy. */
	UE_API void CallUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult);
	/** Helper method to call ExecuteOperation on the appropriate nodes in the hierarchy. */
	UE_API void CallExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation);
	/** Helper method to call Serialize on the appropriate nodes in the hierarchy. */
	UE_API void CallSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar);

private:

	TArray<FCameraNodeEvaluator*> FlattenedHierarchy;

	struct FTaggedRange
	{
		int32 StartIndex;
		int32 EndIndex;
	};
	TMap<FName, FTaggedRange> TaggedRanges;
};

}  // namespace UE::Cameras

#undef UE_API
