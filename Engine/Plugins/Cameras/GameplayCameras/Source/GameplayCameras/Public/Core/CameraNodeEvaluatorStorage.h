// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorFwd.h"
#include "Core/CameraObjectStorage.h"
#include "UObject/ObjectPtr.h"

class FReferenceCollector;
class UCameraRigAsset;
class UCameraNode;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FCameraNodeEvaluator;
class FCameraNodeEvaluatorStorage;
class FCameraSystemEvaluator;

/** Structure for building an entire tree of node evaluators. */
struct FCameraNodeEvaluatorTreeBuildParams
{
	/** The root node of the tree. */
	TObjectPtr<const UCameraNode> RootCameraNode;
	/** An optional allocation information to optimize storage. */
	const FCameraNodeEvaluatorAllocationInfo* AllocationInfo = nullptr;
};

/**
 * A class responsible for storing a tree of camera node evaluators.
 */
class FCameraNodeEvaluatorStorage : public TCameraObjectStorage<FCameraNodeEvaluator>
{
public:

	using Super = TCameraObjectStorage<FCameraNodeEvaluator>;

	/** Build the tree of evaluators for the given tree of camera nodes. */
	FCameraNodeEvaluatorPtr BuildEvaluatorTree(const FCameraNodeEvaluatorTreeBuildParams& Params);

	/** Destroy any allocated evaluators. */
	void DestroyEvaluatorTree(bool bFreeAllocations = false);

	/** Destroy a sub-tree of allocated evaluators. */
	void DestroyEvaluatorTree(FCameraNodeEvaluator* InRootEvaluator, bool bResetMemory = true);

	/** Computes the currently allocated totals. */
	void GetAllocationInfo(FCameraNodeEvaluatorAllocationInfo& OutAllocationInfo);

	/** Build a new evaluator inside this storage. */
	template<typename EvaluatorType, typename ...ArgTypes>
	EvaluatorType* BuildEvaluator(ArgTypes&&... InArgs);
};

template<typename EvaluatorType, typename ...ArgTypes>
EvaluatorType* FCameraNodeEvaluatorStorage::BuildEvaluator(ArgTypes&&... InArgs)
{
	return Super::BuildObject<EvaluatorType>(Forward<ArgTypes>(InArgs)...);
}

}  // namespace UE::Cameras

