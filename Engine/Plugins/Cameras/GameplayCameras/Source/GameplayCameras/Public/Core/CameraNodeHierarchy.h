// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraNode.h"

#define UE_API GAMEPLAYCAMERAS_API

class UBaseCameraObject;

namespace UE::Cameras
{

/**
 * A utility class that stores a flattened hierarchy of camera nodes.
 * Unconnected camera nodes aren't included, of course.
 */
class FCameraNodeHierarchy
{
public:

	/** Build an empty hierarchy. */
	UE_API FCameraNodeHierarchy();
	/** Build a hierarchy starting from the given camera object's root node. */
	UE_API FCameraNodeHierarchy(UBaseCameraObject* InCameraObject);
	/** Build a hierarchy starting from the given root node. */
	UE_API FCameraNodeHierarchy(UCameraNode* InRootCameraNode);

	/** Get the list of camera nodes in depth-first order. */
	UE_API TArrayView<UCameraNode* const> GetFlattenedHierarchy() const;

	/** Returns the number of camera nodes in this hierarchy. */
	UE_API int32 Num() const;

public:

	/** Build a hierarchy starting from the given camera object's root node. */
	UE_API void Build(UBaseCameraObject* InCameraObject);
	/** Build a hierarchy starting from the given root node. */
	UE_API void Build(UCameraNode* InRootCameraNode);
	/** Resets this object to an empty hierarchy. */
	UE_API void Reset();

public:

	/** Executes the given predicate on each camera node in depth-first order. */
	template<typename PredicateClass>
	void ForEach(PredicateClass&& Predicate)
	{
		for (UCameraNode* Node : FlattenedHierarchy)
		{
			Predicate(Node);
		}
	}

public:

	// Internal API.

#if WITH_EDITORONLY_DATA
	UE_API bool FindMissingConnectableObjects(TArrayView<UObject* const> ConnectableObjects, TSet<UObject*>& OutMissingObjects);
	UE_API bool FindMissingConnectableObjects(const TSet<UObject*> ConnectableObjectsSet, TSet<UObject*>& OutMissingObjects);
#endif  // WITH_EDITORONLY_DATA

private:

	TArray<UCameraNode*> FlattenedHierarchy;
};

}  // namespace UE::Cameras

#undef UE_API
