// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraVariableTableFwd.h"
#include "Core/ObjectTreeGraphObject.h"
#include "UObject/ObjectPtr.h"

#include "CameraObjectInterface.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraNode;
struct FCameraContextDataDefinition;
struct FCameraObjectInterfaceParameterDefinition;
struct FCameraVariableDefinition;

/**
 * Base class for interface parameters on a camera rig asset.
 */
UCLASS(MinimalAPI)
class UCameraObjectInterfaceParameterBase 
	: public UObject
{
	GENERATED_BODY()

public:

	/** The exposed name for this parameter. */
	UPROPERTY(EditAnywhere, Category="Camera")
	FString InterfaceParameterName;

	/** Whether to show this parameter on prefab nodes, references, components, etc. */
	UPROPERTY(EditAnywhere, Category="Visibility")
	bool bIsVisible = true;

public:

	/** Gets this parameter's unique ID. */
	const FGuid& GetGuid() const { return Guid; }

	/** Gets the parameter definition. */
	virtual void GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const {}

protected:

	/** The Guid of this parameter. */
	UPROPERTY()
	FGuid Guid;

protected:

	// UObject interface.
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

public:

	// Deprecated

	UPROPERTY(meta=(ObjectTreeGraphHidden=true))
	TObjectPtr<UCameraNode> Target_DEPRECATED;

	UPROPERTY()
	FName TargetPropertyName_DEPRECATED;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	FIntVector2 GraphNodePos_DEPRECATED = FIntVector2::ZeroValue;

	UPROPERTY()
	bool bHasGraphNode_DEPRECATED = false;

#endif  // WITH_EDITORONLY_DATA
};

/**
 * An exposed camera rig parameter that drives a specific parameter on one of
 * its camera nodes.
 */
UCLASS(MinimalAPI)
class UCameraObjectInterfaceBlendableParameter : public UCameraObjectInterfaceParameterBase
{
	GENERATED_BODY()

public:

	/** The type of this parameter. */
	UPROPERTY()
	ECameraVariableType VariableType = ECameraVariableType::Boolean;

	/** The struct type of this parameter if it is a blendable struct. */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> BlendableStructType;

	/**
	 * Whether this parameter's value should be pre-blended.
	 *
	 * Pre-blending means that if two blending camera rigs share this parameter, 
	 * each of their values will be blended in a first evaluation pass, and then
	 * both camera rigs will evaluate with the same blended value.
	 */
	UPROPERTY(EditAnywhere, Category="Blending")
	bool bIsPreBlended = false;

	// Built on save/cook.

	/** The ID to use to access the underlying variable value in the variable table. */
	UPROPERTY()
	FCameraVariableID PrivateVariableID;


	// Deprecated.

	UPROPERTY()
	TObjectPtr<UCameraVariableAsset> PrivateVariable_DEPRECATED;

public:

	/** Gets the camera variable definition for this parameter. */
	GAMEPLAYCAMERAS_API FCameraVariableDefinition GetVariableDefinition() const;

#if WITH_EDITORONLY_DATA
	GAMEPLAYCAMERAS_API FString GetVariableName() const;
#endif

	// UCameraObjectInterfaceParameterBase interface.
	virtual void GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const override;
};

UCLASS(MinimalAPI)
class UCameraObjectInterfaceDataParameter : public UCameraObjectInterfaceParameterBase
{
	GENERATED_BODY()

public:

	/** The type of this parameter. */
	UPROPERTY()
	ECameraContextDataType DataType = ECameraContextDataType::Name;

	/** The type of container for this parameter. */
	UPROPERTY()
	ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;

	/** An additional type object for this parameter. */
	UPROPERTY()
	TObjectPtr<const UObject> DataTypeObject;

	// Built on save/cook.

	/** The ID to use to access the underlying data in the context data table. */
	UPROPERTY()
	FCameraContextDataID PrivateDataID;

public:

	/** Gets the camera context data definition for this parameter. */
	GAMEPLAYCAMERAS_API FCameraContextDataDefinition GetDataDefinition() const;

#if WITH_EDITORONLY_DATA
	GAMEPLAYCAMERAS_API FString GetDataName() const;
#endif

	// UCameraObjectInterfaceParameterBase interface.
	virtual void GetParameterDefinition(FCameraObjectInterfaceParameterDefinition& OutParameterDefinition) const override;
};

/**
 * Structure defining the public data interface of a camera object.
 */
USTRUCT()
struct FCameraObjectInterface
{
	GENERATED_BODY()

public:

	/** The list of exposed blendable parameters on the camera rig. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraObjectInterfaceBlendableParameter>> BlendableParameters;

	/** The list of exposed data parameters on the camera rig. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraObjectInterfaceDataParameter>> DataParameters;

public:
	
	/** Finds an exposed parameter by name. */
	GAMEPLAYCAMERAS_API UCameraObjectInterfaceBlendableParameter* FindBlendableParameterByName(const FString& ParameterName) const;
	GAMEPLAYCAMERAS_API UCameraObjectInterfaceDataParameter* FindDataParameterByName(const FString& ParameterName) const;

	/** Finds an exposed parameter by Guid. */
	GAMEPLAYCAMERAS_API UCameraObjectInterfaceBlendableParameter* FindBlendableParameterByGuid(const FGuid& ParameterGuid) const;
	GAMEPLAYCAMERAS_API UCameraObjectInterfaceDataParameter* FindDataParameterByGuid(const FGuid& ParameterGuid) const;

	/** Returns whether an exposed parameter with the given name exists. */
	GAMEPLAYCAMERAS_API bool HasBlendableParameter(const FString& ParameterName) const;

public:

	// Deprecated methods.
	UE_DEPRECATED(5.6, "Camera rigs are now all standalone assets and don't need a separate display name.")
	FString GetDisplayName() const { return DisplayName_DEPRECATED; }

private:

	// Deprecated properties.

	UPROPERTY()
	FString DisplayName_DEPRECATED;
};

/**
 * Getter object for adding a camera parameter to a camera node graph.
 */
UCLASS(MinimalAPI, meta=(ObjectTreeGraphSelfPinDirection="Output"))
class UCameraObjectInterfaceParameterGetter
	: public UObject
	, public IObjectTreeGraphObject
{
	GENERATED_BODY()

public:

	/** The Guid of the parameter to get. */
	UPROPERTY()
	FGuid ParameterGuid;

	/** The location of this node in the graph. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

public:

	UE_API UCameraObjectInterfaceParameterBase* GetInterfaceParameter() const;

	UE_API FString GetInterfaceParameterName() const;
	
protected:

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
#endif
};

#undef UE_API

