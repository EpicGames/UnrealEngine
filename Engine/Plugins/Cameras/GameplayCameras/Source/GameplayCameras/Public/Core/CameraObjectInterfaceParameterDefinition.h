// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraVariableTableFwd.h"

#include "CameraObjectInterfaceParameterDefinition.generated.h"

/**
 * The type of a camera parameter.
 */
UENUM()
enum class ECameraObjectInterfaceParameterType : uint8
{
	Blendable,
	Data
};

/**
 * Information about a parameter exposed on a camera asset.
 */
USTRUCT()
struct FCameraObjectInterfaceParameterDefinition
{
	GENERATED_BODY()

	/** The name of the parameter. */
	UPROPERTY()
	FName ParameterName;

	/**
	 * The GUID of the parameter.
	 * This matches the GUID on the corresponding UCameraObjectInterfaceBlendableParameter
	 * or UCameraObjectInterfaceDataParameter object.
	 */
	UPROPERTY()
	FGuid ParameterGuid;

	/** The type of this parameter. */
	UPROPERTY()
	ECameraObjectInterfaceParameterType ParameterType = ECameraObjectInterfaceParameterType::Blendable;

	/** Whether to show this parameter on prefab nodes, references, components, etc. */
	UPROPERTY()
	bool bIsVisible = true;


	// Blendable parameter properties.
	// (only valid when ParameterType == Blendable)

	/** The ID of the variable that drives this blendable parameter. */
	UPROPERTY()
	FCameraVariableID VariableID;

	/** The type of the variable that drives this blendable parameter. */
	UPROPERTY()
	ECameraVariableType VariableType = ECameraVariableType::Boolean;

	/** The type of the structure if VariableType is BlendableStruct. */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> BlendableStructType;


	// Data parameter properties.
	// (only valid when ParameterType == Data)

	/** The ID of the data that drives this blendable parameter. */
	UPROPERTY()
	FCameraContextDataID DataID;

	/** The type of the data that drives this blendable parameter. */
	UPROPERTY()
	ECameraContextDataType DataType = ECameraContextDataType::Name;

	/** The type of container that drives this blendable parameter. */
	UPROPERTY()
	ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;

	/** The type object of the data that drives this blendable parameter. */
	UPROPERTY()
	TObjectPtr<const UObject> DataTypeObject;

public:

	bool operator==(const FCameraObjectInterfaceParameterDefinition& Other) const = default;
};

template<>
struct TStructOpsTypeTraits<FCameraObjectInterfaceParameterDefinition> : public TStructOpsTypeTraitsBase2<FCameraObjectInterfaceParameterDefinition>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

