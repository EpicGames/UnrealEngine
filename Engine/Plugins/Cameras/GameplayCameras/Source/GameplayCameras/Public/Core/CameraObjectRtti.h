// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"

namespace UE::Cameras
{

/**
 * Identifier for a given RTTI-enabled class.
 */
struct FCameraObjectTypeID
{
	friend uint32 GetTypeHash(FCameraObjectTypeID In)
	{
		return In.ID;
	}

	friend bool operator<(FCameraObjectTypeID A, FCameraObjectTypeID B)
	{
		return A.ID < B.ID;
	}

	friend bool operator==(FCameraObjectTypeID A, FCameraObjectTypeID B)
	{
		return A.ID == B.ID;
	}

	uint32 GetTypeID() const
	{
		return ID;
	}
	
	bool IsValid() const
	{
		return ID != MAX_uint32;
	}

	FCameraObjectTypeID(uint32 InID)
		: ID(InID)
	{}

	static FCameraObjectTypeID Invalid()
	{
		return FCameraObjectTypeID(MAX_uint32);
	}

protected:

	GAMEPLAYCAMERAS_API static uint32 RegisterNewID();

protected:

	uint32 ID;
};

/**
 * Type information about an RTTI-enabled class.
 */
struct FCameraObjectTypeInfo
{
	FName TypeName;

	uint32 Sizeof = 0;
	uint32 Alignof = 0;

	using FConstructor = void(*)(void*);
	FConstructor Constructor;

	using FDestructor = void(*)(void*);
	FDestructor Destructor;
};

/**
 * Type registry of known RTTI-enabled classes.
 */
class FCameraObjectTypeRegistry
{
public:

	GAMEPLAYCAMERAS_API static FCameraObjectTypeRegistry& Get();

	GAMEPLAYCAMERAS_API void RegisterType(FCameraObjectTypeID TypeID, FCameraObjectTypeInfo&& TypeInfo);
	GAMEPLAYCAMERAS_API FCameraObjectTypeID FindTypeByName(const FName& TypeName) const;
	GAMEPLAYCAMERAS_API const FCameraObjectTypeInfo* GetTypeInfo(FCameraObjectTypeID TypeID) const;
	GAMEPLAYCAMERAS_API FName GetTypeNameSafe(FCameraObjectTypeID TypeID) const;

	GAMEPLAYCAMERAS_API void ConstructObject(FCameraObjectTypeID TypeID, void* Ptr);

private:

	TMap<FName, uint32> TypeIDsByName;
	TSparseArray<FCameraObjectTypeInfo> TypeInfos;
};

/**
 * Strongly-typed ID wrapper for an RTTI-enabled class.
 */
template<typename T>
struct TCameraObjectTypeID : FCameraObjectTypeID
{
private:

	TCameraObjectTypeID(uint32 InID) : FCameraObjectTypeID(InID) {}

	static TCameraObjectTypeID RegisterType(const FName& InClassName)
	{
		TCameraObjectTypeID NewTypeID(FCameraObjectTypeID::RegisterNewID());
		FCameraObjectTypeInfo NewTypeInfo {
			InClassName,
			sizeof(T), alignof(T),
			&TCameraObjectTypeID<T>::StaticConstructor,
			&TCameraObjectTypeID<T>::StaticDestructor
		};
		FCameraObjectTypeRegistry::Get().RegisterType(NewTypeID, MoveTemp(NewTypeInfo));
		return NewTypeID;
	}

	static void StaticConstructor(void* Ptr)
	{
		new(Ptr) T();
	}

	static void StaticDestructor(void* Ptr)
	{
		reinterpret_cast<T*>(Ptr)->~T();
	}

	friend T;
};

}  // namespace UE::Cameras

// Macros for enabling simple RTTI information on a class hierarchy.
//
// The first macro is for the root of the class hierarchy, while the second is for all
// other classes below it. The third macro goes in the cpp file of each class.
//
#define UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(ApiDeclSpec, ClassName)\
	public:\
		using FCameraObjectTypeID = ::UE::Cameras::FCameraObjectTypeID;\
		static const ::UE::Cameras::TCameraObjectTypeID<ClassName>& StaticTypeID() { return ClassName::PrivateTypeID; }\
		virtual const FCameraObjectTypeID& GetTypeID() const { return ClassName::PrivateTypeID; }\
		virtual bool IsKindOf(const FCameraObjectTypeID& InTypeID) const { return InTypeID == ClassName::PrivateTypeID; }\
		template<typename Type> bool IsKindOf() const { return IsKindOf(Type::StaticTypeID()); }\
		template<typename Type> Type* CastThis() { return IsKindOf<Type>() ? static_cast<Type*>(this) : nullptr; }\
		template<typename Type> const Type* CastThis() const { return IsKindOf<Type>() ? static_cast<const Type*>(this) : nullptr; }\
		template<typename Type> Type* CastThisChecked() { check(IsKindOf<Type>()); return static_cast<Type*>(this); }\
		template<typename Type> const Type* CastThisChecked() const { check(IsKindOf<Type>()); return static_cast<const Type*>(this); }\
	private:\
		ApiDeclSpec static const ::UE::Cameras::TCameraObjectTypeID<ClassName> PrivateTypeID;

#define UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)\
	public:\
		using Super = BaseClassName;\
		static const ::UE::Cameras::TCameraObjectTypeID<ClassName>& StaticTypeID() { return ClassName::PrivateTypeID; }\
		virtual const FCameraObjectTypeID& GetTypeID() const override { return ClassName::PrivateTypeID; }\
		virtual bool IsKindOf(const FCameraObjectTypeID& InTypeID) const override { return (InTypeID == ClassName::PrivateTypeID) || BaseClassName::IsKindOf(InTypeID); }\
	private:\
		ApiDeclSpec static const ::UE::Cameras::TCameraObjectTypeID<ClassName> PrivateTypeID;

#define UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)\
	const ::UE::Cameras::TCameraObjectTypeID<ClassName> ClassName::PrivateTypeID = ::UE::Cameras::TCameraObjectTypeID<ClassName>::RegisterType(#ClassName);

