// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Cameras
{

/**
 * Default traits for specific storages.
 */
template<typename BaseObjectType>
struct TCameraObjectStorageTraits
{
	static const uint32 DefaultPageCapacity = 128;
	static const uint32 DefaultPageAlignment = 32;
};

/**
 * A utility class that allocates and stores objects of, or derived from, 
 * a common base class. The storage is a paged buffer composed of one or
 * more pages. If the needed storage size and alignment are known ahead
 * of time, you can pre-allocate the first page appropriately and avoid
 * any further paging.
 */
template<typename BaseObjectType>
class TCameraObjectStorage
{
protected:

	TCameraObjectStorage();
	TCameraObjectStorage(TCameraObjectStorage&& Other);
	TCameraObjectStorage& operator=(TCameraObjectStorage&& Other);
	~TCameraObjectStorage();

	TCameraObjectStorage(const TCameraObjectStorage&) = delete;
	TCameraObjectStorage& operator=(const TCameraObjectStorage&) = delete;

protected:

	/**
	 * Creates an object of the given type. Will allocate a new page buffer
	 * if needed.
	 */
	template<typename ObjectType, typename ...ArgTypes>
	typename TEnableIf<
		TPointerIsConvertibleFromTo<ObjectType, BaseObjectType>::Value,
		ObjectType*>
		::Type
	BuildObject(ArgTypes&&... InArgs);

	/**
	 * Allocates memory for an object of the given size and alignment, but
	 * doesn't initialize anything in that memory block.
	 *
	 * After the caller has constructed the object in-place, they MUST call
	 * RegisterInitializedObject() with the actual object pointer, otherwise 
	 * the storage won't call its destructor.
	 */
	void* BuildObjectUninitialized(uint32 Sizeof, uint32 Alignof);

	/**
	 * Called to register an object after it has been constructed.
	 */
	void RegisterInitializedObject(BaseObjectType* BaseObjectPtr);

	/**
	 * Destroys a given object.
	 */
	void DestroyObject(BaseObjectType* BaseObjectPtr, uint32 MemoryResetSize = 0);

	/**
	 * Destroys all objects in the storage.
	 *
	 * @param bFreeAllocations Whether to also free the memory buffers
	 */
	void DestroyObjects(bool bFreeAllocations = false);

	/**
	 * Computes information about the overall allocated memory.
	 */
	void GetAllocationInfo(uint32& OutTotalUsed, uint32& OutFirstAlignment) const;

	/**
	 * Allocates a new page buffer.
	 */
	void AllocatePage(uint32 InCapacity, uint32 InAlignment);

private:

	/** Allocation page */
	struct FAllocation
	{
		uint8* Memory = nullptr;
		uint32 Alignment = 0;
		uint32 Capacity = 0;
		uint32 Used = 0;
	};
	/** Allocated page buffers */
	TArray<FAllocation> Allocations;

	/** Pointer and info of objects inside the page buffers */
	struct FObjectInfo
	{
		BaseObjectType* Ptr = nullptr;
	};
	/** List of built objects */
	TArray<FObjectInfo> ObjectInfos;
};

template<typename BaseObjectType>
TCameraObjectStorage<BaseObjectType>::TCameraObjectStorage()
{
}

template<typename BaseObjectType>
TCameraObjectStorage<BaseObjectType>::TCameraObjectStorage(TCameraObjectStorage&& Other)
{
	Allocations = MoveTemp(Other.Allocations);
	ObjectInfos = MoveTemp(Other.ObjectInfos);
}

template<typename BaseObjectType>
TCameraObjectStorage<BaseObjectType>& TCameraObjectStorage<BaseObjectType>::operator=(TCameraObjectStorage&& Other)
{
	if (ensure(this != &Other))
	{
		Allocations = MoveTemp(Other.Allocations);
		ObjectInfos = MoveTemp(Other.ObjectInfos);
	}
	return *this;
}

template<typename BaseObjectType>
TCameraObjectStorage<BaseObjectType>::~TCameraObjectStorage()
{
	DestroyObjects(true);
}

template<typename BaseObjectType>
template<typename ObjectType, typename ...ArgTypes>
typename TEnableIf<TPointerIsConvertibleFromTo<ObjectType, BaseObjectType>::Value, ObjectType*>::Type
TCameraObjectStorage<BaseObjectType>::BuildObject(ArgTypes&&... InArgs)
{
	const uint32 Sizeof = sizeof(ObjectType);
	const uint32 Alignof = alignof(ObjectType);
	void* TargetPtr = BuildObjectUninitialized(Sizeof, Alignof);

	ObjectType* NewObject = new(TargetPtr) ObjectType(Forward<ArgTypes>(InArgs)...);

	RegisterInitializedObject(NewObject);

	return NewObject;
}

template<typename BaseObjectType>
void* TCameraObjectStorage<BaseObjectType>::BuildObjectUninitialized(uint32 Sizeof, uint32 Alignof)
{
	// Search for any allocation bucket that has enough room for the object
	// we want to build.
	uint8* TargetPtr = nullptr;
	FAllocation* TargetAllocation = nullptr;
	for (FAllocation& Allocation : Allocations)
	{
		uint8* PossiblePtr = Align(Allocation.Memory + Allocation.Used, Alignof);
		uint32 NewUsed = (PossiblePtr + Sizeof) - Allocation.Memory;
		if (NewUsed <= Allocation.Capacity)
		{
			TargetPtr = PossiblePtr;
			TargetAllocation = &Allocation;
			TargetAllocation->Used = NewUsed;
			break;
		}
	}

	// If we didn't find anything, we need to make a new allocation bucket.
	if (TargetPtr == nullptr)
	{
		using FStorageTraits = TCameraObjectStorageTraits<BaseObjectType>;
		const uint32 DefaultCapacity = FStorageTraits::DefaultPageCapacity;
		const uint32 DefaultAlignment = FStorageTraits::DefaultPageAlignment;

		const uint32 NewCapacity = FMath::Max(DefaultCapacity, Sizeof);
		const uint32 NewAlignment = FMath::Max(DefaultAlignment, Alignof);

		FAllocation& NewAllocation = Allocations.Emplace_GetRef();
		NewAllocation.Memory = reinterpret_cast<uint8*>(FMemory::Malloc(NewCapacity, NewAlignment));
		NewAllocation.Alignment = NewAlignment;
		NewAllocation.Capacity = NewCapacity;
		NewAllocation.Used = Sizeof;

		TargetPtr = NewAllocation.Memory;
		TargetAllocation = &NewAllocation;
	}
	check(TargetPtr && TargetAllocation);

	return TargetPtr;
}

template<typename BaseObjectType>
void TCameraObjectStorage<BaseObjectType>::RegisterInitializedObject(BaseObjectType* BaseObjectPtr)
{
	ObjectInfos.Add({ BaseObjectPtr });
}

template<typename BaseObjectType>
void TCameraObjectStorage<BaseObjectType>::DestroyObject(BaseObjectType* BaseObjectPtr, uint32 MemoryResetSize)
{
	// Check that this is an object we have allocated ourselves.
	const int32 NumRemoved = ObjectInfos.RemoveAll([BaseObjectPtr](FObjectInfo& ObjectInfo)
			{
				return ObjectInfo.Ptr == BaseObjectPtr;
			});
	ensureMsgf(NumRemoved == 1, TEXT("Given object pointer isn't in this storage, or was found multiple times."));
	if (NumRemoved > 0)
	{
		// Destroy the object.
		BaseObjectPtr->~BaseObjectType();

		// Optionally, reset the object's memory, trusting that the caller is giving us the right size.
		// In that case, first check that the memory to reset is contained within our allocation pages.
		if (MemoryResetSize > 0)
		{
			const FAllocation* FoundAllocation = Allocations.FindByPredicate(
					[BaseObjectPtr](const FAllocation& Allocation)
					{
						return (
								Allocation.Memory <= (uint8*)BaseObjectPtr && 
								Allocation.Memory + Allocation.Used > (uint8*)BaseObjectPtr);
					});
			if (ensure(FoundAllocation && (uint8*)BaseObjectPtr + MemoryResetSize <= FoundAllocation->Memory + FoundAllocation->Used))
			{
				FMemory::Memzero((void*)BaseObjectPtr, MemoryResetSize);
			}
		}
	}
}

template<typename BaseObjectType>
void TCameraObjectStorage<BaseObjectType>::DestroyObjects(bool bFreeAllocations)
{
	// Destroy the objects.
	for (FObjectInfo& ObjectInfo : ObjectInfos)
	{
		BaseObjectType* ObjectPtr(ObjectInfo.Ptr);
		ObjectPtr->~BaseObjectType();
	}
	ObjectInfos.Reset();

	// Either destroy the allocations, or reset them to unused.
	if (bFreeAllocations)
	{
		for (FAllocation& Allocation : Allocations)
		{
			FMemory::Free(Allocation.Memory);
		}
		Allocations.Reset();
	}
	else
	{
		for (FAllocation& Allocation : Allocations)
		{
			Allocation.Used = 0;
		}
	}
}

template<typename BaseObjectType>
void TCameraObjectStorage<BaseObjectType>::GetAllocationInfo(uint32& OutTotalUsed, uint32& OutFirstAlignment) const
{
	OutTotalUsed = 0;
	OutFirstAlignment = 0;

	if (!Allocations.IsEmpty())
	{
		OutFirstAlignment = Allocations[0].Alignment;

		for (const FAllocation& Allocation : Allocations)
		{
			if (OutTotalUsed > 0)
			{
				OutTotalUsed = Align(OutTotalUsed, Allocation.Alignment);
			}
			OutTotalUsed += Allocation.Used;
		}
	}
}

template<typename BaseObjectType>
void TCameraObjectStorage<BaseObjectType>::AllocatePage(uint32 InCapacity, uint32 InAlignment)
{
	check(InCapacity > 0 && InAlignment > 0);
	FAllocation& NewAllocation = Allocations.Emplace_GetRef();
	NewAllocation.Memory = reinterpret_cast<uint8*>(FMemory::Malloc(InCapacity, InAlignment));
	NewAllocation.Alignment = InAlignment;
	NewAllocation.Capacity = InCapacity;
	NewAllocation.Used = 0;
}

}  // namespace UE::Cameras

