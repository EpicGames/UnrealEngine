// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/ConsumableValue.h"

namespace UE::Cameras
{

/**
 * Hard-coded type IDs for built-in camera operations.
 */
enum EBuiltInCameraOperationTypes : uint32
{
	YawPitch = 0,
	SingleValue = 1,
	LockUserInput = 2,

	MAX = 3
};

/**
 * Simple type ID for camera operations.
 */
struct FCameraOperationTypeID
{
	FCameraOperationTypeID() : Value(INVALID) {}

	FCameraOperationTypeID(uint32 InValue) : Value(InValue) {}

public:

	/** Generate a new operation type ID. */
	GAMEPLAYCAMERAS_API static FCameraOperationTypeID RegisterNew();

	/** Returns whether this ID is valid. */
	bool IsValid() const { return Value != INVALID; }

	friend bool operator==(FCameraOperationTypeID A, FCameraOperationTypeID B)
	{
		return A.Value == B.Value;
	}

private:

	static const uint32 INVALID = (uint32)-1;

	uint32 Value;
};

/**
 * Base class for an operation to be executed on a camera rig.
 */
struct FCameraOperation
{
public:

	/** Attempts to cast this operation into a sub-class. */
	template<typename OperationType>
	OperationType* CastOperation()
	{
		if (OperationTypeID == OperationType::GetOperationTypeID())
		{
			return static_cast<OperationType*>(this);
		}
		return nullptr;
	}

protected:

	FCameraOperation(FCameraOperationTypeID InOperationTypeID)
		: OperationTypeID(InOperationTypeID)
	{}

private:

	FCameraOperationTypeID OperationTypeID;
};

#define UE_DECLARE_BUILT_IN_CAMERA_OPERATION(OperationName, OperationType)\
	static FCameraOperationTypeID GetOperationTypeID()\
	{\
		return FCameraOperationTypeID(OperationType);\
	}\
	OperationName()\
		: FCameraOperation(OperationName::GetOperationTypeID())\
	{}

/**
 * A camera operation that tries to correct the yaw/pitch of a camera rig.
 */
struct FYawPitchCameraOperation : public FCameraOperation
{
	UE_DECLARE_BUILT_IN_CAMERA_OPERATION(FYawPitchCameraOperation, EBuiltInCameraOperationTypes::YawPitch)

	/** The yaw value to set or use as modifier. */
	FConsumableDouble Yaw;
	/** The pitch value to set or use as modifier. */
	FConsumableDouble Pitch;
};

/**
 * A camera operation that tries to correct a single undetermined input value on a
 * camera rig.
 */
struct FSingleValueCameraOperation : public FCameraOperation
{
	UE_DECLARE_BUILT_IN_CAMERA_OPERATION(FSingleValueCameraOperation, EBuiltInCameraOperationTypes::SingleValue)

	FConsumableDouble Value;
};

/**
 * A camera operation that locks user input for a frame.
 */
struct FLockUserInputCameraOperation : public FCameraOperation
{
	UE_DECLARE_BUILT_IN_CAMERA_OPERATION(FLockUserInputCameraOperation, EBuiltInCameraOperationTypes::LockUserInput)

	/** Whether the targeted nodes should lock their input value for this frame. */
	bool bLockThisFrame = true;
};

#undef UE_DECLARE_BUILT_IN_CAMERA_OPERATION

}  // namespace UE::Cameras

#define UE_DECLARE_CAMERA_OPERATION(ApiDeclSpec, OperationName)\
	public:\
		ApiDeclSpec static UE::Cameras::FCameraOperationTypeID GetOperationTypeID();\
		OperationName()\
			: FCameraOperation(OperationName::GetOperationTypeID())\
		{}

#define UE_DEFINE_CAMERA_OPERATION(OperationName)\
	UE::Cameras::FCameraOperationTypeID OperationName::GetOperationTypeID()\
	{\
		static UE::Cameras::FCameraOperationTypeID StaticOperationTypeID = UE::Cameras::FCameraOperationTypeID::RegisterNew();\
		return StaticOperationTypeID;\
	}\

