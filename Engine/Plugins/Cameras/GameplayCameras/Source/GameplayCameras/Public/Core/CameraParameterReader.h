// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTable.h"
#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"

namespace UE::Cameras
{

/**
 * A utility class for reading the effective value of a camera parameter.
 */
template<typename ValueType>
class TCameraParameterReader
{
public:

	TCameraParameterReader() {}

	template<typename ParameterType>
	TCameraParameterReader(const ParameterType& Parameter)
	{
		Initialize(Parameter);
	}

	/**
	 * Initializes the reader around the given parameter.
	 */
	template<typename ParameterType>
	void Initialize(const ParameterType& Parameter)
	{
		static_assert(
				std::is_same<ValueType, typename ParameterType::ValueType>(),
				"The given parameter is of the wrong type for this reader! Value types must be the same.");

		DefaultValuePtr = &Parameter.Value;
		VariableID = Parameter.VariableID;

		ensureMsgf(DefaultValuePtr, TEXT("The given parameter doesn't have a value!"));
	}

	/**
	 * Gets the actual value for the parameter.
	 */
	const ValueType& Get(const FCameraVariableTable& VariableTable) const
	{
		checkf(DefaultValuePtr, TEXT("Parameter reader has no value pointer!"));
		if (!VariableID.IsValid())
		{
			// No variable is driving the parameter, just return the parameter value.
			return *DefaultValuePtr;
		}
		else
		{
			// The parameter is driven by a variable. Find it in the variable table.
			if (const ValueType* ActualValue = VariableTable.FindValue<ValueType>(VariableID))
			{
				return *ActualValue;
			}
			return *DefaultValuePtr;
		}
	}

	/**
	 * Returns whether the parameter is driven by a variable.
	 */
	bool IsDriven() const
	{
		return VariableID.IsValid();
	}

private:

	/** Pointer to the value in the parameter. */
	const ValueType* DefaultValuePtr = nullptr;
	/** The ID of the variable driving the parameter, if any. */
	FCameraVariableID VariableID;
};

}  // namespace UE::Cameras

