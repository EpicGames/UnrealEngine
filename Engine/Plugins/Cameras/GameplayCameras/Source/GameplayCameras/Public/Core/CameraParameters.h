// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableAssets.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"

#include "CameraParameters.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{
	class FCameraVariableTable;
}

#define UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(ParameterClass)\
	ParameterClass(typename TCallTraits<ParameterClass::ValueType>::ParamType InValue)\
		: Value(InValue)\
	{}\
	bool HasOverride() const { return VariableID.IsValid(); }\
	bool HasUserOverride() const { return Variable != nullptr; }\
	bool HasNonUserOverride() const { return VariableID.IsValid() && (!Variable || Variable->GetVariableID() != VariableID); }\
	ParameterClass::ValueType GetValue(const UE::Cameras::FCameraVariableTable& VariableTable) const;\
	void PostSerialize(const FArchive& Ar);

#define UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(ParameterClass)\
	ParameterClass() {}\
	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(ParameterClass)

// All camera parameters have:
//
// - Value: a value for the user to tweak. This is the "default" value.
// - Variable: a variable chosen by the user to drive this parameter. 
// - VariableID: the ID of the variable driving this parameter.
//
// When Variable is set, VariableID is the ID of that variable.
// When Variable is not set, VariableID is the ID of something else
// driving the parameter, such as a camera rig parameter override.

/** Boolean camera parameter. */
USTRUCT(BlueprintType)
struct FBooleanCameraParameter
{
	GENERATED_BODY()

	using ValueType = bool;
	using VariableAssetType = UBooleanCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	bool Value = false;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UBooleanCameraVariable> Variable;

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FBooleanCameraParameter)
};

/** Integer camera parameter. */
USTRUCT(BlueprintType)
struct FInteger32CameraParameter
{
	GENERATED_BODY()

	using ValueType = int32;
	using VariableAssetType = UInteger32CameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	int32 Value = 0;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UInteger32CameraVariable> Variable;

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FInteger32CameraParameter)
};

/** Float camera parameter. */
USTRUCT(BlueprintType)
struct FFloatCameraParameter
{
	GENERATED_BODY()

	using ValueType = float;
	using VariableAssetType = UFloatCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	float Value = 0.f;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UFloatCameraVariable> Variable;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FFloatCameraParameter)
};

/** Double camera parameter. */
USTRUCT(BlueprintType)
struct FDoubleCameraParameter
{
	GENERATED_BODY()

	using ValueType = double;
	using VariableAssetType = UDoubleCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	double Value = 0.0;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UDoubleCameraVariable> Variable;

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FDoubleCameraParameter)
};

/** Vector2f camera parameter. */
USTRUCT(BlueprintType)
struct FVector2fCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector2f;
	using VariableAssetType = UVector2fCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector2f Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector2fCameraVariable> Variable;

	UE_API FVector2fCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector2fCameraParameter)
};

/** Vector2d camera parameter. */
USTRUCT(BlueprintType)
struct FVector2dCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector2D;
	using VariableAssetType = UVector2dCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector2D Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector2dCameraVariable> Variable;

	UE_API FVector2dCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector2dCameraParameter)
};

/** Vector3f camera parameter. */
USTRUCT(BlueprintType)
struct FVector3fCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector3f;
	using VariableAssetType = UVector3fCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector3f Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector3fCameraVariable> Variable;

	UE_API FVector3fCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector3fCameraParameter)
};

/** Vector3d camera parameter. */
USTRUCT(BlueprintType)
struct FVector3dCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector;
	using VariableAssetType = UVector3dCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector3dCameraVariable> Variable;

	UE_API FVector3dCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector3dCameraParameter)
};

/** Vector4f camera parameter. */
USTRUCT(BlueprintType)
struct FVector4fCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector4f;
	using VariableAssetType = UVector4fCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector4f Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector4fCameraVariable> Variable;

	UE_API FVector4fCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector4fCameraParameter)
};

/** Vector4d camera parameter. */
USTRUCT(BlueprintType)
struct FVector4dCameraParameter
{
	GENERATED_BODY()

	using ValueType = FVector4;
	using VariableAssetType = UVector4dCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FVector4 Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UVector4dCameraVariable> Variable;

	UE_API FVector4dCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FVector4dCameraParameter)
};

/** Rotator3f camera parameter. */
USTRUCT(BlueprintType)
struct FRotator3fCameraParameter
{
	GENERATED_BODY()

	using ValueType = FRotator3f;
	using VariableAssetType = URotator3fCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FRotator3f Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<URotator3fCameraVariable> Variable;

	UE_API FRotator3fCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FRotator3fCameraParameter)
};

/** Rotator3d camera parameter. */
USTRUCT(BlueprintType)
struct FRotator3dCameraParameter
{
	GENERATED_BODY()

	using ValueType = FRotator;
	using VariableAssetType = URotator3dCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FRotator Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<URotator3dCameraVariable> Variable;

	UE_API FRotator3dCameraParameter();
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS(FRotator3dCameraParameter)
};

/** Transform3f camera parameter. */
USTRUCT(BlueprintType)
struct FTransform3fCameraParameter
{
	GENERATED_BODY()

	using ValueType = FTransform3f;
	using VariableAssetType = UTransform3fCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FTransform3f Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UTransform3fCameraVariable> Variable;

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FTransform3fCameraParameter)
};

/** Transform3d camera parameter. */
USTRUCT(BlueprintType)
struct FTransform3dCameraParameter
{
	GENERATED_BODY()

	using ValueType = FTransform;
	using VariableAssetType = UTransform3dCameraVariable;

	UPROPERTY(EditAnywhere, Interp, Category=Common, meta=(SequencerUseParentPropertyName=true))
	FTransform Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UTransform3dCameraVariable> Variable;

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS(FTransform3dCameraParameter)
};

#undef UE_DEFINE_CAMERA_PARAMETER_VALUE_CONSTRUCTORS
#undef UE_DEFINE_CAMERA_PARAMETER_ALL_CONSTRUCTORS

template<typename ValueType>
bool CameraParameterValueEquals(typename TCallTraits<ValueType>::ParamType A, typename TCallTraits<ValueType>::ParamType B)
{
	return A == B;
}

template<>
inline bool CameraParameterValueEquals<FTransform3f>(const FTransform3f& A, const FTransform3f& B)
{
	return A.Equals(B);
}

template<>
inline bool CameraParameterValueEquals<FTransform3d>(const FTransform3d& A, const FTransform3d& B)
{
	return A.Equals(B);
}

// Any camera parameter might replace a previously non-parameterized property (i.e. a "fixed" property
// of the underlying type, like bool, int32, float, etc.)
// When someone upgrades the fixed property to a parameterized property, any previously saved data will
// run into a mismatched tag. So the parameters will handle that by loading the saved value inside of
// them.
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	template<> struct TStructOpsTypeTraits<F##ValueName##CameraParameter>\
		: public TStructOpsTypeTraitsBase2<F##ValueName##CameraParameter>\
	{\
		enum { WithStructuredSerializeFromMismatchedTag = true, WithPostSerialize = true };\
	};
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

#undef UE_API
