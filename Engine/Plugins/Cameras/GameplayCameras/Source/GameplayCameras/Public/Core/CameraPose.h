// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraTypes.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"

#include "CameraPose.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class FArchive;
struct FPostProcessSettings;
enum EAspectRatioAxisConstraint : int;

#define UE_CAMERA_POSE_FOR_TRANSFORM_PROPERTIES()\
	UE_CAMERA_POSE_FOR_PROPERTY(FVector, Location)\
	UE_CAMERA_POSE_FOR_PROPERTY(FRotator3d, Rotation)

#define UE_CAMERA_POSE_FOR_INTERPOLABLE_PROPERTIES()\
	UE_CAMERA_POSE_FOR_PROPERTY(double, TargetDistance)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  Aperture)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  ShutterSpeed)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  FocusDistance)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  SensorWidth)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  SensorHeight)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  SensorHorizontalOffset)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  SensorVerticalOffset)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  ISO)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  SqueezeFactor)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  Overscan)\
	UE_CAMERA_POSE_FOR_PROPERTY(int32,  DiaphragmBladeCount)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  PhysicalCameraBlendWeight)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  NearClippingPlane)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  FarClippingPlane)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  OrthographicWidth)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  FirstPersonFieldOfView)\
	UE_CAMERA_POSE_FOR_PROPERTY(float,  FirstPersonScale)

#define UE_CAMERA_POSE_FOR_FOV_PROPERTIES()\
	UE_CAMERA_POSE_FOR_PROPERTY(float, FieldOfView)\
	UE_CAMERA_POSE_FOR_PROPERTY(float, FocalLength)

#define UE_CAMERA_POSE_FOR_FLIPPING_PROPERTIES()\
	UE_CAMERA_POSE_FOR_PROPERTY(bool, EnableFirstPerson)\
	UE_CAMERA_POSE_FOR_PROPERTY(bool, EnablePhysicalCamera)\
	UE_CAMERA_POSE_FOR_PROPERTY(bool, ConstrainAspectRatio)\
	UE_CAMERA_POSE_FOR_PROPERTY(bool, OverrideAspectRatioAxisConstraint)\
	UE_CAMERA_POSE_FOR_PROPERTY(EAspectRatioAxisConstraint, AspectRatioAxisConstraint)\
	UE_CAMERA_POSE_FOR_PROPERTY(ECameraProjectionMode::Type, ProjectionMode)

#define UE_CAMERA_POSE_FOR_ALL_PROPERTIES()\
	UE_CAMERA_POSE_FOR_TRANSFORM_PROPERTIES()\
	UE_CAMERA_POSE_FOR_INTERPOLABLE_PROPERTIES()\
	UE_CAMERA_POSE_FOR_FOV_PROPERTIES()\
	UE_CAMERA_POSE_FOR_FLIPPING_PROPERTIES()

/**
 * Boolean flags for each of the properties inside FCameraPose.
 */
struct FCameraPoseFlags
{
#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	bool PropName:1 = false;

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

public:

	/** Returns a structure where all flags are set. */
	static UE_API const FCameraPoseFlags& All();

	/** Creates a new flags structure. */
	UE_API FCameraPoseFlags();
	/** Creates a new flags structure with all flags set to the given value. */
	UE_API FCameraPoseFlags(bool bInValue);

	/** Sets all flags to the given value. */
	UE_API FCameraPoseFlags& SetAllFlags(bool bInValue);
	/** Sets the flags that are set in OtherFlags, but checks that no flag is set on both structures. */
	UE_API FCameraPoseFlags& ExclusiveCombine(const FCameraPoseFlags& OtherFlags);

	/** Combines the flags with an AND logical operation. */
	UE_API FCameraPoseFlags& AND(const FCameraPoseFlags& OtherFlags);
	/** Combines the flags with an OR logical operation. */
	UE_API FCameraPoseFlags& OR(const FCameraPoseFlags& OtherFlags);
};

/**
 * Structure describing the state of a camera.
 *
 * Fields are private and can only be accessed via the getters and setters.
 * The ChangedFlags structure keeps track of which fields were changed via the setters.
 */
USTRUCT()
struct FCameraPose
{
	GENERATED_BODY()

public:

	UE_API FCameraPose();

	/** Resets this camera pose to its default values, with all changed flags off. */
	UE_API void Reset();

public:

	// Getters and setters

#define UE_CAMERA_POSE_FOR_PROPERTY(PropType, PropName)\
	PropType Get##PropName() const\
	{\
		return PropName;\
	}\
	void Set##PropName(TCallTraits<PropType>::ParamType InValue, bool bForceSet = false)\
	{\
		if (bForceSet || PropName != InValue)\
		{\
			ChangedFlags.PropName = true;\
			PropName = InValue;\
		}\
	}

UE_CAMERA_POSE_FOR_ALL_PROPERTIES()

#undef UE_CAMERA_POSE_FOR_PROPERTY

public:

	// Changed flags management

	/** Get the changed flags. */
	FCameraPoseFlags& GetChangedFlags() { return ChangedFlags; }
	/** Get the changed flags. */
	const FCameraPoseFlags& GetChangedFlags() const { return ChangedFlags; }

	/** Set the changed flags. */
	void SetChangedFlags(const FCameraPoseFlags& InChangedFlags) { ChangedFlags = InChangedFlags; }
	/** Set all fields as changed. */
	UE_API void SetAllChangedFlags();
	/** Set all fields as clean. */
	UE_API void ClearAllChangedFlags();

public:

	// Utility

	/** Gets the transform of the camera. */
	UE_API FTransform3d GetTransform() const;

	/** Sets the transform of the camera. */
	UE_API void SetTransform(FTransform3d Transform, bool bForceSet = false);

	/**
	 * Computes the horizontal field of view of the camera.
	 * The effective field of view can be driven by the FieldOfView property, or
	 * the FocalLength property in combination with the sensor size.
	 */
	UE_API double GetEffectiveFieldOfView(bool bIncludeOverscan = true) const;

	/** Gets the aspect ratio of the camera sensor. */
	UE_API double GetSensorAspectRatio() const;

	/** Gets the aspect ratio axis constraint if overriden, otherwise a default one. */
	UE_API EAspectRatioAxisConstraint GetEffectiveAspectRatioAxisConstraint() const;

	/** Gets the aspect ratio axis constraint if overriden, otherwise the provided default one. */
	UE_API EAspectRatioAxisConstraint GetEffectiveAspectRatioAxisConstraint(EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint) const;

	/** Gets the horizontal projection offset. */
	UE_API double GetHorizontalProjectionOffset() const;

	/** Gets the vertical projection offset. */
	UE_API double GetVerticalProjectionOffset() const;

	/** Gets the aiming ray of the camera. */
	UE_API FRay3d GetAimRay() const;

	/** Gets the aiming direction of the camera. */
	UE_API FVector3d GetAimDir() const;

	/** Gets the location of the camera's target. */
	UE_API FVector3d GetTarget() const;

	/** Gets the location of the camera's target given a specific distance. */
	UE_API FVector3d GetTarget(double InTargetDistance) const;

public:

	/** Computes the horizontal field of view of a camera. */
	static UE_API double GetEffectiveFieldOfView(float FocalLength, float FieldOfView, float SensorWidth, float SensorHeight, float SqueezeFactor = 1.f, float Overscan = 1.f);
	
	/** Computes the aspect ratio of a camera sensor. */
	static UE_API double GetSensorAspectRatio(float SensorWidth, float SensorHeight);

	/** Gets the default sensor size. */
	static UE_API void GetDefaultSensorSize(float& OutSensorWidth, float& OutSensorHeight);

	/** 
	 * Exports the current camera pose to the given view info.
	 */
	UE_API void GetViewInfo(FMinimalViewInfo& OutViewInfo) const;

	/**
	 * Applies the necessary post-process settings given the current values
	 * on this camera pose.
	 *
	 * This function doesn't do anything if EnablePhysicalCamera is false, or if
	 * PhysicalCameraBlendWeight is zero or less.
	 *
	 * @param PostProcessSettings  The post-process settings to modify
	 * @param bOverwriteSettings   Whether to overwrite values found to already be set
	 * @return  Whether post-process settings were created.
	 */
	UE_API bool ApplyPhysicalCameraSettings(FPostProcessSettings& PostProcessSettings, bool bOverwriteSettings = false) const;

public:

	// Interpolation
	
	/** Takes all properties from OtherPose and sets them on this camera pose. */
	UE_API void OverrideAll(const FCameraPose& OtherPose);
	/** Takes all changed properties from OtherPose and sets them on this camera pose. */
	UE_API void OverrideChanged(const FCameraPose& OtherPose);
	/** Interpolates all properties from ToPose using the given factor. */
	UE_API void LerpAll(const FCameraPose& ToPose, float Factor);
	/** Interpolates all changed properties from ToPose using the given factor. */
	UE_API void LerpChanged(const FCameraPose& ToPose, float Factor);
	/** Interpolates changed properties from ToPose using the given factor. Only properties defined by InMask are taken into account. */
	UE_API void LerpChanged(const FCameraPose& ToPose, float Factor, const FCameraPoseFlags& InMask, bool bInvertMask, FCameraPoseFlags& OutMask);

public:

	/** Serializes the given camera pose including the written-property flags. */
	static UE_API void SerializeWithFlags(FArchive& Ar, FCameraPose& CameraPose);

	/** Serializes this camera pose including the written-property flags. */
	UE_API void SerializeWithFlags(FArchive& Ar);

private:

	UE_API void InternalOverrideChanged(const FCameraPose& OtherPose, bool bChangedOnly);
	UE_API void InternalLerpChanged(const FCameraPose& ToPose, float Factor, const FCameraPoseFlags& InMask, bool bInvertMask, FCameraPoseFlags& OutMask, bool bChangedOnly);

private:

	/** The location of the camera in the world */
	UPROPERTY()
	FVector3d Location = {0, 0, 0};

	/** The rotation of the camera in the world */
	UPROPERTY()
	FRotator3d Rotation = {0, 0, 0};

	/** Distance to the target */
	UPROPERTY()
	double TargetDistance = 1000.0;

	/**
	 * The horizontal field of view of the camera, in degrees 
	 * If zero or less, focal length is used instead
	 */
	UPROPERTY()
	float FieldOfView = -1.f;  // Default to using a focal length

	/**
	 * The focal length of the camera's lens, in millimeters
	 * If zero or less, field of view is used instead
	 */
	UPROPERTY()
	float FocalLength = 35.f;

	/** The horizontal field of view (in degrees) used for primitives tagged as "IsFirstPerson" */
	UPROPERTY()
	float FirstPersonFieldOfView = -1.f;

	/** The scale to apply to primitives tagged as "IsFirstPerson" */
	UPROPERTY()
	float FirstPersonScale = 1.f;
	
	/** The desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY()
	float OrthographicWidth = DEFAULT_ORTHOWIDTH;

	/** The aperture of the camera's lens, in f-stops */
	UPROPERTY()
	float Aperture = 2.8f;

	/** The shutter speed of the camera's lens, in 1/seconds */
	UPROPERTY()
	float ShutterSpeed = 60.f;

	/** The focus distance of the camera's lens, in world units */
	UPROPERTY()
	float FocusDistance = -1.f;

	/** The width of the camera's sensor, in millimeters */
	UPROPERTY()
	float SensorWidth = 24.89f;

	/** The height of the camera's sensor, in millimeters */
	UPROPERTY()
	float SensorHeight = 18.67f;

	/** Horizontal offset of the sensor, in mm. */
	UPROPERTY()
	float SensorHorizontalOffset = 0.f;

	/** Vertical offset of the sensor, in mm. */
	UPROPERTY()
	float SensorVerticalOffset = 0.f;

	/** The camera sensor sensitivity in ISO. */
	UPROPERTY()
	float ISO = 100.f;

	/** Squeeze factor for anamorphic lenses */
	UPROPERTY()
	float SqueezeFactor = 1.f;

	/** Sensor overscan in percentage */
	UPROPERTY()
	float Overscan = 0.f;

	/** Number of blades in the lens diaphragm */
	UPROPERTY()
	int32 DiaphragmBladeCount = 8;

	/** The distance to the near clipping plane, in world units */
	UPROPERTY()
	float NearClippingPlane = 10.f;

	/** The distance to the far clipping plane, in world units */
	UPROPERTY()
	float FarClippingPlane = -1.f;

	/** 
	 * An internal weight for the physical camera post-process settings, used when blending between 
	 * cameras with EnablePhysicalCamera enabled/disabled.
	 */
	UPROPERTY()
	float PhysicalCameraBlendWeight = 0.f;

	/** Whether to enable first-person parameters (FOV, scale) for primitives tagged as "IsFirstPerson" */
	UPROPERTY()
	bool EnableFirstPerson = false;

	/** 
	 * Whether to setup post-process settings based on physical camera properties such as Aperture,
	 * FocusDistance, DiaphragmBladeCount, and so on.
	 */
	UPROPERTY()
	bool EnablePhysicalCamera = false;

	/** Whether to constrain aspect ratio */
	UPROPERTY()
	bool ConstrainAspectRatio = false;

	/** Whether to override the default aspect ratio axis constraint defined on the player controller */
	UPROPERTY()
	bool OverrideAspectRatioAxisConstraint = false;

	/** If ConstrainAspectRatio is false and OverrideAspectRatioAxisConstraint is true, how we should compute FieldOfView */
	UPROPERTY()
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV;

	/** Projection mode - Perspective or orthographic */
	UPROPERTY()
	TEnumAsByte<ECameraProjectionMode::Type> ProjectionMode = ECameraProjectionMode::Perspective;

private:
	
	/**
	 * Flags keeping track of which properties were written to since
	 * last time the flags were cleared.
	 */
	FCameraPoseFlags ChangedFlags;
};

#undef UE_API
