// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildStatus.h"
#include "Core/BaseCameraObject.h"
#include "Core/CameraEventHandler.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraRigTransition.h"
#include "Core/ObjectTreeGraphObject.h"
#include "Core/ObjectTreeGraphRootObject.h"
#include "CoreTypes.h"
#include "GameplayTagAssetInterface.h"
#include "GameplayTagContainer.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigAsset.generated.h"

class UCameraNode;
class UCameraRigAsset;
class UCameraVariableAsset;

namespace UE::Cameras
{
	class FCameraRigAssetBuilder;
	struct FCameraBuildContext;

	/**
	 * Interface for listening to changes on a camera rig asset.
	 */
	class ICameraRigAssetEventHandler
	{
	public:
		virtual ~ICameraRigAssetEventHandler() = default;

		/** Called when the camera rig asset has been built. */
		virtual void OnCameraRigBuilt(const UCameraRigAsset* CameraRigAsset) {}

#if WITH_EDITOR
		virtual void OnObjectAddedToGraph(const FName GraphName, UObject* Object) {}
		virtual void OnObjectRemovedFromGraph(const FName GraphName, UObject* Object) {}
#endif  // WITH_EDITOR
	};
}

/**
 * List of packages that contain the definition of a camera rig.
 * In most cases there's only one, but with nested assets there could be more.
 */
using FCameraRigPackages = TArray<const UPackage*, TInlineAllocator<4>>;

/**
 * A camera rig asset, which runs a hierarchy of camera nodes to drive 
 * the behavior of a camera.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCameraRigAsset
	: public UBaseCameraObject
	, public IGameplayTagAssetInterface
	, public IHasCameraBuildStatus
	, public IObjectTreeGraphObject
	, public IObjectTreeGraphRootObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	GAMEPLAYCAMERAS_API void GatherPackages(FCameraRigPackages& OutPackages) const;
#endif  // WITH_EDITOR

public:

	/** Root camera node. */
	UPROPERTY(Instanced)
	TObjectPtr<UCameraNode> RootNode;

	/** The gameplay tags on this camera rig. */
	UPROPERTY(EditAnywhere, Category=GameplayTags)
	FGameplayTagContainer GameplayTags;

	/** List of enter transitions for this camera rig. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraRigTransition>> EnterTransitions;

	/** List of exist transitions for this camera rig. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCameraRigTransition>> ExitTransitions;
	
	/** Default orientation initialization when this camera rig is activated. */
	UPROPERTY(EditAnywhere, Category="Transition")
	ECameraRigInitialOrientation InitialOrientation = ECameraRigInitialOrientation::None;

public:

	/** The current build state of this camera rig. */
	UPROPERTY(Transient)
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Dirty;

	/**
	 * Builds this camera rig.
	 * This will validate the data, build the allocation info, and create internal
	 * camera variables for any exposed parameters.
	 */
	GAMEPLAYCAMERAS_API void BuildCameraRig();

	/**
	 * Builds this camera rig, similar to BuildCameraRig() but using a given build context.
	 */
	GAMEPLAYCAMERAS_API void BuildCameraRig(UE::Cameras::FCameraBuildContext& InBuildContext);

public:

	// BaseCameraObject interface.
	virtual UCameraNode* GetRootNode() override { return RootNode; }

	// IGameplayTagAssetInterface interface.
	virtual void GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const override;

	// IHasCameraBuildStatus interface.
	virtual ECameraBuildStatus GetBuildStatus() const override { return BuildStatus; }
	virtual void DirtyBuildStatus() override;

public:

	// Graph names for ObjectTreeGraph API.
	GAMEPLAYCAMERAS_API static const FName NodeTreeGraphName;
	GAMEPLAYCAMERAS_API static const FName TransitionsGraphName;

	/** Event handlers to be notified of data changes. */
	UE::Cameras::TCameraEventHandlerContainer<UE::Cameras::ICameraRigAssetEventHandler> EventHandlers;

protected:

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override;
	virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const override;
#endif

	// IObjectTreeGraphRootObject interface.
#if WITH_EDITOR
	virtual void GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const override;
	virtual void AddConnectableObject(FName InGraphName, UObject* InObject) override;
	virtual void RemoveConnectableObject(FName InGraphName, UObject* InObject) override;
#endif

	// UObject interface
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif

private:

#if WITH_EDITORONLY_DATA

	/** Position of the camera rig node in the node graph editor. */
	UPROPERTY()
	FIntVector2 NodeGraphNodePos = FIntVector2::ZeroValue;

	/** Position of the camera rig node in the transition graph editor. */
	UPROPERTY()
	FIntVector2 TransitionGraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the node graph editor. */
	UPROPERTY()
	FString NodeGraphNodeComment;

	/** User-written comment in the transition graph editor. */
	UPROPERTY()
	FString TransitionGraphNodeComment;

	/** 
	 * A list of all the camera nodes, including the 'loose' ones that aren't connected
	 * to the root node, and therefore would be GC'ed if we didn't hold them here.
	 */
	UPROPERTY(Instanced, meta=(ObjectTreeGraphHidden=true))
	TArray<TObjectPtr<UObject>> AllNodeTreeObjects;

	/**
	 * Similar to AllNodeTreeObjects, but for the transitions graph.
	 */
	UPROPERTY(Instanced, meta=(ObjectTreeGraphHidden=true))
	TArray<TObjectPtr<UObject>> AllTransitionsObjects;

	// Deprecated properties.

	UPROPERTY()
	int32 GraphNodePosX_DEPRECATED = 0;
	UPROPERTY()
	int32 GraphNodePosY_DEPRECATED = 0;

#endif  // WITH_EDITORONLY_DATA

	friend class UE::Cameras::FCameraRigAssetBuilder;
};

