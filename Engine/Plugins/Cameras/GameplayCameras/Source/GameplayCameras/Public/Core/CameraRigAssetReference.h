// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCameraObjectReference.h"
#include "Core/CameraContextDataTable.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableTable.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigAssetReference.generated.h"

class UBaseCameraObject;
class UCameraRigAsset;
struct FCameraRigAssetReference;
struct FCustomCameraNodeParameterInfos;
struct FPropertyTag;

namespace UE::Cameras
{
	class FCameraRigAssetReferenceDetailsCustomization;
	struct FCameraNodeEvaluationResult;
}

struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FCameraRigParameterOverrideBase;

struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FBooleanCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FInteger32CameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FFloatCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FDoubleCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector2fCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector2dCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector3fCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector3dCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector4fCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FVector4dCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FRotator3fCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FRotator3dCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FTransform3fCameraRigParameterOverride;
struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FTransform3dCameraRigParameterOverride;

struct UE_DEPRECATED(5.7, "Camera rig references now use instanced property bags.") FCameraRigParameterOverrides;

USTRUCT()
struct FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InterfaceParameterGuid;

	UPROPERTY()
	FGuid PrivateVariableGuid;

	UPROPERTY()
	FString InterfaceParameterName;

	UPROPERTY()
	bool bInvalid = false;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

USTRUCT()
struct FBooleanCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FBooleanCameraParameter Value;
};

USTRUCT()
struct FInteger32CameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FInteger32CameraParameter Value;
};

USTRUCT()
struct FFloatCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FFloatCameraParameter Value;
};

USTRUCT()
struct FDoubleCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FDoubleCameraParameter Value;
};

USTRUCT()
struct FVector2fCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector2fCameraParameter Value;
};

USTRUCT()
struct FVector2dCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector2dCameraParameter Value;
};

USTRUCT()
struct FVector3fCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector3fCameraParameter Value;
};

USTRUCT()
struct FVector3dCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector3dCameraParameter Value;
};

USTRUCT()
struct FVector4fCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector4fCameraParameter Value;
};

USTRUCT()
struct FVector4dCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FVector4dCameraParameter Value;
};

USTRUCT()
struct FRotator3fCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FRotator3fCameraParameter Value;
};

USTRUCT()
struct FRotator3dCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FRotator3dCameraParameter Value;
};

USTRUCT()
struct FTransform3fCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FTransform3fCameraParameter Value;
};

USTRUCT()
struct FTransform3dCameraRigParameterOverride : public FCameraRigParameterOverrideBase
{
	GENERATED_BODY()

	UPROPERTY()
	FTransform3dCameraParameter Value;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * A structure that holds lists of camera rig interface parameter overrides, one list
 * per parameter type.
 */
USTRUCT()
struct FCameraRigParameterOverrides
{
	GENERATED_BODY()

private:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<FBooleanCameraRigParameterOverride> BooleanOverrides;
	UPROPERTY()
	TArray<FInteger32CameraRigParameterOverride> Integer32Overrides;
	UPROPERTY()
	TArray<FFloatCameraRigParameterOverride> FloatOverrides;
	UPROPERTY()
	TArray<FDoubleCameraRigParameterOverride> DoubleOverrides;
	UPROPERTY()
	TArray<FVector2fCameraRigParameterOverride> Vector2fOverrides;
	UPROPERTY()
	TArray<FVector2dCameraRigParameterOverride> Vector2dOverrides;
	UPROPERTY()
	TArray<FVector3fCameraRigParameterOverride> Vector3fOverrides;
	UPROPERTY()
	TArray<FVector3dCameraRigParameterOverride> Vector3dOverrides;
	UPROPERTY()
	TArray<FVector4fCameraRigParameterOverride> Vector4fOverrides;
	UPROPERTY()
	TArray<FVector4dCameraRigParameterOverride> Vector4dOverrides;
	UPROPERTY()
	TArray<FRotator3fCameraRigParameterOverride> Rotator3fOverrides;
	UPROPERTY()
	TArray<FRotator3dCameraRigParameterOverride> Rotator3dOverrides;
	UPROPERTY()
	TArray<FTransform3fCameraRigParameterOverride> Transform3fOverrides;
	UPROPERTY()
	TArray<FTransform3dCameraRigParameterOverride> Transform3dOverrides;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	friend struct FCameraRigAssetReference;
};

/**
 * A structure holding a reference to a camera rig asset, along with the interface parameter
 * override values.
 */
USTRUCT(BlueprintType)
struct FCameraRigAssetReference : public FBaseCameraObjectReference
{
	GENERATED_BODY()

public:

	GAMEPLAYCAMERAS_API FCameraRigAssetReference();
	GAMEPLAYCAMERAS_API FCameraRigAssetReference(UCameraRigAsset* InCameraRig);

	/** Returns whether this reference points to a valid camera rig. */
	bool IsValid() const
	{
		return CameraRig != nullptr;
	}

	/** Gets the referenced camera rig. */
	UCameraRigAsset* GetCameraRig()
	{
		return CameraRig;
	}

	/** Gets the referenced camera rig. */
	const UCameraRigAsset* GetCameraRig() const
	{
		return CameraRig;
	}

	/** Sets the referenced camera rig. */
	void SetCameraRig(UCameraRigAsset* InCameraRig)
	{
		if (CameraRig != InCameraRig)
		{
			CameraRig = InCameraRig;
			RebuildParameters();
		}
	}

public:

	/** Ensure the variable and data tables have the allocations for our referenced camera rig. */
	void EnsureAllocationInfo(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const;

	/** Applies the parameter override values to the given variable table. */
	void ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, bool bDrivenOnly) const;

	/** Applies the parameter override values to the given variable and context data tables. */
	void ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, UE::Cameras::FCameraContextDataTable& OutContextDataTable, bool bDrivenOnly) const;

	/** Applies the parameter override values to the given evaluation result. */
	void ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const;

	/** Applies the parameter override values, or the default value if not overriden, to the given evaluation result. */
	void ApplyParameterOverridesAndDefaults(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const;

private:

	void ApplyParameterOverridesImpl(UE::Cameras::FCameraVariableTable* OutVariableTable, UE::Cameras::FCameraContextDataTable* OutContextDataTable, bool bDrivenOnly, bool bApplyUnwrittenDefaults) const;

public:

	// FBaseCameraObjectReference interface.
	GAMEPLAYCAMERAS_API virtual const UBaseCameraObject* GetCameraObject() const override;

public:

	bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	void PostSerialize(const FArchive& Ar);

private:

	/** The referenced camera rig. */
	UPROPERTY(EditAnywhere, Category="")
	TObjectPtr<UCameraRigAsset> CameraRig;


	// Deprecated

	UPROPERTY()
	TArray<FGuid> ParameterOverrideGuids_DEPRECATED;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	FCameraRigParameterOverrides ParameterOverrides_DEPRECATED;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	friend class UE::Cameras::FCameraRigAssetReferenceDetailsCustomization;
};

template<>
struct TStructOpsTypeTraits<FCameraRigAssetReference> : public TStructOpsTypeTraits<FBaseCameraObjectReference>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithPostSerialize = true
	};
};

