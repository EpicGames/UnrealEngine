// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigInstanceID.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

class UCameraRigAsset;

namespace UE::Cameras
{

class FBlendCameraNodeEvaluator;
class FCameraEvaluationContext;
class FCameraNodeEvaluator;
class FCameraNodeEvaluatorStorage;
struct FCameraNodeEvaluationResult;

/**
 * A structure describing an active camera rig being evaluated, generally
 * inside a blend stack.
 */
struct FCameraRigEvaluationInfo
{
	/** The instance ID of this camera rig. */
	FCameraRigInstanceID InstanceID;
	/** The context inside which the evaluation occurs. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;
	/** The camera rig being evaluated. */
	TObjectPtr<const UCameraRigAsset> CameraRig;
	/** The last evaluated result for this camera rig. */
	const FCameraNodeEvaluationResult* LastResult = nullptr;
	/** The root node evaluator of the camera rig. */
	FCameraNodeEvaluator* RootEvaluator = nullptr;

	FCameraRigEvaluationInfo()
	{}

	FCameraRigEvaluationInfo(
			FCameraRigInstanceID InInstanceID,
			TSharedPtr<const FCameraEvaluationContext> InEvaluationContext,
			TObjectPtr<const UCameraRigAsset> InCameraRig,
			const FCameraNodeEvaluationResult* InLastResult,
			FCameraNodeEvaluator* InRootEvaluator)
		: InstanceID(InInstanceID)
		, EvaluationContext(InEvaluationContext)
		, CameraRig(InCameraRig)
		, LastResult(InLastResult)
		, RootEvaluator(InRootEvaluator)
	{}
};

}  // namespace UE::Cameras

