// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Core/BlendStackEntryID.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "CameraRigInstanceID.generated.h"

/**
 * Defines evaluation layers for camera rigs.
 */
UENUM(BlueprintType)
enum class ECameraRigLayer : uint8
{
	None UMETA(Hidden),
	Base UMETA(DisplayName="Base Layer"),
	Main UMETA(DisplayName="Main Layer"),
	Global UMETA(DisplayName="Global Layer"),
	Visual UMETA(DisplayName="Visual Layer")
};
ENUM_CLASS_FLAGS(ECameraRigLayer)

/**
 * A unique instance ID for a running, or previously running, camera rig.
 */
USTRUCT(BlueprintType)
struct FCameraRigInstanceID
{
	GENERATED_BODY()

public:

	FCameraRigInstanceID() : Value(INVALID), Layer(ECameraRigLayer::None) {}

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

public:

	friend bool operator==(FCameraRigInstanceID A, FCameraRigInstanceID B)
	{
		return A.Value == B.Value && A.Layer == B.Layer;
	}

	friend bool operator!=(FCameraRigInstanceID A, FCameraRigInstanceID B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FCameraRigInstanceID In)
	{
		return HashCombineFast(In.Value, (uint32)In.Layer);
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraRigInstanceID& In)
	{
		Ar << In.Value;
		Ar << In.Layer;
		return Ar;
	}

public:

	/** Gets the layer this camera rig is, or was, running on. */
	ECameraRigLayer GetLayer() const { return Layer; }

	/** Gets a blend stack ID from this instance ID. */
	UE::Cameras::FBlendStackEntryID ToBlendStackEntryID() const { return UE::Cameras::FBlendStackEntryID(Value); }

	/** Creates an instance ID from a blend stack ID and a layer. */
	static FCameraRigInstanceID FromBlendStackEntryID(UE::Cameras::FBlendStackEntryID EntryID, ECameraRigLayer Layer)
	{
		return FCameraRigInstanceID(EntryID, Layer);
	}

private:

	FCameraRigInstanceID(uint32 InValue, ECameraRigLayer InLayer) 
		: Value(InValue)
		, Layer(InLayer)
	{}

	FCameraRigInstanceID(const UE::Cameras::FBlendStackEntryID InBlendStackID, ECameraRigLayer InLayer) 
		: Value(InBlendStackID.Value)
		, Layer(InLayer)
	{}

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;

	UPROPERTY()
	ECameraRigLayer Layer;

	friend class UE::Cameras::FBlendStackCameraNodeEvaluator;
};

/**
 * Blueprint functions for camera rig instance IDs.
 */
UCLASS(MinimalAPI)
class UCameraRigInstanceFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** 
	 * Whether the given camera rig instance ID is valid.
	 * A valid ID doesn't necessarily correspond to a camera rig instance that is still running.
	 */
	UFUNCTION(BlueprintPure, Category="Camera")
	static bool IsValid(FCameraRigInstanceID InstanceID) { return InstanceID.IsValid(); }
};

