// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableFwd.h"
#include "CoreTypes.h"

#define UE_API GAMEPLAYCAMERAS_API

struct FCameraVariableDefinition;

namespace UE::Cameras
{

/**
 * A structure describing a joint in a camera rig.
 */
struct FCameraRigJoint
{
	/** The variable driving the rotation of this joint. */
	FCameraVariableID VariableID;
	/** The position of the this joint. */
	FTransform3d Transform;
};

FArchive& operator<< (FArchive& Ar, FCameraRigJoint& RigJoint);

/**
 * A structure describing the joints of a camera rig.
 * These joints allow for "manipulating" the rig, e.g. to make it point
 * towards a desired target or direction.
 */
class FCameraRigJoints
{
public:

	/** Add a joint. */
	UE_API void AddJoint(const FCameraRigJoint& InJoint);
	/** Add a joint. */
	UE_API void AddJoint(const FCameraVariableDefinition& InVariableDefinition, const FTransform3d& InTransform);
	/** Add a joint related to the yaw/pitch built-in variable. */
	UE_API void AddYawPitchJoint(const FTransform3d& InTransform);

	/** Gets the joints. */
	TArrayView<const FCameraRigJoint> GetJoints() const { return Joints; }

	/** Removes all previously added joints. */
	UE_API void Reset();

	UE_API void Serialize(FArchive& Ar);

public:

	/** Override the joints with another set of joints. */
	UE_API void OverrideAll(const FCameraRigJoints& OtherJoints);

	/** Interpolate the joints towards anoter set of joints. */
	UE_API void LerpAll(const FCameraRigJoints& ToJoints, float BlendFactor);

private:

	using FJointArray = TArray<FCameraRigJoint, TInlineAllocator<2>>;
	FJointArray Joints;
};

}  // namespace UE::Cameras

#undef UE_API
