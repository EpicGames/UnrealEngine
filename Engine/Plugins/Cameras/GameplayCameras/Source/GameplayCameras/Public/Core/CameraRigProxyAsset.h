// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"

#include "CameraRigProxyAsset.generated.h"

/**
 * A proxy for a camera rig asset.
 *
 * This is useful for indicating that a camera rig should be activated in a camera director,
 * but without hard-referencing a particular camera rig. This way, that camera director can
 * be used by multiple cameras with their own camera rigs.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCameraRigProxyAsset : public UObject
{
	GENERATED_BODY()

public:

	/** Gets the camera rig proxy's unique ID. */
	const FGuid& GetGuid() const { return Guid; }

	// UObject interface.
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

private:

	/** The camera rig proxy's unique ID. */
	UPROPERTY()
	FGuid Guid;
};

