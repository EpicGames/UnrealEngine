// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"

#include "CameraRigProxyRedirectTable.generated.h"

class UCameraRigAsset;
class UCameraRigProxyAsset;

/**
 * Parameter structure for resolving a camera rig proxy from a proxy table.
 */
struct FCameraRigProxyResolveParams
{
	/** The camera rig proxy to resolve. */
	const UCameraRigProxyAsset* CameraRigProxy = nullptr;
};

/**
 * An entry in a camera rig proxy table.
 */
USTRUCT()
struct FCameraRigProxyRedirectTableEntry
{
	GENERATED_BODY()

	/** The camera rig proxy for this table entry. */
	UPROPERTY(EditAnywhere, Category="Camera")
	TObjectPtr<UCameraRigProxyAsset> CameraRigProxy;

	/** The actual camera rig that should be mapped to the correspondig proxy. */
	UPROPERTY(EditAnywhere, Category="Camera")
	TObjectPtr<UCameraRigAsset> CameraRig;
};

/**
 * A table that defines mappings between camera rig proxies and actual camera rigs.
 */
USTRUCT()
struct FCameraRigProxyRedirectTable
{
	GENERATED_BODY()

public:

	/**
	 * Resolves a given proxy to an actual camera rig.
	 * Returns nullptr if the given proxy wasn't found, or not mapped to anything in the table.
	 */
	UCameraRigAsset* ResolveProxy(const FCameraRigProxyResolveParams& InParams) const;

public:

	// Internal API.

	bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

public:

	/** The entries in the table. */
	UPROPERTY(EditAnywhere, Category="Camera")
	TArray<FCameraRigProxyRedirectTableEntry> Entries;
};

UCLASS()
class UE_DEPRECATED(5.6, "Use FCameraRigProxyRedirectTable") UCameraRigProxyTable : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FCameraRigProxyRedirectTableEntry> Entries;
};

