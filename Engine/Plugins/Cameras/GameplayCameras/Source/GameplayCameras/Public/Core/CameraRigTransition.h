// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/ObjectTreeGraphObject.h"
#include "CoreTypes.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigTransition.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UBlendCameraNode;
class UCameraAsset;
class UCameraRigAsset;

namespace UE::Cameras
{
	struct FCameraObjectBuildContext;
	class FCameraEvaluationContext;
}

/**
 * Parameter structure for camera transitions.
 */
struct FCameraRigTransitionConditionMatchParams
{
	/** The evaluation context of the previous camera rig. */
	TSharedPtr<const UE::Cameras::FCameraEvaluationContext> FromEvaluationContext;
	/** The previous camera rig. */
	const UCameraRigAsset* FromCameraRig = nullptr;
	/** The previous camera asset. */
	const UCameraAsset* FromCameraAsset = nullptr;

	/** The evaluation context of the next camera rig. */
	TSharedPtr<const UE::Cameras::FCameraEvaluationContext> ToEvaluationContext;
	/** The next camera rig. */
	const UCameraRigAsset* ToCameraRig = nullptr;
	/** The next camera asset. */
	const UCameraAsset* ToCameraAsset = nullptr;
};

/**
 * Base class for a camera transition condition.
 */
UCLASS(MinimalAPI, Abstract, DefaultToInstanced, meta=(
			ObjectTreeGraphCategory="Transition Conditions",
			ObjectTreeGraphSelfPinDirection="Output",
			ObjectTreeGraphDefaultPropertyPinDirection="Input"))
class UCameraRigTransitionCondition
	: public UObject
	, public IObjectTreeGraphObject
{
	GENERATED_BODY()

public:

	using FCameraObjectBuildContext = UE::Cameras::FCameraObjectBuildContext;

	/** Evaluates whether this transition should be used. */
	UE_API bool TransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const;

	/** Build process callback for this transition. */
	UE_API void Build(FCameraObjectBuildContext& BuildContext);

protected:

	/** Evaluates whether this transition should be used. */
	virtual bool OnTransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const { return false; }

	/** Build process callback for this transition. */
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) {}

protected:

	// UObject interface.
	UE_API virtual void PostLoad() override;

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	UE_API virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	UE_API virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override { return EObjectTreeGraphObjectSupportFlags::CommentText; }
	UE_API virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	UE_API virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
#endif

private:

#if WITH_EDITORONLY_DATA

	/** Position of the transition condition node in the transition graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the transition graph editor. */
	UPROPERTY()
	FString GraphNodeComment;


	// Deprecated properties.

	UPROPERTY()
	int32 GraphNodePosX_DEPRECATED = 0;
	UPROPERTY()
	int32 GraphNodePosY_DEPRECATED = 0;

#endif  // WITH_EDITORONLY_DATA
};

/**
 * Determines how a camera rig's initial orientation should be initialized.
 */
UENUM()
enum class ECameraRigInitialOrientation
{
	/** Leave the camera rig to its default orientation. */
	None,
	/** Orient the camera rig in the same direction as its context's initial transform. */
	ContextYawPitch,
	/** Orient the camera rig in the same direction as the previously active camera rig. */
	PreviousYawPitch,
	/** 
	 * Make the camera rig point at the same target as the previously active camera rig's 
	 * last frame target.
	 */
	PreviousAbsoluteTarget,
	/** 
	 * Make the camera rig point at the same target as the previously active camera rig. 
	 * Last frame's target will be moved and turned by an offset equal to how much the 
	 * active evaluation context has moved and turned since last frame.
	 */
	PreviousRelativeTarget
};

/**
 * A camera transition.
 */
UCLASS(MinimalAPI)
class UCameraRigTransition
	: public UObject
	, public IObjectTreeGraphObject
{
	GENERATED_BODY()

public:

	/** The list of conditions that must pass for this transition to be used. */
	UPROPERTY(Instanced, meta=(ObjectTreeGraphPinDirection=Input))
	TArray<TObjectPtr<UCameraRigTransitionCondition>> Conditions;

	/** The blend to use to blend a given camera rig in or out. */
	UPROPERTY(Instanced)
	TObjectPtr<UBlendCameraNode> Blend;

	/** The orientation to set on the camera rig. */
	UPROPERTY(EditAnywhere, Category="Transition", meta=(EditCondition="bOverrideInitialOrientation"))
	ECameraRigInitialOrientation InitialOrientation = ECameraRigInitialOrientation::None;

	/** Whether to override the default orientation to set on the camera rig. */
	UPROPERTY(EditAnywhere, Category="Transition")
	bool bOverrideInitialOrientation = false;

	/** Freeze all previously running camera rigs. */
	UPROPERTY(EditAnywhere, Category="Transition")
	bool bFreezePreviousCameraRigs = false;

	/**
	 * Whether this transition allows merging two similar camera rigs together.
	 * Similar camera rigs run the same underlying camera rig prefab with different parameter
	 * overrides. When merged, instead of pushing a new camera rig instance on the blend stack,
	 * only the parameter overrides are kept. These parameter overrides are blended together
	 * and the underlying camera rig prefab is run only once.
	 */
	UPROPERTY(EditAnywhere, Category="Advanced")
	bool bAllowCameraRigMerging = false;

public:

	using FCameraObjectBuildContext = UE::Cameras::FCameraObjectBuildContext;

	/** Returns whether all transition condition matches the given parameters. */
	UE_API bool AllConditionsMatch(const FCameraRigTransitionConditionMatchParams& Params) const;

	/** Build process callback for this transition. */
	UE_API void Build(FCameraObjectBuildContext& BuildContext);

protected:

	// UObject interface.
	UE_API virtual void PostLoad() override;

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	UE_API virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	UE_API virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override { return EObjectTreeGraphObjectSupportFlags::CommentText; }
	UE_API virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	UE_API virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
#endif

private:

#if WITH_EDITORONLY_DATA

	/** Position of the transition node in the transition graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the transition graph editor. */
	UPROPERTY()
	FString GraphNodeComment;


	// Deprecated properties.

	UPROPERTY()
	int32 GraphNodePosX_DEPRECATED = 0;
	UPROPERTY()
	int32 GraphNodePosY_DEPRECATED = 0;

#endif  // WITH_EDITORONLY_DATA
};

#undef UE_API
