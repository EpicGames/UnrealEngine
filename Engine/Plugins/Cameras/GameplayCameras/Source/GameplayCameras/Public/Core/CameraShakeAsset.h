// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildStatus.h"
#include "Core/BaseCameraObject.h"
#include "Core/ObjectTreeGraphObject.h"
#include "Core/ObjectTreeGraphRootObject.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "CameraShakeAsset.generated.h"

class UCameraShakeAsset;
class UShakeCameraNode;

namespace UE::Cameras
{
	class FCameraShakeAssetBuilder;
	struct FCameraBuildContext;
}

UCLASS(MinimalAPI, BlueprintType)
class UCameraShakeAsset
	: public UBaseCameraObject
	, public IHasCameraBuildStatus
	, public IObjectTreeGraphObject
	, public IObjectTreeGraphRootObject
{
	GENERATED_BODY()

public:

	/** Root camera node. */
	UPROPERTY(Instanced)
	TObjectPtr<UShakeCameraNode> RootNode;

	/** The blend to use for easing into the shake. */
	UPROPERTY(Instanced)
	TObjectPtr<USimpleFixedTimeBlendCameraNode> BlendIn;

	/** The blend to use for easing out of the shake. */
	UPROPERTY(Instanced)
	TObjectPtr<USimpleFixedTimeBlendCameraNode> BlendOut;

	/**
	 * Whether only one instance of this shake can be started via the camera shake functions.
	 * Note that this doesn't prevent this shake from being used "inline" inside a camera rig.
	 */
	UPROPERTY(EditAnywhere, Category="Common")
	bool bIsSingleInstance = false;

public:

	/** The current build state of this camera shake. */
	UPROPERTY(Transient)
	ECameraBuildStatus BuildStatus = ECameraBuildStatus::Dirty;

	/** Builds this camera shake asset. */
	GAMEPLAYCAMERAS_API void BuildCameraShake();

	/** Builds this camera shake asset. */
	GAMEPLAYCAMERAS_API void BuildCameraShake(UE::Cameras::FCameraBuildContext& InBuildContext);

public:

	// BaseCameraObject interface.
	GAMEPLAYCAMERAS_API virtual UCameraNode* GetRootNode() override;

	// IHasCameraBuildStatus interface.
	virtual ECameraBuildStatus GetBuildStatus() const override { return BuildStatus; }
	virtual void DirtyBuildStatus() override;

protected:

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override;
	virtual const FString& GetGraphNodeCommentText(FName InGraphName) const override;
	virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) override;
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const override;
#endif

	// IObjectTreeGraphRootObject interface.
#if WITH_EDITOR
	virtual void GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const override;
	virtual void AddConnectableObject(FName InGraphName, UObject* InObject) override;
	virtual void RemoveConnectableObject(FName InGraphName, UObject* InObject) override;
#endif

	// UObject interface.
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

private:

#if WITH_EDITORONLY_DATA

	/** Position of the camera shake node in the node graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

	/** User-written comment in the node graph editor. */
	UPROPERTY()
	FString GraphNodeComment;

	/** 
	 * A list of all the camera nodes, including the 'loose' ones that aren't connected
	 * to the root node, and therefore would be GC'ed if we didn't hold them here.
	 */
	UPROPERTY(Instanced, meta=(ObjectTreeGraphHidden=true))
	TArray<TObjectPtr<UObject>> AllNodeObjects;

#endif  // WITH_EDITORONLY_DATA

	friend class UE::Cameras::FCameraShakeAssetBuilder;
};

