// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCameraObjectReference.h"
#include "Core/CameraContextDataTable.h"
#include "Core/CameraVariableTable.h"
#include "UObject/ObjectPtr.h"

#include "CameraShakeAssetReference.generated.h"

class UBaseCameraObject;
class UCameraShakeAsset;
struct FCustomCameraNodeParameterInfos;

namespace UE::Cameras
{
	class FCameraShakeAssetReferenceDetailsCustomization;
	struct FCameraNodeEvaluationResult;
}

/**
 * A structure holding a reference to a camera shake asset, along with the interface parameter
 * override values.
 */
USTRUCT(BlueprintType)
struct FCameraShakeAssetReference : public FBaseCameraObjectReference
{
	GENERATED_BODY()

public:

	FCameraShakeAssetReference();
	FCameraShakeAssetReference(UCameraShakeAsset* InCameraShake);

	/** Returns whether this reference points to a valid camera shake. */
	bool IsValid() const
	{
		return CameraShake != nullptr;
	}

	/** Gets the referenced camera shake. */
	UCameraShakeAsset* GetCameraShake()
	{
		return CameraShake;
	}

	/** Gets the referenced camera shake. */
	const UCameraShakeAsset* GetCameraShake() const
	{
		return CameraShake;
	}

	/** Sets the referenced camera shake. */
	void SetCameraShake(UCameraShakeAsset* InCameraShake)
	{
		if (CameraShake != InCameraShake)
		{
			CameraShake = InCameraShake;
			RebuildParameters();
		}
	}

public:

	/** Ensure the variable and data tables have the allocations for our referenced camera shake. */
	void EnsureAllocationInfo(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const;

	/** Applies the parameter override values to the given variable table. */
	void ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, bool bDrivenOnly) const;

	/** Applies the parameter override values to the given variable and context data tables.. */
	void ApplyParameterOverrides(UE::Cameras::FCameraVariableTable& OutVariableTable, UE::Cameras::FCameraContextDataTable& OutContextDataTable, bool bDrivenOnly) const;

	/** Applies the parameter override values to the given evaluation result. */
	void ApplyParameterOverrides(UE::Cameras::FCameraNodeEvaluationResult& OutResult, bool bDrivenOnly) const;

	/** Applies the parameter override values, or the default value if not overriden, to the given evaluation result. */
	void ApplyParameterOverridesAndDefaults(UE::Cameras::FCameraNodeEvaluationResult& OutResult) const;

private:

	void ApplyParameterOverridesImpl(UE::Cameras::FCameraVariableTable* OutVariableTable, UE::Cameras::FCameraContextDataTable* OutContextDataTable, bool bDrivenOnly, bool bApplyUnwrittenDefaults) const;

public:

	// FBaseCameraObjectReference interface.
	GAMEPLAYCAMERAS_API virtual const UBaseCameraObject* GetCameraObject() const override;

private:

	/** The referenced camera shake. */
	UPROPERTY(EditAnywhere, Category="")
	TObjectPtr<UCameraShakeAsset> CameraShake;

	friend class UE::Cameras::FCameraShakeAssetReferenceDetailsCustomization;
};

template<>
struct TStructOpsTypeTraits<FCameraShakeAssetReference> : public TStructOpsTypeTraits<FBaseCameraObjectReference>
{
};

