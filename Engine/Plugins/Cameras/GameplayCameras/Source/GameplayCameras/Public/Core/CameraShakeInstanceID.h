// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "CameraShakeInstanceID.generated.h"

/**
 * A unique instance ID for a running, or previously running, camera shake.
 */
USTRUCT(BlueprintType)
struct FCameraShakeInstanceID
{
	GENERATED_BODY()

public:

	FCameraShakeInstanceID() : Value(INVALID) {}

	FCameraShakeInstanceID(uint32 InValue) 
		: Value(InValue)
	{}

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

public:

	friend bool operator==(FCameraShakeInstanceID A, FCameraShakeInstanceID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCameraShakeInstanceID A, FCameraShakeInstanceID B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FCameraShakeInstanceID In)
	{
		return In.Value;
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraShakeInstanceID& In)
	{
		Ar << In.Value;
		return Ar;
	}

private:

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;
};

/**
 * Blueprint function library for camera shake instance IDs.
 */
UCLASS()
class UCameraShakeInstanceFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Returns whether a camera shake instance ID is valid. 
	 *
	 * Note that this doesn't check if the associated camera shake is still running.
	 */
	UFUNCTION(BlueprintPure, Category="Camera Shake", meta=(DisplayName="Is Valid Camera Shake Instance ID"))
	static bool IsValid(const FCameraShakeInstanceID CameraShakeInstanceID)
	{
		return CameraShakeInstanceID.IsValid();
	}
};

