// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"
#include "Core/CameraEvaluationContextStack.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraNodeEvaluatorStorage.h"
#include "Core/CameraPose.h"
#include "CoreTypes.h"
#include "Debug/CameraDebugBlockStorage.h"
#include "Debug/CameraSystemDebugRegistry.h"
#include "GameplayCameras.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"

class FCanvas;
class FSceneView;
class UCameraDirector;
class UCameraRigAsset;
class UCanvas;
class URootCameraNode;
struct FMinimalViewInfo;

namespace UE::Cameras
{

class FCameraDirectorEvaluator;
class FCameraEvaluationContext;
class FCameraEvaluationService;
class FCameraRigCombinationRegistry;
class FRootCameraNodeEvaluator;
enum class ECameraEvaluationServiceFlags;
enum class ECameraNodeEvaluationType;
struct FCameraRigActivationDeactivationRequest;
struct FRootCameraNodeCameraRigEvent;

#if UE_GAMEPLAY_CAMERAS_DEBUG
class FRootCameraDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

/**
 * An enumeration that describes what a camera system evaluator is used for.
 */
enum class ECameraSystemEvaluatorRole
{
	Game,

#if WITH_EDITOR
	EditorPreview
#endif
};

/**
 * Parameter structure for initializing a new camera system evaluator.
 */
struct FCameraSystemEvaluatorCreateParams
{
	/** The owner of the camera system, if any. */
	TObjectPtr<UObject> Owner;

	/** The role of the camera system. */
	ECameraSystemEvaluatorRole Role = ECameraSystemEvaluatorRole::Game;

	/** An optional factory for creating the root node. */
	using FRootNodeFactory = TFunction<URootCameraNode*()>;
	FRootNodeFactory RootNodeFactory;
};

/**
 * Parameter structure for updating the camera system.
 */
struct FCameraSystemEvaluationParams
{
	/** Time interval for the update. */
	float DeltaTime = 0.f;
};

/**
 * Result structure for updating the camera system.
 */
struct FCameraSystemEvaluationResult
{
	/** The result camera pose. */
	FCameraPose CameraPose;

	/** The result camera variable table. */
	FCameraVariableTable VariableTable;

	/** The result camera context data table. */
	FCameraContextDataTable ContextDataTable;
	
	/** The result post-process settings. */
	FPostProcessSettingsCollection PostProcessSettings;

	/** Whether this evaluation was a camera cut. */
	bool bIsCameraCut = false;

	/** Whether this result is valid. */
	bool bIsValid = false;

public:

	/** Reset this result to its default (non-valid) state.  */
	void Reset();

	/** Set this result to be equivalent to the given evaluation result. */
	void Reset(const FCameraNodeEvaluationResult& NodeResult);
};

/**
 * Result structure for view rotation updates of the camera system.
 */
struct FCameraSystemViewRotationEvaluationResult
{
	/** View rotation for the player. */
	FRotator ViewRotation;

	/** Control rotation for the player. */
	FRotator DeltaRotation;
};

#if UE_GAMEPLAY_CAMERAS_DEBUG

/**
 * Parameter structure for running the debug pass of the camera system.
 */
struct FCameraSystemDebugUpdateParams
{
	/** The canvas to draw upon. */
	UCanvas* CanvasObject = nullptr;

	/** Whether the debug camera is enabled, giving an "outside" view of camera system. */
	bool bIsDebugCameraEnabled = false;

	/**
	 * Whether this camera system should be considered the "default" one, such as when
	 * running inside the active camera manager or the view target.
	 */
	bool bIsDefaultCameraSystem = false;

	/** Whether to force drawing debug info for this camera system. */
	bool bForceDraw = false;
};

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if WITH_EDITOR

/**
 * Parameter structure for rendering the state of the camera system in editor.
 */
struct FCameraSystemEditorPreviewParams
{
	/** The canvas to draw upon. */
	FCanvas* Canvas = nullptr;

	/** The scene view being renderered. */
	const FSceneView* SceneView = nullptr;

	/**
	 * Whether debug drawing is done from "inside" the camera system.
	 * When false, it is acceptable to draw camera frustrums and axes.
	 * When true, these things shouldn't be drawn as they would clutter the view
	 * and obfuscate the scene.
	 */
	bool bIsLockedToCamera = true;

	/**
	 * Whether to enable drawing debug lines and other 3D primitives in the world.
	 */
	bool bDrawWorldDebug = true;
};

#endif  // WITH_EDITOR

/**
 * The main camera system evaluator class.
 */
class FCameraSystemEvaluator : public TSharedFromThis<FCameraSystemEvaluator>
{
public:

	/** Builds a new camera system. Initialize must be called before the system is used. */
	GAMEPLAYCAMERAS_API FCameraSystemEvaluator();

	/** Initializes the camera system. */
	GAMEPLAYCAMERAS_API void Initialize(TObjectPtr<UObject> InOwner = nullptr);
	/** Initializes the camera system. */
	GAMEPLAYCAMERAS_API void Initialize(const FCameraSystemEvaluatorCreateParams& Params);

	GAMEPLAYCAMERAS_API ~FCameraSystemEvaluator();

public:

	/** Gets the owner of this camera system, if any, and if still valid. */
	UObject* GetOwner() const { return WeakOwner.Get(); }

	/** Gets the role of this camera system. */
	ECameraSystemEvaluatorRole GetRole() const { return Role; }

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Gets the debug ID for this camera system. */
	FCameraSystemDebugID GetDebugID() const { return DebugID; }
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

public:

	/** Push a new evaluation context on the stack. */
	GAMEPLAYCAMERAS_API void PushEvaluationContext(TSharedRef<FCameraEvaluationContext> EvaluationContext);
	/** Remove an existing evaluation context from the stack. */
	GAMEPLAYCAMERAS_API void RemoveEvaluationContext(TSharedRef<FCameraEvaluationContext> EvaluationContext);
	/** Pop the active (top) evaluation context from the stack. */
	GAMEPLAYCAMERAS_API void PopEvaluationContext();

	/** Gets the context stack. */
	FCameraEvaluationContextStack& GetEvaluationContextStack() { return ContextStack; }
	/** Gets the context stack. */
	const FCameraEvaluationContextStack& GetEvaluationContextStack() const { return ContextStack; }

public:

	/** Registers an evaluation service on this camera system. */
	GAMEPLAYCAMERAS_API void RegisterEvaluationService(TSharedRef<FCameraEvaluationService> EvaluationService);
	/** Unregisters an evaluation service from this camera system. */
	GAMEPLAYCAMERAS_API void UnregisterEvaluationService(TSharedRef<FCameraEvaluationService> EvaluationService);
	/** Get currently registered evaluation services. */
	GAMEPLAYCAMERAS_API void GetEvaluationServices(TArray<TSharedPtr<FCameraEvaluationService>>& OutEvaluationServices) const;

	/** Finds an evaluation service of the given type. */
	GAMEPLAYCAMERAS_API TSharedPtr<FCameraEvaluationService> FindEvaluationService(const FCameraObjectTypeID& TypeID) const;
	
	/** Finds an evaluation service of the given type. */
	template<typename EvaluationServiceType>
	TSharedPtr<EvaluationServiceType> FindEvaluationService() const
	{
		TSharedPtr<FCameraEvaluationService> EvaluationService = FindEvaluationService(EvaluationServiceType::StaticTypeID());
		if (EvaluationService)
		{
			return StaticCastSharedPtr<EvaluationServiceType>(EvaluationService);
		}
		return nullptr;
	}

	/** Finds or creates/registers an evaluation service of the given type. */
	template<typename EvaluationServiceType>
	TSharedRef<EvaluationServiceType> FindOrRegisterEvaluationService()
	{
		TSharedPtr<EvaluationServiceType> EvaluationService = FindEvaluationService<EvaluationServiceType>();
		if (!EvaluationService)
		{
			EvaluationService = MakeShared<EvaluationServiceType>();
			RegisterEvaluationService(EvaluationService.ToSharedRef());
		}
		return EvaluationService.ToSharedRef();
	}

public:

	/** Run an update of the camera system. */
	GAMEPLAYCAMERAS_API void Update(const FCameraSystemEvaluationParams& Params);

	/** Run a view rotation preview update of the camera system. */
	GAMEPLAYCAMERAS_API void ViewRotationPreviewUpdate(const FCameraSystemEvaluationParams& Params, FCameraSystemViewRotationEvaluationResult& OutResult);

	/** Returns the root node evaluator. */
	FRootCameraNodeEvaluator* GetRootNodeEvaluator() const { return RootEvaluator; }

	/** Gets the evaluated result. */
	const FCameraSystemEvaluationResult& GetEvaluatedResult() const { return Result; }

	/** Gets the evaluated result without the contribution of the visual layer. */
	const FCameraSystemEvaluationResult& GetPreVisualLayerEvaluatedResult() const { return PreVisualResult; }

	/** Get the last evaluated camera. */
	GAMEPLAYCAMERAS_API void GetEvaluatedCameraView(FMinimalViewInfo& DesiredView);

	/** Executes an operation, to be dispatched by the root node evaluator. */
	GAMEPLAYCAMERAS_API void ExecuteOperation(FCameraOperation& Operation);

	/** Collect reference objects for the garbage collector. */
	GAMEPLAYCAMERAS_API void AddReferencedObjects(FReferenceCollector& Collector);

#if WITH_EDITOR
	/** Run an update suitable for an in-editor preview. */
	GAMEPLAYCAMERAS_API void EditorPreviewUpdate(const FCameraSystemEvaluationParams& Params);

	/** Render information about the state of the camera system in-editor. */
	GAMEPLAYCAMERAS_API void DrawEditorPreview(const FCameraSystemEditorPreviewParams& Params);
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	GAMEPLAYCAMERAS_API void DebugUpdate(const FCameraSystemDebugUpdateParams& Params);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void UpdateImpl(float DeltaTime, ECameraNodeEvaluationType EvaluationType);

	void UpdateCameraDirector(float DeltaTime, FCameraDirectorEvaluator* CameraDirectorEvaluator);
	void GetCombinedCameraRigRequest(TConstArrayView<FCameraRigActivationDeactivationRequest> Requests, FCameraRigActivationDeactivationRequest& OutCombinedRequest);

	void PreUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType);
	void PostCameraDirectorUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType);
	void PostUpdateServices(float DeltaTime, ECameraNodeEvaluationType EvaluationType);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	static bool IsDebugTraceEnabled();
	static bool ShouldBuildOrDrawDebugBlocks();
	void BuildDebugBlocksIfNeeded();
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	/** The owner (if any) of this camera system evaluator. */
	TWeakObjectPtr<> WeakOwner;

	/** The root camera node. */
	TObjectPtr<URootCameraNode> RootNode;

	/** The stack of active evaluation context. */
	FCameraEvaluationContextStack ContextStack;

	/** The list of evaluation services. */
	TArray<TSharedPtr<FCameraEvaluationService>> EvaluationServices;

	/** Registry for programmatically building combinations of camera rigs. */
	TSharedPtr<FCameraRigCombinationRegistry> CameraRigCombinationRegistry;

	/** Storage buffer for the root evaluator. */
	FCameraNodeEvaluatorStorage RootEvaluatorStorage;

	/** The root evaluator. */
	FRootCameraNodeEvaluator* RootEvaluator = nullptr;

	/** The current result of the root camera node. */
	FCameraNodeEvaluationResult RootNodeResult;

	/** The current overall result of the camera system. */
	FCameraSystemEvaluationResult Result;

	/** The current overall result of the camera system without the contribution of the visual layer. */
	FCameraSystemEvaluationResult PreVisualResult;

	/** Reusable buffer for node tree snapshots (e.g. view rotation preview update). */
	TArray<uint8> EvaluatorSnapshot;

	/** The role of this camera system. */
	ECameraSystemEvaluatorRole Role = ECameraSystemEvaluatorRole::Game;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** The debug ID for this camera system. */
	FCameraSystemDebugID DebugID;

	/** Storage for debug drawing blocks. */
	FCameraDebugBlockStorage DebugBlockStorage;

	/** The root debug drawing block. */
	FRootCameraDebugBlock* RootDebugBlock = nullptr;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	friend class FRootCameraNodeEvaluator;
};

}  // namespace UE::Cameras

