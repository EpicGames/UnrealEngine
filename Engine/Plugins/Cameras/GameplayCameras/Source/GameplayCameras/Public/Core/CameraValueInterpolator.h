// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Serialization/Archive.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

#include "CameraValueInterpolator.generated.h"

class FArchive;
class UCameraValueInterpolator;

namespace UE::Cameras
{

class FCameraVariableTable;

/** 
 * Parameter structure for updating a value interpolator.
 */
struct FCameraValueInterpolationParams
{
	float DeltaTime = 0.f;
	bool bIsCameraCut = false;
};

/**
 * Result structure for updating a value interpolator.
 */
struct FCameraValueInterpolationResult
{
	FCameraValueInterpolationResult(FCameraVariableTable& InVariableTable)
		: VariableTable(InVariableTable)
	{}

	FCameraVariableTable& VariableTable;
};

/**
 * Parameter structure for serializing a value interpolator.
 */
struct FCameraValueInterpolatorSerializeParams
{
};

/**
 * A value interpolator is a stand-alone object that can interpolate a given value
 * towards a given target value. Various sub-classes of this base class should 
 * implement various interpolation algorithms.
 */
template<typename ValueType>
class TCameraValueInterpolator
{
public:

	using ValueTypeParam = typename TCallTraits<ValueType>::ParamType;

	/** 
	 * Creates a new value interpolator given a piece of data containing user-defined
	 * settings.
	 */
	TCameraValueInterpolator(const UCameraValueInterpolator* InParameters)
		: Parameters(InParameters)
		, CurrentValue(ValueType())
		, TargetValue(ValueType())
		, bIsFinished(false)
	{}

	/** Destructor */
	virtual ~TCameraValueInterpolator() = default;

	/** Gets the parameters class and casts it to the given sub-class.  */
	template<typename TInterpolatorClass>
	const TInterpolatorClass* GetParametersAs() const { return Cast<TInterpolatorClass>(Parameters); }

	/** Gets the current value. */
	ValueType GetCurrentValue() const { return CurrentValue; }
	
	/** Gets the target value. */
	ValueType GetTargetValue() const { return TargetValue; }

	/** 
	 * Returns whether the interpolation has ended. 
	 * The current value may not be equal to the target value if for some reason the 
	 * implemented interpolation algorithm cannot reach the target value.
	 */
	bool IsFinished() const { return bIsFinished; }

	/** Resets the current and target values. */
	void Reset(ValueTypeParam NewCurrentValue, ValueTypeParam NewTargetValue)
	{
		ValueType OldCurrentValue = CurrentValue;
		ValueType OldTargetValue = TargetValue;

		CurrentValue = NewCurrentValue;
		TargetValue = NewTargetValue;

		OnReset(OldCurrentValue, OldTargetValue);
	}

	/** Evaluates the interpolator, advancing the current value towards the target. */
	ValueType Run(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult)
	{
		if (Params.DeltaTime != 0.f)
		{
			OnRun(Params, OutResult);
		}
		return CurrentValue;
	}

	/** Serializes the value interpolator to/from the given archive. */
	void Serialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar)
	{
		Ar << CurrentValue;
		Ar << TargetValue;
		Ar << bIsFinished;

		OnSerialize(Params, Ar);
	}

protected:

	/** Called when the current and/or target values are forcibly changed. */
	virtual void OnReset(ValueTypeParam OldCurrentValue, ValueTypeParam OldTargetValue) {}
	/** Evaluates the interpolator, advancing the current value towards the target. */
	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) {}
	/** Serializes the value interpolator to/from the given archive. */
	virtual void OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar) {}

protected:

	const UCameraValueInterpolator* Parameters;
	ValueType CurrentValue;
	ValueType TargetValue;
	bool bIsFinished;
};

/**
 * Generic traits for interpolated value types.
 * Empty by default, overriden for specifically supported value types.
 */
template<typename ValueType>
struct TCameraValueInterpolationTraits
{};

template<>
struct TCameraValueInterpolationTraits<double>
{
	static double Distance(double A, double B)
	{
		return FMath::Abs(B - A);
	}

	static double Direction(double A, double B)
	{
		return A < B ? 1 : -1;
	}
};

template<>
struct TCameraValueInterpolationTraits<FVector2d>
{
	static double Distance(const FVector2d& A, const FVector2d& B)
	{
		return FVector2d::Distance(A, B);
	}

	static FVector2d Direction(const FVector2d& A, const FVector2d& B)
	{
		return (B - A).GetSafeNormal();
	}
};

template<>
struct TCameraValueInterpolationTraits<FVector3d>
{
	static double Distance(const FVector3d& A, const FVector3d& B)
	{
		return FVector3d::Distance(A, B);
	}

	static FVector3d Direction(const FVector3d& A, const FVector3d& B)
	{
		return (B - A).GetSafeNormal();
	}
};

using FCameraDoubleValueInterpolator = TCameraValueInterpolator<double>;
using FCameraVector2dValueInterpolator = TCameraValueInterpolator<FVector2d>;
using FCameraVector3dDoubleValueInterpolator = TCameraValueInterpolator<FVector3d>;

/** A simple value interpolator that immediately "pops" to the target value. */
template<typename ValueType>
class TPopValueInterpolator : public TCameraValueInterpolator<ValueType>
{
public:

	TPopValueInterpolator()
		: TCameraValueInterpolator<ValueType>(nullptr)
	{}

	TPopValueInterpolator(const UPopValueInterpolator* InInterpolator)
		: TCameraValueInterpolator<ValueType>(InInterpolator)
	{}

protected:

	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) override
	{
		this->CurrentValue = this->TargetValue;
		this->bIsFinished = true;
	}
};

}  // namespace UE::Cameras

/**
 * Base class for value interpolator parameters.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, DefaultToInstanced, EditInlineNew, CollapseCategories)
class UCameraValueInterpolator : public UObject
{
	GENERATED_BODY()

public:

	template<typename ValueType> 
	using TCameraValueInterpolator = UE::Cameras::TCameraValueInterpolator<ValueType>;

	/** Creates a value interpolator for double precision floating point numbers. */
	virtual TUniquePtr<TCameraValueInterpolator<double>> BuildDoubleInterpolator() const;
	/** Creates a value interpolator for 2D vectors. */
	virtual TUniquePtr<TCameraValueInterpolator<FVector2d>> BuildVector2dInterpolator() const;
	/** Creates a value interpolator for 3D vectors. */
	virtual TUniquePtr<TCameraValueInterpolator<FVector3d>> BuildVector3dInterpolator() const;

protected:

	/** Creates a value interpolator for double precision floating point numbers. */
	virtual TUniquePtr<TCameraValueInterpolator<double>> OnBuildDoubleInterpolator() const { return nullptr; }
	/** Creates a value interpolator for 2D vectors. */
	virtual TUniquePtr<TCameraValueInterpolator<FVector2d>> OnBuildVector2dInterpolator() const { return nullptr; }
	/** Creates a value interpolator for 3D vectors. */
	virtual TUniquePtr<TCameraValueInterpolator<FVector3d>> OnBuildVector3dInterpolator() const { return nullptr; }
};

#define UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()\
	protected:\
		virtual TUniquePtr<UE::Cameras::TCameraValueInterpolator<double>> OnBuildDoubleInterpolator() const override;\
		virtual TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector2d>> OnBuildVector2dInterpolator() const override;\
		virtual TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector3d>> OnBuildVector3dInterpolator() const override;\
	private:

#define UE_DEFINE_CAMERA_VALUE_INTERPOLATOR_GENERIC(ThisClass, GenericInterpolatorClass)\
	TUniquePtr<UE::Cameras::TCameraValueInterpolator<double>> ThisClass::OnBuildDoubleInterpolator() const\
	{\
		return MakeUnique<GenericInterpolatorClass<double>>(this);\
	}\
	TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector2d>> ThisClass::OnBuildVector2dInterpolator() const\
	{\
		return MakeUnique<GenericInterpolatorClass<FVector2d>>(this);\
	}\
	TUniquePtr<UE::Cameras::TCameraValueInterpolator<FVector3d>> ThisClass::OnBuildVector3dInterpolator() const\
	{\
		return MakeUnique<GenericInterpolatorClass<FVector3d>>(this);\
	}

/** Simple interpolator that immediately "pops" to the target value. */
UCLASS(meta=(DisplayName="Immediate Pop"))
class UPopValueInterpolator : public UCameraValueInterpolator
{
	GENERATED_BODY()

	UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()
};

