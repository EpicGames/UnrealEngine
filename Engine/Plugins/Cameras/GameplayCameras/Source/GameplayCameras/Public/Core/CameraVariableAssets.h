// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Core/CameraVariableTableAllocationInfo.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"

#include "CameraVariableAssets.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

/**
 * The base asset class for all camera variables.
 */
UCLASS(MinimalAPI, Abstract)
class UCameraVariableAsset : public UObject
{
	GENERATED_BODY()

public:

	UE_API UCameraVariableAsset(const FObjectInitializer& ObjectInit);

	UE_API FCameraVariableID GetVariableID() const;

	UE_API FCameraVariableDefinition GetVariableDefinition() const;

	const FGuid& GetGuid() const { return Guid; }

	virtual ECameraVariableType GetVariableType() const PURE_VIRTUAL(UCameraVariableAsset::GetVariableType, return ECameraVariableType::Boolean;);
	virtual const uint8* GetDefaultValuePtr() const PURE_VIRTUAL(UCameraVariableAsset::GetDefaultValuePtr, return nullptr;);

#if WITH_EDITORONLY_DATA
	UE_API FString GetDisplayName() const;
#endif  // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API FText GetDisplayText() const;

	virtual FString FormatDefaultValue() const PURE_VIRTUAL(UCameraVariableAsset::FormatDefaultValue, return FString(););
#endif  // WITH_EDITOR

public:

	// UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

public:

#if WITH_EDITORONLY_DATA
	/** The name of the variable. */
	UPROPERTY()
	FString DisplayName;
#endif  // WITH_EDITORONLY_DATA

	/** Whether this variable auto-resets to its default value every frame. */
	UPROPERTY(EditAnywhere, Category=Camera)
	bool bAutoReset = false;

	/**
	 * Whether this variable should be pre-blended.
	 *
	 * Pre-blending means that if two blending camera rigs share this variable, 
	 * each of their values will be blended in a first evaluation pass, and then
	 * both camera rigs will evaluate with the same blended value.
	 */
	UPROPERTY(EditAnywhere, Category=Camera)
	bool bIsPreBlended = false;

	/** Whether this variable is private and shouldn't be propagated with evaluation results. */
	UPROPERTY()
	bool bIsPrivate = false;

private:

	UPROPERTY()
	FGuid Guid;
};

/** Boolean camera variable. */
UCLASS(MinimalAPI)
class UBooleanCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = bool;

	bool GetDefaultValue() const { return bDefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Boolean; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&bDefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return LexToString(bDefaultValue); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	bool bDefaultValue = false;
};

/** Integer camera variable. */
UCLASS(MinimalAPI)
class UInteger32CameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = int32;

	int32 GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Integer32; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return LexToString(DefaultValue); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	int32 DefaultValue = 0;
};

/** Float camera variable. */
UCLASS(MinimalAPI)
class UFloatCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = float;

	float GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Float; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return LexToString(DefaultValue); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	float DefaultValue = 0.f;
};

/** Double camera variable. */
UCLASS(MinimalAPI)
class UDoubleCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = double;

	double GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Double; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return LexToString(DefaultValue); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	double DefaultValue = 0.0;
};

/** Vector2f camera variable. */
UCLASS(MinimalAPI)
class UVector2fCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector2f;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector2f; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector2f DefaultValue;
};

/** Vector2d camera variable. */
UCLASS(MinimalAPI)
class UVector2dCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector2d;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector2d; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector2D DefaultValue;
};

/** Vector3f camera variable. */
UCLASS(MinimalAPI)
class UVector3fCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector3f;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector3f; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector3f DefaultValue;
};

/** Vector3d camera variable. */
UCLASS(MinimalAPI)
class UVector3dCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector3d;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector3d; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector3d DefaultValue;
};

/** Vector4f camera variable. */
UCLASS(MinimalAPI)
class UVector4fCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector4f;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector4f; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector4f DefaultValue;
};

/** Vector4d camera variable. */
UCLASS(MinimalAPI)
class UVector4dCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FVector4d;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Vector4d; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FVector4d DefaultValue;
};

/** Rotator3f camera variable. */
UCLASS(MinimalAPI)
class URotator3fCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FRotator3f;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Rotator3f; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FRotator3f DefaultValue;
};

/** Rotator3d camera variable. */
UCLASS(MinimalAPI)
class URotator3dCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FRotator3d;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Rotator3d; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FRotator3d DefaultValue;
};

/** Transform3f camera variable. */
UCLASS(MinimalAPI)
class UTransform3fCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FTransform3f;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Transform3f; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FTransform3f DefaultValue;
};

/** Transform3d camera variable. */
UCLASS(MinimalAPI)
class UTransform3dCameraVariable : public UCameraVariableAsset
{
	GENERATED_BODY()

public:

	using ValueType = FTransform3d;

	const ValueType& GetDefaultValue() const { return DefaultValue; }

	virtual ECameraVariableType GetVariableType() const override { return ECameraVariableType::Transform3d; }
	virtual const uint8* GetDefaultValuePtr() const override { return reinterpret_cast<const uint8*>(&DefaultValue); }

#if WITH_EDITOR
	virtual FString FormatDefaultValue() const override { return DefaultValue.ToString(); }
#endif  // WITH_EDITOR

public:

	/** The default value of this variable. */
	UPROPERTY(EditAnywhere, Category=Camera)
	FTransform3d DefaultValue;
};

#undef UE_API
