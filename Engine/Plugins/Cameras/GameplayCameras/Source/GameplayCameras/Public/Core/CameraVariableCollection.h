// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "CameraVariableCollection.generated.h"

class UCameraVariableAsset;

/**
 * An asset that represents a collection of camera variables.
 */
UCLASS(MinimalAPI)
class UCameraVariableCollection : public UObject
{
	GENERATED_BODY()

public:

	UCameraVariableCollection(const FObjectInitializer& ObjectInit);

protected:

	// UObject interface
	virtual void PostLoad() override;

private:

#if WITH_EDITOR
	void CleanUpStrayObjects();
#endif

public:

	/** The variables in this collection. */
	UPROPERTY()
	TArray<TObjectPtr<UCameraVariableAsset>> Variables;
};

