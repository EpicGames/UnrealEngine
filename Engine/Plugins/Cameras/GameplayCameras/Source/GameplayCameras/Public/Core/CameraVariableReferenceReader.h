// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTable.h"
#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"

namespace UE::Cameras
{

/**
 * A utility class for reading the effective value of variable reference.
 */
template<typename ValueType>
class TCameraVariableReferenceReader
{
public:

	TCameraVariableReferenceReader() {}

	template<typename VariableReferenceType>
	TCameraVariableReferenceReader(const VariableReferenceType& Reference)
	{
		Initialize(Reference);
	}

	/**
	 * Initializes the reader around the given variable reference.
	 */
	template<typename VariableReferenceType>
	void Initialize(const VariableReferenceType& Reference)
	{
		Initialize(Reference, ValueType());
	}

	/**
	 * Initializes the reader around the given variable reference.
	 */
	template<typename VariableReferenceType>
	void Initialize(const VariableReferenceType& Reference, typename TCallTraits<ValueType>::ParamType DefaultValueIfNoReference)
	{
		static_assert(
				std::is_same<ValueType, typename VariableReferenceType::VariableAssetType::ValueType>(),
				"The given variable reference is of the wrong type for this reader! Value types must be the same.");

		if (Reference.Variable)
		{
			DefaultValue = Reference.Variable->GetDefaultValue();
			VariableID = Reference.Variable->GetVariableID();
		}
		else
		{
			DefaultValue = DefaultValueIfNoReference;
			VariableID = Reference.VariableID;
		}
	}

	/**
	 * Gets the actual value for the referenced variable.
	 */
	const ValueType& Get(const FCameraVariableTable& VariableTable) const
	{
		if (!VariableID.IsValid())
		{
			return DefaultValue;
		}
		else
		{
			if (const ValueType* ActualValue = VariableTable.FindValue<ValueType>(VariableID))
			{
				return *ActualValue;
			}
			return DefaultValue;
		}
	}

	/**
	 * Gets the actual value for the referenced variable, or returns false if the variable doesn't
	 * have a value yet.
	 */
	bool TryGet(const FCameraVariableTable& VariableTable, ValueType& OutValue) const
	{
		if (VariableID.IsValid())
		{
			return VariableTable.TryGetValue(VariableID, OutValue);
		}
		return false;
	}

	/**
	 * Returns whether his reference points to a variable.
	 */
	bool IsDriven() const
	{
		return VariableID.IsValid();
	}

private:

	/** The default value for the variable. */
	ValueType DefaultValue;
	/** The ID of the variable, if any. */
	FCameraVariableID VariableID;
};

}  // namespace UE::Cameras

