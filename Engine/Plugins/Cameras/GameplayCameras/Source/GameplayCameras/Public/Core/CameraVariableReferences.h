// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableAssets.h"

#include "CameraVariableReferences.generated.h"

namespace UE::Cameras
{
	class FCameraVariableTable;
}

#define UE_DEFINE_CAMERA_VARIABLE_REFERENCE(ValueName)\
	F##ValueName##CameraVariableReference() {}\
	F##ValueName##CameraVariableReference(VariableAssetType* InVariable) : Variable(InVariable) {}\
	bool IsValid() const { return VariableID.IsValid(); }\
	bool HasVariable() const { return Variable != nullptr; }\
	bool HasNonUserOverride() const { return VariableID.IsValid() && (!Variable || Variable->GetVariableID() != VariableID); }\
	const F##ValueName##CameraVariableReference::ValueType* GetValue(const UE::Cameras::FCameraVariableTable& VariableTable) const;

// All camera references have:
//
// - Variable: a variable chosen by the user.
// - VariableID: the ID of the variable to use for this reference.
//
// When Variable is set, VariableID is the ID of that variable.
// When Variable is not set, VariableID is the ID of something else that
// the caller code should use, such as a camera rig parameter override.

USTRUCT()
struct FBooleanCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = bool;
	using VariableAssetType = UBooleanCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UBooleanCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Boolean)
};

USTRUCT()
struct FInteger32CameraVariableReference
{
	GENERATED_BODY()

	using ValueType = int32;
	using VariableAssetType = UInteger32CameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UInteger32CameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Integer32)
};

USTRUCT()
struct FFloatCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = float;
	using VariableAssetType = UFloatCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UFloatCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Float)
};

USTRUCT()
struct FDoubleCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = double;
	using VariableAssetType = UDoubleCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UDoubleCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Double)
};

USTRUCT()
struct FVector2fCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector2f;
	using VariableAssetType = UVector2fCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector2fCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector2f)
};

USTRUCT()
struct FVector2dCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector2d;
	using VariableAssetType = UVector2dCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector2dCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector2d)
};

USTRUCT()
struct FVector3fCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector3f;
	using VariableAssetType = UVector3fCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector3fCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector3f)
};

USTRUCT()
struct FVector3dCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector3d;
	using VariableAssetType = UVector3dCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector3dCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector3d)
};

USTRUCT()
struct FVector4fCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector4f;
	using VariableAssetType = UVector4fCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector4fCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector4f)
};

USTRUCT()
struct FVector4dCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FVector4d;
	using VariableAssetType = UVector4dCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UVector4dCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Vector4d)
};

USTRUCT()
struct FRotator3fCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FRotator3f;
	using VariableAssetType = URotator3fCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<URotator3fCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Rotator3f)
};

USTRUCT()
struct FRotator3dCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FRotator3d;
	using VariableAssetType = URotator3dCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<URotator3dCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Rotator3d)
};

USTRUCT()
struct FTransform3fCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FTransform3f;
	using VariableAssetType = UTransform3fCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UTransform3fCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Transform3f)
};

USTRUCT()
struct FTransform3dCameraVariableReference
{
	GENERATED_BODY()

	using ValueType = FTransform3d;
	using VariableAssetType = UTransform3dCameraVariable;

	UPROPERTY()
	FCameraVariableID VariableID;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UTransform3dCameraVariable> Variable;

	UE_DEFINE_CAMERA_VARIABLE_REFERENCE(Transform3d)
};

