// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTable.h"
#include "Misc/InlineValue.h"
#include "Templates/UnrealTypeTraits.h"
#include <type_traits>

#include "CameraVariableSetter.generated.h"

namespace UE::Cameras
{
	class FCameraParameterSetterService;
}

/**
 * The blend type for a camera variable setter.
 */
UENUM(BlueprintType)
enum class ECameraVariableSetterBlendType : uint8
{
	None,
	Linear,
	SmoothStep,
	SmootherStep
};

/**
 * A handle to an ongoing camera variable setter.
 */
USTRUCT(BlueprintType)
struct FCameraVariableSetterHandle
{
	GENERATED_BODY()

public:

	FCameraVariableSetterHandle() : Value(INVALID), SerialNumber(0) {}

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

public:

	friend bool operator==(FCameraVariableSetterHandle A, FCameraVariableSetterHandle B)
	{
		return A.Value == B.Value && A.SerialNumber == B.SerialNumber;
	}

	friend bool operator!=(FCameraVariableSetterHandle A, FCameraVariableSetterHandle B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FCameraVariableSetterHandle In)
	{
		return HashCombineFast(In.Value, In.SerialNumber);
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraVariableSetterHandle& In)
	{
		Ar << In.Value;
		Ar << In.SerialNumber;
		return Ar;
	}

private:

	FCameraVariableSetterHandle(uint32 InValue, uint32 InSerialNumber) 
		: Value(InValue)
		, SerialNumber(InSerialNumber)
	{}

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;

	UPROPERTY()
	uint32 SerialNumber;

	friend class UE::Cameras::FCameraParameterSetterService;
};

namespace UE::Cameras
{

/**
 * A base structure for overriding a camera variable value until told to stop.
 * Overriding supports blending in and out.
 *
 * This base structure doesn't apply the values, see TCameraVariableSetter for that.
 */
struct FCameraVariableSetter
{
	ECameraVariableSetterBlendType BlendType = ECameraVariableSetterBlendType::Linear;
	float BlendInTime = 1.f;
	float BlendOutTime = 1.f;

	virtual ~FCameraVariableSetter() = default;

	virtual void Apply(FCameraVariableTable& VariableTable) {}

	void Update(float DeltaTime);
	void Stop(bool bImmediately);
	bool IsActive() const { return State != EState::Inactive; }
	
protected:

	FCameraVariableSetter() {}
	FCameraVariableSetter(FCameraVariableID InVariableID) : VariableID(InVariableID) {}

	void UpdateState(float DeltaTime);
	float GetBlendFactor();

protected:

	enum class EState { Inactive, BlendIn, Full, BlendOut };

	float CurrentTime = 0.f;
	EState State = EState::BlendIn;
	FCameraVariableID VariableID;
};

using FCameraVariableSetterPtr = TInlineValue<FCameraVariableSetter, 64>;

namespace Internal
{

template<typename ValueType>
struct TVariableSetterValueInterpolator
{
	static ValueType Lerp(typename TCallTraits<ValueType>::ParamType FromValue, typename TCallTraits<ValueType>::ParamType ToValue, float BlendFactor)
	{
		return FMath::Lerp(FromValue, ToValue, BlendFactor);
	}
};

template<>
struct TVariableSetterValueInterpolator<bool>
{
	static bool Lerp(bool FromValue, bool ToValue, float BlendFactor)
	{
		return (BlendFactor >= 0.5f) ? ToValue : FromValue;
	}
};

template<typename T>
struct TVariableSetterValueInterpolator<UE::Math::TTransform<T>>
{
	static UE::Math::TTransform<T> Lerp(const UE::Math::TTransform<T>& FromValue, const UE::Math::TTransform<T>& ToValue, float BlendFactor)
	{
		UE::Math::TTransform<T> CurrentValue(FromValue);
		CurrentValue.BlendWith(ToValue, BlendFactor);
		return CurrentValue;
	}
};

}  // namespace Internal

/**
 * An actual camera variable setter for a given type of values.
 */
template<typename ValueType>
struct TCameraVariableSetter : public FCameraVariableSetter
{
	ValueType ToValue;

	TCameraVariableSetter() = default;
	TCameraVariableSetter(FCameraVariableID InVariableID, typename TCallTraits<ValueType>::ParamType InToValue)
		: FCameraVariableSetter(InVariableID)
		, ToValue(InToValue)
	{}

	virtual void Apply(FCameraVariableTable& VariableTable) override
	{
		ValueType FromValue;
		const bool bIsValid = VariableTable.TryGetValue<ValueType>(VariableID, FromValue);
		if (bIsValid)
		{
			const float Factor = GetBlendFactor();
			const ValueType CurrentValue = Internal::TVariableSetterValueInterpolator<ValueType>::Lerp(FromValue, ToValue, Factor);
			VariableTable.SetValue<ValueType>(VariableID, CurrentValue);
		}
	}
};

}  // namespace UE::Cameras

