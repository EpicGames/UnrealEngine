// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Core/CameraVariableTableFwd.h"
#include "Core/CameraVariableTableAllocationInfo.h"
#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UnrealTypeTraits.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

template<typename ValueType>
struct TCameraVariableTraits
{
	static const ECameraVariableType Type = ECameraVariableType::BlendableStruct;
};

template<typename ValueType>
struct TCameraVariableInterpolation;

/**
 * Filter for variable table operations.
 */
enum class ECameraVariableTableFilter
{
	None = 0,
	/** Don't include private variables. */
	PublicOnly = 1 << 0,
	/** Only include input variables. */
	InputOnly = 1 << 1,
	/** Only include data that is common to both tables. */
	KnownOnly = 1 << 2,
	/** Only include variables that were written this frame. */
	ChangedOnly = 1 << 3
};
ENUM_CLASS_FLAGS(ECameraVariableTableFilter)

/**
 * A structure that keeps track of which variables have been processed in a
 * camera variable table.
 */
struct FCameraVariableTableFlags
{
	/** The list of processed variable IDs. */
	TSet<FCameraVariableID> VariableIDs;
};

/**
 * The camera variable table is a container for a collection of arbitrary values
 * of various types. Only certain basic types are supported (most primitive types).
 * 
 * This table serves both as an implementation of the usual "blackboard" design, where
 * gameplay systems can push any appropriate values into the camera system, and as a
 * place for camera node evaluators to stash various things.
 *
 * The main function of the variable table is that it is blended along with the camera
 * rig it belongs to. Any matching values between to blended tables with be themselves
 * blended, except for values flagged as "private".
 *
 * Internally, the variable table is allocated as one continuous block of memory, plus
 * a map of metadata keyed by variable ID. A variable ID can be anything, but will
 * generally be the hash of the variable name.
 */
class FCameraVariableTable
{
public:

	UE_API FCameraVariableTable();
	UE_API FCameraVariableTable(FCameraVariableTable&& Other);
	UE_API FCameraVariableTable& operator=(FCameraVariableTable&& Other);
	UE_API ~FCameraVariableTable();

	FCameraVariableTable(const FCameraVariableTable&) = delete;
	FCameraVariableTable& operator=(const FCameraVariableTable&) = delete;

	/** Initializes the variable table so that it fits the provided allocation info. */
	UE_API void Initialize(const FCameraVariableTableAllocationInfo& AllocationInfo);

	/**
	 * Ensure that all the variables referenced in the given allocation info exist in the table.
	 * Returns false if some variables had to be added.
	 */
	UE_API bool EnsureVariables(const FCameraVariableTableAllocationInfo& AllocationInfo);

	/** Adds a variable to the table.
	 *
	 * This may re-allocate the internal memory buffer. It's recommended to pre-compute
	 * the allocation information needed for a table, and initialize it once.
	 */
	UE_API void AddVariable(const FCameraVariableDefinition& VariableDefinition);

public:

	// Getter methods.

	template<typename ValueType>
	const ValueType* FindValue(FCameraVariableID VariableID) const;

	template<typename ValueType>
	const ValueType& GetValue(FCameraVariableID VariableID) const;

	template<typename ValueType>
	ValueType GetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType DefaultValue) const;

	template<typename VariableAssetType>
	typename VariableAssetType::ValueType GetValue(const VariableAssetType* VariableAsset) const;

	template<typename ValueType>
	bool TryGetValue(FCameraVariableID VariableID, ValueType& OutValue) const;

	template<typename VariableAssetType>
	bool TryGetValue(const VariableAssetType* VariableAsset, typename VariableAssetType::ValueType& OutValue) const;

	UE_API bool ContainsValue(FCameraVariableID VariableID) const;

public:

	// Setter methods.
	
	template<typename ValueType>
	void SetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType Value);

	template<typename ValueType>
	bool TrySetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType Value);

	template<typename ValueType>
	void SetValue(
			const FCameraVariableDefinition& VariableDefinition, 
			typename TCallTraits<ValueType>::ParamType Value,
			bool bCreateIfMissing = false);

	template<typename VariableAssetType>
	void SetValue(
			const VariableAssetType* VariableAsset, 
			typename TCallTraits<typename VariableAssetType::ValueType>::ParamType Value, 
			bool bCreateIfMissing = false);

public:

	// Interpolation.
	
	UE_API void OverrideAll(const FCameraVariableTable& OtherTable, bool bIncludePrivateValues = false);
	UE_API void Override(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter);
	UE_API void Override(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter, const FCameraVariableTableFlags& InMask, bool bInvertMask, FCameraVariableTableFlags& OutMask);

	UE_API void LerpAll(const FCameraVariableTable& ToTable, float Factor, bool bIncludePrivateValues = false);
	UE_API void Lerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor);
	UE_API void Lerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor, const FCameraVariableTableFlags& InMask, bool bInvertMask, FCameraVariableTableFlags& OutMask);

public:

	// Lower level API.

	UE_API const uint8* GetValue(
			FCameraVariableID VariableID,
			ECameraVariableType ExpectedVariableType,
			const UScriptStruct* ExpectedBlendableStructType,
			bool bOnlyIfWritten = true) const;

	UE_API const uint8* TryGetValue(
			FCameraVariableID VariableID,
			ECameraVariableType ExpectedVariableType,
			const UScriptStruct* ExpectedBlendableStructType,
			bool bOnlyIfWritten = true) const;

	UE_API uint8* TryGetMutableValue(
			FCameraVariableID VariableID,
			ECameraVariableType ExpectedVariableType,
			const UScriptStruct* ExpectedBlendableStructType,
			bool bOnlyIfWritten = true);

	UE_API void SetValue(
			FCameraVariableID VariableID, 
			ECameraVariableType ExpectedVariableType, 
			const UScriptStruct* ExpectedBlendableStructType,
			const uint8* InRawValuePtr,
			bool bMarkAsWrittenThisFrame = true);

	UE_API bool TrySetValue(
			FCameraVariableID VariableID, 
			ECameraVariableType ExpectedVariableType, 
			const UScriptStruct* ExpectedBlendableStructType,
			const uint8* InRawValuePtr,
			bool bMarkAsWrittenThisFrame = true);

	UE_API bool IsValueWritten(FCameraVariableID VariableID) const;
	UE_API void UnsetValue(FCameraVariableID VariableID);
	UE_API void UnsetAllValues();

	UE_API bool IsValueWrittenThisFrame(FCameraVariableID VariableID) const;
	UE_API void ClearAllWrittenThisFrameFlags();

	UE_API void AutoResetValues();

	UE_API bool TryGetVariableDefinition(FCameraVariableID VariableID, FCameraVariableDefinition& OutVariableDefinition) const;

	UE_API void Serialize(FArchive& Ar);

private:

	struct FEntry;

	static UE_API void CacheBlendableStructs();
	static UE_API FBlendableStructTypeErasedInterpolator GetBlendableStructInterpolator(const UScriptStruct* StructType);

	static UE_API bool GetVariableTypeAllocationInfo(ECameraVariableType VariableType, const UScriptStruct* StructType, uint32& OutSizeOf, uint32& OutAlignOf);

	static void CreateEntry(const FCameraVariableDefinition& VariableDefinition, FEntry& OutEntry);

	template<typename ValueType>
	static bool CheckVariableType(ECameraVariableType InType)
	{
		return ensure(TCameraVariableTraits<ValueType>::Type == InType);
	}

	UE_API void ReallocateBuffer(uint32 MinRequired = 0);

	UE_API FEntry* FindEntry(FCameraVariableID VariableID);
	UE_API const FEntry* FindEntry(FCameraVariableID VariableID) const;

	UE_API void InternalOverride(const FCameraVariableTable& OtherTable, ECameraVariableTableFilter Filter, const FCameraVariableTableFlags* InMask, bool bInvertMask, FCameraVariableTableFlags* OutMask);
	UE_API void InternalLerp(const FCameraVariableTable& ToTable, ECameraVariableTableFilter Filter, float Factor, const FCameraVariableTableFlags* InMask, bool bInvertMask, FCameraVariableTableFlags* OutMask);

	void InitializeEntries(int32 StartIndex = 0, int32 EndIndex = -1);

private:

	static const uint32 GDefaultCapacity;
	static const uint32 GDefaultAlignment;

	enum class EEntryFlags : uint8
	{
		None = 0,
		Private = 1 << 0,
		Input = 1 << 1,
		AutoReset = 1 << 2,
		Written = 1 << 3,
		WrittenThisFrame = 1 << 4
	};
	FRIEND_ENUM_CLASS_FLAGS(EEntryFlags)

	struct FEntry
	{
		FCameraVariableID ID;
		ECameraVariableType Type;
		const UScriptStruct* StructType = nullptr;
		uint32 Offset;
		EEntryFlags Flags;
#if WITH_EDITORONLY_DATA
		FString DebugName;
#endif
	};

	TArray<FEntry> Entries;
	TMap<FCameraVariableID, int32> EntryLookup;

	uint8* Memory = nullptr;
	uint32 Capacity = 0;
	uint32 Alignment = 0;
	uint32 Used = 0;

	static UE_API TArray<FBlendableStructInfo> CachedBlendableStructs;
	static UE_API bool bCachedBlendableStructs;

	template<typename T>
	friend struct TCameraVariableInterpolation;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	friend class FVariableTableDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

ENUM_CLASS_FLAGS(FCameraVariableTable::EEntryFlags)

template<typename ValueType>
const ValueType* FCameraVariableTable::FindValue(FCameraVariableID VariableID) const
{
	if (const FEntry* Entry = FindEntry(VariableID))
	{
		CheckVariableType<ValueType>(Entry->Type);
		if (EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written))
		{
			return reinterpret_cast<ValueType*>(Memory + Entry->Offset);
		}
	}
	return nullptr;
}

template<typename ValueType>
const ValueType& FCameraVariableTable::GetValue(FCameraVariableID VariableID) const
{
	const FEntry* Entry = FindEntry(VariableID);
	if (ensureMsgf(Entry, TEXT("Can't get camera variable (ID '%d') because it doesn't exist in the table."), VariableID.GetValue()))
	{
		CheckVariableType<ValueType>(Entry->Type);
#if WITH_EDITORONLY_DATA
		checkf(
				EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written),
				TEXT("Variable '%s' has never been written to. GetValue() will return uninitialized memory!"),
				*Entry->DebugName);
#else
		checkf(
				EnumHasAnyFlags(Entry->Flags, EEntryFlags::Written),
				TEXT("Variable '%s' has never been written to. GetValue() will return uninitialized memory!"),
				*LexToString(VariableID.GetValue()));
#endif
		return *reinterpret_cast<ValueType*>(Memory + Entry->Offset);
	}

	static ValueType DefaultValue = ValueType();
	return DefaultValue;
}

template<typename ValueType>
ValueType FCameraVariableTable::GetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType DefaultValue) const
{
	if (const ValueType* Value = FindValue<ValueType>(VariableID))
	{
		return *Value;
	}
	return DefaultValue;
}

template<typename VariableAssetType>
typename VariableAssetType::ValueType FCameraVariableTable::GetValue(const VariableAssetType* VariableAsset) const
{
	return GetValue<typename VariableAssetType::ValueType>(VariableAsset->GetVariableID(), VariableAsset->GetDefaultValue());
}

template<typename ValueType>
bool FCameraVariableTable::TryGetValue(FCameraVariableID VariableID, ValueType& OutValue) const
{
	if (const ValueType* Value = FindValue<ValueType>(VariableID))
	{
		OutValue = *Value;
		return true;
	}
	return false;
}

template<typename VariableAssetType>
bool FCameraVariableTable::TryGetValue(const VariableAssetType* VariableAsset, typename VariableAssetType::ValueType& OutValue) const
{
	return TryGetValue(VariableAsset->GetVariableID(), OutValue);
}

template<typename ValueType>
void FCameraVariableTable::SetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType Value)
{
	FEntry* Entry = FindEntry(VariableID);
	if (ensureMsgf(Entry, TEXT("Can't set camera variable (ID '%d') because it doesn't exist in the table."), VariableID.GetValue()))
	{
		CheckVariableType<ValueType>(Entry->Type);
		ValueType* ValuePtr = reinterpret_cast<ValueType*>(Memory + Entry->Offset);
		*ValuePtr = Value;
		Entry->Flags |= EEntryFlags::Written | EEntryFlags::WrittenThisFrame;
	}
}

template<typename ValueType>
bool FCameraVariableTable::TrySetValue(FCameraVariableID VariableID, typename TCallTraits<ValueType>::ParamType Value)
{
	if (FEntry* Entry = FindEntry(VariableID))
	{
		CheckVariableType<ValueType>(Entry->Type);
		ValueType* ValuePtr = reinterpret_cast<ValueType*>(Memory + Entry->Offset);
		*ValuePtr = Value;
		Entry->Flags |= EEntryFlags::Written | EEntryFlags::WrittenThisFrame;
		return true;
	}
	return false;
}

template<typename ValueType>
void FCameraVariableTable::SetValue(
		const FCameraVariableDefinition& VariableDefinition, 
		typename TCallTraits<ValueType>::ParamType Value,
		bool bCreateIfMissing)
{
	const bool bDidSet = TrySetValue<ValueType>(VariableDefinition.VariableID, Value);
#if WITH_EDITORONLY_DATA
	ensureMsgf(
			bDidSet || bCreateIfMissing, 
			TEXT("Can't set camera variable '%s' (ID '%d') because it doesn't exist in the table."),
			*VariableDefinition.VariableName, VariableDefinition.VariableID.GetValue());
#else
	ensureMsgf(
			bDidSet || bCreateIfMissing, 
			TEXT("Can't set camera variable '%s' (ID '%d') because it doesn't exist in the table."),
			*LexToString(VariableDefinition.VariableID.GetValue()), VariableDefinition.VariableID.GetValue());
#endif
	if (bDidSet)
	{
		return;
	}

	if (bCreateIfMissing)
	{
		AddVariable(VariableDefinition);
		SetValue<ValueType>(VariableDefinition.VariableID, Value);
	}
}

template<typename VariableAssetType>
void FCameraVariableTable::SetValue(
		const VariableAssetType* VariableAsset, 
		typename TCallTraits<typename VariableAssetType::ValueType>::ParamType Value, 
		bool bCreateIfMissing)
{
	if (ensure(VariableAsset))
	{
		const bool bDidSet = TrySetValue<typename VariableAssetType::ValueType>(VariableAsset->GetVariableID(), Value);
		ensureMsgf(
				bDidSet || bCreateIfMissing, 
				TEXT("Can't set camera variable '%s' (ID '%d') because it doesn't exist in the table."),
				*GetNameSafe(VariableAsset), VariableAsset->GetVariableID().GetValue());
		if (bDidSet)
		{
			return;
		}

		if (bCreateIfMissing)
		{
			FCameraVariableDefinition VariableDefinition = VariableAsset->GetVariableDefinition();
			AddVariable(VariableDefinition);
			SetValue<typename VariableAssetType::ValueType>(VariableDefinition.VariableID, Value);
		}
	}
}

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	template<>\
	struct TCameraVariableTraits<ValueType>\
	{\
		static const ECameraVariableType Type = ECameraVariableType::ValueName;\
	};
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

template<typename ValueType>
struct TCameraVariableInterpolation
{
	using ValueParam = typename TCallTraits<ValueType>::ParamType;
	static ValueType Interpolate(const FCameraVariableTable::FEntry& TableEntry, ValueParam From, ValueParam To, float Factor)
	{
		return FMath::LerpStable(From, To, Factor);
	}
};

template<typename T>
struct TCameraVariableInterpolation<UE::Math::TTransform<T>>
{
	using ValueType = UE::Math::TTransform<T>;
	using ValueParam = typename TCallTraits<ValueType>::ParamType;
	static ValueType Interpolate(const FCameraVariableTable::FEntry& TableEntry, ValueParam From, ValueParam To, float Factor)
	{
		ValueType Result(From);
		Result.BlendWith(To, Factor);
		return Result;
	}
};

}  // namespace UE::Cameras

#undef UE_API
