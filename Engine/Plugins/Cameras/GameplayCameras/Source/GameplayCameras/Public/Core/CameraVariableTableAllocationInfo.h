// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableFwd.h"
#include "UObject/ObjectMacros.h"

#include "CameraVariableTableAllocationInfo.generated.h"

/**
 * A structure that describes a camera variable.
 */
USTRUCT()
struct FCameraVariableDefinition
{
	GENERATED_BODY()

	/** The ID of the variable. */
	UPROPERTY()
	FCameraVariableID VariableID;

	/** The type of the variable. */
	UPROPERTY()
	ECameraVariableType VariableType = ECameraVariableType::Boolean;

	/** The type of a blendable struct (only valid when VariableType == BlendableStruct). */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> BlendableStructType; 

	/**
	 * Whether the variable is private. 
	 *
	 * Private variables are not propagated from one table to another when
	 * interpolating or overriding a table.
	 */
	UPROPERTY()
	bool bIsPrivate = false;

	/**
	 * Whether the variable is an input variable.
	 *
	 * Input variables are blended during the pre-blend parameter update phase.
	 */
	UPROPERTY()
	bool bIsInput = false;

	/** Whether the variable should auto-reset to an "unset" state after every evaluation. */
	UPROPERTY()
	bool bAutoReset = false;

#if WITH_EDITORONLY_DATA
	/** The name of the variable, for debugging purposes. */
	UPROPERTY()
	FString VariableName;
#endif

	/** Returns whether this definition has a valid variable ID. */
	bool IsValid() const
	{
		return VariableID.IsValid();
	}

	/** Implicit conversion to a camera variable ID. */
	operator FCameraVariableID() const
	{
		return VariableID;
	}

	/** Creates a variant of this camera variable definition. */
	FCameraVariableDefinition CreateVariant(const FString& VariantID) const
	{
		FCameraVariableDefinition VariantDefinition(*this);
		VariantDefinition.VariableID = FCameraVariableID::FromHashValue(
				HashCombineFast(VariableID.GetValue(), GetTypeHash(VariantID)));
#if WITH_EDITORONLY_DATA
		if (!VariableName.IsEmpty())
		{
			VariantDefinition.VariableName += FString::Format(TEXT("_{0}Variant"), { VariantID });
		}
#endif
		return VariantDefinition;
	}

	bool operator==(const FCameraVariableDefinition& Other) const = default;
};

template<>
struct TStructOpsTypeTraits<FCameraVariableDefinition> : public TStructOpsTypeTraitsBase2<FCameraVariableDefinition>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * A structure that describes the required camera variable table setup of a camera rig.
 */
USTRUCT()
struct FCameraVariableTableAllocationInfo
{
	GENERATED_BODY()

	/** The list of variables that should be allocated in a table. */
	UPROPERTY()
	TArray<FCameraVariableDefinition> VariableDefinitions;

	/**Combines the given allocation info with this one. */
	GAMEPLAYCAMERAS_API void Combine(const FCameraVariableTableAllocationInfo& OtherInfo);

	/** Returns whether the given variable definitions already exist in our list. */
	GAMEPLAYCAMERAS_API bool Contains(const FCameraVariableTableAllocationInfo& OtherInfo) const;

	bool operator==(const FCameraVariableTableAllocationInfo& Other) const = default;
};

template<>
struct TStructOpsTypeTraits<FCameraVariableTableAllocationInfo> : public TStructOpsTypeTraitsBase2<FCameraVariableTableAllocationInfo>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

