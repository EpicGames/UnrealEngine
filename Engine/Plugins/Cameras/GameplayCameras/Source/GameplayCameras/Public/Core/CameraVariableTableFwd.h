// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/TypeHash.h"

#include "CameraVariableTableFwd.generated.h"

class UCameraVariableAsset;

#define UE_CAMERA_VARIABLE_FOR_ALL_TYPES()\
	UE_CAMERA_VARIABLE_FOR_TYPE(bool, Boolean)\
	UE_CAMERA_VARIABLE_FOR_TYPE(int32, Integer32)\
	UE_CAMERA_VARIABLE_FOR_TYPE(float, Float)\
	UE_CAMERA_VARIABLE_FOR_TYPE(double, Double)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector2f, Vector2f)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector2d, Vector2d)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector3f, Vector3f)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector3d, Vector3d)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector4f, Vector4f)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FVector4d, Vector4d)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FRotator3f, Rotator3f)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FRotator3d, Rotator3d)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FTransform3f, Transform3f)\
	UE_CAMERA_VARIABLE_FOR_TYPE(FTransform3d, Transform3d)

/**
 * The type of a camera variable. 
 *
 * Only a fixed set of types are supported for camera variables because of
 * simplicity, and because these types need to be blendable.
 */
UENUM()
enum class ECameraVariableType
{
	Boolean,
	Integer32,
	Float,
	Double,
	Vector2f,
	Vector2d,
	Vector3f,
	Vector3d,
	Vector4f,
	Vector4d,
	Rotator3f,
	Rotator3d,
	Transform3f,
	Transform3d,
	BlendableStruct
};

/**
 * The ID of a camera variable, used to refer to it in a camera variable table.
 */
USTRUCT()
struct FCameraVariableID
{
	GENERATED_BODY()

public:

	FCameraVariableID() : Value(INVALID) {}

	uint32 GetValue() const { return Value; }

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

	static FCameraVariableID FromHashValue(uint32 InValue)
	{
		return FCameraVariableID(InValue);
	}

public:

	friend bool operator<(FCameraVariableID A, FCameraVariableID B)
	{
		return A.Value < B.Value;
	}

	friend bool operator==(FCameraVariableID A, FCameraVariableID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCameraVariableID A, FCameraVariableID B)
	{
		return A.Value != B.Value;
	}

	friend uint32 GetTypeHash(FCameraVariableID In)
	{
		return In.Value;
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraVariableID& In)
	{
		Ar << In.Value;
		return Ar;
	}

private:

	FCameraVariableID(uint32 InValue) : Value(InValue) {}

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;
};

namespace UE::Cameras
{

using FBlendableStructTypeErasedInterpolator = void(*)(uint8* From, const uint8* To, float Alpha);

struct FBlendableStructInfo
{
	TObjectPtr<const UScriptStruct> StructType;
	FBlendableStructTypeErasedInterpolator Interpolator;
};

}  // namespace UE::Cameras

