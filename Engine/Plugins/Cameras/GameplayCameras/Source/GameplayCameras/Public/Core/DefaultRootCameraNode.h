// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/RootCameraNode.h"
#include "Core/BlendStackCameraRigEvent.h"
#include "Templates/SharedPointerFwd.h"

#include "DefaultRootCameraNode.generated.h"

class UBlendStackCameraNode;

/**
 * The default implementation of a root camera node.
 */
UCLASS(MinimalAPI, Hidden)
class UDefaultRootCameraNode : public URootCameraNode
{
	GENERATED_BODY()

public:

	UDefaultRootCameraNode(const FObjectInitializer& ObjectInit);

protected:

	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	UPROPERTY(Instanced)
	TObjectPtr<UBlendStackCameraNode> BaseLayer;

	UPROPERTY(Instanced)
	TObjectPtr<UBlendStackCameraNode> MainLayer;

	UPROPERTY(Instanced)
	TObjectPtr<UBlendStackCameraNode> GlobalLayer;

	UPROPERTY(Instanced)
	TObjectPtr<UBlendStackCameraNode> VisualLayer;
};

namespace UE::Cameras
{

class FCameraParameterSetterService;
class FPersistentBlendStackCameraNodeEvaluator;
class FTransientBlendStackCameraNodeEvaluator;

/**
 * Evaluator for the default root camera node.
 */
class FDefaultRootCameraNodeEvaluator : public FRootCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FDefaultRootCameraNodeEvaluator, FRootCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// FRootCameraNodeEvaluator interface.
	virtual FCameraRigInstanceID OnActivateCameraRig(const FActivateCameraRigParams& Params) override;
	virtual void OnDeactivateCameraRig(const FDeactivateCameraRigParams& Params) override;
	virtual void OnDeactivateAllCameraRigs(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately) override;
	virtual void OnGetActiveCameraRigInfo(FCameraRigEvaluationInfo& OutCameraRigInfo) const override;
	virtual bool OnHasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const override;
	virtual void OnGetCameraRigInfo(const FCameraRigInstanceID InstanceID, FCameraRigEvaluationInfo& OutCameraRigInfo) const override;
	virtual const FCameraVariableTable* OnGetBlendedParameters() const override;
	virtual void OnBuildSingleCameraRigHierarchy(const FSingleCameraRigHierarchyBuildParams& Params, FCameraNodeEvaluatorHierarchy& OutHierarchy) override;
	virtual void OnRunSingleCameraRig(const FSingleCameraRigEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual TSharedPtr<FCameraActionScope> OnGetActiveCameraRigActionScope(bool bCreateIfNeeded) override;

private:

	template<typename EvaluatorType>
	EvaluatorType* BuildBlendStackEvaluator(const FCameraNodeEvaluatorBuildParams& Params, UBlendStackCameraNode* BlendStackNode);

	void OnBlendStackEvent(const FBlendStackCameraRigEvent& InEvent);

private:

	FPersistentBlendStackCameraNodeEvaluator* BaseLayer;
	FTransientBlendStackCameraNodeEvaluator* MainLayer;
	FPersistentBlendStackCameraNodeEvaluator* GlobalLayer;
	FPersistentBlendStackCameraNodeEvaluator* VisualLayer;

	TSharedPtr<FCameraParameterSetterService> ParameterSetterService;
};

}  // namespace UE::Cameras

