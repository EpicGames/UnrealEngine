// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigAsset.h"
#include "Containers/Array.h"
#include "UObject/Interface.h"

#include "IAssetReferenceCameraNode.generated.h"

class UCameraNode;

UINTERFACE(MinimalAPI)
class UAssetReferenceCameraNode : public UInterface
{
	GENERATED_BODY()
};

class IAssetReferenceCameraNode
{
	GENERATED_BODY()

public:

	virtual void GatherPackages(FCameraRigPackages& OutPackages) const = 0;
};

