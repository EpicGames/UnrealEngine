// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableFwd.h"
#include "Core/CameraContextDataTableAllocationInfo.h"
#include "UObject/Interface.h"
#include "UObject/ObjectPtr.h"

#include "ICustomCameraNodeParameterProvider.generated.h"

class UCameraNode;

namespace UE::Cameras
{

class FCameraNodeHierarchyBuilder;
class FCameraObjectInterfaceBuilder;
class FCameraObjectInterfaceParameterBuilder;
namespace Internal { struct FInterfaceParameterBindingBuilder; }

/** Information about a blendable parameter on a camera node. */
struct FCameraNodeBlendableParameterInfo
{
	FName ParameterName;
	ECameraVariableType VariableType = ECameraVariableType::Boolean;
	const UScriptStruct* BlendableStructType = nullptr;
	const uint8* DefaultValue = nullptr;
	FCameraVariableID* OverrideVariableID = nullptr;
	UCameraVariableAsset* OverrideVariable = nullptr;
};

/** Information about a data parameter on a camera node. */
struct FCameraNodeDataParameterInfo
{
	FName ParameterName;
	ECameraContextDataType DataType = ECameraContextDataType::Name;
	ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;
	const UObject* DataTypeObject = nullptr;
	const uint8* DefaultValue = nullptr;
	FCameraContextDataID* OverrideDataID = nullptr;
};

/**
 * A structure providing information about the exposed parameters on a camera node.
 */
struct FCameraNodeParameterInfos
{
	using FBlendableParameterInfo = FCameraNodeBlendableParameterInfo;
	using FDataParameterInfo = FCameraNodeDataParameterInfo;

public:

	/** 
	 * Declares a blendable parameter. 
	 * Pass a null pointer for OverrideVariableID if this parameter should not be overridable
	 * by a camera rig parameter.
	 */
	GAMEPLAYCAMERAS_API void AddBlendableParameter(
			FName ParameterName, 
			ECameraVariableType VariableType, 
			const UScriptStruct* BlendableStructType,
			const uint8* DefaultValue,
			FCameraVariableID* OverrideVariableID,
			UCameraVariableAsset* OverrideVariable = nullptr);

	GAMEPLAYCAMERAS_API void AddBlendableParameter(FCustomCameraNodeBlendableParameter& Parameter, const uint8* DefaultValue);

	/** 
	 * Declares a data parameter.
	 * Pass a null pointer for OverrideDataID if this parameter should not be overridable
	 * by a camera rig parameter.
	 */
	GAMEPLAYCAMERAS_API void AddDataParameter(
			FName ParameterName, 
			ECameraContextDataType DataType,
			ECameraContextDataContainerType DataContainerType,
			const UObject* DataTypeObject,
			const uint8* DefaultValue,
			FCameraContextDataID* OverrideDataID);

	GAMEPLAYCAMERAS_API void AddDataParameter(FCustomCameraNodeDataParameter& Parameter, const uint8* DefaultValue);

	/** Clears the list of parameters in this structure. */
	GAMEPLAYCAMERAS_API void Reset();

public:

	/** Returns whether there are any blendable or data parameters. */
	bool HasAnyParameters() const { return !BlendableParameters.IsEmpty() || !DataParameters.IsEmpty(); }

	/** Gets the list of blendable parameters. */
	TConstArrayView<FBlendableParameterInfo> GetBlendableParameters() const { return BlendableParameters; }
	/** Gets the list of data parameters. */
	TConstArrayView<FDataParameterInfo> GetDataParameters() const { return DataParameters; }

	/** Finds a blendable parameter of the given name. */
	GAMEPLAYCAMERAS_API const FBlendableParameterInfo* FindBlendableParameter(FName ParameterName) const;
	/** Finds a data parameter of the given name. */
	GAMEPLAYCAMERAS_API const FDataParameterInfo* FindDataParameter(FName ParameterName) const;

public:

	/** Build this structure from the given camera node. */
	GAMEPLAYCAMERAS_API void BuildFrom(UCameraNode* InCameraNode);

private:

	TArray<FBlendableParameterInfo> BlendableParameters;
	TArray<FDataParameterInfo> DataParameters;

	friend class FCameraNodeHierarchyBuilder;
	friend class FCameraObjectInterfaceBuilder;
	friend class FCameraObjectInterfaceParameterBuilder;
	friend struct Internal::FInterfaceParameterBindingBuilder;
};

}  // namespace UE::Cameras

/** Describes a custom camera blendable parameter. */
USTRUCT()
struct FCustomCameraNodeBlendableParameter
{
	GENERATED_BODY()

	/** The name of the parameter. */
	UPROPERTY()
	FName ParameterName;

	/** The type of the parameter. */
	UPROPERTY()
	ECameraVariableType VariableType = ECameraVariableType::Boolean;

	/** The struct type of a blendable struct. */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> BlendableStructType;

	/** An optional camera variable ID for dynamically driving the parameter's value. */
	UPROPERTY()
	FCameraVariableID OverrideVariableID;

	/** An optional user-defined camera variable for dynamically driving the parameter's value. */
	UPROPERTY()
	TObjectPtr<UCameraVariableAsset> OverrideVariable;

	bool operator==(const FCustomCameraNodeBlendableParameter& Other) const = default;
};

/** Describes a custom camera data parameter. */
USTRUCT()
struct FCustomCameraNodeDataParameter
{
	GENERATED_BODY()

	/** The name of the parameter. */
	UPROPERTY()
	FName ParameterName;

	/** The type of the parameter. */
	UPROPERTY()
	ECameraContextDataType DataType = ECameraContextDataType::Name;

	/** The type of the parameter container. */
	UPROPERTY()
	ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;

	/** An extra type object for the parameter. */
	UPROPERTY()
	TObjectPtr<const UObject> DataTypeObject;

	/** An optional context data ID for dynamically driving the parameter's value. */
	UPROPERTY()
	FCameraContextDataID OverrideDataID;

	bool operator==(const FCustomCameraNodeDataParameter& Other) const = default;
};

/** 
 * Describes custom camera parameters. This structure is suitable for being owned by a camera node
 * that wants to manage a variable number of custom parameters.
 */
USTRUCT()
struct FCustomCameraNodeParameters
{
	GENERATED_BODY()

	/** The list of blendable parameters. */
	UPROPERTY()
	TArray<FCustomCameraNodeBlendableParameter> BlendableParameters;

	/** The list of data parameters. */
	UPROPERTY()
	TArray<FCustomCameraNodeDataParameter> DataParameters;

	/** Returns whether this struct has any blendable or data parameter. */
	bool HasAnyParameters() const { return !BlendableParameters.IsEmpty() || !DataParameters.IsEmpty(); }

	/** Removes all parameters from this structure. */
	void Reset() { BlendableParameters.Reset(); DataParameters.Reset(); }

	bool operator==(const FCustomCameraNodeParameters& Other) const = default;
};

UINTERFACE(MinimalAPI)
class UCustomCameraNodeParameterProvider : public UInterface
{
	GENERATED_BODY()
};

/**
 * An interface for camera nodes that want to expose a custom interface of
 * blendable parameters and data parameters.
 */
class ICustomCameraNodeParameterProvider
{
	GENERATED_BODY()

public:

	using FCameraNodeParameterInfos = UE::Cameras::FCameraNodeParameterInfos;

	/** Gathers the custom parameters on this node. */
	virtual void GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos) {}

	/** Utility function for sub-classes to broadcast when the custom parameters have changed. */
	GAMEPLAYCAMERAS_API void OnCustomCameraNodeParametersChanged(const UCameraNode* ThisAsCameraNode) const;
};

