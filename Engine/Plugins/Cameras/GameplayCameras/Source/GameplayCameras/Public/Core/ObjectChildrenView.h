// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Misc/TVariant.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/ObjectPtr.h"

#include <initializer_list>
#include <type_traits>

namespace UE::Cameras
{

/**
 * Structure that describes a list of children of an object.
 *
 * This structure can either provide a TArrayView<> on an existing container of children,
 * or store within itself a list of arbitrary camera node children.
 */
template<typename ChildType>
struct TObjectChildrenView
{
public:

	using FArrayView = TArrayView<ChildType>;
	using FArray = TArray<ChildType, TInlineAllocator<4>>;

	/** An empty view. */
	TObjectChildrenView()
	{
	}

	/** Sets the view to the given TArrayView<>. */
	TObjectChildrenView(FArrayView&& InArrayView)
	{
		Storage.template Set<FArrayView>(MoveTemp(InArrayView));
	}

	/** Sets the view to pointer storage and adds the given list of children pointers. */
	TObjectChildrenView(std::initializer_list<ChildType> InChildren)
	{
		Storage.template Set<FArray>(FArray(InChildren));
	}

	/** Sets the view to pointer storage and adds the given list of children pointers. */
	template<typename OtherType, bool V = std::is_convertible_v<OtherType, ChildType>>
	TObjectChildrenView(TArrayView<OtherType> InRange)
	{
		Storage.template Set<FArray>(FArray(InRange));
	}

	/**
	 * Sets the view to pointer storage (if not already done) and adds the given
	 * pointer to the list.
	 */
	void Add(typename TCallTraits<ChildType>::ParamType InChild)
	{
		if (Storage.GetIndex() != FStorage::template IndexOfType<FArray>())
		{
			Storage.template Set<FArray>(FArray());
		}
		FArray& Array = Storage.template Get<FArray>();
		Array.Add(InChild);
	}

	/** Whether this view has any children. */
	bool IsEmpty() const
	{
		switch (Storage.GetIndex())
		{
			case FStorage::template IndexOfType<FArrayView>():
				return Storage.template Get<FArrayView>().IsEmpty();
			case FStorage::template IndexOfType<FArray>():
				return Storage.template Get<FArray>().IsEmpty();
			default:
				return true;
		}
	}

	/** Returns the number of children. */
	int32 Num() const
	{
		switch (Storage.GetIndex())
		{
			case FStorage::template IndexOfType<FArrayView>():
				return Storage.template Get<FArrayView>().Num();
			case FStorage::template IndexOfType<FArray>():
				return Storage.template Get<FArray>().Num();
			default:
				return 0;
		}
	}

	/** Gets the i'th child. */
	ChildType operator[](int32 Index) const
	{
		switch (Storage.GetIndex())
		{
			case FStorage::template IndexOfType<FArrayView>():
				return Storage.template Get<FArrayView>()[Index];
			case FStorage::template IndexOfType<FArray>():
				return Storage.template Get<FArray>()[Index];
			default:
				return nullptr;
		}
	}

public:

	// Range iteration

	struct FBaseIterator
	{
		const TObjectChildrenView* Owner;
		int32 Index;

		inline ChildType operator*()
		{
			return (*Owner)[Index];
		}

		inline bool operator== (const FBaseIterator& Other) const
		{
			return Owner == Other.Owner
				&& Index == Other.Index;
		}

		inline bool operator!= (const FBaseIterator& Other) const
		{
			return !(*this == Other);
		}
	};

	struct FIterator : FBaseIterator
	{
		inline FIterator& operator++()
		{
			++FBaseIterator::Index;
			return *this;
		}
	};

	inline FIterator begin() const { return FIterator{ this, 0 }; }
	inline FIterator end() const { return FIterator{ this, Num() }; }

	struct FReverseIterator : FBaseIterator
	{
		inline FReverseIterator& operator++()
		{
			--FBaseIterator::Index;
			return *this;
		}
	};

	inline FReverseIterator rbegin() const { return FReverseIterator{ this, Num() - 1 }; }
	inline FReverseIterator rend() const { return FReverseIterator{ this, -1 }; }

private:

	using FStorage = TVariant<FArrayView, FArray>;
	FStorage Storage;
};

}  // namespace UE::Cameras

