// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/ObjectTreeGraphObject.h"
#include "UObject/Object.h"

#include "ObjectTreeGraphComment.generated.h"

/**
 * A generic comment node for object tree graphs.
 */
UCLASS(MinimalAPI)
class UObjectTreeGraphComment 
	: public UObject
	, public IObjectTreeGraphObject
{
	GENERATED_BODY()

public:

	UObjectTreeGraphComment(const FObjectInitializer& ObjInit);

#if WITH_EDITOR

public:

	// IObjectTreeGraphObject interface.
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const override;
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) override;
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const override;
	virtual void OnRenameGraphNode(FName InGraphName, const FString& NewName) override;

#endif  // WITH_EDITOR

#if WITH_EDITORONLY_DATA

public:

	/** The text of the comment node. */
	UPROPERTY()
	FString CommentText;

	/** Color of the comment node in the node graph editor. */
	UPROPERTY()
	FLinearColor CommentColor = FLinearColor::White;

	/** Position of the comment node in the node graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodePos = FIntVector2::ZeroValue;

	/** Size of the comment node in the node graph editor. */
	UPROPERTY()
	FIntVector2 GraphNodeSize = FIntVector2(400, 100);

#endif  // WITH_EDITORONLY_DATA
};

