// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/Interface.h"

#include "ObjectTreeGraphObject.generated.h"

UINTERFACE(MinimalAPI)
class UObjectTreeGraphObject : public UInterface
{
	GENERATED_BODY()
};

#if WITH_EDITOR

/** Flags representing supported optional APIs of an IObjectTreeGraphObject. */
enum class EObjectTreeGraphObjectSupportFlags
{
	/** None supported, only graph node position. */
	None,
	/** Supports storing a comment text. */
	CommentText = 1 << 0,
	/** Has a custom graph node title. */
	CustomTitle = 1 << 1,
	/** Supports custom renaming the graph node. */
	CustomRename = 1 << 2
};
ENUM_CLASS_FLAGS(EObjectTreeGraphObjectSupportFlags);

#endif  // WITH_EDITOR

/**
 * An interface that UObjects can implement to customize how they are represented
 * and interacted with inside an object tree graph (see UObjectTreeGraph).
 */
class IObjectTreeGraphObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	// Minimal API.
	
	/** Gets the canvas position for the graph node representing this object. */
	virtual void GetGraphNodePosition(FName InGraphName, int32& NodePosX, int32& NodePosY) const {}
	/** Called to save the canvas position of the graph node representing this object. */
	virtual void OnGraphNodeMoved(FName InGraphName, int32 NodePosX, int32 NodePosY, bool bMarkDirty) {}

	// Optional API.

	/** Gets optional APIs support flags. */
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const { return EObjectTreeGraphObjectSupportFlags::None; }
	/** Gets whether this object supports the given optional API. */
	bool HasAnySupportFlags(FName InGraphName, EObjectTreeGraphObjectSupportFlags InFlags) const { return EnumHasAnyFlags(GetSupportFlags(InGraphName), InFlags); }

	/** Gets the graph node's comment text. */
	virtual const FString& GetGraphNodeCommentText(FName InGraphName) const { static const FString EmptyString; return EmptyString; }
	/** Called to save a new comment text. */
	virtual void OnUpdateGraphNodeCommentText(FName InGraphName, const FString& NewComment) {}

	/** Gets the custom name for the graph node. */
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const {}
	/** Called to save a new custom name. */
	virtual void OnRenameGraphNode(FName InGraphName, const FString& NewName) {}

#endif  // WITH_EDITOR
};

