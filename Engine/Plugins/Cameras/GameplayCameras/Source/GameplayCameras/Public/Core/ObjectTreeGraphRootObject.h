// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "ObjectTreeGraphRootObject.generated.h"

UINTERFACE(MinimalAPI)
class UObjectTreeGraphRootObject : public UInterface
{
	GENERATED_BODY()
};

class IObjectTreeGraphRootObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	virtual void GetConnectableObjects(FName InGraphName, TSet<UObject*>& OutObjects) const {}
	virtual void AddConnectableObject(FName InGraphName, UObject* InObject) {}
	virtual void RemoveConnectableObject(FName InGraphName, UObject* InObject) {}

#endif  // WITH_EDITOR

};

