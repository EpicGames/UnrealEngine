// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendStackCameraNode.h"

namespace UE::Cameras
{

/**
 * Parameter structure for inserting a camera rig into a persistent blend stack.
 */
struct FBlendStackCameraInsertParams
{
	/** The evaluation context within which a camera rig's node tree should run. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset to instantiate and push on the blend stack. */
	TObjectPtr<const UCameraRigAsset> CameraRig;

	/** A transition to use, instead of looking one up. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;

	/**
	 * An ordering value for where to insert the camera rig in the stack.
	 * Higher values place the camera rig higher in the stack (i.e. evaluating later)
	 * Lower values place the camera rig lower in the stack (i.e. evluating earlier)
	 * Insertion of an equal pre-existing value will be done on top of (after) existing 
	 * entries in the stack.
	 */
	int32 StackOrder = 0;

	/** 
	 * Whether to force insert a new instance of the camera rig, even if there is already 
	 * one in the stack with the same StackOrder.
	 */
	bool bForceInsert = false;
};

/**
 * Parameter structure for removing a camera rig from a persistent blend stack.
 * The camera rig to remove is identified first by a given ID or, if no ID is given,
 * by finding a camera rig matching the provided EvaluationContext and CameraRig.
 */
struct FBlendStackCameraRemoveParams
{
	/** The ID of the blend stack entry to remove. */
	FBlendStackEntryID EntryID;

	/** The evaluation context within which the camera rig to remove is being run. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset used by the instanced to remove. */
	TObjectPtr<const UCameraRigAsset> CameraRig;

	/** A transition to use, instead of looking one up. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;

	/** 
	 * Whether to immediately remove the given camera rig instead of blending it out.
	 * Equivalent to passing a pop blend as the TransitionOverride.
	 */
	bool bRemoveImmediately = false;
};

/**
 * Evaluator for a persistent blend stack, i.e. a blend stack in which camera rigs blend additively on top of 
 * each other, but without automatically popping out any fully blended-out entries. 
 * This is a stack suitable for a "camera modifier stack" of sorts.
 */
class FPersistentBlendStackCameraNodeEvaluator 
	: public FBlendStackCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FPersistentBlendStackCameraNodeEvaluator, FBlendStackCameraNodeEvaluator)

public:

	/** Insert a new camera rig onto the blend stack. */
	FBlendStackEntryID Insert(const FBlendStackCameraInsertParams& Params);

	/** Remove an existing camera rig from the blend stack. */
	void Remove(const FBlendStackCameraRemoveParams& Params);

	/** Remove all existing camera rigs with the given context from the blend stack. */
	void RemoveAll(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately);

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if WITH_EDITOR
	// FBlendStackCameraNodeEvaluator interface.
	virtual void OnEntryReinitialized(int32 EntryIndex) override;
#endif

private:

	void InternalUpdate(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

	void RemoveEntry(int32 EntryIndex, const UCameraRigTransition* TransitionOverride, bool bImmediately);

	const UCameraRigTransition* FindEnterTransition(const FBlendStackCameraInsertParams& Params) const;
	const UCameraRigTransition* FindExitTransition(const FCameraRigEntry& Entry, const UCameraRigTransition* TransitionOverride) const;

private:

	enum class EBlendStatus
	{
		None,
		BlendIn,
		BlendOut
	};

	struct FCameraRigEntryExtraInfo
	{
		int32 StackOrder = 0;
		EBlendStatus BlendStatus = EBlendStatus::None;
		bool bIsBlendFull = false;
		bool bIsBlendFinished = false;
	};

	TArray<FCameraRigEntryExtraInfo> EntryExtraInfos;
};

}  // namespace UE::Cameras

