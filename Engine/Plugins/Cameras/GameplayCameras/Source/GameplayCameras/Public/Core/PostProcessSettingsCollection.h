// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

/**
 * A class that can collect post-process settings, combining them with their associated
 * blend weights.
 */
struct FPostProcessSettingsCollection
{
	/** Gets the effective post-process settings. */
	FPostProcessSettings& Get() { return PostProcessSettings; }

	/** Gets the effective post-process settings. */
	const FPostProcessSettings& Get() const { return PostProcessSettings; }

	/** Resets this collection to the default post-process settings. */
	UE_API void Reset();

	/** 
	 * Overwrites the post-process settings in this collection with the values in the other.
	 */
	UE_API void OverrideAll(const FPostProcessSettingsCollection& OtherCollection);

	/** 
	 * Overwrites the post-process settings in this collection with any changed values in the other.
	 * Changed values are those whose bOverride_Xxx flag is true. Functionally equivalent to
	 * LerpAll with a blend factor of 100%.
	 */
	UE_API void OverrideChanged(const FPostProcessSettingsCollection& OtherCollection);
	UE_API void OverrideChanged(const FPostProcessSettings& OtherPostProcessSettings);

	/**
	 * Interpolates the post-process settings towards the values in the given other collection. All
	 * values are interpolated if either post-process settings have the bOverride_Xxx flag set.
	 * This means that some values will interpolate to and/or from default values.
	 */
	UE_API void LerpAll(const FPostProcessSettingsCollection& ToCollection, float BlendFactor);
	UE_API void LerpAll(const FPostProcessSettings& ToPostProcessSettings, float BlendFactor);

	/** Serializes this collection into the given archive. */
	UE_API void Serialize(FArchive& Ar);

private:

	UE_API void InternalLerp(const FPostProcessSettings& ToPostProcessSettings, float BlendFactor);

private:

	FPostProcessSettings PostProcessSettings;
};

}  // namespace UE::Cameras

#undef UE_API
