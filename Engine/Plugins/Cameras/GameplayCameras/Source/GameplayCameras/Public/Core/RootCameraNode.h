// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SparseArray.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraRigEvaluationInfo.h"
#include "Core/CameraRigInstanceID.h"

#include "RootCameraNode.generated.h"

class UCameraRigAsset;
class UCameraRigTransition;

/**
 * The base class for a camera node that can act as the root of the
 * camera system evaluation.
 */
UCLASS(MinimalAPI, Abstract)
class URootCameraNode : public UCameraNode
{
	GENERATED_BODY()
};

namespace UE::Cameras
{

class FCameraActionScope;
class FCameraEvaluationContext;
class FCameraSystemEvaluator;
struct FCameraRigActivationDeactivationRequest;
struct FRootCameraNodeCameraRigEvent;

/**
 * Parameter structure for activating a new camera rig.
 */
struct FActivateCameraRigParams
{
	/** The evaluation context in which the camera rig runs. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset that will be instantiated. */
	TObjectPtr<const UCameraRigAsset> CameraRig;

	/** A transition to use, instead of looking one up. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;

	/** The evaluation layer on which to instantiate the camera rig. */
	ECameraRigLayer Layer = ECameraRigLayer::Main;

	/** If the given layer supports ordering, this specifies a priority key. */
	int32 OrderKey = 0;

	/** Whether a new instance of the camera rig should be created, even if it is already active in the given layer. */
	bool bForceActivate = false;
};

/**
 * Parameter structure for deactivating a running camera rig.
 */
struct FDeactivateCameraRigParams
{
	/** The instance to deactivate. */
	FCameraRigInstanceID InstanceID;

	/** The evaluation context in which the camera rig runs. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset that was instantiated. */
	TObjectPtr<const UCameraRigAsset> CameraRig;

	/** The evaluation layer on which the camera rig is running. */
	ECameraRigLayer Layer = ECameraRigLayer::Main;

	/** If deactivation uses a transition (e.g. blending out), then force using the given transition. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;

	/** If deactivation can take time (e.g. blending out), then force an immediate deactivation. */
	bool bDeactiveImmediately = false;
};

/**
 * Parameter structure for building a single camera rig hierarchy.
 */
struct FSingleCameraRigHierarchyBuildParams
{
	/** The camera rig to build the hierachy for. */
	FCameraRigEvaluationInfo CameraRigInfo;

	/** The name of the range to tag for the camera rig's nodes. */
	FName CameraRigRangeName = TEXT("ActiveCameraRig");
};

/**
 * Parameter structure for evaluating a single camera rig.
 */
struct FSingleCameraRigEvaluationParams
{
	/** The evaluation parameters. */
	FCameraNodeEvaluationParams EvaluationParams;

	/** The camera rig to evaluate. */
	FCameraRigEvaluationInfo CameraRigInfo;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRootCameraNodeCameraRigEvent, const FRootCameraNodeCameraRigEvent&);

/**
 * Base class for the evaluator of a root camera node.
 */
class FRootCameraNodeEvaluator : public FCameraNodeEvaluator
{
public:

	/** 
	 * Activates a camera rig.
	 * What it means to activate a camera rig may differ depending on the layer it runs on.
	 */
	GAMEPLAYCAMERAS_API FCameraRigInstanceID ActivateCameraRig(const FActivateCameraRigParams& Params);

	/** 
	 * Deactivates a camera rig. 
	 * What it means to deactivate a camera rig may differ depending on the layer it runs on.
	 */
	GAMEPLAYCAMERAS_API void DeactivateCameraRig(const FDeactivateCameraRigParams& Params);

	/** Deactivates all camera rigs with the given evaluation context. */
	GAMEPLAYCAMERAS_API void DeactivateAllCameraRigs(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately);

	/** Execute a request to activate or deactivate a camera rig. */
	GAMEPLAYCAMERAS_API void ExecuteCameraDirectorRequest(const FCameraRigActivationDeactivationRequest& Request);

	/** Gets information about the active camera rig in the main layer. */
	GAMEPLAYCAMERAS_API void GetActiveCameraRigInfo(FCameraRigEvaluationInfo& OutCameraRigInfo) const;

	/** Gets information about a specified camera rig. */
	GAMEPLAYCAMERAS_API void GetCameraRigInfo(const FCameraRigInstanceID InstanceID, FCameraRigEvaluationInfo& OutCameraRigInfo) const;

	/** Gets whether any camera rig is running in the main layer. */
	GAMEPLAYCAMERAS_API bool HasAnyActiveCameraRig() const;

	/** Gets whether any camera rig is running with the given context in the main layer. */
	GAMEPLAYCAMERAS_API bool HasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const;

	/** Gets the variable table containing the blended camera object interfaces parameters. */
	GAMEPLAYCAMERAS_API const FCameraVariableTable* GetBlendedParameters() const;

	/** Returns the evaluation result without the contribution of the visual layer. */
	GAMEPLAYCAMERAS_API const FCameraNodeEvaluationResult& GetPreVisualLayerResult() const;

	/**
	 * Builds the hierarchy of the system for a given single camera rig.
	 * This is expected to return the nodes of all the layers, except for the main layer which
	 * should only have the nodes of the given camera rig (i.e. it shouldn't have nodes of
	 * other currently active camera rigs).
	 */
	GAMEPLAYCAMERAS_API void BuildSingleCameraRigHierarchy(const FSingleCameraRigHierarchyBuildParams& Params, FCameraNodeEvaluatorHierarchy& OutHierarchy);

	/**
	 * Evaluates a single camera rig.
	 * This is expected to run all layers as usual, except for the main layer which should
	 * only run the given camera rig instead.
	 */
	GAMEPLAYCAMERAS_API void RunSingleCameraRig(const FSingleCameraRigEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Gets the action scope for the active camera rig in the main layer. */
	GAMEPLAYCAMERAS_API TSharedPtr<FCameraActionScope> GetActiveCameraRigActionScope(bool bCreateIfNeeded = false);

	/** Gets the delegate for camera rig events. */
	FOnRootCameraNodeCameraRigEvent& OnCameraRigEvent() { return OnCameraRigEventDelegate; }

protected:

	// FCameraNodeEvaluator interface.
	GAMEPLAYCAMERAS_API virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;

protected:

	/** Activates a camera rig. */
	virtual FCameraRigInstanceID OnActivateCameraRig(const FActivateCameraRigParams& Params) { return FCameraRigInstanceID(); }
	
	/** Deactivates a camera rig. */
	virtual void OnDeactivateCameraRig(const FDeactivateCameraRigParams& Params) {}

	/** Deactivates all camera rigs with the given evaluation context. */
	virtual void OnDeactivateAllCameraRigs(TSharedPtr<const FCameraEvaluationContext> InContext, bool bImmediately) {}

	/** Gets information about the active camera rig in the main layer. */
	virtual void OnGetActiveCameraRigInfo(FCameraRigEvaluationInfo& OutCameraRigInfo) const {}

	/** Gets whether any camera rig is running with the given context in the main layer. */
	virtual bool OnHasAnyRunningCameraRig(TSharedPtr<const FCameraEvaluationContext> InContext) const { return false; }

	/** Gets information about a specified camera rig. */
	virtual void OnGetCameraRigInfo(const FCameraRigInstanceID InstanceID, FCameraRigEvaluationInfo& OutCameraRigInfo) const {}

	/** Gets the variable table containing the blended camera object interfaces parameters. */
	virtual const FCameraVariableTable* OnGetBlendedParameters() const { return nullptr; }

	/* Builds the hierarchy of the system for a given single camera rig. */
	virtual void OnBuildSingleCameraRigHierarchy(const FSingleCameraRigHierarchyBuildParams& Params, FCameraNodeEvaluatorHierarchy& OutHierarchy) {}

	/** Evaluates a single camera rig. See comments on RunSingleCameraRig. */
	virtual void OnRunSingleCameraRig(const FSingleCameraRigEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) {}

	/** Gets the action scope for the active camera rig in the main layer. */
	virtual TSharedPtr<FCameraActionScope> OnGetActiveCameraRigActionScope(bool bCreateIfNeeded) { return nullptr; }

protected:

	GAMEPLAYCAMERAS_API void SetPreVisualLayerResult(const FCameraNodeEvaluationResult& InResult);

	GAMEPLAYCAMERAS_API void BroadcastCameraRigEvent(const FRootCameraNodeCameraRigEvent& InEvent) const;

private:

	/** The camera system that owns this root node. */
	FCameraSystemEvaluator* OwningEvaluator = nullptr;

	/** Evaluation result before the contribution of the visual layer. */
	FCameraNodeEvaluationResult PreVisualResult;

	/** The delegate to notify when an event happens. */
	FOnRootCameraNodeCameraRigEvent OnCameraRigEventDelegate;
};

}  // namespace UE::Cameras

