// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigEvaluationInfo.h"
#include "UObject/ObjectPtr.h"

class UCameraRigTransition;
enum class ECameraRigLayer : uint8;

namespace UE::Cameras
{

/** The type of root node event. */
enum class ERootCameraNodeCameraRigEventType
{
	/** A camera rig was activated. */
	Activated,
	/** A camera rig was deactivated. */
	Deactivated
};

/**
 * A structure describing an event on a root node.
 */
struct FRootCameraNodeCameraRigEvent
{
	/** The type of event. */
	ERootCameraNodeCameraRigEventType EventType = ERootCameraNodeCameraRigEventType::Activated;
	/** The layer on which the event happened. */
	ECameraRigLayer EventLayer = ECameraRigLayer::Main;
	/** The evaluation information of the associated camera rig. */
	FCameraRigEvaluationInfo CameraRigInfo;
	/** If a camera rig was activated, the transition used to activate it, if any. */
	TObjectPtr<const UCameraRigTransition> Transition;
};

}  // namespace UE::Cameras


