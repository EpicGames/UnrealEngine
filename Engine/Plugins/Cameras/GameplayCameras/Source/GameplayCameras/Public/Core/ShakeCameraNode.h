// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraTypes.h"
#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"

#include "ShakeCameraNode.generated.h"

/**
 * Base class for shake camera nodes.
 */
UCLASS(MinimalAPI, Abstract, meta=(CameraNodeCategories="Shakes"))
class UShakeCameraNode : public UCameraNode
{
	GENERATED_BODY()
};

namespace UE::Cameras
{

/**
 * Parameters for applying a shake to a camera result.
 */
struct FCameraNodeShakeParams
{
	FCameraNodeShakeParams(const FCameraNodeEvaluationParams& InChildParams)
		: ChildParams(InChildParams) 
	{}

	/** The parameters that the shake received during the evaluation. */
	const FCameraNodeEvaluationParams& ChildParams;
	/** The intensity to use for the camera shake. */
	float ShakeScale = 1.f;
	/** The play space to modify the result by */
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;
	/** The custom space to use for the shake. Only used when PlaySpace is UserDefined. */
	FMatrix UserPlaySpaceMatrix = FMatrix::Identity;
};

/**
 * A structure that describes a group of offset values representing a shake.
 */
struct FCameraNodeShakeDelta
{
	/** Position shake, which will be scaled and applied as a delta in the desired space. */
	FVector3d Location = FVector3d::ZeroVector;

	/** Rotation shake, which will be scaled and applied as a delta in the desired space. */
	FRotator3d Rotation = FRotator3d::ZeroRotator;

	/** Field of view shake, which will be scaled and applied as a delta. */
	float FieldOfView = 0.f;

public:

	/** Combines this shake delta with another, optionally scaled, shake delta. */
	void Combine(const FCameraNodeShakeDelta& Other, float OtherScale = 1.f);
};

/**
 * Result structure for applying a shake to a camera result.
 */
struct FCameraNodeShakeResult
{
	FCameraNodeShakeResult(FCameraNodeEvaluationResult& InShakenResult)
		: ShakenResult(InShakenResult)
	{}

	/** The result that should be shaken. */
	FCameraNodeEvaluationResult& ShakenResult;

	/** 
	 * Delta values that should be applied to the shaken result.
	 *
	 * These are simpler to set and combine inside camera shake nodes, since they will be applied
	 * with the correct scale and in the correct space at the end. By comparison, writing to the
	 * ShakenResult field directly requires taking into account all the parameters first (see 
	 * FCameraNodeShakeParams).
	 */
	FCameraNodeShakeDelta ShakeDelta;

	/** The time left in this shake, if applicable. Negative values indicate an infinite shake. */
	float ShakeTimeLeft = 0.f;

public:

	/** Apply the delta values to the shaken result. */
	void ApplyDelta(const FCameraNodeShakeParams& Params);
};

/**
 * Parameters for restarting a running camera shake.
 */
struct FCameraNodeShakeRestartParams
{
};

class FShakeCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FShakeCameraNodeEvaluator)

public:

	/** Applies the shake to the given result. */
	GAMEPLAYCAMERAS_API void ShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult);

	/** Restart a running camera shake. */
	GAMEPLAYCAMERAS_API void RestartShake(const FCameraNodeShakeRestartParams& Params);

protected:

	/** Applies the shake to the given result. */
	virtual void OnShakeResult(const FCameraNodeShakeParams& Params, FCameraNodeShakeResult& OutResult) {}

	/** Called when the intensity of the shake changes, but before ShakeScale is set. */
	virtual void OnSetShakeScale(float InShakeScale) {}

	/** Restart a running camera shake. */
	virtual void OnRestartShake(const FCameraNodeShakeRestartParams& Params) {}
};

}  // namespace UE::Cameras

// Macros for declaring and defining new shake node evaluators. They are the same
// as the base ones for generic node evaluators, but the first one prevents you
// from having to specify FShakeCameraNodeEvaluator as the base class, which saves
// a little bit of typing.
//
#define UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR(ApiDeclSpec, ClassName)\
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, ::UE::Cameras::FShakeCameraNodeEvaluator)

#define UE_DECLARE_SHAKE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_SHAKE_CAMERA_NODE_EVALUATOR(ClassName)\
	UE_DEFINE_CAMERA_NODE_EVALUATOR(ClassName)

