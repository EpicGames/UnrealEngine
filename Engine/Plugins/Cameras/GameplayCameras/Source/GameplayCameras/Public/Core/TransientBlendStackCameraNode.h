// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendStackCameraNode.h"
#include "Core/BlendStackEntryID.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::Cameras
{

class FCameraActionScope;
class FCameraActionService;
class FCameraParameterSetterService;

/**
 * Parameter structure for pushing a camera rig onto a transient blend stack.
 */
struct FBlendStackCameraPushParams
{
	/** The evaluation context within which a camera rig's node tree should run. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset to instantiate and push on the blend stack. */
	TObjectPtr<const UCameraRigAsset> CameraRig;

	/** A transition to use, instead of looking one up. */
	TObjectPtr<const UCameraRigTransition> TransitionOverride;

	/** Whether to force pushing a new instance of the camera rig, even if it is currently active. */
	bool bForcePush = false;
};

/**
 * Parameter structure for freezing a camera rig inside a transient blend stack.
 */
struct FBlendStackCameraFreezeParams
{
	/** The ID of the blend stack entry to freeze. */
	FBlendStackEntryID EntryID;

	/** The evaluation context within which a camera rig's node tree is running. */
	TSharedPtr<const FCameraEvaluationContext> EvaluationContext;

	/** The source camera rig asset that is running. */
	TObjectPtr<const UCameraRigAsset> CameraRig;
};

/**
 * Evaluator for a transient blend stack, i.e. a blend stack where camera rigs evaluate together,
 * blending together and blend out those below them, which are then automatically popped out of
 * the stack.
 * This is a stack suitable for "camera modes" of sorts.
 */
class FTransientBlendStackCameraNodeEvaluator 
	: public FBlendStackCameraNodeEvaluator
{

	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FTransientBlendStackCameraNodeEvaluator, FBlendStackCameraNodeEvaluator)

public:

	/** Push a new camera rig onto the blend stack. */
	FBlendStackEntryID Push(const FBlendStackCameraPushParams& Params);

	/** Freeze a camera rig. */
	void Freeze(const FBlendStackCameraFreezeParams& Params);

	/** Freeze all camera rigs that belong to a given evaluation context. */
	void FreezeAll(TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	/** Gets the variable table containing the blended camera object interfaces parameters. */
	const FCameraVariableTable& GetBlendedParameters() const { return PreBlendVariableTable; }

	/** Gets the active camera rig's action scope, optionally creating one if needed. */
	TSharedPtr<FCameraActionScope> GetActiveCameraRigActionScope(bool bCreateIfNeeded = false);

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif

#if WITH_EDITOR
	// FBlendStackCameraNodeEvaluator interface.
	virtual void OnEntryReinitialized(int32 EntryIndex) override;
#endif

private:

	// Update methods.
	void InternalPreBlendPrepare(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void InternalPreBlendExecute(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void InternalUpdate(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void InternalPostBlendExecute(TArrayView<FResolvedEntry> ResolvedEntries, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void InternalRunFinished(FCameraNodeEvaluationResult& OutResult);

	const UCameraRigTransition* FindTransition(const FBlendStackCameraPushParams& Params) const;

	FBlendStackEntryID PushNewEntry(const FBlendStackCameraPushParams& Params, const UCameraRigTransition* Transition);
	FBlendStackEntryID PushMergedEntry(const FBlendStackCameraPushParams& Params, const UCameraRigTransition* Transition);

private:

	/** Extra blending-related info for each camera rig entry. */
	struct FCameraRigEntryExtraInfo
	{
		/** Action scope for this entry. */
		TSharedPtr<FCameraActionScope> ActionScope;

		/** Whether input slots were run. */
		bool bInputRunThisFrame = false;
		/** Whether the blend node was run. */
		bool bBlendRunThisFrame = false;
		/** Whether this camera rig has any parameters to pre-blend. */
		bool bHasPreBlendedParameters = false;
		/** Whether this camera rig's pre-blend was full and finished this frame. */
		bool bIsPreBlendFull = false;
	};

	TArray<FCameraRigEntryExtraInfo> EntryExtraInfos;

	/** Variable table for pre-blending. */
	FCameraVariableTable PreBlendVariableTable;

	/** Cached pointer to the parameter setter service. */
	TSharedPtr<FCameraParameterSetterService> ParameterSetterService;
};

}  // namespace UE::Cameras

