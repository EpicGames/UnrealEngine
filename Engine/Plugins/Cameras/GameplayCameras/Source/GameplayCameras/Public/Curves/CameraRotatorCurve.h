// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "Math/MathFwd.h"

#include "CameraRotatorCurve.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

USTRUCT()
struct FCameraRotatorCurve
{
	GENERATED_BODY()

	UPROPERTY()
	FRichCurve Curves[3];

	UE_API FRotator GetValue(float InTime) const;
	UE_API bool HasAnyData() const;
};

#undef UE_API
