// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "Math/MathFwd.h"

#include "CameraSingleCurve.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

USTRUCT()
struct FCameraSingleCurve
{
	GENERATED_BODY()

	UPROPERTY()
	FRichCurve Curve;

	UE_API float GetValue(float InTime) const;
	UE_API bool HasAnyData() const;
};

#undef UE_API
