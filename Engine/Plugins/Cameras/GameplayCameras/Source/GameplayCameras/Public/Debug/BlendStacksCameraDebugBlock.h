// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FBlendStackCameraDebugBlock;

/**
 * Debug block that shows multiple blend stacks.
 */
class FBlendStacksCameraDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FBlendStacksCameraDebugBlock)

public:

	FBlendStacksCameraDebugBlock();

	/**
	 * Adds the given blend stack block as a child, which will be rendered with the given title.
	 * The blend stack block can be obtained from the blend stack evaluator.
	 */
	void AddBlendStack(const FString& InBlendStackName, FBlendStackCameraDebugBlock* InDebugBlock);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	TArray<FString> BlendStackNames;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

