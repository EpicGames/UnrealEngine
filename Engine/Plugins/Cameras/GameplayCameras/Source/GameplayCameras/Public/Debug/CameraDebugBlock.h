// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraObjectRtti.h"
#include "CoreTypes.h"
#include "GameplayCameras.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FArchive;

namespace UE::Cameras
{

class FCameraDebugRenderer;
class FCameraDebugBlock;

/**
 * Parameter structure for debug block drawing.
 */
struct FCameraDebugBlockDrawParams
{
	/** The list of active debug categories. */
	TSet<FString> ActiveCategories;

	GAMEPLAYCAMERAS_API bool IsCategoryActive(const FString& InCategory) const;
};

/**
 * Metadata for a debug block's member field.
 * This is mostly used for auto-serialization of debugging info.
 */
struct FCameraDebugBlockField
{
	virtual ~FCameraDebugBlockField() = default;
	virtual void SerializeField(FCameraDebugBlock* This, FArchive& Ar) = 0;

	FName FieldName;
	uint16 FieldIndex = 0;
	uint16 FieldOffset = 0;
};

/**
 * Templated version of the debug block field.
 */
template<typename FieldType>
struct TCameraDebugBlockField : FCameraDebugBlockField
{
	TCameraDebugBlockField(const FName& InName, uint16 InOffset) 
	{
		FieldName = InName;
		FieldOffset = InOffset;
	}

	virtual void SerializeField(FCameraDebugBlock* This, FArchive& Ar) override
	{
		FieldType* Value = reinterpret_cast<FieldType*>(reinterpret_cast<uint8*>(This) + FieldOffset);
		Ar << (*Value);
	}
};

/**
 * Debug drawing structure responsible for displaying information about some aspect
 * of the camera system evaluation. For instance, there is generally one debug block
 * per camera node.
 */
class FCameraDebugBlock
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraDebugBlock)

public:

	virtual ~FCameraDebugBlock() = default;

	/** Attaches a block to this block. */
	GAMEPLAYCAMERAS_API void Attach(FCameraDebugBlock* InAttachment);
	/** Gets the list of blocks attached to this block. */
	TArrayView<FCameraDebugBlock*> GetAttachments() { return Attachments; }

	/** Adds a child block to this block. */
	GAMEPLAYCAMERAS_API void AddChild(FCameraDebugBlock* InChild);
	/** Gets the children of this block. */
	TArrayView<FCameraDebugBlock*> GetChildren() { return Children; }

	/** Called to let this block display its information on screen. */
	GAMEPLAYCAMERAS_API void DebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer);

	/** Serializes this debug block into a buffer, for recording/replaying purposes. */
	GAMEPLAYCAMERAS_API void Serialize(FArchive& Ar);

protected:

	/** Called to let this block display its information on screen. */
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) {}
	/** Called after attached and children blocks' debug draw. */
	virtual void OnPostDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) {}

	/** Serializes this debug block into a buffer, for recording/replaying purposes. */
	virtual void OnSerialize(FArchive& Ar) {}

protected:

	using FStaticFieldArray = TArray<FCameraDebugBlockField*>;

	template<typename FieldType>
	static TCameraDebugBlockField<FieldType> CreateField(const FName& FieldName, uint16 FieldOffset)
	{
		return TCameraDebugBlockField<FieldType>{ FieldName, FieldOffset };
	}

	static int32 RegisterField(FCameraDebugBlockField* InField, FStaticFieldArray& InStaticFields)
	{
		InStaticFields.Add(InField);
		return InStaticFields.Num() - 1;
	}

private:

	using FRelatedDebugBlockArray = TArray<FCameraDebugBlock*>;
	FRelatedDebugBlockArray Attachments;
	FRelatedDebugBlockArray Children;
};

}  // namespace UE::Cameras

// Macros for defining debug blocks.
//
#define UE_DECLARE_CAMERA_DEBUG_BLOCK(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, ::UE::Cameras::FCameraDebugBlock)

#define UE_DECLARE_CAMERA_DEBUG_BLOCK_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_DEBUG_BLOCK(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#if UE_GAMEPLAY_CAMERAS_DEBUG

// Macros for defining "simple" debug blocks with primitive types as fields.
// This will setup debug block serialization automatically, compared to the macros
// above where you have to write your own OnSerialize().
//
// Example:
//
//     UE_DECLARE_CAMERA_DEBUG_BLOCK_START(MYMODULE_API, FMyDebugBlock)
//			UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(float, Something)
//			UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(bool, bSomethingElse)
//	   UE_DECLARE_CAMERA_DEBUG_BLOCK_END()
//
//	   UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(FMyDebugBlock)
//
#define UE_DECLARE_CAMERA_DEBUG_BLOCK_START(ApiDeclSpec, ClassName)\
	class ClassName : public ::UE::Cameras::FCameraDebugBlock\
	{\
		UE_DECLARE_CAMERA_DEBUG_BLOCK(ApiDeclSpec, ClassName)\
	private:\
		using ThisClassName = ClassName;\
		static FStaticFieldArray& GetStaticFields()\
		{\
			static FStaticFieldArray StaticFields;\
			return StaticFields;\
		}\
	protected:\
		virtual void OnSerialize(FArchive& Ar) override\
		{\
			Super::OnSerialize(Ar);\
			for (FCameraDebugBlockField* StaticField : ClassName::GetStaticFields())\
			{\
				StaticField->SerializeField(this, Ar);\
			}\
		}\
	private:

#define UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FieldType, FieldName)\
	public:\
		FieldType FieldName;\
	private:\
		inline static PTRINT Get##FieldName##Offset() { return (PTRINT)(&(((ThisClassName*)0)->FieldName)); }\
		inline static TCameraDebugBlockField<FieldType>* Get##FieldName##Field()\
		{\
			using FField = TCameraDebugBlockField<FieldType>;\
			static FField StaticField = CreateField<FieldType>(TEXT(#FieldName), Get##FieldName##Offset());\
			return &StaticField;\
		}\
		inline static const int32 FieldName##FieldIndex = RegisterField(Get##FieldName##Field(), GetStaticFields());

#define UE_DECLARE_CAMERA_DEBUG_BLOCK_END()\
		virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;\
	};

#define UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(ClassName)\
	UE_DEFINE_CAMERA_DEBUG_BLOCK(ClassName)\

#else  // UE_GAMEPLAY_CAMERAS_DEBUG

// Empty macros for shipping builds.
//
#define UE_DECLARE_CAMERA_DEBUG_BLOCK_START(ApiDeclSpec, ClassName)
#define UE_DECLARE_CAMERA_DEBUG_BLOCK_FIELD(FieldType, FieldName)
#define UE_DECLARE_CAMERA_DEBUG_BLOCK_END()
#define UE_DEFINE_CAMERA_DEBUG_BLOCK_WITH_FIELDS(ClassName)

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

