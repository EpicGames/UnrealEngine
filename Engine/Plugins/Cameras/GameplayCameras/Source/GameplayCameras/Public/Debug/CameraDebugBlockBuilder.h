// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraDebugBlockStorage.h"
#include "GameplayCameras.h"
#include "Templates/SharedPointer.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraDebugBlock;
class FCameraNodeEvaluatorDebugBlock;
class FRootCameraDebugBlock;
struct FCameraDebugBlockBuilder;

enum class ECameraDebugBlockBuildVisitFlags
{
	None,
	SkipChildren = 1 << 0
};
ENUM_CLASS_FLAGS(ECameraDebugBlockBuildVisitFlags);

/**
 * Builder class for camera debug drawing blocks.
 */
struct FCameraDebugBlockBuilder
{
public:

	/** Creates a new builder structure. */
	FCameraDebugBlockBuilder(FCameraDebugBlockStorage& InStorage, FRootCameraDebugBlock& InRootBlock);

	/** Gets the storage used by this builder. */
	FCameraDebugBlockStorage& GetStorage() const { return Storage; }

	/** Gets the root debug block. */
	FRootCameraDebugBlock& GetRootDebugBlock() const { return RootBlock; }

	/** Gets the current parent debug block. */
	FCameraDebugBlock& GetParentDebugBlock() const { return *CurrentHierarchy.Last(); }

	/**
	 * Creates a new unassociated debug block.
	 * It won't render unless it's referenced by or parented under another debug block!
	 */
	template<typename BlockType, typename ...ArgTypes>
	BlockType& BuildDebugBlock(ArgTypes&&... InArgs)
	{
		return *Storage.BuildDebugBlock<BlockType>(Forward<ArgTypes>(InArgs)...);
	}

	/**
	 * Creates a new debug block and attaches it to the current active block.
	 *
	 * Attached debug blocks are rendered/not rendered along with their "anchor" debug block.
	 * This differs from children debug blocks which may be rendered/not rendered independently
	 * of their parent.
	 */
	template<typename BlockType, typename ...ArgTypes>
	BlockType& AttachDebugBlock(ArgTypes&&... InArgs)
	{
		BlockType* NewBlock = Storage.BuildDebugBlock<BlockType>(Forward<ArgTypes>(InArgs)...);
		OnAttachDebugBlock(NewBlock);
		return *NewBlock;
	}

	/**
	 * Creates a new debug block and adds it to the current hierarhcy.
	 * This sets the new block as the "active" block, and adds it as a child of the
	 * previously active block.
	 */
	template<typename BlockType, typename ...ArgTypes>
	BlockType& StartChildDebugBlock(ArgTypes&& ...InArgs)
	{
		BlockType* NewBlock = Storage.BuildDebugBlock<BlockType>(Forward<ArgTypes>(InArgs)...);
		OnStartChildDebugBlock(NewBlock);
		return *NewBlock;
	}

	/** Ends the currently active debug drawing block. */
	GAMEPLAYCAMERAS_API void EndChildDebugBlock();

	/** Gets current hierarchy level. */
	int32 GetHierarchyLevel() const { return CurrentHierarchy.Num(); }

	/**
	 * Don't visit children node evaluators when building the hierarchy of debug blocks.
	 *
	 * This implies that a node evaluator will visit its children "manually", otherwise these
	 * children node evaluators won't have any debugging information available.
	 */
	GAMEPLAYCAMERAS_API void SkipChildren();
	/** Gets visiting flags. */
	ECameraDebugBlockBuildVisitFlags GetVisitFlags() const { return VisitFlags; }
	/** Resets visiting flags. */
	GAMEPLAYCAMERAS_API void ResetVisitFlags();

	/** Temporarily overrides the attachment/children to work on the new parent. */
	GAMEPLAYCAMERAS_API void StartParentDebugBlockOverride(FCameraDebugBlock& InNewParent);
	/** Ends a temporary attachment/children override. */
	GAMEPLAYCAMERAS_API void EndParentDebugBlockOverride();

private:

	GAMEPLAYCAMERAS_API void OnAttachDebugBlock(FCameraDebugBlock* InNewBlock);
	GAMEPLAYCAMERAS_API void OnStartChildDebugBlock(FCameraDebugBlock* InNewBlock);

private:

	FCameraDebugBlockStorage& Storage;
	FRootCameraDebugBlock& RootBlock;
	TArray<FCameraDebugBlock*> CurrentHierarchy;

	ECameraDebugBlockBuildVisitFlags VisitFlags = ECameraDebugBlockBuildVisitFlags::None;
	int32 HierarchyOverrideStart = INDEX_NONE;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

