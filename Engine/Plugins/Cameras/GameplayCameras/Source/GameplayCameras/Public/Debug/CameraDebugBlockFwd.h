// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayCameras.h"

namespace UE::Cameras
{

#if UE_GAMEPLAY_CAMERAS_DEBUG

struct FCameraDebugBlockBuilder;

/**
 * Structure for creating the node evaluator's debug blocks.
 */
struct FCameraDebugBlockBuildParams
{
	// Empty for now, but defined for later API changes.
};

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

}  // namespace UE::Cameras

