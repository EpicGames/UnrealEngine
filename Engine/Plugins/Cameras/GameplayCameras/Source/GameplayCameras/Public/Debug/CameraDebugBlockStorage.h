// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraObjectStorage.h"
#include "CoreTypes.h"
#include "Debug/CameraDebugBlock.h"
#include "GameplayCameras.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

/**
 * A class responsible for storing a tree of debug blocks
 */
class FCameraDebugBlockStorage : public TCameraObjectStorage<FCameraDebugBlock>
{
public:

	using Super = TCameraObjectStorage<FCameraDebugBlock>;

	/** Destroy any allocated debug blocks. */
	GAMEPLAYCAMERAS_API void DestroyDebugBlocks(bool bFreeAllocations = false);

	/** Build a new debug block inside this storage. */
	template<typename BlockType, typename ...ArgTypes>
	BlockType* BuildDebugBlock(ArgTypes&&... InArgs);

public:

	// Internal API.
	void* BuildDebugBlockUninitialized(uint32 Sizeof, uint32 Alignof)
	{
		return BuildObjectUninitialized(Sizeof, Alignof);
	}
};

template<typename BlockType, typename ...ArgTypes>
BlockType* FCameraDebugBlockStorage::BuildDebugBlock(ArgTypes&&... InArgs)
{
	return Super::BuildObject<BlockType>(Forward<ArgTypes>(InArgs)...);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

