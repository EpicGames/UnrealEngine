// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "GameplayCameras.h"

#define UE_API GAMEPLAYCAMERAS_API

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

/**
 * Standard debug categories for showing different "debug modes".
 */
struct FCameraDebugCategories
{
	static UE_API const FString NodeTree;
	static UE_API const FString DirectorTree;
	static UE_API const FString BlendStacks;
	static UE_API const FString Services;
	static UE_API const FString PoseStats;
	static UE_API const FString Viewfinder;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

#undef UE_API
