// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/TVariant.h"
#include "Serialization/Archive.h"

#define UE_API GAMEPLAYCAMERAS_API

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FCanvas;

namespace UE::Cameras
{

class FCameraDebugClock;

/**
 * Parameter structure for drawing a debug clock.
 */
struct FCameraDebugClockDrawParams
{
	FCameraDebugClockDrawParams();

	/** The position of the clock card on screen. */
	FVector2f ClockPosition;
	/** The total size of the clock card on screen. */
	FVector2f ClockSize;
	/** The color of the clock card's background. */
	FLinearColor ClockBackgroundColor;

	/** The name of the clock, displayed at the bottom of the card. */
	FText ClockName;
	/** The color of the clock name text. */
	FLinearColor ClockNameColor;

	/** The color of the clock face circle. */
	FLinearColor ClockFaceColor;
	/** The color of the arrow inside the clock. */
	FLinearColor ClockValueLineColor;
};

namespace Internal
{

class FCameraDebugClockRenderer
{
public:

	FCameraDebugClockRenderer(FCanvas* InCanvas, const FCameraDebugClockDrawParams& InDrawParams);

	void DrawVectorClock(const FVector2d& Value, double MaxLength) const;
	void DrawAngleClock(double Angle) const;

private:

	void DrawFrame() const;
	void DrawCurrentValue(const FText& CurrentValueStr) const;
	void GetClockFaceParams(FVector2f& OutClockCenter, double& OutClockRadius) const;

private:

	FCanvas* Canvas = nullptr;
	FCameraDebugClockDrawParams DrawParams;
};

}  // namespace Internal

/**
 * A debug clock, for showing a real-time angle or 2D vector in a graphical way.
 */
class FCameraDebugClock
{
public:

	/**
	 * Update the clock with the given angle. The clock's arrow will reach the
	 * edge of the clock face and oriented according to this angle, relative to
	 * the direction pointing straight upwards on the screen.
	 */
	UE_API void Update(double InAngle);

	/**
	 * Update the clock with the given 2D vector. The clock's arrow will represent
	 * this vector, relative to the direction pointing straight updwards on the
	 * screen.
	 */
	UE_API void Update(const FVector2d& InValue);

	/**
	 * Draw the debug clock onto the given canvas.
	 */
	UE_API void Draw(FCanvas* Canvas, const FCameraDebugClockDrawParams& DrawParams);

public:

	UE_API void Serialize(FArchive& Ar);

	friend FCameraDebugClock& operator<< (FArchive& Ar, FCameraDebugClock& This)
	{
		This.Serialize(Ar);
		return This;
	}

private:

	struct FVectorValue
	{
		FVector2d Vector = FVector2d::ZeroVector;
		double CurrentMaxLength = 0.f;
	};
	struct FAngleValue
	{
		double Angle = 0.f;
	};
	using FVariant = TVariant<FVectorValue, FAngleValue>;
	FVariant Value;

	friend FVectorValue& operator<< (FArchive& Ar, FVectorValue& This);
	friend FAngleValue& operator<< (FArchive& Ar, FAngleValue& This);
};

inline FCameraDebugClock::FVectorValue& operator<< (FArchive& Ar, FCameraDebugClock::FVectorValue& This)
{
	Ar << This.Vector;
	Ar << This.CurrentMaxLength;
	return This;
}

inline FCameraDebugClock::FAngleValue& operator<< (FArchive& Ar, FCameraDebugClock::FAngleValue& This)
{
	Ar << This.Angle;
	return This;
}

}  // namespace UE::Cameras

#endif

#undef UE_API
