// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayCameras.h"
#include "Math/Color.h"
#include "Misc/OptionalFwd.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

/**
 * Standard debug drawing colors for the camera system.
 */
class FCameraDebugColors
{
public:

	/** Debug category titles and section titles. */
	FColor Title;
	/** Normal text. */
	FColor Default;
	/** Unimportant text. */
	FColor Passive;
	FColor VeryPassive;
	/** Important or notable text. */
	FColor Hightlighted;
	/** Text with no specific importance, but that needs to be separate from the rest. */
	FColor Notice;
	FColor Notice2;
	/** Positive text. */
	FColor Good;
	/** Warning text. */
	FColor Warning;
	/** Error text. */
	FColor Error;
	/** Background. Should only be used for the background tile. */
	FColor Background;

public:

	/** Gets the current color scheme. */
	static const FCameraDebugColors& Get();

	/** Gets a specific color by name on the current scheme. */
	static TOptional<FColor> GetFColorByName(const FString& InColorName);

	/** Gets the current color scheme name. */
	GAMEPLAYCAMERAS_API static const FString& GetName();
	/** Sets the current color scheme. */
	GAMEPLAYCAMERAS_API static void Set(const FString& InColorSchemeName);
	/** Sets the current color scheme. */
	GAMEPLAYCAMERAS_API static void Set(const FCameraDebugColors& InColorScheme);

	/** Registers a new color scheme. */
	GAMEPLAYCAMERAS_API static void RegisterColorScheme(const FString& InColorSchemeName, const FCameraDebugColors& InColorScheme);
	/** Gets all registered color scheme names. */
	GAMEPLAYCAMERAS_API static void GetColorSchemeNames(TArray<FString>& OutColorSchemeNames);

public:

	// Internal API.
	static void RegisterBuiltinColorSchemes();

private:

	static void UpdateColorMap(const FCameraDebugColors& Instance);

private:

	static FString CurrentColorSchemeName;
	static FCameraDebugColors CurrentColorScheme;
	static TMap<FString, FColor> ColorMap;

	static TMap<FString, FCameraDebugColors> ColorSchemes;
};

GAMEPLAYCAMERAS_API FLinearColor LerpLinearColorUsingHSV(
	const FLinearColor& Start, const FLinearColor& End,
	int32 Increment, int32 TotalIncrements);

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

