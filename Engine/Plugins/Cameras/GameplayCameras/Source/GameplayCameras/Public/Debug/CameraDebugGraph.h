// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StridedView.h"
#include "CoreTypes.h"
#include "Debug/CameraDebugColors.h"
#include "GameplayCameras.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Serialization/Archive.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FCanvas;

namespace UE::Cameras
{

extern float GGameplayCamerasDebugBackgroundOpacity;

/**
 * Parameter structure for drawing a debug graph.
 */
struct FCameraDebugGraphDrawParams
{
	/** Gets the default maximum time shown on the graph. */
	static GAMEPLAYCAMERAS_API float GetDefaultMaxHistoryTime();

	FCameraDebugGraphDrawParams();

	/** The position of the graph card on screen. */
	FVector2f GraphPosition;
	/** The total size of the graph card on screen. */
	FVector2f GraphSize;
	/** The color of the card's background. */
	FLinearColor GraphBackgroundColor;

	/** The name of the graph, displayed at the bottom of the card. */
	FText GraphName;
	/** The color of the graph name text. */
	FLinearColor GraphNameColor;

	/** How far back in the past the graph goes. */
	float HistoryTime;
	/** The colors for the lines of the graph. */
	TArray<FLinearColor, TInlineAllocator<4>> GraphLineColors;

	/** Setup line colors using the active color scheme. */
	template<uint8 NumValues>
	void SetupDefaultLineColors();
};

template<uint8 NumValues>
inline void FCameraDebugGraphDrawParams::SetupDefaultLineColors()
{
	const FCameraDebugColors& ColorScheme = FCameraDebugColors::Get();
	GraphLineColors.Reset();
	GraphLineColors.Add(ColorScheme.Error);
	GraphLineColors.Add(ColorScheme.Good);
	GraphLineColors.Add(ColorScheme.Notice);
	GraphLineColors.Add(ColorScheme.Notice2);
}

template<>
inline void FCameraDebugGraphDrawParams::SetupDefaultLineColors<2>()
{
	const FCameraDebugColors& ColorScheme = FCameraDebugColors::Get();
	GraphLineColors.Reset();
	GraphLineColors.Add(ColorScheme.Good);
	GraphLineColors.Add(ColorScheme.Notice);
}

template<>
inline void FCameraDebugGraphDrawParams::SetupDefaultLineColors<1>()
{
	const FCameraDebugColors& ColorScheme = FCameraDebugColors::Get();
	GraphLineColors.Reset();
	GraphLineColors.Add(ColorScheme.Notice);
}

namespace Internal
{

class FCameraDebugGraphRenderer
{
public:

	struct FLineDrawParams
	{
		FLinearColor LineColor;
		float MaxValue;
		float MinValue;
	};

	FCameraDebugGraphRenderer(FCanvas* InCanvas, const FCameraDebugGraphDrawParams& InDrawParams);

	void DrawEmptyFrame() const;
	void DrawFrame(TArrayView<float> CurrentValues) const;
	void DrawGraphLine(const FLineDrawParams& LineDrawParams, TStridedView<float> Times, TStridedView<float> Values) const;

private:

	void DrawFrameImpl() const;

private:

	FCanvas* Canvas = nullptr;
	FCameraDebugGraphDrawParams DrawParams;
};

}   // namespace Internal

/**
 * An entry on a debug graph, defined by a timestamp and one or more graph values.
 */
template<uint8 NumValues>
struct TCameraDebugGraphEntry
{
	/** The absolute time of the entry, from an arbitrary start time. */
	float Time;
	/** The values of each line on the owning graph. */
	float Values[NumValues];

	/** Creates a new uninitialized entry. */
	TCameraDebugGraphEntry() = default;

	/** Creates new entry, zero-initialized. */
	TCameraDebugGraphEntry(EForceInit)
		: Time(0)
	{
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			Values[Index] = 0.f;
		}
	}

	/** Creates a new entry given an absolute time and some graph values. */
	template<typename... InValueTypes>
	TCameraDebugGraphEntry(float InTime, float InFirstValue, InValueTypes... InOtherValues)
		: Time(InTime)
	{
		SetValues<0u>(InFirstValue, InOtherValues...);
	}

	/** Sets the values on this entry. */
	template<typename... InValueTypes>
	void SetValues(float InFirstValue, InValueTypes... InOtherValues)
	{
		SetValues<0u>(InFirstValue, InOtherValues...);
	}

public:

	void Serialize(FArchive& Ar)
	{
		Ar << Time;
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			Ar << Values[Index];
		}
	}

	friend TCameraDebugGraphEntry<NumValues>& operator<< (FArchive& Ar, TCameraDebugGraphEntry<NumValues>& This)
	{
		This.Serialize(Ar);
		return This;
	}

private:

	template<uint8 InValueIndex, typename... InValueTypes>
	void SetValues(float InNextValue, InValueTypes... InOtherValues)
	{
		static_assert(InValueIndex < NumValues, "Incorrect number of values passed");
		Values[InValueIndex] = InNextValue;
		SetValues<InValueIndex + 1>(InOtherValues...);
	}
	
	template<uint8 InEndIndex>
	void SetValues()
	{
		static_assert(InEndIndex == NumValues, "Incorrect number of values passed");
	}
};

/**
 * A debug graph, showing one or more lines. The lines progress as new timestamped values
 * are added to the graph, with older values being discarded when they go past the maximum
 * history time of the grpah.
 */
template<uint8 NumValues>
class TCameraDebugGraph
{
	using FGraphEntry = TCameraDebugGraphEntry<NumValues>;

public:

	/** Creates a new debug graph. */
	TCameraDebugGraph()
	{
		Entries.Reserve(20);
	}

	/**
	 * Adds a new entry to the graph, timestamped relative to the last added entry.
	 *
	 * @param InDeltaTime  The delta-time elapsed since the last entry was added.
	 * @param InValues     The values for the new entry.
	 */
	template<typename... InValueTypes>
	void Add(float InDeltaTime, InValueTypes... InValues)
	{
		const float Time = GetNewEntryTime(InDeltaTime);
		Add(FGraphEntry(Time, InValues...));
	}

	/**
	 * Draw this debug graph to the given canvas.
	 */
	void Draw(FCanvas* InCanvas, const FCameraDebugGraphDrawParams& InDrawParams)
	{
		using namespace Internal;

		Update(InDrawParams.HistoryTime);

		FCameraDebugGraphRenderer Renderer(InCanvas, InDrawParams);
		if (Entries.IsEmpty())
		{
			Renderer.DrawEmptyFrame();
		}
		else
		{
			Renderer.DrawFrame(MakeArrayView(Entries.Last().Values));

			const int32 NumLineColors = InDrawParams.GraphLineColors.Num();
			TStridedView<float> TimesView = MakeStridedView(Entries, &FGraphEntry::Time);
			const int32 StrideBetweenValues = sizeof(float) + NumValues * sizeof(float);
			for (int32 Index = 0; Index < NumValues; ++Index)
			{
				const int32 LineColorIndex = Index % NumLineColors;
				const FLinearColor& LineColor = InDrawParams.GraphLineColors[LineColorIndex];
				TStridedView<float> ValuesView = MakeStridedView(
						StrideBetweenValues, 
						reinterpret_cast<float*>(
							reinterpret_cast<uint8*>(&Entries[0]) + sizeof(float) + Index * sizeof(float)),
						Entries.Num());

				FCameraDebugGraphRenderer::FLineDrawParams LineDrawParams;
				LineDrawParams.LineColor = LineColor;
				LineDrawParams.MinValue = CurrentMinValue;
				LineDrawParams.MaxValue = CurrentMaxValue;
				Renderer.DrawGraphLine(LineDrawParams, TimesView, ValuesView);
			}
		}
	}

public:

	void Serialize(FArchive& Ar)
	{
		Ar << CurrentMinValue;
		Ar << CurrentMaxValue;
		Ar << Entries;
	}

	friend TCameraDebugGraph<NumValues>& operator<< (FArchive& Ar, TCameraDebugGraph<NumValues>& This)
	{
		This.Serialize(Ar);
		return This;
	}

private:

	void Add(const FGraphEntry& InEntry)
	{
		Entries.Add(InEntry);
	}

	float GetNewEntryTime(float InDeltaTime) const
	{
		if (!Entries.IsEmpty())
		{
			return Entries.Last().Time + InDeltaTime;
		}
		return 0.f;
	}

	void Update(float InMaxHistoryTime)
	{
		float MaxHistoryTime = InMaxHistoryTime;
		if (InMaxHistoryTime <= 0.f)
		{
			MaxHistoryTime = FCameraDebugGraphDrawParams::GetDefaultMaxHistoryTime();
		}

		if (!Entries.IsEmpty())
		{
			int32 TrimBeforeIndex = 0;
			const float NewestTime = Entries.Last().Time;

			CurrentMinValue = TNumericLimits<float>::Max();
			CurrentMaxValue = TNumericLimits<float>::Lowest();

			float HistoryTime = 0.f;
			for (int32 EntryIndex = Entries.Num() - 1; EntryIndex >= 0; --EntryIndex)
			{
				const FGraphEntry& Entry = Entries[EntryIndex];
				for (int32 ValueIndex = 0; ValueIndex < NumValues; ++ValueIndex)
				{
					CurrentMinValue = FMath::Min(CurrentMinValue, Entry.Values[ValueIndex]);
					CurrentMaxValue = FMath::Max(CurrentMaxValue, Entry.Values[ValueIndex]);
				}

				if (NewestTime - Entry.Time >= MaxHistoryTime)
				{
					TrimBeforeIndex = EntryIndex;
					break;
				}
			}

			if (TrimBeforeIndex > 0)
			{
				Entries.RemoveAt(0, TrimBeforeIndex, EAllowShrinking::No);
			}
		}
		else
		{
			CurrentMinValue = 0.f;
			CurrentMaxValue = 0.f;
		}
	}

private:

	float CurrentMinValue = 0;
	float CurrentMaxValue = 0;

	TArray<FGraphEntry> Entries;
};

}  // namespace UE::Cameras

#endif

