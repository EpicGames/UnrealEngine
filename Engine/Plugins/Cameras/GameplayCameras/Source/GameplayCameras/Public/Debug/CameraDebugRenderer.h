// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Debug/CameraDebugGraph.h"
#include "GameplayCameras.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Misc/StringBuilder.h"

class FCanvas;
class FSceneView;
class UCanvas;
class UFont;
class ULineBatchComponent;
class UWorld;
struct FCameraPose;

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FDebugTextRenderer;
class FCameraDebugClock;
template<uint8> class TCameraDebugGraph;

enum class ECameraDebugDrawVisitFlags
{
	None = 0,
	SkipAttachedBlocks = 1 << 0,
	SkipChildrenBlocks = 1 << 1
};
ENUM_CLASS_FLAGS(ECameraDebugDrawVisitFlags)

/**
 * Utility class for camera-related debug drawing.
 */
class FCameraDebugRenderer
{
public:

	/** Creates a new debug renderer. */
	GAMEPLAYCAMERAS_API FCameraDebugRenderer(UWorld* InWorld, UCanvas* InCanvasObject, bool bInIsExternalRendering = false);
	/** Creates a new debug renderer. */
	GAMEPLAYCAMERAS_API FCameraDebugRenderer(UWorld* InWorld, const FSceneView* InSceneView, FCanvas* InCanvas, bool bInIsExternalRendering = false);
	/** Destroys the debug renderer. */
	GAMEPLAYCAMERAS_API ~FCameraDebugRenderer();

	/** Whether we are looking at the camera system from the "outside". */
	bool IsExternalRendering() const { return bIsExternalRendering; }

	GAMEPLAYCAMERAS_API void BeginDrawing();
	GAMEPLAYCAMERAS_API void EndDrawing();

	/** Adds text to the text wall. */
	GAMEPLAYCAMERAS_API void AddText(const FString& InString);
	GAMEPLAYCAMERAS_API void AddText(const TCHAR* Fmt, ...);

	/** 
	 * Move to a new line on the text wall.
	 *
	 * @return Whether a new line was added.
	 */
	GAMEPLAYCAMERAS_API bool NewLine(bool bSkipIfEmptyLine = false);

	/** Gets the current text color. */
	GAMEPLAYCAMERAS_API FColor GetTextColor() const;
	/** Sets the text color for further calls. Returns the previous color. */
	GAMEPLAYCAMERAS_API FColor SetTextColor(const FColor& Color);

	/** Increases the indent of the next text wall entry. This will make a new line. */
	GAMEPLAYCAMERAS_API void AddIndent();
	/** Decreases the indent of the next text wall entry. This will make a new line. */
	GAMEPLAYCAMERAS_API void RemoveIndent();

public:

	/**
	 * Draw a debug clock showing an angle or 2D vector at the next position available
	 * for a "card" debug item.
	 */
	GAMEPLAYCAMERAS_API void DrawClock(FCameraDebugClock& InClock, const FText& InClockName);

	/**
	 * Draw a debug graph showing one or more graph lines at the next position available
	 * for a "card" debug item.
	 */
	template<uint8 NumValues>
	void DrawGraph(TCameraDebugGraph<NumValues>& InGraph, const FText& InGraphName);

	/**
	 * Draws the given camera pose using the DrawCamera method.
	 */
	GAMEPLAYCAMERAS_API void DrawCameraPose(const FCameraPose& InCameraPose, const FLinearColor& LineColor, float CameraSize = 0.f);

public:

	/** Draws a 2D cross at a point. */
	GAMEPLAYCAMERAS_API void Draw2DPointCross(const FVector2D& Location, float CrossSize, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 2D line. */
	GAMEPLAYCAMERAS_API void Draw2DLine(const FVector2D& Start, const FVector2D& End, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 2D box. */
	GAMEPLAYCAMERAS_API void Draw2DBox(const FBox2D& Box, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 2D box. */
	GAMEPLAYCAMERAS_API void Draw2DBox(const FVector2D& BoxPosition, const FVector2D& BoxSize, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 2D circle. */
	GAMEPLAYCAMERAS_API void Draw2DCircle(const FVector2D& Center, float Radius, const FLinearColor& LineColor, float LineThickness = 1.f, int32 NumSides = 0);

	/** Draws a 3D point. */
	GAMEPLAYCAMERAS_API void DrawPoint(const FVector3d& Location, float PointSize, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 3D line. */
	GAMEPLAYCAMERAS_API void DrawLine(const FVector3d& Start, const FVector3d& End, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 3D box. */
	GAMEPLAYCAMERAS_API void DrawBox(const FVector3d& Center, const FVector3d& Size, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 3D box. */
	GAMEPLAYCAMERAS_API void DrawBox(const FTransform3d& Transform, const FVector3d& Size, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 3D sphere. */
	GAMEPLAYCAMERAS_API void DrawSphere(const FVector3d& Center, float Radius, int32 Segments, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a 3D arrow. */
	GAMEPLAYCAMERAS_API void DrawDirectionalArrow(const FVector3d& Start, const FVector3d& End, float ArrowSize, const FLinearColor& LineColor, float LineThickness = 1.f);
	/** Draws a camera. */
	GAMEPLAYCAMERAS_API void DrawCamera(const FTransform3d& Transform, float HorizontalFieldOfView, float AspectRatio, float TargetDistance, const FLinearColor& LineColor, float CameraSize = 0.f, float LineThickness = 1.f);
	/** Draws a 3D coordinate system. */
	GAMEPLAYCAMERAS_API void DrawCoordinateSystem(const FVector3d& Location, const FRotator3d& Rotation, float AxesLength = 0.f);
	/** Draws a 3D coordinate system. */
	GAMEPLAYCAMERAS_API void DrawCoordinateSystem(const FTransform3d& Transform, float AxesLength = 0.f);
	/** Draws text at a projected 3D position. */
	GAMEPLAYCAMERAS_API void DrawText(const FVector3d& WorldPosition, const FString& Text, const FLinearColor& TextColor, UFont* TextFont = nullptr, float TextScale = 1.f);
	/** Draws text at a projected 3D position. */
	GAMEPLAYCAMERAS_API void DrawTextView(const FVector3d& WorldPosition, FStringView Text, const FLinearColor& TextColor, UFont* TextFont = nullptr, float TextScale = 1.f);
	/** Draws text at a projected 3D position, with an added screen-space offset. */
	GAMEPLAYCAMERAS_API void DrawText(const FVector3d& WorldPosition, const FVector2d& ScreenOffset, const FString& Text, const FLinearColor& TextColor, UFont* TextFont = nullptr, float TextScale = 1.f);
	/** Draws text at a projected 3D position, with an added screen-space offset. */
	GAMEPLAYCAMERAS_API void DrawTextView(const FVector3d& WorldPosition, const FVector2d& ScreenOffset, FStringView Text, const FLinearColor& TextColor, UFont* TextFont = nullptr, float TextScale = 1.f);

public:

	/** Request skipping drawing any blocks attached to the current block. */
	GAMEPLAYCAMERAS_API void SkipAttachedBlocks();
	/** Request skipping drawing any children blocks of the current block. */
	GAMEPLAYCAMERAS_API void SkipChildrenBlocks();
	/** Skip all related blocks (attached, children, etc.) */
	GAMEPLAYCAMERAS_API void SkipAllBlocks();
	/**Gets block visiting flags. */
	GAMEPLAYCAMERAS_API ECameraDebugDrawVisitFlags GetVisitFlags() const;
	/** Resets block visiting flags. */
	GAMEPLAYCAMERAS_API void ResetVisitFlags();

public:

	/** Gets the drawing canvas. */
	FCanvas* GetCanvas() const { return Canvas; }

	/** Gets the size of the canvas. */
	FVector2D GetCanvasSize() const { return CanvasSize; }

	/** Returns whether this renderer has a valid canvas to draw upon. */
	bool HasCanvas() const { return Canvas != nullptr; }

public:

	// Internal API.
	void DrawTextBackgroundTile(float Opacity);

private:

	void Initialize(UWorld* InWorld, const FSceneView* InSceneView, FCanvas* InCanvas, bool bInIsExternalRendering);

	void AddTextFmtImpl(const TCHAR* Fmt, va_list Args);
	void AddTextImpl(const TCHAR* Buffer);

	float GetIndentMargin() const;
	void FlushText();

	FVector2f GetNextCardPosition();
	void GetNextDrawGraphParams(FCameraDebugGraphDrawParams& OutDrawParams, const FText& InGraphName);

	ULineBatchComponent* GetDebugLineBatcher() const;

private:

	/** The world in which we might draw debug primitives. */
	UWorld* World = nullptr;
	/** The canvas used to draw the text wall. */
	FCanvas* Canvas = nullptr;

	const FSceneView* SceneView = nullptr;
	/** The size of the canvas. */
	FVector2D CanvasSize;
	/** Whether we are looking from the "outside" of the camera system. */
	bool bIsExternalRendering = false;

	/** The draw color of the canvas. */
	FColor DrawColor;

	/** The font used to render the text wall. */
	const UFont* RenderFont;
	/** The height of one line of the text wall. */
	int32 MaxCharHeight;

	/** Temporary string formatter for variadic methods. */
	TStringBuilder<512> Formatter;
	/** String formatter for building a line up until the point it needs to be rendered. */
	TStringBuilder<512> LineBuilder;

	/** Current indent level. */
	int8 IndentLevel = 0;
	/** The screenspace coordinates for the next block of text on the wall. */
	FVector2f NextDrawPosition;
	/** The maximum horizontal extent of the text rendered so far. */
	float RightMargin = 0;

	/** The next available position for a card item. */
	FVector2f NextCardPosition;
	/** The index of the column for displaying the next card item. */
	int8 NextCardColumn;

	/** How to visit the next debug blocks. */
	ECameraDebugDrawVisitFlags VisitFlags;
};

template<uint8 NumValues>
inline void FCameraDebugRenderer::DrawGraph(TCameraDebugGraph<NumValues>& InGraph, const FText& InGraphName)
{
	FCameraDebugGraphDrawParams DrawParams;
	GetNextDrawGraphParams(DrawParams, InGraphName);
	DrawParams.SetupDefaultLineColors<NumValues>();
	InGraph.Draw(GetCanvas(), DrawParams);
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

