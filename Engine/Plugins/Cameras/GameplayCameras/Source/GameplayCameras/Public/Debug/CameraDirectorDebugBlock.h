// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"
#include "Math/Transform.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraEvaluationContext;
struct FCameraDebugBlockBuilder;
struct FCameraEvaluationContextStack;

/**
 * A debug block for a camera director entry in the evaluation stack.
 */
class FCameraDirectorDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraDirectorDebugBlock)

public:

	void Initialize(TSharedPtr<FCameraEvaluationContext> Context, FCameraDebugBlockBuilder& Builder);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FName ContextClassName;
	FName OwnerClassName;
	FString OwnerName;
	FString CameraAssetName;
	FName CameraDirectorClassName;
	FTransform3d InitialContextTransform = FTransform3d::Identity;
	bool bIsValid = false;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

