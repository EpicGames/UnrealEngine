// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraEvaluationContext;
struct FCameraDebugBlockBuilder;
struct FCameraEvaluationContextStack;

/**
 * A debug block for showing the list of camera directors in the camera system's context stack.
 */
class FCameraDirectorTreeDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraDirectorTreeDebugBlock)

public:

	void Initialize(const FCameraEvaluationContextStack& ContextStack, FCameraDebugBlockBuilder& Builder);
	void Initialize(TArrayView<const TSharedPtr<FCameraEvaluationContext>> Contexts, FCameraDebugBlockBuilder& Builder);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

