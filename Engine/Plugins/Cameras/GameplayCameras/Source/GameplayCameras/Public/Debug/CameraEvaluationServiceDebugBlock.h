// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Debug/CameraDebugBlock.h"
#include "GameplayCameras.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraEvaluationService;

/**
 * Basic debug block for an evaluation service.
 */
class FCameraEvaluationServiceDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraEvaluationServiceDebugBlock)

public:

	/** Constructs a new evaluation service debug block. */
	FCameraEvaluationServiceDebugBlock();
	/** Constructs a new evaluation service debug block. */
	FCameraEvaluationServiceDebugBlock(TSharedPtr<const FCameraEvaluationService> InEvaluationService);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FString ServiceClassName;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

