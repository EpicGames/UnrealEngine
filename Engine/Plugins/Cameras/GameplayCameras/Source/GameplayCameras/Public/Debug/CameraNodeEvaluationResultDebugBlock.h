// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraPoseDebugBlock;
class FContextDataTableDebugBlock;
class FPostProcessSettingsDebugBlock;
class FVariableTableDebugBlock;
struct FCameraDebugBlockBuilder;
struct FCameraNodeEvaluationResult;
struct FCameraSystemEvaluationResult;

/**
 * A debug block that prints an evaluation result.
 * 
 * It will print a few basic bits of information, followed by the result camera pose (using 
 * FCameraPoseDebugBlock) and the result variable table (using FVariableTableDebugBlock).
 */
class FCameraNodeEvaluationResultDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraNodeEvaluationResultDebugBlock)

public:

	FCameraNodeEvaluationResultDebugBlock();

	/**
	 * Initializes this block with the given camera node result.
	 *
	 * @param InResult   The result to display
	 * @param Builder    The builder to use to make the children debug blocks
	 */
	void Initialize(const FCameraNodeEvaluationResult& InResult, FCameraDebugBlockBuilder& Builder);

	/**
	 * Initializes this block with the given camera system result.
	 *
	 * @param InResult   The result to display
	 * @param Builder    The builder to use to make the children debug blocks
	 */
	void Initialize(const FCameraSystemEvaluationResult& InResult, FCameraDebugBlockBuilder& Builder);

	/** Gets the pose stats debug block. */
	FCameraPoseDebugBlock* GetCameraPoseDebugBlock();
	/** Gets the variable table debug block. */
	FVariableTableDebugBlock* GetVariableTableDebugBlock();
	/** Gets the context data table debug block. */
	FContextDataTableDebugBlock* GetContextDataTableDebugBlock();
	/** Gets the pose stats debug block. */
	FPostProcessSettingsDebugBlock* GetPostProcessSettingsDebugBlock();

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	bool bIsCameraCut;
	bool bIsValid;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

