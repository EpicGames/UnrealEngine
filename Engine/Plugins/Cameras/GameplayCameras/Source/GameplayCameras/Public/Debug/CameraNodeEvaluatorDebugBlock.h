// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Debug/CameraDebugBlock.h"
#include "GameplayCameras.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

class UCameraNode;

namespace UE::Cameras
{

/**
 * Basic debug block for a camera node evaluator.
 */
class FCameraNodeEvaluatorDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraNodeEvaluatorDebugBlock)

public:

	/** Constructs a new node evaluator debug block. */
	FCameraNodeEvaluatorDebugBlock();
	/** Constructs a new node evaluator debug block. */
	FCameraNodeEvaluatorDebugBlock(TObjectPtr<const UCameraNode> InCameraNode);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FString NodeClassName;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

