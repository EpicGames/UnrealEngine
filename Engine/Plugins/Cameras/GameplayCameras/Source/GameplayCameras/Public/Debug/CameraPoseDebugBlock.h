// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraPose.h"
#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

/**
 * Sets the global options for external rendering of a camera pose.
 */
struct FScopedGlobalCameraPoseRenderingParams
{
	GAMEPLAYCAMERAS_API FScopedGlobalCameraPoseRenderingParams(const FString& Label);
	GAMEPLAYCAMERAS_API FScopedGlobalCameraPoseRenderingParams(const FLinearColor& LineColor);
	GAMEPLAYCAMERAS_API FScopedGlobalCameraPoseRenderingParams(const FString& Label, const FLinearColor& LineColor);
	GAMEPLAYCAMERAS_API ~FScopedGlobalCameraPoseRenderingParams();

private:

	FString PreviousLabel;
	FLinearColor PreviousLineColor;
};

/**
 * A debug block that displays information about a camera pose.
 */
class FCameraPoseDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraPoseDebugBlock)

public:

	/** Creates a new camera pose debug block. */
	FCameraPoseDebugBlock();
	/** Creates a new camera pose debug block. */
	FCameraPoseDebugBlock(const FCameraPose& InCameraPose);

	/** Sets whether the camera pose values should be printed in the text HUD. */
	FCameraPoseDebugBlock& ShouldDrawText(bool bShouldDraw)
	{
		bDrawText = bShouldDraw;
		return *this;
	}

	/** Sets whether the camera pose should be drawn when in external debug rendering. */
	FCameraPoseDebugBlock& ShouldDrawInExternalRendering(bool bShouldDraw)
	{
		bDrawInExternalRendering = bShouldDraw;
		return *this;
	}

	/** 
	 * Specifies the console variable to use to toggle between only showing camera pose
	 * properties that were written to, or showing all camera pose properties.
	 */
	FCameraPoseDebugBlock& WithShowUnchangedCVar(const TCHAR* InShowUnchangedCVarName)
	{
		ShowUnchangedCVarName = InShowUnchangedCVarName;
		return *this;
	}

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	static FLinearColor GlobalCameraPoseLineColor;
	static FString GlobalCameraPoseLabel;

	FCameraPose CameraPose;
	FString ShowUnchangedCVarName;
	bool bDrawText = true;
	bool bDrawInExternalRendering = true;

	friend struct FScopedGlobalCameraPoseRenderingParams;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

