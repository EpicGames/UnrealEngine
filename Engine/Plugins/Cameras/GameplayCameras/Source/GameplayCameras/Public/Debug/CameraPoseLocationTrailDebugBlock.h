// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

struct FCameraNodeEvaluationResult;

/**
 * A debug block that draws a camera pose location trail.
 */
class FCameraPoseLocationTrailDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraPoseLocationTrailDebugBlock)

public:

	FCameraPoseLocationTrailDebugBlock();
	FCameraPoseLocationTrailDebugBlock(const FCameraNodeEvaluationResult& InResult);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	void DrawCameraPoseLocationTrail(FCameraDebugRenderer& Renderer) const;

private:

	TArray<FVector3d> Trail;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

