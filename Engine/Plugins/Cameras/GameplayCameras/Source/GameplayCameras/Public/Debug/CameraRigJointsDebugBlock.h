// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigJoints.h"
#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraVariableTable;

/**
 * A debug block that displays information about a camera rig's joints.
 */
class FCameraRigJointsDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCameraRigJointsDebugBlock)

public:

	/** Creates a new camera rig joints debug block. */
	FCameraRigJointsDebugBlock();
	/** Creates a new camera rig joints debug block. */
	FCameraRigJointsDebugBlock(const FCameraRigJoints& InCameraRigJoints, const FCameraVariableTable& InVariableTable);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	struct FEntry
	{
		FVector3d Location;
		FRotator3d Rotation;
		FCameraVariableID VariableID;
		FString VariableName;
	};
	TArray<FEntry, TInlineAllocator<2>> Entries;

	friend FArchive& operator<< (FArchive& Ar, FEntry& Entry);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

