// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SparseArray.h"
#include "GameplayCameras.h"
#include "Math/NumericLimits.h"
#include "Templates/SharedPointerFwd.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

class FArchive;

namespace UE::Cameras
{

class FCameraSystemEvaluator;

/**
 * An identifier for a camera system instance.
 */
struct FCameraSystemDebugID
{
	FCameraSystemDebugID() : Value(INVALID) {}
	FCameraSystemDebugID(int32 InValue) : Value(InValue) {}

	bool IsValid() const { return Value >= 0; }
	bool IsAuto() const { return Value == AUTO; }
	bool IsAny() const { return Value == ANY; }
	operator bool() const { return IsValid(); }

	int32 GetValue() const { return Value; }

	static FCameraSystemDebugID Invalid() { return FCameraSystemDebugID(INVALID); }
	static FCameraSystemDebugID Auto() { return FCameraSystemDebugID(AUTO); }
	static FCameraSystemDebugID Any() { return FCameraSystemDebugID(ANY); }

public:

	friend bool operator==(FCameraSystemDebugID A, FCameraSystemDebugID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCameraSystemDebugID A, FCameraSystemDebugID B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FCameraSystemDebugID In)
	{
		return In.Value;
	}

	friend FString LexToString(const FCameraSystemDebugID In)
	{
		return LexToString(In.Value);
	}

	friend FArchive& operator <<(FArchive& Ar, FCameraSystemDebugID& InDebugID);

private:

	constexpr static int32 INVALID = -1;
	constexpr static int32 AUTO = 0;
	constexpr static int32 ANY = MAX_int32;

	int32 Value;

	friend class FCameraSystemDebugRegistry;
	friend class FRootCameraDebugBlock;
};

FArchive& operator <<(FArchive& Ar, FCameraSystemDebugID& InDebugID);

/**
 * A registry for any running camera system instance for which we may want to display debug info.
 */
class FCameraSystemDebugRegistry
{
public:

	GAMEPLAYCAMERAS_API static FCameraSystemDebugRegistry& Get();

	GAMEPLAYCAMERAS_API FCameraSystemDebugID RegisterCameraSystemEvaluator(TSharedRef<FCameraSystemEvaluator> InEvaluator);
	GAMEPLAYCAMERAS_API void UnregisterCameraSystemEvaluator(FCameraSystemDebugID InDebugID);

	using FRegisteredCameraSystems = TArray<TSharedPtr<FCameraSystemEvaluator>>;
	GAMEPLAYCAMERAS_API void GetRegisteredCameraSystemEvaluators(FRegisteredCameraSystems& OutEvaluators) const;

	GAMEPLAYCAMERAS_API int32 NumRegisteredCameraSystemEvaluators() const;

private:

	struct FEntry
	{
		TWeakPtr<FCameraSystemEvaluator> WeakEvaluator;
		FString EvaluatorOwnerName;
	};

	TSparseArray<FEntry> Entries;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

