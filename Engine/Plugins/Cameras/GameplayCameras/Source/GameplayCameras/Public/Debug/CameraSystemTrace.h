// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Trace/Config.h"
#include "TraceFilter.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

class UWorld;

namespace UE::Cameras
{

class FCameraDebugBlock;
class FCameraDebugBlockStorage;
class FRootCameraDebugBlock;
struct FCameraSystemEvaluationResult;

/**
 * Trace utility class for the camera system.
 */
class FCameraSystemTrace
{
public:

	GAMEPLAYCAMERAS_API static FString ChannelName;
	GAMEPLAYCAMERAS_API static FString LoggerName;
	GAMEPLAYCAMERAS_API static FString EvaluationEventName;

public:

	/** Returns whether tracing of camera system evaluation is enabled. */
	GAMEPLAYCAMERAS_API static bool IsTraceEnabled();
	/** Records one frame of camera system evaluation. */
	GAMEPLAYCAMERAS_API static void TraceEvaluation(UWorld* InWorld, const FCameraSystemEvaluationResult& InResult, FRootCameraDebugBlock& InRootDebugBlock);
	/** Reads back one frame of camera system evaluation. */
	GAMEPLAYCAMERAS_API static FCameraDebugBlock* ReadEvaluationTrace(TArray<uint8> InSerializedBlocks, FCameraDebugBlockStorage& InStorage);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

