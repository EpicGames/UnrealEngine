// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraNodeEvaluatorDebugBlock;

/**
 * A utility debug block that prints a title for a debug category, if that debug category is active.
 * By default, if the debug category is inactive, it will skip any attachments and children blocks.
 */
class FCategoryTitleDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FCategoryTitleDebugBlock)

public:

	GAMEPLAYCAMERAS_API FCategoryTitleDebugBlock();
	GAMEPLAYCAMERAS_API FCategoryTitleDebugBlock(const FString& InCategory, const FString& InTitle);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

public:

	FString Category;
	FString Title;

	bool bSkipAttachedBlocksIfInactive = true;
	bool bSkipChildrenBlocksIfInactive = true;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

