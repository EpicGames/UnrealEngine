// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraContextDataTable;

/**
 * A debug block that prints the contents of a context data table.
 */
class FContextDataTableDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FContextDataTableDebugBlock)

public:

	/** Creates a new context data table debug block. */
	GAMEPLAYCAMERAS_API FContextDataTableDebugBlock();
	/** Creates a new context data table debug block. */
	GAMEPLAYCAMERAS_API FContextDataTableDebugBlock(const FCameraContextDataTable& InContextDataTable);

	/** Specifies the console variable to use to toggle the printing of variable IDs. */
	FContextDataTableDebugBlock& WithShowDataIDsCVar(const TCHAR* InShowDataIDsCVarName)
	{
		ShowDataIDsCVarName = InShowDataIDsCVarName;
		return *this;
	}

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

	void Initialize(const FCameraContextDataTable& InContextDataTable);

	static FString GetDebugValueString(
			ECameraContextDataType DataType, 
			ECameraContextDataContainerType DataContainerType,
			const UObject* DataTypeObject,
			const uint8* DataPtr);

	static FString GetDebugValueString(
			ECameraContextDataType DataType, 
			const UObject* DataTypeObject,
			const uint8* DataPtr);

private:

	struct FEntryDebugInfo
	{
		uint32 ID;
		FString Name;
		FName TypeName;
		FString Value;
		bool bWritten;
		bool bWrittenThisFrame;
	};
	TArray<FEntryDebugInfo> Entries;

	FString ShowDataIDsCVarName;

	friend FArchive& operator<< (FArchive&, FEntryDebugInfo&);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

