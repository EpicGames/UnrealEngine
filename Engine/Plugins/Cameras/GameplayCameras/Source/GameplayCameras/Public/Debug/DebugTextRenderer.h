// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Camera/CameraTypes.h"
#include "Containers/StringView.h"
#include "Engine/EngineTypes.h"
#include "GameplayCameras.h"
#include "Math/UnrealMath.h"
#include "Misc/TVariant.h"
#include <type_traits>

class FCanvas;
class UFont;

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

// Utility function to print something. Most types have a LexToString implementation, but not all.
template<typename FieldType>
FString ToDebugString(const FieldType& FieldValue)
{
	if constexpr (std::is_enum_v<FieldType>)
	{
		const UEnum* EnumClass = StaticEnum<FieldType>();
		return EnumClass->GetValueAsString(FieldValue);
	}
	else
	{
		return LexToString(FieldValue);
	}
}
template<typename EnumType>
FString ToDebugString(TEnumAsByte<EnumType> EnumValue)
{
	const UEnum* EnumClass = StaticEnum<EnumType>();
	return EnumClass->GetValueAsString(EnumValue);
}
template<typename T>
FString ToDebugString(const UE::Math::TVector<T>& FieldValue)
{
	return FieldValue.ToString();
}
template<typename T>
FString ToDebugString(const UE::Math::TVector2<T>& FieldValue)
{
	return FieldValue.ToString();
}
template<typename T>
FString ToDebugString(const UE::Math::TVector4<T>& FieldValue)
{
	return FieldValue.ToString();
}
template<typename T>
FString ToDebugString(const UE::Math::TRotator<T>& FieldValue)
{
	return FieldValue.ToString();
}
template<typename T>
FString ToDebugString(const UE::Math::TTransform<T>& FieldValue)
{
	return FieldValue.ToString();
}
template<>
inline FString ToDebugString(const FLinearColor& FieldValue)
{
	return FieldValue.ToString();
}
template<>
inline FString ToDebugString(const EAspectRatioAxisConstraint& FieldValue)
{
	switch (FieldValue)
	{
		case AspectRatio_MaintainYFOV: return TEXT("Maintain Y-Axis FOV");
		case AspectRatio_MaintainXFOV: return TEXT("Maintain X-Axis FOV");
		case AspectRatio_MajorAxisFOV: return TEXT("Maintain Major Axis FOV");
	}
	return TEXT("Invalid");
}
template<>
inline FString ToDebugString(const ECameraProjectionMode::Type& FieldValue)
{
	switch (FieldValue)
	{
	case ECameraProjectionMode::Perspective: return TEXT("Perspective");
	case ECameraProjectionMode::Orthographic: return TEXT("Orthographic");
	}
	return TEXT("Invalid");
}

/** Command for drawing text on a canvas. */
struct FDebugTextDrawCommand
{
	TStringView<TCHAR> TextView;

	void Execute(FCanvas* Canvas, const FColor& DrawColor, const UFont* Font, FVector2f& InOutDrawPosition) const;
};

/** Command for moving the drawing position to a new line. */
struct FDebugTextNewLineCommand
{
	float LineSpacing = 0.f;
	float LeftMargin = 0.f;

	void Execute(FVector2f& InOutDrawPosition) const;
};

/** Command for setting the text color on a canvas. */
struct FDebugTextSetColorCommand
{
	FColor DrawColor;

	void Execute(FColor* OutDrawColor) const;
};

/** A debug text drawing command, which can be of multiple types. */
using FDebugTextCommand = TVariant<
	FDebugTextDrawCommand, 
	FDebugTextNewLineCommand,
	FDebugTextSetColorCommand>;

/** A command queue for drawing text on a canvas. */
using FDebugTextCommandArray = TArray<FDebugTextCommand>;

/**
 * Rendering utility for colored debug text.
 */
class FDebugTextRenderer
{
public:

	/** Space between the lines, defaults to max font character height. */
	float LineSpacing;
	/** The X coordinate for where text drawing starts, and where new lines start from. */
	float LeftMargin;
	/** Moves the next draw position to a new line at the end of the text render. */
	bool bEndWithNewLine = false;

	/** Creates a new debug text renderer. */
	FDebugTextRenderer(FCanvas* InCanvas, const FColor& InDrawColor, const UFont* InFont);

	/** Renders the given text to the canvas. */
	void RenderText(float StartingDrawY, const TStringView<TCHAR> TextView);
	void RenderText(FVector2f StartingDrawPosition, const TStringView<TCHAR> TextView);

	/** Executes the given command queue. */
	void ExecuteCommands(float StartingDrawY, FDebugTextCommandArray& Commands);
	void ExecuteCommands(FVector2f StartingDrawPosition, FDebugTextCommandArray& Commands);

	/** Gets the coordinate of where any new text would go, just after the last render. */
	FVector2f GetEndDrawPosition() const { return NextDrawPosition; }

	/** Gets the maximum horizontal extent of the rendered text. */
	float GetRightMargin() const { return RightMargin; }

public:

	static float GetStringViewSize(const UFont* Font, TStringView<TCHAR> TextView);

private:

	void ParseText(const TStringView<TCHAR> TextView, FDebugTextCommandArray& OutCommands);
	void ExecuteCommands(FDebugTextCommandArray& Commands);
	void UpdateRightMargin();

	void AddDrawCommand(const TCHAR* RangeStart, const TCHAR* RangeEnd, bool bNewLine, FDebugTextCommandArray& OutCommands);
	void AddTokenCommand(const TCHAR* RangeStart, const TCHAR* RangeEnd, FDebugTextCommandArray& OutCommands);

	FColor InterpretColor(const FString& ColorName);

private:

	FCanvas* Canvas;
	FColor DrawColor;
	const UFont* Font;
	FVector2f NextDrawPosition;
	float RightMargin = 0;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

