// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"
#include "Engine/EngineTypes.h"
#include "Math/UnrealMath.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

struct FCameraDebugBlockBuilder;

/**
 * A debug block for showing the list of local player controllers and their view targets.
 */
class FPlayerControllersDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FPlayerControllersDebugBlock)

public:

	FPlayerControllersDebugBlock();

	void Initialize(UWorld* World);

protected:

	// FCameraDebugBlock interface.
	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	struct FPlayerControllerDebugInfo
	{
		FString PlayerControllerName;
		FString CameraManagerName;
		FString LocalPlayerName;

		FIntPoint ViewportSize;
		TEnumAsByte<EAspectRatioAxisConstraint> DefaultAspectRatioAxisConstraint = AspectRatio_MaintainXFOV;
		bool bHasDefaultAspectRatioAxisConstraint = true;

		FString ViewTargetName;
		FVector3d ViewTargetLocation;
		FRotator3d ViewTargetRotation;
		float ViewTargetFOV;
		float ViewTargetAspectRatio;
		TEnumAsByte<EAspectRatioAxisConstraint> ViewTargetAspectRatioAxisConstraint = AspectRatio_MaintainXFOV;
		bool bViewTargetConstrainAspectRatio;
		bool bViewTargetOverrideAspectRatioAxisConstraint = false;
	};
	TArray<FPlayerControllerDebugInfo> PlayerControllers;
	bool bHadValidWorld = false;

	friend FArchive& operator<< (FArchive&, FPlayerControllerDebugInfo&);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

