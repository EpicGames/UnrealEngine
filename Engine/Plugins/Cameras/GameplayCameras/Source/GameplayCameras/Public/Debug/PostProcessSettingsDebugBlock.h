// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/PostProcessSettingsCollection.h"
#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

/**
 * A debug block that displays information about post-process settings.
 */
class FPostProcessSettingsDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FPostProcessSettingsDebugBlock)

public:

	/** Creates a new post-process settings debug block. */
	FPostProcessSettingsDebugBlock();
	/** Creates a new post-process settings debug block. */
	FPostProcessSettingsDebugBlock(const FPostProcessSettingsCollection& InPostProcessSettings);

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FPostProcessSettingsCollection PostProcessSettings;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

