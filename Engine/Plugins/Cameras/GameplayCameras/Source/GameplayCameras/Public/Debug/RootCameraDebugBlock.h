// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"
#include "Debug/CameraSystemDebugRegistry.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraSystemEvaluator;
struct FCameraDebugBlockBuildParams;
struct FCameraDebugBlockBuilder;
struct FCameraSystemDebugID;

GAMEPLAYCAMERAS_API extern bool GGameplayCamerasDebugEnable;
GAMEPLAYCAMERAS_API extern int32 GGameplayCamerasDebugSystemID;
GAMEPLAYCAMERAS_API extern FString GGameplayCamerasDebugCategories;

/**
 * Parameters for drawing a hierarchy of debug blocks.
 */
struct FRootCameraDebugDrawParams
{
	/**
	 * Whether the debug blocks belong to a camera system that is considered the "default" one,
	 * such as the one running inside the camera manager, or the active view target.
	 */
	bool bFromDefaultCameraSystem = false;

	/** Whether to force draw the debug blocks. */
	bool bForceDraw = false;
};

/**
 * The root debug block for the camera system.
 */
class FRootCameraDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FRootCameraDebugBlock)

public:

	/** Build all debug blocks for the last evaluation frame. */
	GAMEPLAYCAMERAS_API void BuildDebugBlocks(const FCameraSystemEvaluator& CameraSystem, const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);

	/** Initiate the debug drawing. */
	GAMEPLAYCAMERAS_API void RootDebugDraw(const FRootCameraDebugDrawParams& Params, FCameraDebugRenderer& Renderer);

	/** Gets the debug ID of the camera system that generated this debug info. */
	const FCameraSystemDebugID& GetDebugID() const { return DebugID; }

public:

	/** Checks whether the debug data with the given ID would be drawn on screen. */
	GAMEPLAYCAMERAS_API static bool ShouldDebugDraw(FCameraSystemDebugID InDebugID, bool bIsDefaultCameraSystem);

protected:

	// FCameraDebugBlock interface.
	virtual void OnSerialize(FArchive& Ar) override;

private:

	FCameraSystemDebugID DebugID; 
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

