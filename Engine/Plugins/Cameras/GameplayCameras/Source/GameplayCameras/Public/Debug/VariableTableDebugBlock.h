// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/CameraDebugBlock.h"

#if UE_GAMEPLAY_CAMERAS_DEBUG

namespace UE::Cameras
{

class FCameraVariableTable;

/**
 * A debug block that prints the contents of a variable table.
 */
class FVariableTableDebugBlock : public FCameraDebugBlock
{
	UE_DECLARE_CAMERA_DEBUG_BLOCK(GAMEPLAYCAMERAS_API, FVariableTableDebugBlock)

public:

	/** Creates a new variable table debug block. */
	GAMEPLAYCAMERAS_API FVariableTableDebugBlock();
	/** Creates a new variable table debug block. */
	GAMEPLAYCAMERAS_API FVariableTableDebugBlock(const FCameraVariableTable& InVariableTable);

	/** Specifies the console variable to use to toggle the printing of variable IDs. */
	FVariableTableDebugBlock& WithShowVariableIDsCVar(const TCHAR* InShowVariableIDsCVarName)
	{
		ShowVariableIDsCVarName = InShowVariableIDsCVarName;
		return *this;
	}

protected:

	virtual void OnDebugDraw(const FCameraDebugBlockDrawParams& Params, FCameraDebugRenderer& Renderer) override;
	virtual void OnSerialize(FArchive& Ar) override;

	void Initialize(const FCameraVariableTable& InVariableTable);

private:

	struct FEntryDebugInfo
	{
		uint32 ID;
		FString Name;
		FString Value;
		bool bIsInput;
		bool bIsPrivate;
		bool bWritten;
		bool bWrittenThisFrame;
	};
	TArray<FEntryDebugInfo> Entries;

	FString ShowVariableIDsCVarName;

	friend FArchive& operator<< (FArchive&, FEntryDebugInfo&);
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

