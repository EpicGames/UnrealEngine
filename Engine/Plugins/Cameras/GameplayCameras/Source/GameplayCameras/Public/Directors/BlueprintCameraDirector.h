// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Core/CameraEvaluationContext.h"
#include "GameFramework/BlueprintCameraEvaluationDataRef.h"
#include "Templates/SubclassOf.h"

#include "BlueprintCameraDirector.generated.h"

class UCameraRigAsset;
class UCameraRigProxyAsset;
enum class ECameraRigLayer : uint8;

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.6, "This parameter structure is deprecated, parameters are now passed directly to RunCameraDirector.") FBlueprintCameraDirectorEvaluationParams
{
	GENERATED_BODY()

	/** The elapsed time since the last evaluation. */
	UPROPERTY(BlueprintReadWrite, Category="Evaluation", meta=(Deprecated, DeprecationMessage="Use the main DeltaTime parameter on RunCameraDirector"))
	float DeltaTime = 0.f;

	/** The owner (if any) of the evaluation context we are running inside of. */
	UPROPERTY(BlueprintReadWrite, Category="Evaluation", meta=(Deprecated, DeprecationMessage="Use the main EvaluationContextOwner parameter on RunCameraDirector"))
	TObjectPtr<UObject> EvaluationContextOwner;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.6, "This parameter structure is deprecated, parameters are now passed directly to ActivateCameraDirector.") FBlueprintCameraDirectorActivateParams
{
	GENERATED_BODY()

	/** The owner (if any) of the evaluation context we are running inside of. */
	UPROPERTY(BlueprintReadWrite, Category="Evaluation")
	TObjectPtr<UObject> EvaluationContextOwner;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.6, "This parameter structure is deprecated, parameters are now passed directly to DeactivateCameraDirector.") FBlueprintCameraDirectorDeactivateParams
{
	GENERATED_BODY()

	/** The owner (if any) of the evaluation context we were running inside of. */
	UPROPERTY(BlueprintReadWrite, Category="Evaluation")
	TObjectPtr<UObject> EvaluationContextOwner;
};

/**
 * Base class for a Blueprint camera director evaluator.
 */
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UBlueprintCameraDirectorEvaluator : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Override this method in Blueprint to execute custom logic when this
	 * camera director gets activated.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Camera Director|Activation")
	void ActivateCameraDirector(UObject* EvaluationContextOwner, const FBlueprintCameraDirectorActivateParams& Params);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Override this method in Blueprint to execute custom logic when this
	 * camera director gets deactivated.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Camera Director|Activation")
	void DeactivateCameraDirector(UObject* EvaluationContextOwner, const FBlueprintCameraDirectorDeactivateParams& Params);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * Override this method in Blueprint to execute the custom logic that determines
	 * what camera rig(s) should be active every frame.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Evaluation")
	void RunCameraDirector(float DeltaTime, UObject* EvaluationContextOwner, const FBlueprintCameraDirectorEvaluationParams& Params);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Evaluation")
	GAMEPLAYCAMERAS_API FName AddChildEvaluationContext(UObject* ChildEvaluationContextOwner);

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Evaluation")
	GAMEPLAYCAMERAS_API bool RemoveChildEvaluationContext(UObject* ChildEvaluationContextOwner, FName ChildSlotName);

	UFUNCTION(BlueprintCallable, Category="Evaluation")
	GAMEPLAYCAMERAS_API bool RunChildCameraDirector(float DeltaTime, FName ChildSlotName);

public:

	/** Activates the given camera rig prefab in the base layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRigPrefab);

	/** Activates the given camera rig prefab in the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRigPrefab);

	/** Activates the given camera rig prefab in the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRigPrefab);

	/** Deactivates the given camera rig prefab in the base layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void DeactivatePersistentBaseCameraRig(UCameraRigAsset* CameraRigPrefab);

	/** Deactivates the given camera rig prefab in the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void DeactivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRigPrefab);

	/** Deactivates the given camera rig prefab in the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Activation")
	GAMEPLAYCAMERAS_API void DeactivatePersistentVisualCameraRig(UCameraRigAsset* CameraRigPrefab);

public:

	/** Specifies a camera rig to be active this frame. */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Evaluation")
	GAMEPLAYCAMERAS_API void ActivateCameraRig(UCameraRigAsset* CameraRig, bool bForceNewInstance = false);

	/**
	 * Specifies a camera rig to be active this frame, via a proxy which is later resolved
	 * via the proxy table of the Blueprint camera director.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Director|Evaluation")
	GAMEPLAYCAMERAS_API void ActivateCameraRigViaProxy(UCameraRigProxyAsset* CameraRigProxy, bool bForceNewInstance = false);

public:

	/** Resolves the camera rig proxy using the camera director's proxy table. */
	UFUNCTION(BlueprintPure, Category="Camera Director|Evaluation", meta=(HideSelfPin=true))
	GAMEPLAYCAMERAS_API UCameraRigAsset* ResolveCameraRigProxy(const UCameraRigProxyAsset* CameraRigProxy) const;

public:

	/**
	 * A utility function that tries to find if an actor owns the evaluation context.
	 * Handles the situation where the evaluation context is an actor component (like a
	 * UGameplayCameraComponent) or an actor itself.
	 */
	UFUNCTION(BlueprintPure, Category="Camera Director|Evaluation", meta=(DeterminesOutputType="ActorClass"))
	GAMEPLAYCAMERAS_API AActor* FindEvaluationContextOwnerActor(TSubclassOf<AActor> ActorClass) const;

	/** Gets the shared evaluation context data. */
	UFUNCTION(BlueprintPure, Category="Camera Director|Evaluation", meta=(DisplayName="Get Shared Camera Data"))
	GAMEPLAYCAMERAS_API FBlueprintCameraEvaluationDataRef GetInitialContextResult() const;

	/** Gets the evaluation context data for a sub-set of camera rigs. */	
	UFUNCTION(BlueprintPure, Category="Camera Director|Evaluation", meta=(DisplayName="Get Conditional Camera Data"))
	GAMEPLAYCAMERAS_API FBlueprintCameraEvaluationDataRef GetConditionalContextResult(ECameraEvaluationDataCondition Condition) const;

public:

	// UObject interface.
	virtual UWorld* GetWorld() const override;
#if WITH_EDITOR
	virtual bool ImplementsGetWorld() const override { return true; }
#endif  // WITH_EDITOR

public:

	using FCameraEvaluationContext = UE::Cameras::FCameraEvaluationContext;
	using FCameraDirectorEvaluationResult = UE::Cameras::FCameraDirectorEvaluationResult;

	// Internal API.

	const FCameraDirectorEvaluationResult& GetEvaluationResult() const { return EvaluationResult; }

public:

	/** Initialize this camera director evaluator. */
	void NativeInitializeCameraDirector(UE::Cameras::FCameraDirectorEvaluator* InOwningDirectorEvaluator, const UE::Cameras::FCameraDirectorInitializeParams& Params);

	/** Abandon this camera director evalutor. */
	void NativeAbandonCameraDirector();

public:

	/** Native wrapper for ActivateCameraDirector. */
	void NativeActivateCameraDirector(const UE::Cameras::FCameraDirectorActivateParams& Params);

	/** Native wrapper for DeactivateCameraDirector. */
	void NativeDeactivateCameraDirector(const UE::Cameras::FCameraDirectorDeactivateParams& Params);

	/** Native wrapper for RunCameraDirector. */
	void NativeRunCameraDirector(const UE::Cameras::FCameraDirectorEvaluationParams& Params);

	bool NativeAddChildEvaluationContext(UObject* ChildEvaluationContextOwner);

	bool NativeRemoveChildEvaluationContext(UObject* ChildEvaluationContextOwner);

private:

	/** The owning camera director evaluator. */
	UE::Cameras::FCameraDirectorEvaluator* OwningDirectorEvaluator = nullptr;

	/** The current evaluation result. */
	FCameraDirectorEvaluationResult EvaluationResult;

	/** The current evaluation context. */
	TSharedPtr<FCameraEvaluationContext> EvaluationContext;

	/** Currently registered children contexts. */
	TArray<FName> ChildrenContextSlotNames;

	/** Cached world. */
	mutable TWeakObjectPtr<UWorld> WeakCachedWorld;
};

/**
 * A camera director that will instantiate the given Blueprint and run it.
 */
UCLASS(MinimalAPI, EditInlineNew)
class UBlueprintCameraDirector : public UCameraDirector
{
	GENERATED_BODY()

public:

	/** The blueprint class that we should instantiate and run. */
	UPROPERTY(EditAnywhere, Category="Evaluation")
	TSubclassOf<UBlueprintCameraDirectorEvaluator> CameraDirectorEvaluatorClass;

protected:

	// UCameraDirector interface.
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const override;
	virtual void OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext) override;
	virtual void OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const override;
	virtual void OnExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
};

