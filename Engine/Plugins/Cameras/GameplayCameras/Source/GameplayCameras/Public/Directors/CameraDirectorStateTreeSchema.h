// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "StateTreeConditionBase.h"
#include "StateTreeSchema.h"
#include "StateTreeTaskBase.h"
#include "StateTreeTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#include "CameraDirectorStateTreeSchema.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraRigAsset;
class UCameraRigProxyAsset;
struct FStateTreeExternalDataDesc;

namespace UE::Cameras
{

struct FStateTreeContextDataNames
{
	const static FName ContextOwner;
};

}  // namespace UE::Cameras

/**
 * The schema of the StateTree for a StateTree camera director.
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta=(DisplayName="Gameplay Camera Director"))
class UCameraDirectorStateTreeSchema : public UStateTreeSchema
{
	GENERATED_BODY()

public:

	UE_API UCameraDirectorStateTreeSchema();

protected:

	// UStateTreeSchema interface.
	UE_API virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
	UE_API virtual bool IsClassAllowed(const UClass* InScriptStruct) const override;
	UE_API virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override;
	virtual TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const override { return ContextDataDescs; }

private:

	UPROPERTY()
	TArray<FStateTreeExternalDataDesc> ContextDataDescs;
};

/** The evaluation data for the StateTree camera director. */
USTRUCT(BlueprintType)
struct FCameraDirectorStateTreeEvaluationData
{
	GENERATED_BODY()

	/** Camera rigs activated during a StateTree's execution frame. */
	UPROPERTY()
	TArray<TObjectPtr<UCameraRigAsset>> ActiveCameraRigs;

	/** Camera rig proxies activated during a StateTree's execution frame. */
	UPROPERTY()
	TArray<TObjectPtr<UCameraRigProxyAsset>> ActiveCameraRigProxies;

public:

	/** Reset this evaluation data for a new frame. */
	UE_API void Reset();
};

/** Base classs for camera director StateTree tasks. */
USTRUCT(meta = (Hidden))
struct FGameplayCamerasStateTreeTask : public FStateTreeTaskBase
{
	GENERATED_BODY()
};

/** Base classs for camera director StateTree conditions. */
USTRUCT(meta = (Hidden))
struct FGameplayCamerasStateTreeCondition : public FStateTreeConditionBase
{
	GENERATED_BODY()
};

#undef UE_API
