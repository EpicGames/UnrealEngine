// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"
#include "Core/CameraDirectorEvaluator.h"
#include "Misc/TVariant.h"

#include "PriorityQueueCameraDirector.generated.h"

/**
 * A camera director that holds multiple sub-directors, and runs the one that has the highest priority.
 */
UCLASS(MinimalAPI, EditInlineNew)
class UPriorityQueueCameraDirector : public UCameraDirector
{
	GENERATED_BODY()

public:

	UPriorityQueueCameraDirector(const FObjectInitializer& ObjectInit);

protected:

	// UCameraDirector interface.
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const override;
};

namespace UE::Cameras
{

/**
 * Interface for sub-directors with dynamic priorities.
 */
class IPriorityQueueEntry
{
public:

	virtual ~IPriorityQueueEntry() = default;

	/** Gets the current priority of the sub-director. */
	virtual int32 GetPriority() = 0;
};

/**
 * Evaluator for the priority queue camera director.
 *
 * Use the overriden AddChildEvaluationContext methods here to add sub-directors with a specific priority. Using the base class
 * version will add a sub-director with a priority of 0.
 *
 * Highest priority values mean more priority, which means higher chances of being picked from the queue.
 */
class FPriorityQueueCameraDirectorEvaluator : public FCameraDirectorEvaluator
{
	UE_DECLARE_CAMERA_DIRECTOR_EVALUATOR(GAMEPLAYCAMERAS_API, FPriorityQueueCameraDirectorEvaluator)

public:

	/**
	 * Adds a sub-director with the given fixed priority.
	 */
	GAMEPLAYCAMERAS_API void AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext, int32 Priority);

	/**
	 * Adds a sub-director with the given dynamic priority.
	 * It's customary to have the evaluation context itself implement the IPriorityQueueEntry interface, and therefore passing
	 * the pointer twice to the function.
	 */
	GAMEPLAYCAMERAS_API void AddChildEvaluationContext(TSharedRef<FCameraEvaluationContext> InContext, IPriorityQueueEntry* PriorityEntry);

protected:

	virtual void OnRun(const FCameraDirectorEvaluationParams& Params, FCameraDirectorEvaluationResult& OutResult) override;

	virtual void OnAddChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) override;
	virtual void OnRemoveChildEvaluationContext(const FChildContextManulationParams& Params, FChildContextManulationResult& Result) override;

private:

	using FPriorityGiver = TVariant<int32, IPriorityQueueEntry*>;
	struct FEntry
	{
		TSharedPtr<FCameraEvaluationContext> ChildContext;
		FPriorityGiver PriorityGiver;

		int32 GetPriority() const;
	};

	using FEntryArray = TArray<FEntry>;
	FEntryArray Entries;
};

}  // namespace UE::Cameras

