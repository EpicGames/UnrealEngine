// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"

#include "SingleCameraDirector.generated.h"

/**
 * A simple camera director that only ever returns one single camera rig.
 */
UCLASS(MinimalAPI, EditInlineNew)
class USingleCameraDirector : public UCameraDirector
{
	GENERATED_BODY()

public:

	USingleCameraDirector(const FObjectInitializer& ObjectInit);

protected:

	// UCameraDirector interface.
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const override;
	virtual void OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext) override;
	virtual void OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const override;

public:

	/** The camera rig to run every frame. */
	UPROPERTY(EditAnywhere, Category=Common, meta=(UseSelfCameraRigPicker=true))
	TObjectPtr<UCameraRigAsset> CameraRig;
};

