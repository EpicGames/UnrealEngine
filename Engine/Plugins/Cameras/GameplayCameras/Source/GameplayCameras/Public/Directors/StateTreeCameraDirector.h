// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraDirector.h"
#include "StateTreeReference.h"

#include "StateTreeCameraDirector.generated.h"

/**
 * A camera director that runs a StateTree to specify which camera rigs should be active
 * any given frame.
 */
UCLASS(MinimalAPI, EditInlineNew)
class UStateTreeCameraDirector : public UCameraDirector
{
	GENERATED_BODY()

public:

	UStateTreeCameraDirector(const FObjectInitializer& ObjectInit);

protected:

	// UCameraDirector interface.
	virtual FCameraDirectorEvaluatorPtr OnBuildEvaluator(FCameraDirectorEvaluatorBuilder& Builder) const override;
	virtual void OnBuildCameraDirector(UE::Cameras::FCameraBuildContext& BuildContext) override;
	virtual void OnGatherRigUsageInfo(FCameraDirectorRigUsageInfo& UsageInfo) const override;
	virtual void OnExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

public:

	/** The StateTree to execute. Must have been created with the CameraDirectorStateTreeSchema. */
	UPROPERTY(EditAnywhere, Category="StateTree", DisplayName="State Tree",
			meta=(Schema="/Script/GameplayCameras.CameraDirectorStateTreeSchema"))
	FStateTreeReference StateTreeReference;
};

