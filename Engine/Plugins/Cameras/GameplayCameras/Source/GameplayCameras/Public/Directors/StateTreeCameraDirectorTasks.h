// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigAsset.h"
#include "Directors/CameraDirectorStateTreeSchema.h"

#include "StateTreeCameraDirectorTasks.generated.h"

/** Instance data for the "Activate Camera Rig" task. */
USTRUCT()
struct FGameplayCamerasActivateCameraRigTaskInstanceData
{
	GENERATED_BODY()

	/** The camera rig to activate. */
	UPROPERTY(EditAnywhere, Category="Cameras", meta=(UseCameraDirectorRigPicker=true))
	TObjectPtr<UCameraRigAsset> CameraRig;
};

/**
 * A task that activates a given camera rig inside a StateTreeCameraDirector.
 */
USTRUCT(meta=(DisplayName="Activate Camera Rig", Category="Cameras"))
struct FGameplayCamerasActivateCameraRigTask : public FGameplayCamerasStateTreeTask
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayCamerasActivateCameraRigTaskInstanceData;

protected:

	// FStateTreeTaskBase interface.
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;

private:

	bool UpdateResult(FStateTreeExecutionContext& Context) const;

private:

	TStateTreeExternalDataHandle<FCameraDirectorStateTreeEvaluationData> CameraDirectorEvaluationDataHandle;

public:

	/** If true, the task will complete immediately. If false, the task will run until a transition triggers. */
	UPROPERTY(EditAnywhere, Category="State Tree")
	bool bRunOnce = false;
};

/** Instance data for the "Activate Camera Rig via Proxy" task. */
USTRUCT()
struct FGameplayCamerasActivateCameraRigViaProxyTaskInstanceData
{
	GENERATED_BODY()

	/** The camera rig proxy to activate. */
	UPROPERTY(EditAnywhere, Category="Cameras")
	TObjectPtr<UCameraRigProxyAsset> CameraRigProxy;
};

/**
 * A task that activates a camera rig via the given camera rig proxy inside a
 * StateTreeCameraDirector. The given proxy must be mapped to an actual proxy
 * in each camera asset using this StateTree.
 */
USTRUCT(meta=(DisplayName="Activate Camera via Proxy", Category="Cameras"))
struct FGameplayCamerasActivateCameraRigViaProxyTask : public FGameplayCamerasStateTreeTask
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayCamerasActivateCameraRigViaProxyTaskInstanceData;

protected:

	// FStateTreeTaskBase interface.
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;

private:

	bool UpdateResult(FStateTreeExecutionContext& Context) const;

private:

	TStateTreeExternalDataHandle<FCameraDirectorStateTreeEvaluationData> CameraDirectorEvaluationDataHandle;

public:

	/** If true, the task will complete immediately. If false, the task will run until a transition triggers. */
	UPROPERTY(EditAnywhere, Category="State Tree")
	bool bRunOnce = false;
};

