// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ActivateCameraRigFunctions.generated.h"

class APlayerController;
class UCameraRigAsset;
enum class ECameraRigLayer : uint8;

/**
 * Blueprint functions for activating camera rigs in the base/global/visual layers.
 *
 * These camera rigs run with a global, shared evaluation context that doesn't provide any
 * meaningful initial result. They are activated on the camera system found to be running
 * on the given player controller.
 *
 * Deprecated in 5.7.0
 */
UCLASS()
class UActivateCameraRigFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Activates the given camera rig prefab in the base layer. */
	UFUNCTION(BlueprintCallable, Category="Camera", meta=(WorldContext="WorldContextObject", DeprecatedFunction, DeprecationMessage="Use the similar functions on the GameplayCamera components or camera manager"))
	static void ActivatePersistentBaseCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera", meta = (WorldContext = "WorldContextObject", DeprecatedFunction, DeprecationMessage="Use the similar functions on the GameplayCamera components or camera manager"))
	static void ActivatePersistentGlobalCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera", meta = (WorldContext = "WorldContextObject", DeprecatedFunction, DeprecationMessage="Use the similar functions on the GameplayCamera components or camera manager"))
	static void ActivatePersistentVisualCameraRig(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig);

private:

	/** Activates the given camera rig in the given layer. Should not be used with Main layer. */
	static void ActivateCameraRigImpl(UObject* WorldContextObject, APlayerController* PlayerController, UCameraRigAsset* CameraRig, ECameraRigLayer EvaluationLayer);
};

