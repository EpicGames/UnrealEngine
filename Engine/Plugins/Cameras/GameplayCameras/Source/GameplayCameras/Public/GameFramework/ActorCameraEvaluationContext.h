// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNode.h"

#include "ActorCameraEvaluationContext.generated.h"

class AActor;
class UCameraComponent;
struct FMinimalViewInfo;

namespace UE::Cameras
{

/**
 * An evaluation context for an arbitrary actor.
 * It procedurally creates a single camera director that runs a rig only composed of
 * a camera node that simply copies the actor's properties.
 */
class FActorCameraEvaluationContext : public FCameraEvaluationContext
{
	UE_DECLARE_CAMERA_EVALUATION_CONTEXT(GAMEPLAYCAMERAS_API, FActorCameraEvaluationContext)

public:

	FActorCameraEvaluationContext();
	FActorCameraEvaluationContext(UCameraComponent* InCameraComponent);
	FActorCameraEvaluationContext(AActor* InActor);

public:

	static void ApplyMinimalViewInfo(const FMinimalViewInfo& ViewInfo, FCameraNodeEvaluationResult& OutResult);

private:

	static UCameraAsset* MakeCameraComponentCameraAsset(UObject* OuterObject);
	static UCameraAsset* MakeCalcCameraActorCameraAsset(UObject* OuterObject);
	static UCameraAsset* MakeSimpleCameraAsset(UObject* OuterObject, UCameraNode* RootNode);
};

}  // namespace UE::Cameras

/**
 * A simple camera node that copies the properties of a camera component and applies them to
 * the evaluation output.
 * Note that this node expects to run inside an evaluation context owned by the camera component's
 * parent actor.
 */
UCLASS(MinimalAPI, Hidden)
class UCameraComponentCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

/**
 * A simple camera node that calls an actor's CalcCamera functoin and copies the result to
 * the evaluation output.
 * Note that this node expects to run inside an evaluation context owned by that actor.
 */
UCLASS(MinimalAPI, Hidden)
class UCalcCameraActorCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

