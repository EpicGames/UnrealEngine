// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraVariableAssets.h"
#include "GameFramework/BlueprintCameraPose.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "BlueprintCameraEvaluationDataRef.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraRigAsset;

namespace UE::Cameras
{

struct FCameraNodeEvaluationResult;

}  // namespace UE::Cameras

/**
 * Blueprint wrapper for camera evaluation data.
 */
USTRUCT(BlueprintType, DisplayName="Camera Evaluation Data Ref", 
		meta=(HasNativeMake="/Script/GameplayCameras.BlueprintCameraEvaluationDataFunctionLibrary.MakeCameraEvaluationData"))
struct FBlueprintCameraEvaluationDataRef
{
	GENERATED_BODY()

public:

	using FCameraNodeEvaluationResult = UE::Cameras::FCameraNodeEvaluationResult;

	UE_API FBlueprintCameraEvaluationDataRef();

	bool IsValid() const { return Result != nullptr; }

	FCameraNodeEvaluationResult* GetResult() const { return Result; }

public:

	static UE_API FBlueprintCameraEvaluationDataRef MakeExternalRef(FCameraNodeEvaluationResult* InResult);
	static UE_API FBlueprintCameraEvaluationDataRef MakeOwningRef();

private:

	/** The underlying camera evaluation result. */
	FCameraNodeEvaluationResult* Result = nullptr;
	
	/** If the underlying camera evaluation result is owned by this data ref, a shared pointer to keep it alive. */
	TSharedPtr<FCameraNodeEvaluationResult> SharedResult;
};

/**
 * Blueprint function library for camera evaluation data references.
 */
UCLASS()
class UBlueprintCameraEvaluationDataFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Creates a camera evaluation data. */
	UFUNCTION(BlueprintPure, Category="Camera", meta=(ReturnDisplayName="Camera Evaluation Data"))
	static FBlueprintCameraEvaluationDataRef MakeCameraEvaluationData();

public:

	/** Gets the camera pose. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FBlueprintCameraPose GetCameraPose(const FBlueprintCameraEvaluationDataRef& CameraData);

	/** Sets the camera pose. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	static void SetCameraPose(const FBlueprintCameraEvaluationDataRef& CameraData, const FBlueprintCameraPose& CameraPose);

public:

	/** Interpolates one camera data towards another. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	static void BlendCameraEvaluationData(const FBlueprintCameraEvaluationDataRef& FromCameraData, const FBlueprintCameraEvaluationDataRef& ToCameraData, float Factor);

	/** Sets the default values for all parameters in the given camera rig. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static void SetDefaultCameraRigParameters(const FBlueprintCameraEvaluationDataRef& CameraData, const UCameraRigAsset* CameraRig);
};

/**
 * Blueprint function library for camera variable tables.
 */
UCLASS()
class UBlueprintCameraVariableTableFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Boolean Variable", Category=Camera, meta=(CompactNodeTitle="Get Boolean Variable"))
	static bool GetBooleanCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UBooleanCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Integer Variable", Category=Camera, meta=(CompactNodeTitle="Get Integer Variable"))
	static int32 GetInteger32CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UInteger32CameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Float Variable", Category=Camera, meta=(CompactNodeTitle="Get Float Variable"))
	static float GetFloatCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UFloatCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Double Variable", Category=Camera, meta=(CompactNodeTitle="Get Double Variable"))
	static double GetDoubleCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UDoubleCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Vector 2D Variable", Category=Camera, meta=(CompactNodeTitle="Get Vector 2D Variable"))
	static FVector2D GetVector2CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector2dCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Vector Variable", Category=Camera, meta=(CompactNodeTitle="Get Vector Variable"))
	static FVector GetVector3CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector3dCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Vector 4 Variable", Category=Camera, meta=(CompactNodeTitle="Get Vector 4 Variable"))
	static FVector4 GetVector4CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector4dCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Rotator Variable", Category=Camera, meta=(CompactNodeTitle="Get Rotator Variable"))
	static FRotator GetRotatorCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, URotator3dCameraVariable* Variable);

	/** Gets a camera variable's value from the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Get Transform Variable", Category=Camera, meta=(CompactNodeTitle="Get Transform Variable"))
	static FTransform GetTransformCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UTransform3dCameraVariable* Variable);

public:

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Boolean Variable", Category=Camera)
	static void SetBooleanCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UBooleanCameraVariable* Variable, bool Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Integer Variable", Category=Camera)
	static void SetInteger32CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UInteger32CameraVariable* Variable, int32 Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Float Variable", Category=Camera)
	static void SetFloatCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UFloatCameraVariable* Variable, float Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Double Variable", Category=Camera)
	static void SetDoubleCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UDoubleCameraVariable* Variable, double Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Vector 2D Variable", Category=Camera)
	static void SetVector2CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector2dCameraVariable* Variable, const FVector2D& Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Vector Variable", Category=Camera)
	static void SetVector3CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector3dCameraVariable* Variable, const FVector& Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Vector 4 Variable", Category=Camera)
	static void SetVector4CameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UVector4dCameraVariable* Variable, const FVector4& Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Rotator Variable", Category=Camera)
	static void SetRotatorCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, URotator3dCameraVariable* Variable, const FRotator& Value);

	/** Sets a camera variable's value in the given table. */
	UFUNCTION(BlueprintCallable, DisplayName="Set Transform Variable", Category=Camera)
	static void SetTransformCameraVariable(const FBlueprintCameraEvaluationDataRef& CameraData, UTransform3dCameraVariable* Variable, const FTransform& Value);
};

/**
 * Utility Blueprint functions for camera context data tables.
 */
UCLASS()
class UBlueprintCameraContextDataTableFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static FName GetNameData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID);

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static FString GetStringData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID);

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera, meta=(DeterminesOutputType="EnumType"))
	static uint8 GetEnumData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UEnum* EnumType);

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static FInstancedStruct GetStructData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UScriptStruct* DataStructType);

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static UObject* GetObjectData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID);

	/** Gets a value from the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static UClass* GetClassData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID);

public:

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetNameData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FName& Data);

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetStringData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FString& Data);

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetEnumData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const UEnum* EnumType, uint8 Data);

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetStructData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, const FInstancedStruct& Data);

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetObjectData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, UObject* Data);

	/** Sets a value in the given camera context data table. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static bool SetClassData(const FBlueprintCameraEvaluationDataRef& CameraData, FCameraContextDataID DataID, UClass* Data);
};

/**
 * Utility Blueprint functions for camera post process settings.
 */
UCLASS()
class UBlueprintPostProcessSettingsCollectionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets the post-process settings. */
	UFUNCTION(BlueprintPure, Category=Camera)
	static void GetPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, FPostProcessSettings& OutPostProcessSettings);

	/** Sets the post-process settings. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static void SetPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings);

	/** Overrides the post-process settings with the given ones. Only set properties are written. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static void OverrideChangedPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings);

	/** Interpolate the post-process settings towards the given ones. Only set properties are interpolated. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	static void LerpPostProcessSettings(const FBlueprintCameraEvaluationDataRef& CameraData, const FPostProcessSettings& PostProcessSettings, float BlendFactor);
};

#undef UE_API
