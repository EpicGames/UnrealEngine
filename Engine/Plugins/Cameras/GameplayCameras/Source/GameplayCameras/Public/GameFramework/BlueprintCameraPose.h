// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraPose.h"
#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"

#include "BlueprintCameraPose.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraComponent;
class UCineCameraComponent;

namespace UE::Cameras
{

class FCameraVariableTable;
struct FCameraNodeEvaluationResult;

}  // namespace UE::Cameras

/** Represents a camera pose. */
USTRUCT(BlueprintType, meta=(DisplayName="Camera Pose"))
struct FBlueprintCameraPose
{
	GENERATED_BODY()

public:

	/** The location of the camera. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	FVector Location = {0, 0, 0};

	/** The rotation of the camera. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	FRotator Rotation = {0, 0, 0};

	/** The distance of the target of the camera. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	double TargetDistance = 1000.0;

	/** The field of view of the camera. May be negative if driven by focal length. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FieldOfView = -1.f;

	/** The focal length of the camera. May be negative if driven directly by field of view. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FocalLength = 35.f;

	/** The horizontal field of view (in degrees) used for primitives tagged as "IsFirstPerson" */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FirstPersonFieldOfView = 90.f;

	/** The scale to apply to primitives tagged as "IsFirstPerson" */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FirstPersonScale = 1.f;

	/** The desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Camera")
	float OrthographicWidth = DEFAULT_ORTHOWIDTH;

	/** The aperture (f-stop) of the camera's lens. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float Aperture = 2.8f;

	/** The shutter speed of the camera's lens, in 1/seconds */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float ShutterSpeed = 60.f;

	/** The focus distance of the camera, if different from target distance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FocusDistance = -1.f;

		/** The width of the camera's sensor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float SensorWidth = 24.89f;

	/** The height of the camera's sensor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float SensorHeight = 18.67f;

	/** Horizontal offset of the sensor, in mm. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float SensorHorizontalOffset = 0.f;

	/** Vertical offset of the sensor, in mm. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float SensorVerticalOffset = 0.f;

	/** The camera sensor sensitivity in ISO. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float ISO = 100.f;

	/** The squeeze factor of the camera's lens. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float SqueezeFactor = 1.f;

	/** Sensor overscan in percentage */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float Overscan = 0.f;

	/** Number of blades in the lens diaphragm */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	int32 DiaphragmBladeCount = 8;

	/** The camera's near clipping plane. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float NearClippingPlane = 10.f;

	/** The camera's far clipping plane. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	float FarClippingPlane = -1.f;

	/** Internal weight for physical camera post-process settings. */
	UPROPERTY()
	float PhysicalCameraBlendWeight = 0.f;
	
	/** Whether to enable first-person parameters (FOV, scale) for primitives tagged as "IsFirstPerson" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera")
	bool EnableFirstPerson = false;

	/** 
	 * Whether to setup post-process settings based on physical camera properties such as Aperture,
	 * FocusDistance, DiaphragmBladeCount, and so on.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	bool EnablePhysicalCamera = false;

	/** Whether the camera should constrain aspect ratio. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	bool ConstrainAspectRatio = false;

	/** Whether to override the default aspect ratio axis constraint defined on the player controller */
	UPROPERTY()
	bool OverrideAspectRatioAxisConstraint = false;

	/** The aspect ratio axis constraint to use if aspect ratio is constrained. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Camera")
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV;

	/** Projection mode - Perspective or orthographic */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Camera")
	TEnumAsByte<ECameraProjectionMode::Type> ProjectionMode = ECameraProjectionMode::Perspective;

public:

	/** Creates a new BP camera pose from a raw camera pose structure. */
	static UE_API FBlueprintCameraPose FromCameraPose(const FCameraPose& InCameraPose);

	/** Applies this BP camera pose's values to the given raw camera pose. */
	UE_API void ApplyTo(FCameraPose& OutCameraPose) const;
};

/**
 * Utility Blueprint functions for camera poses.
 */
UCLASS()
class UBlueprintCameraPoseFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets the transform matrix of the camera pose. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FTransform GetTransform(const FBlueprintCameraPose& CameraPose);

	/** Gets the effective field of view of the camera pose, possibly computed from focal length. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static double GetEffectiveFieldOfView(const FBlueprintCameraPose& CameraPose);

	/** Gets the effective aspect ratio of the camera pose, computed from the sensor size. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static double GetSensorAspectRatio(const FBlueprintCameraPose& CameraPose);

	/** Gets the aim ray of the camera pose. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FRay GetAimRay(const FBlueprintCameraPose& CameraPose);

	/** Gets the facing direction of the camera pose. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FVector GetAimDir(const FBlueprintCameraPose& CameraPose);

	/** Gets the target of the camera pose. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FVector GetTarget(const FBlueprintCameraPose& CameraPose);

	/** Gets the target of the camera pose given a specific target distance. */
	UFUNCTION(BlueprintPure, Category="Camera")
	static FVector GetTargetAtDistance(const FBlueprintCameraPose& CameraPose, double TargetDistance);

public:

	/** Creates a copy of the given camera pose with the given location and rotation. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	static FBlueprintCameraPose SetTransform(const FBlueprintCameraPose& CameraPose, const FTransform& Transform);

public:

	/** Creates a new camera pose given a camera component. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	static FBlueprintCameraPose MakeCameraPoseFromCameraComponent(const UCameraComponent* CameraComponent);

	/** Creates a new camera pose given a cine-camera component. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	static FBlueprintCameraPose MakeCameraPoseFromCineCameraComponent(const UCineCameraComponent* CameraComponent);

public:

	UFUNCTION(BlueprintPure, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Break node"))
	static inline FVector GetLocation(const FBlueprintCameraPose& CameraPose) { return CameraPose.Location; }

	UFUNCTION(BlueprintPure, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Break node"))
	static inline FRotator GetRotation(const FBlueprintCameraPose& CameraPose) { return CameraPose.Rotation; }

	UFUNCTION(BlueprintPure, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Break node"))
	static inline double GetTargetDistance(const FBlueprintCameraPose& CameraPose) { return CameraPose.TargetDistance; }

	UFUNCTION(BlueprintPure, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Break node"))
	static inline double GetFieldOfView(const FBlueprintCameraPose& CameraPose) { return CameraPose.FieldOfView; }

	UFUNCTION(BlueprintPure, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Break node"))
	static inline double GetFocalLength(const FBlueprintCameraPose& CameraPose) { return CameraPose.FocalLength; }
	
	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Set Members node"))
	static FBlueprintCameraPose SetLocation(const FBlueprintCameraPose& CameraPose, const FVector& Location);

	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Set Members node"))
	static FBlueprintCameraPose SetRotation(const FBlueprintCameraPose& CameraPose, const FRotator& Rotation);

	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Set Members node"))
	static FBlueprintCameraPose SetTargetDistance(const FBlueprintCameraPose& CameraPose, double TargetDistance);

	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Set Members node"))
	static FBlueprintCameraPose SetFieldOfView(const FBlueprintCameraPose& CameraPose, float FieldOfView);

	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecationMessage="Please use the standard Set Members node"))
	static FBlueprintCameraPose SetFocalLength(const FBlueprintCameraPose& CameraPose, float FocalLength);
};

#undef UE_API
