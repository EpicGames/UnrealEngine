// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigParameterInterop.generated.h"

class UCameraRigAsset;
struct FBlueprintCameraEvaluationDataRef;

/**
 * Blueprint internal methods to set values on a camera rig's exposed parameters.
 *
 * These functions are internal because users are supposed to use the K2Node_SetCameraRigParameters node instead. That node then
 * gets compiled into one or more of these internal functions.
 */
UCLASS(MinimalAPI)
class UCameraRigParameterInterop : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UCameraRigParameterInterop(const FObjectInitializer& ObjectInit);

public:

	UFUNCTION(BlueprintPure, CustomThunk, Category="Camera", meta=(BlueprintInternalUseOnly="true", CustomStructureParam="ReturnValue"))
	static void GetCameraParameter(UPARAM(Ref) const FBlueprintCameraEvaluationDataRef& CameraData, UCameraRigAsset* CameraRig, FName ParameterName, int32& ReturnValue);

	UFUNCTION(BlueprintCallable, CustomThunk, Category="Camera", meta=(BlueprintInternalUseOnly="true", CustomStructureParam="NewValue"))
	static void SetCameraParameter(UPARAM(Ref) const FBlueprintCameraEvaluationDataRef& CameraData, UCameraRigAsset* CameraRig, FName ParameterName, UPARAM(Ref) const int32& NewValue);

private:

	DECLARE_FUNCTION(execGetCameraParameter);
	DECLARE_FUNCTION(execSetCameraParameter);
};

/**
 * Internal Blueprint function library for creating default values of the SetCameraParameter node above.
 */
UCLASS(MinimalAPI, Hidden)
class UCameraRigParameterInteropLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category="Camera", meta=(BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static FVector MakeLiteralVector(FVector Value);

	UFUNCTION(BlueprintPure, Category="Camera", meta=(BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static FVector3f MakeLiteralVector3f(FVector3f Value);

	UFUNCTION(BlueprintPure, Category="Camera", meta=(BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static FVector2D MakeLiteralVector2D(FVector2D Value);
	
	UFUNCTION(BlueprintPure, Category="Camera", meta=(BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static FRotator MakeLiteralRotator(FRotator Value);

	UFUNCTION(BlueprintPure, Category="Camera", meta=(BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static FLinearColor MakeLiteralLinearColor(FLinearColor Value);
};

