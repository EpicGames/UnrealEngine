// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"

#include "ControllerGameplayCameraEvaluationComponent.generated.h"

class IGameplayCameraSystemHost;
class UCameraRigAsset;
enum class ECameraRigLayer : uint8;

namespace UE::Cameras
{
	class FCameraEvaluationContext;
}

/**
 * A component, attached to a player controller, that can run camera rigs activated from
 * a global place like the Blueprint functions inside UActivateCameraRigFunctions.
 */
UCLASS(Hidden)
class UControllerGameplayCameraEvaluationComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UControllerGameplayCameraEvaluationComponent(const FObjectInitializer& ObjectInitializer);

	/** Initializes this component's evaluation context. */
	void Initialize(TScriptInterface<IGameplayCameraSystemHost> InCameraSystemHost, APlayerController* InPlayerController = nullptr);

	/** Activates a new camera rig. */
	void ActivateCameraRig(UCameraRigAsset* CameraRig, ECameraRigLayer EvaluationLayer);

public:

	static UControllerGameplayCameraEvaluationComponent* FindComponent(AActor* OwnerActor);
	static UControllerGameplayCameraEvaluationComponent* FindOrAddComponent(AActor* OwnerActor, bool* bOutCreated = nullptr);

public:

	// UActorComponent interface.
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

private:

	void ActivateCameraRigs();

private:

	struct FCameraRigInfo
	{
		TObjectPtr<UCameraRigAsset> CameraRig;
		ECameraRigLayer EvaluationLayer;
		bool bActivated = false;
	};

	TArray<FCameraRigInfo> CameraRigInfos;

	TSharedPtr<UE::Cameras::FCameraEvaluationContext> EvaluationContext;

	UPROPERTY()
	TScriptInterface<IGameplayCameraSystemHost> CameraSystemHost;
};

