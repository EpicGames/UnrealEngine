// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameplayCameraActorBase.h"

#include "GameplayCameraActor.generated.h"

class UCineCameraComponent;
class UGameplayCameraComponent;

/**
 * An actor that can run a camera asset.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, HideCategories=(OutputCamera))
class AGameplayCameraActor : public AGameplayCameraActorBase
{
	GENERATED_BODY()

public:

	GAMEPLAYCAMERAS_API AGameplayCameraActor(const FObjectInitializer& ObjectInit);

public:

	/** Gets the camera component. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UGameplayCameraComponent* GetCameraComponent() const { return CameraComponent; }

	/** Gets the output camera component. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UCineCameraComponent* GetOutputCameraComponent() const { return OutputCameraComponent; }

public:

	// AActor interface.
	GAMEPLAYCAMERAS_API virtual USceneComponent* GetDefaultAttachComponent() const override;

protected:

	// AGameplayCameraActorBase interface.
	GAMEPLAYCAMERAS_API virtual UGameplayCameraComponentBase* GetCameraComponentBase() const override;

private:

	UPROPERTY(VisibleAnywhere, Category=GameplayCamera, BlueprintGetter="GetCameraComponent", meta=(ExposeFunctionCategories="Camera"))
	TObjectPtr<UGameplayCameraComponent> CameraComponent;

	UPROPERTY(Transient, VisibleAnywhere, Category=OutputCamera, BlueprintGetter="GetOutputCameraComponent")
	TObjectPtr<UCineCameraComponent> OutputCameraComponent;
};

