// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"

#include "GameplayCameraActorBase.generated.h"

class UGameplayCameraComponentBase;
class UGameplayCameraSystemHost;

/**
 * A base class for actors that can run a camera object.
 *
 * When the actor becomes the view target, it is able to start updating itself by instantiating
 * a private instance of the camera system. It does this if no camera system was found attached
 * under the player controller.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera)
class AGameplayCameraActorBase : public AActor
{
	GENERATED_BODY()

public:

	GAMEPLAYCAMERAS_API AGameplayCameraActorBase(const FObjectInitializer& ObjectInit);

public:

	// AActor interface.
	GAMEPLAYCAMERAS_API virtual void CalcCamera(float DeltaTime, FMinimalViewInfo& OutResult) override;

protected:

	virtual UGameplayCameraComponentBase* GetCameraComponentBase() const { return nullptr; }
};

