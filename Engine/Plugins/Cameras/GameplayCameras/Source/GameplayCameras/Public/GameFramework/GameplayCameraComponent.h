// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraAssetReference.h"
#include "GameFramework/GameplayCameraComponentBase.h"

#include "GameplayCameraComponent.generated.h"

/**
 * A component that can run a camera asset inside its own camera evaluation context.
 */
UCLASS(Blueprintable, MinimalAPI, 
		ClassGroup=Camera, 
		meta=(BlueprintSpawnableComponent))
class UGameplayCameraComponent : public UGameplayCameraComponentBase
{
	GENERATED_BODY()

public:

	/** Create a new camera component. */
	GAMEPLAYCAMERAS_API UGameplayCameraComponent(const FObjectInitializer& ObjectInit);

public:

	// UActorComponent interface.
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	// UObject interface.
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	/** The camera asset to run. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera, meta=(InterpNotify="NotifyChangeCameraReference", SequencerHideProperty=true))
	FCameraAssetReference CameraReference;

protected:

	UFUNCTION()
	void NotifyChangeCameraReference();

	// UGameplayCameraComponentBase interface.
	virtual UCameraAsset* OnCreateEvaluationContext() override;
	virtual void OnUpdateEvaluationContext(bool bForceApplyParameterOverrides) override;

private:

	void ApplyChangedParameterOverrides();

#if WITH_EDITOR
	void OnCameraAssetBuilt(const UCameraAsset* InCameraAsset);
#endif

private:

	/** Cached parameter overrides, for detecting when some of them change. */
	UPROPERTY(Transient)
	FInstancedPropertyBag CachedParameterOverrides;


	// Deprecated

	UPROPERTY()
	TObjectPtr<UCameraAsset> Camera_DEPRECATED;
};

