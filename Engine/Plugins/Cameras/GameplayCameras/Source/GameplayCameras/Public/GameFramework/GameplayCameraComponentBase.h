// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Core/CameraAssetReference.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraRigInstanceID.h"
#include "Core/CameraShakeInstanceID.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/BlueprintCameraEvaluationDataRef.h"
#include "GameFramework/IGameplayCameraSystemHost.h"
#include "Services/CameraActionInstanceID.h"
#include "UObject/ObjectMacros.h"

#include "GameplayCameraComponentBase.generated.h"

class APlayerController;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class UCameraAction;
class UCameraAsset;
class UCameraShakeAsset;
class UCanvas;
class UCineCameraComponent;
struct FCameraContextDataTableAllocationInfo;
struct FCameraVariableTableAllocationInfo;

namespace UE::Cameras
{

class FCameraSystemEvaluator;
class FGameplayCameraComponentEvaluationContext;

}  // namespace UE::Cameras

/**
 * A component that can run a camera asset inside its own camera evaluation context.
 */
UCLASS(Blueprintable, MinimalAPI, Abstract, ClassGroup=Camera,
		HideCategories=(Object, ActorComponent, Physics, Rendering, Mobility, LOD),
		PrioritizeCategories=(Camera, EditorCamera, Activation),
		meta=(BlueprintSpawnableComponent))
class UGameplayCameraComponentBase 
	: public USceneComponent
	, public IGameplayCameraSystemHost
{
	GENERATED_BODY()

public:

	/** Create a new camera component. */
	GAMEPLAYCAMERAS_API UGameplayCameraComponentBase(const FObjectInitializer& ObjectInit);

public:

	/** Get the camera evaluation context used by this component. */
	GAMEPLAYCAMERAS_API TSharedPtr<const UE::Cameras::FCameraEvaluationContext> GetEvaluationContext() const;

	/** Get the camera evaluation context used by this component. */
	GAMEPLAYCAMERAS_API TSharedPtr<UE::Cameras::FCameraEvaluationContext> GetEvaluationContext();

	/** Returns whether an evaluation context exists in this component. */
	bool HasEvaluationContext() const { return EvaluationContext.IsValid(); }

public:

	/** Gets the child camera component used as the "output" for the gameplay/procedural camera. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UCineCameraComponent* GetOutputCameraComponent() const { return OutputCameraComponent; }

	/** 
	 * Activates the camera for the given player.
	 *
	 * @param PlayerIndex        The player to activate the camera for.
	 * @param bSetAsViewTarget   Whether to set this component's actor as the view target for the player.
	 */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void ActivateCameraForPlayerIndex(int32 PlayerIndex, bool bSetAsViewTarget = true);

	/** 
	 * Activates the camera for the given player.
	 *
	 * @param PlayerController   The player to activate the camera for.
	 * @param bSetAsViewTarget   Whether to set this component's actor as the view target for the player.
	 */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void ActivateCameraForPlayerController(APlayerController* PlayerController, bool bSetAsViewTarget = true);

	/** Gets the shared camera evaluation data for this component's evaluation context. */
	UFUNCTION(BlueprintPure, Category=Camera, meta=(DisplayName="Get Shared Camera Data"))
	GAMEPLAYCAMERAS_API FBlueprintCameraEvaluationDataRef GetInitialResult() const;

	/** Gets the camera evaluation data for a given sub-set of camera rigs in this component's evaluation context. */
	UFUNCTION(BlueprintPure, Category=Camera, meta=(DisplayName="Get Conditional Camera Data"))
	GAMEPLAYCAMERAS_API FBlueprintCameraEvaluationDataRef GetConditionalResult(ECameraEvaluationDataCondition Condition) const;

	/** Gets the last evaluated orientation of the camera. */
	UFUNCTION(BlueprintPure, Category=Camera)
	GAMEPLAYCAMERAS_API FRotator GetEvaluatedCameraRotation() const;

public:

	/** Activates the given camera rig prefab in the base layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraRigInstanceID ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraRigInstanceID ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraRigInstanceID ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRig);

	/** Deactivates a previously activated base, global, or visual camera rig. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API void DeactivateCameraRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

public:

	/** Starts a camera modifier rig on the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	/** Starts a camera modifier rig on the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	/** Stops a camera modifier rig on previously started on the global or visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API void StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

public:

	/** Starts a new camera shake. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraShakeInstanceID StartCameraShakeAsset(
			const UCameraShakeAsset* CameraShake,
			float ShakeScale = 1.f,
			ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal,
			FRotator UserPlaySpaceRotation = FRotator::ZeroRotator);

	/** Checks if a camera shake is running. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API bool IsCameraShakeAssetPlaying(FCameraShakeInstanceID InInstanceID) const;

	/** Stops a running camera shake. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API bool StopCameraShakeAsset(FCameraShakeInstanceID InInstanceID, bool bImmediately = false);

public:

	/** Starts the given camera action. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API FCameraActionInstanceID StartAction(const UCameraAction* CameraAction);

	/** Returns whether the given camera action instance is still running. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API bool IsActionRunning(FCameraActionInstanceID InInstanceID);

	/** Stops the given camera action instance. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API bool StopAction(FCameraActionInstanceID InInstanceID);

	/** Stops all camera actions of a given class. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	GAMEPLAYCAMERAS_API bool StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass);

public:

	// UActorComponent interface
	GAMEPLAYCAMERAS_API virtual void OnRegister() override;
	GAMEPLAYCAMERAS_API virtual void BeginPlay() override;
	GAMEPLAYCAMERAS_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	GAMEPLAYCAMERAS_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	GAMEPLAYCAMERAS_API virtual void Activate(bool bReset = false) override;
	GAMEPLAYCAMERAS_API virtual void Deactivate() override;
	GAMEPLAYCAMERAS_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
#if WITH_EDITOR
	GAMEPLAYCAMERAS_API virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
#endif 

	// USceneComponent interface.
	GAMEPLAYCAMERAS_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;

	// UObject interface.
	GAMEPLAYCAMERAS_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	GAMEPLAYCAMERAS_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	GAMEPLAYCAMERAS_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// IGameplayCameraSystemHost interface.
	virtual UObject* GetAsObject() override { return this; }

public:

	// Internal API

#if WITH_EDITOR
	GAMEPLAYCAMERAS_API void OnDrawVisualizationHUD(const FViewport* Viewport, const FSceneView* SceneView, FCanvas* Canvas) const;
#endif  // WITH_EDITOR

public:

	// Deprecated API
	
	UFUNCTION(BlueprintCallable, Category="Camera", meta=(DeprecatedFunction, DeprecatedMessage="Please use Deactivate"))
	void DeactivateCamera(bool bImmediately = false) { Deactivate(); }

protected:

	// UGameplayCameraComponentBase interface.
	virtual UCameraAsset* OnCreateEvaluationContext() { return nullptr; }
	virtual void OnUpdateEvaluationContext(bool bForceApplyParameterOverrides) {}

	void UpdateEvaluationContext(bool bForceApplyParameterOverrides);

	void GetChangedParameterOverrides(
			const FInstancedPropertyBag& InParameterOverrides,
			FInstancedPropertyBag& InOutCachedParameterOverrides,
			TArray<FGuid>& OutChangedParameterGuids);
	
	bool IsEditorWorld() const;
	void UpdateControlRotationIfNeeded();

#if WITH_EDITOR
	void ReinitializeEvaluationContext(
			const FCameraVariableTableAllocationInfo& VariableTableAllocationInfo,
			const FCameraContextDataTableAllocationInfo& ContextDataTableAllocationInfo);
	void RecreateEditorWorldEvaluationContext();

	void CreateCameraSpriteComponent(const FString& SpriteTexturePath);
#endif  // WITH_EDITOR

private:

	bool CanRunCameraSystem() const;
	bool EnsureCameraSystemHostIfNeeded();
	void DestroyCameraSystemHost();

	void EnsureEvaluationContext(APlayerController* PlayerController);
	void DestroyEvaluationContext();

	void EnsureOutputCameraComponent();
	void UpdateOutputCameraComponent();

#if WITH_EDITOR
	void AutoManageEditorPreviewEvaluator();
	void OnEditorPreviewCameraRigIndexChanged();
	void UpdateCameraSpriteComponent();
#endif  // WITH_EDITOR

public:

	/**
	 * The player controller to bind to by default.
	 *
	 * Auto-activation and explicit calls to Activate will use this player, if valid. Calls to 
	 * ActivateCameraForPlayerIndex and ActivateCameraForPlayerController will use their given player, 
	 * but fall back to this player if they are given an invalid parameter.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Activation)
	TEnumAsByte<EAutoReceiveInput::Type> DefaultPlayer = EAutoReceiveInput::Player0;

	/**
	 * When enabled, this camera component runs its own camera system when activated, which
	 * makes it possible for it to run independently. When disabled, this camera component
	 * only creates an evaluation context that must be run with an external camera system,
	 * such as the Gameplay Cameras Player Camera Manager.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	bool bRunStandaloneCameraSystem = true;

	/**
	 * Specifies whether this component should set the player controller's control rotation 
	 * to the computed point of view's orientation every frame. This is only used when a 
	 * player controller is associated with this component, and the view target is that
	 * component.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera, meta=(EditCondition="bRunStandaloneCameraSystem"))
	bool bSetControlRotationWhenViewTarget = false;

	/**
	 * Enables or disables playback mode, which turns of any standalone camera system evaluation
	 * and lets the output camera component be manipulated externally. This is useful when the
	 * evaluated camera was recorded (e.g. with Take Recorder) and turned into keyframed animation.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	bool bPlaybackMode = false;

#if WITH_EDITORONLY_DATA

	/** Whether to run this camera in editor. */
	UPROPERTY(EditAnywhere, Category=EditorCamera)
	bool bRunInEditor = true;

	/** The camera rig to run in the editor. */
	UPROPERTY(EditAnywhere, Category=EditorCamera, meta=(EditCondition="bRunInEditor"))
	int32 EditorPreviewCameraRigIndex = 0;

	/**
	 * When the output camera's location is within this distance of the root, hide the editor sprite.
	 * Set this value to zero or negative to never hide the editor sprite.
	 */
	UPROPERTY(EditAnywhere, Category=EditorCamera)
	float EditorSpriteHiddenWhenOutputCameraWithinDistance = 100.f;

#endif  // WITH_EDITORONLY_DATA

private:

	/**
	 * Camera component that receives the "output" of the hosted camera system when bRunStandaloneCameraSystem
	 * is enabled.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UCineCameraComponent> OutputCameraComponent;

	/** Whether the output camera component was auto-created, or obtained from the parent actor. */
	UPROPERTY()
	bool bIsOwnedOutputCameraComponent = false;

private:

	using FGameplayCameraComponentEvaluationContext = UE::Cameras::FGameplayCameraComponentEvaluationContext;
	using FCameraEvaluationContext = UE::Cameras::FCameraEvaluationContext;

	/** Evaluation context. */
	TSharedPtr<FGameplayCameraComponentEvaluationContext> EvaluationContext;

	/** Whether to force a camera cut next frame. */
	bool bIsCameraCutNextFrame = false;

#if WITH_EDITOR
	
	/** Whether this component is running in an editor world. */
	bool bIsEditorWorld = false;

	/** The show flag for camera system debug rendering. */
	int32 CustomShowFlag = INDEX_NONE;

#endif  // WITH_EDITOR
};

namespace UE::Cameras
{

/**
 * Evaluation context for the gameplay camera component.
 */
class FGameplayCameraComponentEvaluationContext : public FCameraEvaluationContext
{
	UE_DECLARE_CAMERA_EVALUATION_CONTEXT(GAMEPLAYCAMERAS_API, FGameplayCameraComponentEvaluationContext)

#if WITH_EDITOR

public:

	void UpdateForEditorPreview();

#endif  // WITH_EDITOR
};

}  // namespace UE::Cameras

