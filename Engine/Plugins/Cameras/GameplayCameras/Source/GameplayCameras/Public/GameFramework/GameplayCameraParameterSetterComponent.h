// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Core/CameraRigAssetReference.h"
#include "Core/CameraVariableSetter.h"

#include "GameplayCameraParameterSetterComponent.generated.h"

namespace UE::Cameras
{
	class FCameraParameterSetterService;
	struct FCameraVariableSetter;
}

UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, meta=(BlueprintSpawnableComponent))
class UGameplayCameraParameterSetterComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UGameplayCameraParameterSetterComponent(const FObjectInitializer& ObjInit);

public:

	/** The camera asset whose parameters to override. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	FCameraRigAssetReference CameraRigReference;

	/** The blend-in time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	float BlendInTime = 1.f;

	/** The blend-out time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	float BlendOutTime = 1.f;

	/** The blend type. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	ECameraVariableSetterBlendType BlendType = ECameraVariableSetterBlendType::Linear;

public:

	/** Start setting parameters. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	void StartParameterSetters();

	/**
	 * Stop setting parameters.
	 *
	 * @param bImmediately  If false, blend out the parameter overrides. If true, stop overrides immediately.
	 */
	UFUNCTION(BlueprintCallable, Category=Camera)
	void StopParameterSetters(bool bImmediately = false);

public:

	// UActorComponent interface.
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

private:

	UFUNCTION()
	void OnActorBeginOverlap(AActor* OverlappedActor, AActor* OtherActor);

	UFUNCTION()
	void OnActorEndOverlap(AActor* OverlappedActor, AActor* OtherActor);

private:

	TSharedPtr<UE::Cameras::FCameraParameterSetterService> GetParameterSetterService();

	void InitializeParameterSetter(UE::Cameras::FCameraVariableSetter& VariableSetter);

private:

	TArray<FCameraVariableSetterHandle> SetterHandles;
};

