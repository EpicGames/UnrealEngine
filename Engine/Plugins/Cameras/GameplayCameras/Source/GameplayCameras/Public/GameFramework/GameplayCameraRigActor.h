// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameplayCameraActorBase.h"

#include "GameplayCameraRigActor.generated.h"

class UCineCameraComponent;
class UGameplayCameraRigComponent;

/**
 * An actor that can run a camera asset.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, HideCategories=(OutputCamera))
class AGameplayCameraRigActor : public AGameplayCameraActorBase
{
	GENERATED_BODY()

public:

	AGameplayCameraRigActor(const FObjectInitializer& ObjectInit);

public:

	/** Gets the camera component. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UGameplayCameraRigComponent* GetCameraRigComponent() const { return CameraRigComponent; }

	/** Gets the output camera component. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UCineCameraComponent* GetOutputCameraComponent() const { return OutputCameraComponent; }

public:

	// AActor interface.
	virtual USceneComponent* GetDefaultAttachComponent() const override;

protected:

	// AGameplayCameraActorBase interface.
	virtual UGameplayCameraComponentBase* GetCameraComponentBase() const override;

private:

	UPROPERTY(VisibleAnywhere, Category=GameplayCamera, BlueprintGetter="GetCameraRigComponent", meta=(ExposeFunctionCategories="Camera"))
	TObjectPtr<UGameplayCameraRigComponent> CameraRigComponent;

	UPROPERTY(Transient, VisibleAnywhere, Category=OutputCamera, BlueprintGetter="GetOutputCameraComponent")
	TObjectPtr<UCineCameraComponent> OutputCameraComponent;
};

