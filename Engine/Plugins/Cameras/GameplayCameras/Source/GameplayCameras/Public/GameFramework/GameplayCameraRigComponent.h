// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigAssetReference.h"
#include "GameFramework/GameplayCameraComponentBase.h"

#include "GameplayCameraRigComponent.generated.h"

/**
 * A component that can run a camera rig asset inside its own camera evaluation context.
 */
UCLASS(Blueprintable, MinimalAPI, 
		ClassGroup=Camera, 
		meta=(BlueprintSpawnableComponent))
class UGameplayCameraRigComponent : public UGameplayCameraComponentBase
{
	GENERATED_BODY()

public:

	/** Create a new camera component. */
	GAMEPLAYCAMERAS_API UGameplayCameraRigComponent(const FObjectInitializer& ObjectInit);

public:

	// UActorComponent interface.
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	// UObject interface.
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	/** The camera rig asset to run. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera, meta=(InterpNotify="NotifyChangeCameraRigReference", SequencerHideProperty=true))
	FCameraRigAssetReference CameraRigReference;

protected:

	UFUNCTION()
	void NotifyChangeCameraRigReference();

	// UGameplayCameraComponentBase interface.
	virtual UCameraAsset* OnCreateEvaluationContext() override;
	virtual void OnUpdateEvaluationContext(bool bForceApplyParameterOverrides) override;

private:

	void BuildGeneratedCamera();
	void OnCameraRigAssetBuiltImpl();

	void ApplyChangedParameterOverrides();

#if WITH_EDITOR
	void OnCameraRigAssetBuilt(const UCameraRigAsset* InCameraRigAsset);
#endif

private:

	/** Generated camera asset to run our camera rig. */
	UPROPERTY(Transient)
	TObjectPtr<UCameraAsset> GeneratedCameraAsset;

	/** Cached parameter overrides, for detecting when some of them change. */
	UPROPERTY(Transient)
	FInstancedPropertyBag CachedParameterOverrides;

	bool bIsBuildingGeneratedCameraAsset = false;
};

