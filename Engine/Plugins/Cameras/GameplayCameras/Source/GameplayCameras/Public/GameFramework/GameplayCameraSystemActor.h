// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraPose.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "GameplayCameras.h"
#include "UObject/ObjectMacros.h"

#include "GameplayCameraSystemActor.generated.h"

class APlayerController;
class UGameplayCameraSystemComponent;

/**
 * An actor that hosts a camera system.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, HideCategories=(Input, Rendering))
class AGameplayCameraSystemActor : public AActor
{
	GENERATED_BODY()

public:

	AGameplayCameraSystemActor(const FObjectInitializer& ObjectInit);

public:

	/** Gets the camera system component. */
	UFUNCTION(BlueprintGetter, Category=Camera)
	UGameplayCameraSystemComponent* GetCameraSystemComponent() const { return CameraSystemComponent; }

public:

	// AActor interface.
	virtual void CalcCamera(float DeltaTime, struct FMinimalViewInfo& OutResult) override;

private:

	static AGameplayCameraSystemActor* GetAutoSpawnedCameraSystemActor(APlayerController* PlayerController, bool bSpawnIfMissing, bool* bOutSpawned);

private:

	UPROPERTY(VisibleAnywhere, Category=Camera, BlueprintGetter="GetCameraSystemComponent", meta=(ExposeFunctionCategories="CameraSystem"))
	TObjectPtr<UGameplayCameraSystemComponent> CameraSystemComponent;
};

