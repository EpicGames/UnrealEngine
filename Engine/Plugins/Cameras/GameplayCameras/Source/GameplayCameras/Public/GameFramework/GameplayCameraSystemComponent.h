// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "GameFramework/IGameplayCameraSystemHost.h"

#include "GameplayCameraSystemComponent.generated.h"

class APlayerController;
class UCameraRigAsset;
class UCanvas;
struct FCameraRigInstanceID;
struct FMinimalViewInfo;

namespace UE::Cameras
{
	class FCameraSystemEvaluator;
}

/**
 * A component that hosts a camera system.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, HideCategories=(Mobility, Rendering, LOD))
class UGameplayCameraSystemComponent 
	: public USceneComponent
	, public IGameplayCameraSystemHost
{
	GENERATED_BODY()

public:

	using FCameraSystemEvaluator = UE::Cameras::FCameraSystemEvaluator;

	UGameplayCameraSystemComponent(const FObjectInitializer& ObjectInit);

	/** Updates the camera system and returns the computed view. */
	GAMEPLAYCAMERAS_API void GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView);

	/** Sets this component's actor as the view target for the given player. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void ActivateCameraSystemForPlayerIndex(int32 PlayerIndex);

	/** Sets this component's actor as the view target for the given player. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void ActivateCameraSystemForPlayerController(APlayerController* PlayerController);

	/** Returns whether this component's actor is set as the view target for the given player. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API bool IsCameraSystemActiveForPlayController(APlayerController* PlayerController) const;

	/** Removes this component's actor from being the view target. */
	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void DeactivateCameraSystem(AActor* NextViewTarget = nullptr);

public:

	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	UFUNCTION(BlueprintCallable, Category=Camera)
	GAMEPLAYCAMERAS_API void StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

public:

	// UActorComponent interface.
	virtual void OnRegister() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	// IGameplayCameraSystemHost interface.
	virtual UObject* GetAsObject() { return this; }

private:

#if WITH_EDITOR
	void CreateCameraSystemSpriteComponent();
#endif  // WITH_EDITOR

public:

	/**
	 * If AutoActivate is set, auto-activates the camera system for the given player.
	 * This sets this actor as the view target, and is equivalent to calling ActivateCameraSystem on BeginPlay.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Activation, meta=(EditCondition="bAutoActivate"))
	TEnumAsByte<EAutoReceiveInput::Type> AutoActivateForPlayer;

	/**
	 * If enabled, sets the evaluated camera orientation as the player controller rotation every frame.
	 * This is set on the player controller that this component was activated for.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Camera)
	bool bSetPlayerControllerRotation = false;

private:

	UPROPERTY(Transient)
	TWeakObjectPtr<APlayerController> WeakPlayerController;

#if WITH_EDITORONLY_DATA

	/** Sprite scaling for the editor. */
	UPROPERTY(transient)
	float EditorSpriteTextureScale = 0.5f;

#endif	// WITH_EDITORONLY_DATA
};

