// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/PlayerCameraManager.h"
#include "Core/CameraEvaluationContext.h"
#include "Core/CameraShakeInstanceID.h"
#include "GameFramework/IGameplayCameraSystemHost.h"

#include "GameplayCamerasPlayerCameraManager.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UGameplayCameraComponentBase;
struct FCameraRigInstanceID;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FCameraSystemEvaluator;
class FViewTargetContextReferencerService;

}  // namespace UE::Cameras

/**
 * Defines how to activate a gameplay camera component.
 */
UENUM()
enum class EGameplayCameraComponentActivationMode : uint8
{
	/** Push the camera director over any existing ones. */
	Push,
	/** Inserts the camera director as a child of the active one, or push it if there is no active one. */
	InsertOrPush
};

/**
 * Defines how the GameplayCamerasPlayerCameraManager should handle the player's view rotation.
 */
UENUM()
enum class EGameplayCamerasViewRotationMode
{
	/**
	 * Don't do anything with the view rotation. This is suitable if the player camera input
	 * and control rotation are handled by custom logic, and the Gameplay Cameras only use
	 * the resulting control rotation (i.e. none of the camera rigs use player input nodes).
	 */
	None,
	/**
	 * Runs a "light" update of the whole camera system and comes up with a good approximation of
	 * the resulting camera orientation. Computationally more expensive than just computing a
	 * yaw and pitch angle, but handles blending camera rigs with different orientations while
	 * preserving aim.
	 */
	PreviewUpdate
};

/**
 * A player camera manager that runs the GameplayCameras camera system.
 *
 * Setting the view target does the following:
 * - Push a new evaluation context for the provided view target actor.
 *    - If that actor contains a GameplayCameraComponent, use its evaluation context directly.
 *    - If that actor contains a CameraComponent, make an evaluation context that wraps it
 *      and runs by simply copying that camera's properties (see FCameraActorCameraEvaluationContext).
 *    - For other actors, do as above, but convert the output of the actor's CalcCamera function.
 * - The old view target's evaluation context is immediately removed from the evaluation stack.
 *   For other way to handle evaluation contexts, call methods directly on the camera system
 *   evaluator instead of going through the base APlayerCameraManager class.
 *
 * There is only ever one active view target, the "pending" view target isn't used. This is
 * because we may be blending between more than two camera rigs that may belong to more than
 * two actors.
 */
UCLASS(notplaceable, MinimalAPI)
class AGameplayCamerasPlayerCameraManager 
	: public APlayerCameraManager
	, public IGameplayCameraSystemHost
{
	GENERATED_BODY()

public:

	UE_API AGameplayCamerasPlayerCameraManager(const FObjectInitializer& ObjectInitializer);

public:

	/** Replace the camera manager currently set on the provided controller with this camera manager. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	void StealPlayerController(APlayerController* PlayerController);

	/** Restore an originally stolen camera manager (see StealPlayerController). */
	UFUNCTION(BlueprintCallable, Category="Camera")
	void ReleasePlayerController();

public:

	/** Activates the given component inside the manager's camera system. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API void ActivateGameplayCamera(UGameplayCameraComponentBase* GameplayCamera, EGameplayCameraComponentActivationMode ActivationMode = EGameplayCameraComponentActivationMode::Push);

	/** Deactivates a previously activated component. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API void DeactivateGameplayCamera(UGameplayCameraComponentBase* GameplayCamera, bool bDeactivateAllCameraRigs = false);

public:

	/** Activates the given camera rig prefab in the base layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraRigInstanceID ActivatePersistentBaseCameraRig(UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraRigInstanceID ActivatePersistentGlobalCameraRig(UCameraRigAsset* CameraRig);

	/** Activates the given camera rig prefab in the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraRigInstanceID ActivatePersistentVisualCameraRig(UCameraRigAsset* CameraRig);

	/** Deactivates a previously activated base, global, or visual camera rig. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API void DeactivateCameraRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

public:

	/** Starts a camera modifier rig on the global layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraRigInstanceID StartGlobalCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	/** Starts a camera modifier rig on the visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraRigInstanceID StartVisualCameraModifierRig(const UCameraRigAsset* CameraRig, int32 OrderKey = 0);

	/** Stops a camera modifier rig on previously started on the global or visual layer. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API void StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

public:

	/** Starts a new camera shake. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API FCameraShakeInstanceID StartCameraShakeAsset(
			const UCameraShakeAsset* CameraShake,
			float ShakeScale = 1.f,
			ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal,
			FRotator UserPlaySpaceRotation = FRotator::ZeroRotator);

	/** Checks if a camera shake is running. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API bool IsCameraShakeAssetPlaying(FCameraShakeInstanceID InInstanceID) const;

	/** Stops a running camera shake. */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UE_API bool StopCameraShakeAsset(FCameraShakeInstanceID InInstanceID, bool bImmediately = false);

public:

	// APlayerCameraManager interface.
	UE_API virtual void InitializeFor(APlayerController* PlayerController) override;
	UE_API virtual void SetViewTarget(AActor* NewViewTarget, FViewTargetTransitionParams TransitionParams = FViewTargetTransitionParams()) override;
	UE_API virtual void ProcessViewRotation(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot) override;

	// AActor interface.
	UE_API virtual void DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;

	// UObject interface.
	UE_API virtual void BeginDestroy() override;
	UE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// IGameplayCameraSystemHost interface.
	virtual UObject* GetAsObject() override { return this; }

protected:

	// APlayerCameraManager interface.
	UE_API virtual void DoUpdateCamera(float DeltaTime) override;

private:

	void RunViewRotationPreviewUpdate(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot);

	void EnsureNullContext();

	void OnContextStackChanged();
	void CleanUpViewTargetContexts();

public:

	/** The view rotation handling mode to use. */
	UPROPERTY(EditAnywhere, Category="Camera", meta=(EditCondition="bOverrideViewRotationMode"))
	EGameplayCamerasViewRotationMode ViewRotationMode = EGameplayCamerasViewRotationMode::None;

	/** Whether the default view rotation mode setting should be overriden. */
	UPROPERTY(EditAnywhere, Category="Camera")
	bool bOverrideViewRotationMode = false;

private:

	UPROPERTY(Transient)
	TObjectPtr<APlayerCameraManager> OriginalCameraManager;

	TArray<TSharedRef<UE::Cameras::FCameraEvaluationContext>> ViewTargetContexts;

	TSharedPtr<UE::Cameras::FCameraEvaluationContext> NullContext;

	bool bIsSettingNewViewTarget = false;
};

#undef UE_API

