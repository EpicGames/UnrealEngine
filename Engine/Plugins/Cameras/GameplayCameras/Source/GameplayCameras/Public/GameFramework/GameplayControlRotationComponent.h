// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "GameplayControlRotationComponent.generated.h"

class IGameplayCameraSystemHost;
class APlayerController;
class UCanvas;
class UInputAction;

namespace UE::Cameras
{
	class FCameraSystemEvaluator; 
	class FPlayerControlRotationEvaluationService;
}

/**
 * An example component that works with the GameplayCameraComponent to manage a player's
 * control rotation when the camera changes or moves in a way that was not initiated 
 * by the player themselves.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup=Camera, HideCategories=(Mobility, Rendering, LOD), meta=(BlueprintSpawnableComponent))
class UGameplayControlRotationComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	/** Create a new control rotation component. */
	UGameplayControlRotationComponent(const FObjectInitializer& ObjectInit);

	/**
	 * Activates management of a player controller's control rotation. The component will set
	 * the control rotation every frame based on the latest camera system update.
	 */
	UFUNCTION(BlueprintCallable, Category="Control Rotation")
	void ActivateControlRotationManagementForPlayerIndex(int32 PlayerIndex);

	/**
	 * Activates management of a player controller's control rotation. The component will set
	 * the control rotation every frame based on the latest camera system update.
	 */
	UFUNCTION(BlueprintCallable, Category="Control Rotation")
	void ActivateControlRotationManagementForPlayerController(APlayerController* PlayerController);

	/**
	 * Deactivates management of a player controller's control rotation.
	 */
	UFUNCTION(BlueprintCallable, Category="Control Rotation")
	void DeactivateControlRotationManagement();

public:

	// UActorComponent interface
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

public:

	/** The axis input action(s) to read from. */
	UPROPERTY(EditAnywhere, Category="Input")
	TArray<TObjectPtr<UInputAction>> AxisActions;

	/**
	 * The angular speed, in degrees per second, past which a change in the player input
	 * will thaw a frozen control rotation.
	 */
	UPROPERTY(EditAnywhere, Category="Input")
	float AxisActionAngularSpeedThreshold = 20.f;

	/**
	 * The player input magnitude under which the frozen control rotation is thawed.
	 */
	UPROPERTY(EditAnywhere, Category="Input")
	float AxisActionMagnitudeThreshold = 0.1f;

	/**
	 * If AutoActivate is set, auto-activates control rotation management for the given player.
	 * This is equivalent to calling ActivateControlRotationManagement on BeginPlay.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Activation", meta=(EditCondition="bAutoActivate"))
	TEnumAsByte<EAutoReceiveInput::Type> AutoActivateForPlayer;

private:

	void InitializeControlRotationService(APlayerController* PlayerController);
	void TeardownControlRotationService(bool bAllowUninitialized);

private:

	UPROPERTY()
	TObjectPtr<APlayerController> PlayerController;

	UPROPERTY()
	TScriptInterface<IGameplayCameraSystemHost> CameraSystemHost;

	TSharedPtr<UE::Cameras::FPlayerControlRotationEvaluationService> ControlRotationService;
};

