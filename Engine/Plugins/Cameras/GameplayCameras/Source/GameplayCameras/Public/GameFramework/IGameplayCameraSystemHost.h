// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraTypes.h"
#include "Core/CameraRigInstanceID.h"
#include "Core/CameraShakeInstanceID.h"
#include "GameplayCameras.h"
#include "Services/CameraActionInstanceID.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/Interface.h"

#include "IGameplayCameraSystemHost.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraAction;
class UCameraRigAsset;
class UCameraShakeAsset;
class UCanvas;
enum class ECameraRigLayer : uint8;

namespace UE::Cameras
{
	class FCameraEvaluationContext;
	class FCameraSystemEvaluator;
	struct FCameraSystemEvaluatorCreateParams;
}

/**
 * An interface for objects that host a camera system evaluator.
 */
UINTERFACE(MinimalAPI)
class UGameplayCameraSystemHost : public UInterface
{
	GENERATED_BODY()
};

/**
 * An interface for objects that host a camera system evaluator.
 */
class IGameplayCameraSystemHost
{
	GENERATED_BODY()

public:

	using FCameraSystemEvaluator = UE::Cameras::FCameraSystemEvaluator;
	using FCameraEvaluationContext = UE::Cameras::FCameraEvaluationContext;

	/** Gets the camera system evaluator. */
	UE_API TSharedPtr<FCameraSystemEvaluator> GetCameraSystemEvaluator();

	/** Returns whether a camera system evaluation has been created in this host. */
	bool HasCameraSystem() const { return CameraSystemEvaluator.IsValid(); }

	/** Should be implemented by the underlying class to return itself as a UObject. */
	virtual UObject* GetAsObject() { return nullptr; }

	/** Returns this object as a script interface. */
	UE_API TScriptInterface<IGameplayCameraSystemHost> GetAsScriptInterface();

public:

	/** Finds a valid host on the player controller's camera manager, or its view target. */
	UE_API static IGameplayCameraSystemHost* FindActiveHost(APlayerController* PlayerController);

protected:

	/** Creates a new camera system. Asserts if there is already one. */
	UE_API void InitializeCameraSystem();
	/** Creates a new camera system. Asserts if there is already one. */
	UE_API void InitializeCameraSystem(const UE::Cameras::FCameraSystemEvaluatorCreateParams& Params);
	/** Ensures that the camera system is created. */
	UE_API void EnsureCameraSystemInitialized();
	/** Destroys the camera system. */
	UE_API void DestroyCameraSystem();

	/** Should be called by the underlying object for garbage collection. */
	UE_API void OnAddReferencedObjects(FReferenceCollector& Collector);

	/** Updates the camera system, if it exists. */
	UE_API void UpdateCameraSystem(float DeltaTime);

#if WITH_EDITOR
	/** Updates the camera system, if it exists, for an editor world preview. */
	UE_API void UpdateCameraSystemForEditorPreview(float DeltaTime);
#endif  // WITH_EDITOR

protected:

	/** Activates the given camera rig in the given layer. Should not be used with Main layer. */
	UE_API FCameraRigInstanceID ActivateCameraRig(
			const UCameraRigAsset* CameraRig,
			TSharedPtr<FCameraEvaluationContext> EvaluationContext,
			ECameraRigLayer EvaluationLayer);

	/** Deactivates a previously activated camera rig. */
	UE_API void DeactivateCameraRig(FCameraRigInstanceID InInstanceID, bool bImmediately = false);

	/** Starts a camera modifier rig on the given layer. */
	UE_API FCameraRigInstanceID StartCameraModifierRig(
			const UCameraRigAsset* CameraRig,
			TSharedPtr<FCameraEvaluationContext> EvaluationContext,
			ECameraRigLayer EvaluationLayer,
			int32 OrderKey = 0);

	/** Stops a camera modifier rig on previously started on the global or visual layer. */
	UE_API void StopCameraModifierRig(FCameraRigInstanceID InstanceID, bool bImmediately = false);

	/** Starts a new camera shake. */
	UE_API FCameraShakeInstanceID StartCameraShake(
			const UCameraShakeAsset* CameraShake,
			float ShakeScale = 1.f,
			ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal,
			FRotator UserPlaySpaceRotation = FRotator::ZeroRotator);

	/** Checks if a camera shake is running. */
	UE_API bool IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const;

	/** Stops a running camera shake. */
	UE_API bool StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately = false);

	/** Starts a new camera action. */
	FCameraActionInstanceID StartAction(const UCameraAction* CameraAction);

	/** Returns whether a given camera action instance is running. */
	bool IsActionRunning(const FCameraActionInstanceID InInstanceID) const;

	/** Stops a given camera action instance. */
	bool StopAction(const FCameraActionInstanceID InInstanceID);

	/** Stops all camera action instances of a given class. */
	bool StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass);

protected:

	/** The camera system evaluator. */
	TSharedPtr<FCameraSystemEvaluator> CameraSystemEvaluator;

private:

#if UE_GAMEPLAY_CAMERAS_DEBUG
	void DebugDraw(UCanvas* Canvas, APlayerController* PlayerController);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FDelegateHandle DebugDrawDelegateHandle;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

#undef UE_API

