// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EngineDefines.h"
#include "HAL/Platform.h"
#include "Stats/Stats.h"

#ifndef UE_GAMEPLAY_CAMERAS_EVENT_HANDLERS
	#define UE_GAMEPLAY_CAMERAS_EVENT_HANDLERS WITH_EDITOR
#endif

#ifndef UE_GAMEPLAY_CAMERAS_DEBUG
	#if UE_ENABLE_DEBUG_DRAWING
		#define UE_GAMEPLAY_CAMERAS_DEBUG 1
	#else
		#define UE_GAMEPLAY_CAMERAS_DEBUG 0
	#endif
#endif

#if UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING && !UE_BUILD_TEST
#define UE_GAMEPLAY_CAMERAS_TRACE 1
#else
#define UE_GAMEPLAY_CAMERAS_TRACE 0
#endif

DECLARE_STATS_GROUP(TEXT("Camera System Evaluation"), STATGROUP_CameraSystem, STATCAT_Advanced)

DECLARE_LOG_CATEGORY_EXTERN(LogCameraSystem, Log, All);

