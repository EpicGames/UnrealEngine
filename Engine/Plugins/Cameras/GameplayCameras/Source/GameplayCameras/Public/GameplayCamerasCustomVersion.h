// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

namespace FGameplayCamerasCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		AddLensNodeParameterFlags = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	const FGuid GUID(0x8BCB8CA9, 0x642B497C, 0xAB759854, 0x44875CE1);
}

