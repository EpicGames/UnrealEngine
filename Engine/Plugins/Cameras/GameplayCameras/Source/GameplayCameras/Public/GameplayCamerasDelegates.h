// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraAsset;
class UCameraNode;
class UCameraRigAsset;
class UCameraShakeAsset;

namespace UE::Cameras
{

class FCameraBuildLog;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraAssetBuilt, const UCameraAsset*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraRigAssetBuilt, const UCameraRigAsset*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraShakeAssetBuilt, const UCameraShakeAsset*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraNodeChanged, const UCameraNode*);

/**
 * Global delegates for the GameplayCameras module.
 */
class FGameplayCamerasDelegates
{
public:

	/** Broadcast when a camera asset has been built. */
	static inline FOnCameraAssetBuilt& OnCameraAssetBuilt()
	{
		return OnCameraAssetBuiltDelegate;
	}

	/** Broadcast when a camera rig has been built. */
	static inline FOnCameraRigAssetBuilt& OnCameraRigAssetBuilt()
	{
		return OnCameraRigAssetBuiltDelegate;
	}

	/** Broadcast when a camera shake has been built. */
	static inline FOnCameraShakeAssetBuilt& OnCameraShakeAssetBuilt()
	{
		return OnCameraShakeAssetBuiltDelegate;
	}

	/** Broadcast when a custom camera parameter provider node changes it parameters. */
	static inline FOnCameraNodeChanged& OnCustomCameraNodeParametersChanged()
	{
		return OnCustomCameraNodeParametersChangedDelegate;
	}

private:

	static UE_API FOnCameraAssetBuilt OnCameraAssetBuiltDelegate;
	static UE_API FOnCameraRigAssetBuilt OnCameraRigAssetBuiltDelegate;
	static UE_API FOnCameraShakeAssetBuilt OnCameraShakeAssetBuiltDelegate;
	static UE_API FOnCameraNodeChanged OnCustomCameraNodeParametersChangedDelegate;
};

}  // namespace UE::Cameras

#undef UE_API
