// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/DeveloperSettings.h"
#include "GameFramework/GameplayCamerasPlayerCameraManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "GameplayCamerasSettings.generated.h"

/**
 * The settings for the Gameplay Cameras runtime.
 */
UCLASS(Config=GameplayCameras, DefaultConfig, MinimalAPI, meta=(DisplayName="Gameplay Cameras"))
class UGameplayCamerasSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UGameplayCamerasSettings(const FObjectInitializer& ObjectInitializer);	

public:

	/**
	 * Build camera assets before using them in PIE, so that they automatically run with the
	 * latest user edits.
	 */
	UPROPERTY(EditAnywhere, Config, Category="General")
	bool bAutoBuildInPIE = true;

	/** 
	 * The default view rotation handling mode to use when the game's player controller uses a
	 * GameplayCamerasPlayerCameraManager instance as its camera manager.
	 */
	UPROPERTY(EditAnywhere, Config, Category="General")
	EGameplayCamerasViewRotationMode DefaultViewRotationMode = EGameplayCamerasViewRotationMode::None;

	/**
	 * The number of camera rigs combined in one frame past which the camera system emits a warning.
	 */
	UPROPERTY(EditAnywhere, Config, Category="General")
	int32 CombinedCameraRigNumThreshold = 10;

public:

	/**
	 * How many packages away to look for references to camera rigs and rig proxies from a camera director.
	 */
	UPROPERTY(EditAnywhere, Config, Category="Camera Director", meta=(ClampMin=1, ClampMax=5))
	int32 MaxUsageSearchDistance = 2;

public:

	/** The default angle tolerance to accept an aiming operation. */
	UPROPERTY(EditAnywhere, Config, Category="IK Aiming")
	double DefaultIKAimingAngleTolerance = 0.1;  // 0.1 degrees

	/** The default distance tolerance to accept an aiming operation. */
	UPROPERTY(EditAnywhere, Config, Category="IK Aiming")
	double DefaultIKAimingDistanceTolerance = 1.0;  // 1cm

	/** The default number of iterations for an aiming operation. */
	UPROPERTY(EditAnywhere, Config, Category="IK Aiming")
	uint8 DefaultIKAimingMaxIterations = 3;

	/** The distance below which any IK aiming operation is disabled. */
	UPROPERTY(EditAnywhere, Config, Category="IK Aiming")
	double DefaultIKAimingMinDistance = 100.0;  // 1m
};

