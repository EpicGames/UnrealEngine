// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Serialization/ArchiveUObject.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

/**
 * A utility class for finding references from a given package to a set of UObjects.
 */
class FObjectReferenceFinder : public FArchiveUObject
{
public:

	/** Creates a new instance of the reference finder. */
	UE_API FObjectReferenceFinder(UObject* InRootObject, TArrayView<UObject*> InReferencedObjects);

	/** Runs the reference finding. */
	UE_API void CollectReferences();

	/** Whether any object was found in the package that references any of the target objects. */
	UE_API bool HasAnyObjectReference() const;

	/** Returns the number of found references to the given target object. */
	UE_API int32 GetObjectReferenceCount(UObject* InObject) const;

protected:

	// FArchive interface.
	UE_API virtual FArchive& operator<<(UObject*& ObjRef) override;

private:

	void Initialize(UObject* InRootObject);

private:

	UObject* RootObject;
	UPackage* PackageScope;

	TSet<UObject*> TargetObjects;

	TArray<UObject*> ObjectsToVisit;
	TSet<UObject*> VisitedObjects;

	TMap<UObject*, int32> ObjectReferenceCounts;
};

}  // namespace UE::Cameras

#undef UE_API
