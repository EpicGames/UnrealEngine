// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{

/**
 * A utility class for finding outgoing references from a given package to external UObjects
 * of a given class.
 */
class FOutgoingReferenceFinder : public FArchiveUObject
{
public:

	/**
	 * The maximum distance (in number of packages) to search for references.
	 * Zero means only search the root object package.
	 * One means search directly referenced packages.
	 * Higher distances exponentially grow the number of objects to search, so the maximum
	 * value settable is 5.
	 */
	int32 MaxDistance = 1;

	/**
	 * EPackageFlags that determine what packages to skip.
	 */
	uint32 ExcludePackageFlags = 0;

	/**
	 * Classes to not inspect for outgoing references.
	 */
	TArray<UClass*> ExcludeClasses;

public:

	/** Creates a new instance of the reference finder. */
	UE_API FOutgoingReferenceFinder(UObject* InRootObject, UClass* InReferencedObjectClass);

	/** Creates a new instance of the reference finder. */
	UE_API FOutgoingReferenceFinder(UObject* InRootObject, TArrayView<UClass*> InReferencedObjectClasses);

	/** Runs the reference finding. */
	UE_API void CollectReferences();

	/** Gets the references of a given class. */
	template<typename ObjectClass>
	bool GetReferencesOfClass(TArray<ObjectClass*>& OutReferencedObjects) const;

	/** Gets all referenced objects. */
	UE_API bool GetAllReferences(TArray<UObject*>& OutReferencedObjects) const;

protected:

	// FArchive interface.
	UE_API virtual FArchive& operator<<(UObject*& ObjRef) override;

private:

	void Initialize(UObject* InRootObject);

	bool MatchesAnyTargetClass(UClass* InObjClass) const;
	bool IsIgnoredClass(UClass* InObjClass) const;

private:

	struct FObjectToVisit
	{
		UObject* Obj = nullptr;
		UPackage* Package = nullptr;
		int32 Distance = 0;
	};

	struct FSerializeState
	{
		UObject* CurrentObject = nullptr;
		UPackage* CurrentPackage = nullptr;
		int32 CurrentDistance = 0;
	};

	UObject* RootObject;
	UPackage* PackageScope;
	int32 ValidMaxDistance = 1;

	TSet<UClass*> TargetObjectClasses;

	TArray<FObjectToVisit> ObjectsToVisit;
	TSet<UObject*> VisitedObjects;
	FSerializeState SerializeState;

	TMap<UClass*, TSet<UObject*>> ReferencedObjects;
};

template<typename ObjectClass>
bool FOutgoingReferenceFinder::GetReferencesOfClass(TArray<ObjectClass*>& OutReferencedObjects) const
{
	bool bHadAny = false;
	UClass* ObjectClassClass = ObjectClass::StaticClass();
	for (const auto& Pair : ReferencedObjects)
	{
		if (Pair.Key->IsChildOf(ObjectClassClass))
		{
			bHadAny |= (Pair.Value.Num() > 0);
			for (UObject* Obj : Pair.Value)
			{
				OutReferencedObjects.Add(CastChecked<ObjectClass>(Obj));
			}
		}
	}
	return bHadAny;
}

}  // namespace UE::Cameras

#undef UE_API
