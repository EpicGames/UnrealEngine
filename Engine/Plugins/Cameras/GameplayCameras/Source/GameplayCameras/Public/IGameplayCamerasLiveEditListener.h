// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"

class UCameraNode;
class UPackage;
struct FPropertyChangedEvent;

namespace UE::Cameras
{

#if WITH_EDITOR

struct FGameplayCameraAssetBuildEvent
{
	const UPackage* AssetPackage = nullptr;
};

/**
 * Interface for an object that can handle an asset being hot-reloaded at runtime.
 */
class IGameplayCamerasLiveEditListener
{
public:

	virtual ~IGameplayCamerasLiveEditListener() = default;

	/** Called when a camera asset has been (re)built. */
	void PostBuildAsset(const FGameplayCameraAssetBuildEvent& BuildEvent) { OnPostBuildAsset(BuildEvent); }
	/** Called when a camera node's property has been changed. */
	void PostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) { OnPostEditChangeProperty(InCameraNode, PropertyChangedEvent); }

protected:

	/** Called when a camera asset has been (re)built. */
	virtual void OnPostBuildAsset(const FGameplayCameraAssetBuildEvent& BuildEvent) {}
	/** Called when a camera node's property has been changed. */
	virtual void OnPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) {}
};

#endif  // WITH_EDITOR

}  // namespace UE::Cameras

