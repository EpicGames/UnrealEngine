// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"

class UCameraNode;
class UPackage;
struct FPropertyChangedEvent;

namespace UE::Cameras
{

#if WITH_EDITOR

class IGameplayCamerasLiveEditListener;

/**
 * Interface for an object that can centralize the live-editing features of the camera system.
 */
class IGameplayCamerasLiveEditManager : public TSharedFromThis<IGameplayCamerasLiveEditManager>
{
public:

	virtual ~IGameplayCamerasLiveEditManager() = default;

	/** Whether cameras should be run in editor. */
	virtual bool CanRunInEditor() const = 0;

	/** Notify all listeners to reload cameras related to the given package. */
	virtual void NotifyPostBuildAsset(const UPackage* InAssetPackage) const = 0;

	/** Add a listener for the given package. */
	virtual void AddListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener) = 0;
	/** Remove a listener for the given package. */
	virtual void RemoveListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener) = 0;

	/** Notify all listeners that a property was changed on the given camera node. */
	virtual void NotifyPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) const = 0;

	/** Add a listener for the given node. */
	virtual void AddListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener) = 0;
	/** Remove a listener for the given node. */
	virtual void RemoveListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener) = 0;

	/** Remove the given listener from all notifications. */
	virtual void RemoveListener(IGameplayCamerasLiveEditListener* Listener) = 0;
};

#endif  // WITH_EDITOR

}  // namespace UE::Cameras

