// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableFwd.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::Cameras
{

#if WITH_EDITOR
class IGameplayCamerasLiveEditManager;
#endif

}  // namespace UE::Cameras

class IGameplayCamerasModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to ICameraModule
	 *
	 * @return The ICameraModule instance, loading the module on demand if needed
	 */
	GAMEPLAYCAMERAS_API static IGameplayCamerasModule& Get();

	/** Register a new blendable structure. */
	virtual void RegisterBlendableStruct(const UScriptStruct* StructType, UE::Cameras::FBlendableStructTypeErasedInterpolator Interpolator) = 0;
	/** Gets the currently registered blendable structures. */
	virtual TConstArrayView<UE::Cameras::FBlendableStructInfo> GetBlendableStructs() const = 0;
	/** Unregisters a blendable structure. */
	virtual void UnregisterBlendableStruct(const UScriptStruct* StructType) = 0;

#if WITH_EDITOR
	using IGameplayCamerasLiveEditManager = UE::Cameras::IGameplayCamerasLiveEditManager;

	/**
	 * Gets the live edit manager.
	 */
	virtual TSharedPtr<IGameplayCamerasLiveEditManager> GetLiveEditManager() const = 0;

	/**
	 * Sets the live edit manager.
	 */
	virtual void SetLiveEditManager(TSharedPtr<IGameplayCamerasLiveEditManager> InLiveEditManager) = 0;
#endif
};
