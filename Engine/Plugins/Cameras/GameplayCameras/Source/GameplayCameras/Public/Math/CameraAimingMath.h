// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

struct FCameraPose;

namespace UE::Cameras
{

class FCameraRigJoints;
struct FCameraNodeEvaluationResult;
struct FCameraRigJoint;

/** Result for an aiming operation. */
enum class ECameraAimingResult
{
	/** The aiming was successful. */
	Success,
	/** The aiming did not have a valid solution. */
	NoSolution,
	/** The aiming was not possible for the given camera rig. */
	NoSolver
};

/**
 * A utility class for math operations pertaining to aiming a camera at something.
 */
class FCameraAimingMath
{
public:

	/** Computes the rotation correction needed for the given camera to be aimed at the given target. */
	static ECameraAimingResult ComputeCorrection(const FCameraNodeEvaluationResult& Result, const FVector3d& DesiredTarget, FRotator3d& OutCorrection);

	/** Finds the pivot joint in the given evaluation result. */
	static const FCameraRigJoint* FindPivotJoint(const FCameraNodeEvaluationResult& Result);
	/** Finds the pivot joint in the given camera rig joints. */
	static const FCameraRigJoint* FindPivotJoint(const FCameraRigJoints& CameraRigJoints);

	/** Computes the rotation correction needed for the given camera to be aimed at the given target.*/
	static bool ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection);

	/**
	 * As per the other ComputeTwoBonesCorrection function, but take into account an additional angular offset for the
	 * camera's line of sight. These angles should be expressed in the pivot's coordinate system.
	 */
	static bool ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FRotator3d& AimOffsetAngles, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection);
	static bool ComputeTwoBonesCorrection(const FCameraPose& CurrentPose, const FVector3d& CustomAim, const FVector3d& PivotLocation, const FVector3d& DesiredTarget, FRotator3d& OutCorrection);

	/** Computes the difference between two aim vectors. */
	static double GetAngleBetween(const FVector3d& AimA, const FVector3d& AimB);

	/** Computes the rotation between two aim vectors. */
	static FRotator3d GetRotationBetween(const FVector3d& AimA, const FVector3d& AimB);

	/** Computes the rotation between two aim vectors, using only yaw and pitch. */
	static FRotator3d GetNoRollRotationBetween(const FVector3d& AimA, const FVector3d& AimB);

private:

	static bool RaySphereIntersectExit(const FRay3d& Ray, const FVector3d& SphereOrigin, double SphereRadius, double& OutRayIntersectDistance);
	static bool RaySphereIntersectExit(const FVector3d& RayStart, const FVector3d& RayDir, const FVector3d& SphereOrigin, double SphereRadius, double& OutRayIntersectDistance);

};

}  // namespace UE::Cameras

