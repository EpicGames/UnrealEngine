// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/UnrealMath.h"

#define UE_API GAMEPLAYCAMERAS_API

class FArchive;
struct FCameraFramingZone;
struct FCameraPose;

namespace UE::Cameras
{

struct FCameraFieldsOfView;

/**
 * Effective coordinates for a rectangular screen-space zone.
 * Unlike FCameraFramingZone, whose values can mean anything (margins, offsets, etc)
 * this struct is expected to store actual screen coordinates in 0..1 UI space.
 */
struct FFramingZone
{
	double LeftBound = 0;
	double TopBound = 0;
	double RightBound = 0;
	double BottomBound = 0;

	/** Builds an empty framing zone. */
	UE_API FFramingZone();

	/** Gets the width of the framing zone. */
	double Width() const { return FMath::Max(0.0, RightBound - LeftBound); }

	/** Gets the height of the framing zone. */
	double Height() const { return FMath::Max(0.0, BottomBound - TopBound); }

	/** Gets the center of the framing zone. */
	FVector2d Center() const { return FVector2d((LeftBound + RightBound) / 2.0, (TopBound + BottomBound) / 2.0); }

	/** 
	 * Returns whether this framing zone is well-formed (bounds in 0..1 UI space, 
	 * with left/top bounds lesser than or equal to right/bottom bounds).
	 */
	UE_API bool IsValid() const;

	/** Makes sure all the bounds have valid values between 0 and 1. */
	UE_API void ClampBounds();

	/**
	 * Makes sure all the bounds have valid values between 0 and 1, and that the
	 * enclosed rectangle contains the given target point.
	 */
	UE_API void ClampBounds(const FVector2d& MustContain);

	/**
	 * Makes sure all the bounds have valid values between 0 and 1, and that the
	 * enclosed rectangle contains the given target point, with a uniform margin.
	 */
	UE_API void ClampBounds(const FVector2d& MustContain, double Margin);

	/**
	 * Makes sure all the bounds have valid values between 0 and 1, and that the
	 * enclosed rectangle contains the given inner rectangle.
	 */
	UE_API void ClampBounds(const FFramingZone& MustContain);

	/**
	 * Expands this framing zone to include the other framing zone.
	 */
	UE_API void Add(const FFramingZone& Other);

	/** Checks whether the given point (in 0..1 UI space) is inside this zone. */
	UE_API bool Contains(const FVector2d& Point) const;

	/** 
	 * Computes intersections between a line and the zone's box, and returns the one closest
	 * to the line's origin point.
	 */
	UE_API FVector2d ComputeClosestIntersection(const FVector2d& Origin, const FVector2d& LineDir, bool bLineDirIsNormalized = false) const;

	/** Gets the inner margins of this zone compared to the screen's center. */
	UE_API FVector4d GetNormalizedBounds() const;

	/** Gets the coordinates of the top-left corner of the zone, in 0..Width/Height canvas units. */
	UE_API FVector2d GetCanvasPosition(const FVector2d& CanvasSize) const;
	/** Gets the size of the zone, in 0..Width/Height canvas units. */
	UE_API FVector2d GetCanvasSize(const FVector2d& CanvasSize) const;

	UE_API void Serialize(FArchive& Ar);

public:

	/** Build a framing zone from a set of margins relative to the screen edge. */
	UE_API static FFramingZone FromScreenMargins(const FCameraFramingZone& Margins);

	/** Build a framing zone from a set of margins relative to a screen location. */
	UE_API static FFramingZone FromRelativeMargins(const FVector2d& ScreenLocation, const FCameraFramingZone& Margins);

	/** Build a framing zone that encompasses all the given points (in 0..1 UI space). */
	UE_API static FFramingZone FromPoints(TConstArrayView<FVector2d> ScreenPoints);

private:

	static double GetNormalizedBound(double Bound);
};

FArchive& operator <<(FArchive& Ar, FFramingZone& FramingZone);

/**
 * The half-angles (in radians) of a rectangular screen framing zone, relative to the
 * camera pose's aim direction.
 */
struct FFramingZoneAngles
{
	double LeftHalfAngle = 0;
	double TopHalfAngle = 0;
	double RightHalfAngle = 0;
	double BottomHalfAngle = 0;
};

}  // namespace UE::Cameras

#undef UE_API

