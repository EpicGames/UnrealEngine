// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/CameraNodeTypes.h"

#include "Math/Transform.h"

namespace UE::Cameras
{

class FCameraEvaluationContext;
struct FCameraNodeEvaluationParams;
struct FCameraNodeEvaluationResult;

/**
 * A structure that can wrap node evaluation parameters and results, and spit out information needed
 * to compute positions and offsets in various evaluation coordinate systems.
 */
struct FCameraNodeSpaceParams
{
	/** Creates a new FCameraNodeSpaceParams. */
	FCameraNodeSpaceParams(const FCameraNodeEvaluationParams& InEvaluationParams, const FCameraNodeEvaluationResult& InEvaluationResult);

	/** Gets the active context on the root node's main stack. */
	TSharedPtr<const FCameraEvaluationContext> GetActiveContext() const;
	/** Gets the context owning the caller camera node. */
	TSharedPtr<const FCameraEvaluationContext> GetOwningContext() const;
	/** Finds a pivot in the evaluation result. */
	bool FindPivotTransform(FTransform3d& OutPivotTransform) const;

public:

	const FCameraNodeEvaluationParams& EvaluationParams;
	const FCameraNodeEvaluationResult& EvaluationResult;
};

class FCameraNodeSpaceMath
{
public:

	/** Gets the location of the specified camera node origin. */
	static bool GetCameraNodeOriginPosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, ECameraNodeOriginPosition InOriginPosition, FVector3d& OutOrigin);

	/** Gets the location of the specified camera node origin. */
	static bool GetCameraNodeOriginPosition(const FCameraNodeSpaceParams& Params, ECameraNodeOriginPosition InOriginPosition, FVector3d& OutOrigin);

	/** Gets the transform of the specified camera node space. */
	static bool GetCameraNodeSpaceTransform(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, ECameraNodeSpace InSpace, FTransform3d& OutTransform);

	/** Gets the transform of the specified camera node space. */
	static bool GetCameraNodeSpaceTransform(const FCameraNodeSpaceParams& Params, ECameraNodeSpace InSpace, FTransform3d& OutTransform);

	/** Offsets a given world position in the specified camera node space. */
	static bool OffsetCameraNodeSpacePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& Result, const FVector3d& InPosition, const FVector3d& InOffset, ECameraNodeSpace InSpace, FVector3d& OutPosition);

	/** Offsets a given world position in the specified camera node space. */
	static bool OffsetCameraNodeSpacePosition(const FCameraNodeSpaceParams& Params, const FVector3d& InPosition, const FVector3d& InOffset, ECameraNodeSpace InSpace, FVector3d& OutPosition);

	/** Offsets a given world transform in the specified camera node space. */
	static bool OffsetCameraNodeSpaceTransform(const FCameraNodeSpaceParams& Params, const FTransform3d& InTransform, const FVector3d& InLocationOffset, const FRotator3d& InRotationOffset, ECameraNodeSpace InSpace, FTransform3d& OutTransform);
};

}  // namespace UE::Cameras

