// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"

#define UE_API GAMEPLAYCAMERAS_API

class FArchive;
struct FCameraFieldsOfView;
struct FCameraPose;
struct FSceneViewProjectionData;
enum EAspectRatioAxisConstraint : int;

namespace UE::Cameras
{

class FCameraEvaluationContext;

/**
 * Simple struct for holding horizontal and vertical fields of view.
 */
struct FCameraFieldsOfView
{
	double HorizontalFieldOfView;
	double VerticalFieldOfView;
};
FArchive& operator<< (FArchive& Ar, FCameraFieldsOfView& FieldsOfView);

/**
 * A utility class for mathematical functions related to a camera pose.
 */
class FCameraPoseMath
{
public:

	/** Gets both horizontal and vertical effective fields of view, using the sensor aspect ratio. */
	UE_API static FCameraFieldsOfView GetEffectiveFieldsOfView(const FCameraPose& CameraPose);

	/** Gets both horizontal and vertical effective fields of view, using the given player controller's viewport aspect ratio. */
	UE_API static FCameraFieldsOfView GetEffectiveFieldsOfView(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	/** Gets both horizontal and vertical effective fields of view, using the given aspect ratio. */
	UE_API static FCameraFieldsOfView GetEffectiveFieldsOfView(const FCameraPose& CameraPose, double ActualAspectRatio, EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint);

	/** Gets the aspect ratio of the viewport associated with the given player controller. */
	UE_API static double GetEffectiveAspectRatio(const FCameraPose& CameraPose, TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	/** Gets the default aspect ratio axis constraint associated with the given player controller. */
	UE_API static EAspectRatioAxisConstraint GetDefaultAspectRatioAxisConstraint(TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	/**
	 * Builds the projection matrices of the given camera pose.
	 */
	UE_API static void BuildProjectionData(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			FSceneViewProjectionData& OutProjectionData);
	/**
	 * Builds the projection matrices of the given camera pose.
	 */
	UE_API static void BuildProjectionData(
			const FCameraPose& CameraPose,
			const FIntPoint& ViewportSize,
			EAspectRatioAxisConstraint DefaultAspectRatioAxisConstraint,
			FSceneViewProjectionData& OutProjectionData);
	/** 
	 * Builds the projection matrix of the given camera pose. 
	 * This matrix is suitable for projecting camera-space points onto screen-space.
	 */
	UE_API static FMatrix BuildProjectionMatrix(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext);
	/** 
	 * Builds the view-projection matrix of the given camera pose.
	 * This matrix combines the camera transform and the projection matrix, making it
	 * suitable for projecting world-space points onto screen-space.
	 */
	UE_API static FMatrix BuildViewProjectionMatrix(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext);
	/**
	 * Builds a view-projection matrix using an identity view, so that using for unprojecting 
	 * yields camera-local positions.
	 */
	UE_API static FMatrix BuildLocalViewProjectionMatrix(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext);

	/** Projects the given world point onto screen-space. */
	UE_API static TOptional<FVector2d> ProjectWorldToScreen(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector3d& WorldLocation, bool bForceLocationInsideFrustum = false);
	/** Projects the given camera-local point onto screen-space. */
	UE_API static TOptional<FVector2d> ProjectCameraToScreen(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector3d& CameraSpaceLocation, bool bForceLocationInsideFrustum = false);

	/**
	 * Projects the given point onto screen-space.
	 * The caller is responsible for making sure the view-projection matrix and coordinate system 
	 * for the given point are compatible.
	 */
	UE_API static TOptional<FVector2d> ProjectToScreen(
			const FMatrix& ViewProjectionMatrix, const FVector3d& Location, bool bForceLocationInsideFrustum = false);

	/** Unproject a screen-space point into a camera-local ray. */
	UE_API static FRay3d UnprojectScreenToCamera(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector2D& ScreenSpacePoint);
	/** Unproject a screen-space point into a camera-local point given an expected distance. */
	UE_API static FVector3d UnprojectScreenToCamera(
			const FCameraPose& CameraPose, 
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector2D& ScreenSpacePoint,
			double PredictedDistance);
	/** Unproject a screen-space point into a world-space ray. */
	UE_API static FRay3d UnprojectScreenToWorld(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector2D& ScreenSpacePoint);
	/** Unproject a screen-space point into a world-space point given an expected distance. */
	UE_API static FVector3d UnprojectScreenToWorld(
			const FCameraPose& CameraPose,
			TSharedPtr<const FCameraEvaluationContext> EvaluationContext,
			const FVector2D& ScreenSpacePoint,
			double PredictedDistance);

	/** Unprojects the given screen-space point into a ray. */
	UE_API static FRay3d UnprojectFromScreen(
			const FMatrix& InverseViewProjectionMatrix, const FVector2D& ScreenSpacePoint);
	/** Unprojects the given screen-space point into a point given an expected distance. */
	UE_API static FVector3d UnprojectFromScreen(
			const FMatrix& InverseViewProjectionMatrix, const FVector2D& ScreenSpacePoint, double PredictedDistance);

private:

	static FMatrix InverseProjectionMatrix(const FMatrix& ProjectionMatrix);
};

}  // namespace UE::Cameras

#undef UE_API

