// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

namespace UE::Cameras
{

/**
 * A value that can be "consumed" and applied to another value.
 * The consumable value can be a "delta", which is added to another value,
 * or an "absolute", which is set on another value.
 */
template<typename ValueType>
struct TConsumableValue
{
public:

	/** Returns an absolute consumable with the given value. */
	static TConsumableValue<ValueType> Absolute(ValueType InValue)
	{
		return TConsumableValue<ValueType>(InValue, false);
	}

	/** Returns a delta consumable with the given value. */
	static TConsumableValue<ValueType> Delta(ValueType InValue)
	{
		return TConsumableValue<ValueType>(InValue, true);
	}

	/** Creates a new consumable with no value. */
	TConsumableValue()
		: bHasValue(false)
	{}

	/** Creates a new consumable. */
	TConsumableValue(ValueType InValue, bool bInIsDelta)
		: Value(InValue)
		, bHasValue(true)
		, bIsDelta(bInIsDelta)
	{}

public:

	/** Gets the current value. Asserts if there is no value. */
	ValueType Get() const
	{
		check(bHasValue);
		return Value;
	}

	/** Gets the current value, or the default one if there's no value. */
	ValueType GetOrDefault(ValueType DefaultValue) const
	{
		return bHasValue ? Value : DefaultValue;
	}

	/** Returns whether there's any value left in the consumable. */
	bool HasValue() const
	{
		return bHasValue;
	}

	/** Returns whether the consumable is a delta. */
	bool IsDelta() const
	{
		return bIsDelta;
	}

public:

	/**
	 * Applies the consumable to the given value and returns the new value.
	 *
	 * This is meant to be used as follows:
	 *
	 *   MyFooBar = Consumable.Apply(MyFooBar);
	 *
	 * After this, the consumable doesn't have any value left in it anymore,
	 * so HasValue() will return false and further calls to Apply() will just
	 * return the same value as the given parameter.
	 */
	ValueType Apply(const ValueType InTarget)
	{
		if (bHasValue)
		{
			if (bIsDelta)
			{
				ValueType Result = InTarget + Value;
				Value = ValueType();
				bHasValue = false;
				return Result;
			}
			else
			{
				ValueType Result = Value;
				Value = ValueType();
				bHasValue = false;
				return Result;
			}
		}
		return InTarget;
	}

	/**
	 * Applies the consumable to the given value and returns the new value. However,
	 * check that the return value falls within the given min/max bounds. If not, only
	 * consume enough to reach those bounds and leave the rest to be consumed later.
	 *
	 * This is meant to be used as follows:
	 *
	 *   MyFooBar = Consumable.Apply(MyFooBar, MyMin, MyMax);
	 *
	 * After this, the consumable may or may not have any value left in it, depending
	 * on whether the given min/max bounds were reached.
	 */
	ValueType Apply(const ValueType InTarget, const ValueType InMinTarget, const ValueType InMaxTarget)
	{
		check(InMinTarget <= InMaxTarget);
		if (bHasValue)
		{
			if (bIsDelta)
			{
				ValueType Result = InTarget + Value;
				if (Result < InMinTarget)
				{
					ValueType ActuallyConsumable = InMinTarget - InTarget;
					Value -= ActuallyConsumable;
					return InMinTarget;
				}
				else if (Result > InMaxTarget)
				{
					ValueType ActuallyConsumable = InMaxTarget - InTarget;
					Value -= ActuallyConsumable;
					return InMaxTarget;
				}
				else
				{
					Value = ValueType();
					bHasValue = false;
					return Result;
				}
			}
			else
			{
				if (Value < InMinTarget)
				{
					ValueType Remaining = Value - InMinTarget;
					Value = Remaining;
					bIsDelta = true;
					return InMinTarget;
				}
				else if (Value > InMaxTarget)
				{
					ValueType Remaining = Value - InMaxTarget;
					Value = Remaining;
					bIsDelta = true;
					return InMaxTarget;
				}
				else
				{
					ValueType Result = Value;
					Value = ValueType();
					bHasValue = false;
					return Result;
				}
			}
		}
		return InTarget;
	}

private:

	ValueType Value = ValueType();
	bool bHasValue = false;
	bool bIsDelta = false;
};

using FConsumableFloat = TConsumableValue<float>;
using FConsumableDouble = TConsumableValue<double>;

}  // namespace UE::Cameras

