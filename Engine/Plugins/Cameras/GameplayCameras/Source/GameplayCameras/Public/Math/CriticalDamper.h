// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FArchive;

namespace UE::Cameras
{

/**
 * Implements a critically damped spring-mass system.
 * See https://en.wikipedia.org/wiki/Damping
 *
 * Important notes:
 *
 * - A critically damped spring-mass system converges as fast as possible
 *   towards its state of equilibrium without overshooting and/or oscillating.
 *   That's the theory, though. In practice, probably because of floating 
 *   point precision, the system can slightly overshoot, so we clamp the result
 *   when it's just about to "settle".
 *   See the `GameplayCameras.CriticalDamper.StabilizationThreshold` cvar for
 *   tweaking how that happens.
 *
 * - In this implementation, the state of equilibrium is 0 (zero). That is, the
 *   target that the system is trying to reach is 0. In most practical cases, 
 *   however, we want to converge towards an arbitrary value, which can even
 *   move if the system is being "dragged around". The different overloads of
 *   the `Update` method are there for these different use-cases. Don't mix and
 *   match calls, though: it's recommended to stick to one of these overloads.
 *
 * - See the math notes in the corresponding cpp file for details about the
 *   algorithm and its implementation.
 */
struct FCriticalDamper
{
public:

	FCriticalDamper();
	FCriticalDamper(float InW0);

	/**
	 * Resets the initial conditions (aka the previous frame's state) of the
	 * spring-mass system.
	 */
	void Reset(float InX0, float InX0Derivative);

	/**
	 * Gets the damping factor.
	 * This is the undamped angular frequency of the oscillator system.
	 * It's technically expressed in seconds.
	 */
	float GetW0() const { return W0; }
	/**
	 * Sets the damping factor.
	 * This is the undamped angular frequency of the oscillator system.
	 * It's technically expressed in seconds.
	 */
	void SetW0(float InW0) { W0 = InW0; }

	/** Gets the last position of the spring-mass system. */
	float GetX0() const { return X0; }
	/** Gets the last velocity of the spring-mass system. */
	float GetX0Derivative() const { return X0Derivative; }

	/**
	 * Updates the system to make it converge towards 0.
	 *
	 * @param DeltaTime  The elapsed time.
	 */
	float Update(float DeltaTime);
	/**
	 * Updates the system, given a possibly forced position the system was moved to.
	 * The forced position should be last frame's result (also available with GetX0())
	 * if the target wasn't moved.
	 * The system will converge towards 0.
	 *
	 * @param X          The forced position of the system.
	 * @param DeltaTime  The elapsed time.
	 * @return           The new position of the system.
	 */
	float Update(float X, float DeltaTime);
	/** 
	 * Updates the system for damping towards arbitrary non-zero values.
	 * This call takes last frame's target and this frame's target. The difference 
	 * between the two indicates by how much the target was forcibly moved, if at all.
	 * The result (also available with GetX0()) is returned relative to the current
	 * frame's target (i.e. the second parameter).
	 *
	 * @param PreviousDamped  The previous value returned by this function.
	 * @param NextTarget      The new target for the system.
	 * @param DeltaTime       The elapsed time.
	 * @return			      The new damped value.
	 */
	float Update(float PreviousDamped, float NextTarget, float DeltaTime);

	/** Saves or loads the state of this critical damper. */
	void Serialize(FArchive& Ar);

private:

	void InternalUpdate(float ForcedMovement, float DeltaTime);

private:

	/** Angular frequency of the oscillator. */
	float W0;
	/** Initial position of the spring-mass system, aka the previous frame's position. */
	float X0;
	/** Initial velocity of the spring-mass system, aka the previous frame's velocity. */
	float X0Derivative;

	friend FArchive& operator <<(FArchive& Ar, FCriticalDamper& Damper);
};

FArchive& operator <<(FArchive& Ar, FCriticalDamper& Damper);

}  // namespace UE::Cameras

