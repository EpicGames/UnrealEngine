// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Cameras
{

/**
 * Smoothstep interpolation
 * 
 * @param Value  A value between 0 and 1
 */
inline float SmoothStep(float Value)
{
	return (Value * Value * (3.f - 2.f * Value));
}

/**
 * Smootherstep interpolation
 *
 * @param Value  A value between 0 and 1
 */
inline float SmootherStep(float Value)
{
	return (Value * Value * Value * (Value * (Value * 6.f - 15.f) + 10.f));
}

}  // namespace UE::Cameras

