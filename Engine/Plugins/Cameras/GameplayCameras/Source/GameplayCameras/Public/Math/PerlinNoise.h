// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "PerlinNoise.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class FArchive;

USTRUCT()
struct FPerlinNoiseData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	float Amplitude = 1.f;

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	float Frequency = 1.f;
};

namespace UE::Cameras
{

/**
 * Implements a 1-dimensional perlin noise generator.
 *
 * Unlike the FMath::PerlinNoise1D method, this one doesn't use a hard-coded lookup
 * table of pre-generated numbers. Since camera noise doesn't need to generate
 * numbers for thousands or millions of pixels at a time (only a handful every frame),
 * we can afford to get slightly better noise quality here.
 *
 * Other differences include:
 *
 *   - Multi-octave noise.
 *   - The ability to dynamically change the frequency.
 *   - Input is a delta-time, instead of an absolute undefined value.
 */
struct FPerlinNoise
{
public:

	UE_API FPerlinNoise();
	UE_API FPerlinNoise(const FPerlinNoiseData& InData, uint8 InOctaves = 1);
	UE_API FPerlinNoise(float InAmplitude, float InFrequency, uint8 InOctaves = 1);

	float GetAmplitude() const { return Amplitude; }
	void SetAmplitude(float InAmplitude) { Amplitude = InAmplitude; }

	float GetFrequency() const { return Octaves[0].Frequency; }
	UE_API void SetFrequency(float InFrequency);

	float GetLacunarity() const { return Lacunarity; }
	void SetLacunarity(float InLacunarity) { Lacunarity = InLacunarity; }

	float GetOctaveGain() const { return OctaveGain; }
	void SetOctaveGain(float InOctaveGain) { OctaveGain = InOctaveGain; }

	uint8 GetNumOctaves() const { return NumOctaves; }
	UE_API void SetNumOctaves(uint8 InNumOctaves);

	UE_API float GenerateValue(float DeltaTime);

	UE_API void Serialize(FArchive& Ar);

	friend FArchive& operator<< (FArchive& Ar, FPerlinNoise& This)
	{
		This.Serialize(Ar);
		return Ar;
	}

private:

	UE_API void Initialize(float InFrequency);
	
	static FVector2f ComputeTangent(float InPrev, float InNext, float Interval, float Tension);

private:

	static const uint8 MAX_OCTAVES = 4;

	struct FSinglePerlinNoise
	{
		float Frequency = 1.f;
		float CurTime = 0.f;
		float SecondPrev = 0.f;
		float Prev = 0.f;
		float Next = 0.f;
		float SecondNext = 0.f;
	};

	float Amplitude = 1.f;

	float Lacunarity = 2.f;
	float OctaveGain = 0.5f;

	uint8 NumOctaves = 1;

	FSinglePerlinNoise Octaves[MAX_OCTAVES];
};

}  // namespace UE::Cameras

#undef UE_API
