// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneDoubleChannel.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "Nodes/Framing/CameraFramingZone.h"

#include "MovieSceneCameraFramingZoneSection.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneCameraFramingZoneSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	FMovieSceneDoubleChannel LeftMarginCurve;

	UPROPERTY()
	FMovieSceneDoubleChannel TopMarginCurve;

	UPROPERTY()
	FMovieSceneDoubleChannel RightMarginCurve;

	UPROPERTY()
	FMovieSceneDoubleChannel BottomMarginCurve;

private:

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
};

