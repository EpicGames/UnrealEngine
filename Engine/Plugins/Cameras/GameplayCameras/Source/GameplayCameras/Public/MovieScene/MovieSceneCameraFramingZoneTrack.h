// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieScenePropertyTrack.h"

#include "MovieSceneCameraFramingZoneTrack.generated.h"

UCLASS( MinimalAPI )
class UMovieSceneCameraFramingZoneTrack : public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:

	UMovieSceneCameraFramingZoneTrack(const FObjectInitializer& Init);
	
	// UMovieSceneTrack interface.
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
};

