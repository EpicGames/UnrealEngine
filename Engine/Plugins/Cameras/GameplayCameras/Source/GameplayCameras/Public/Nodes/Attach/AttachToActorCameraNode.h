// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/Attach/CameraActorAttachmentInfo.h"

#include "AttachToActorCameraNode.generated.h"

/**
 * A camera node that moves the camera to an actor.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Attachment"))
class UAttachToActorCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The actor to attach to. */
	UPROPERTY(EditAnywhere, Category=Common, meta=(CameraContextData=true))
	FCameraActorAttachmentInfo Attachment;

	/** The data ID for the attachment info. */
	UPROPERTY()
	FCameraContextDataID AttachmentDataID;

	/** Whether to move the camera to the actor's location. */
	UPROPERTY(EditAnywhere, Category=Common)
	FBooleanCameraParameter AttachToLocation = true;

	/** Whether to align the camera rotation to the actor's orientation. */
	UPROPERTY(EditAnywhere, Category=Common)
	FBooleanCameraParameter AttachToRotation = false;
};

