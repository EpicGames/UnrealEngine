// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraNode.h"
#include "Nodes/Attach/CameraActorAttachmentInfo.h"

#include "AttachToActorGroupCameraNode.generated.h"

/**
 * A camera node that moves the camera to the weighted center of a group of actors.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Attachment"))
class UAttachToActorGroupCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The actors to attach to. */
	UPROPERTY(EditAnywhere, Category=Common, meta=(CameraContextData=true))
	TArray<FCameraActorAttachmentInfo> Attachments;

	UPROPERTY()
	FCameraContextDataID AttachmentsDataID;
};

