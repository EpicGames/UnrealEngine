// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "AttachToPlayerPawnCameraNode.generated.h"

/**
 * A camera node that moves the camera to the player pawn.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Attachment"))
class UAttachToPlayerPawnCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Whether to move the camera to the pawn's location. */
	UPROPERTY(EditAnywhere, Category=Common)
	FBooleanCameraParameter AttachToLocation = true;

	/** Whether to align the camera rotation to the pawn's orientation. */
	UPROPERTY(EditAnywhere, Category=Common)
	FBooleanCameraParameter AttachToRotation = false;

	/** An optional socket to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category=Common)
	FName SocketName;

	/** An optional bone to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category=Common)
	FName BoneName;
};

