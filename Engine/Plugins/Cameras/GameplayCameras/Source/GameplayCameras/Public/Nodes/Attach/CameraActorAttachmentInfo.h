// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Core/CameraContextDataTableFwd.h"

#include "CameraActorAttachmentInfo.generated.h"

class AActor;

/**
 * Attachment information for a camera rig.
 */
USTRUCT(BlueprintType)
struct FCameraActorAttachmentInfo
{
	GENERATED_BODY()

	/** The actor to attach to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attachment")
	TObjectPtr<AActor> Actor;

	/** An optional socket to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Attachment")
	FName SocketName;

	/** An optional bone to attach to on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	FName BoneName;

	/** The weight of this attachment. Unused if only one attachment is used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Attachment")
	float Weight = 1.f;

	bool operator== (const FCameraActorAttachmentInfo& Other) const = default;
};

namespace UE::Cameras
{

/** A special reader class for attachment information. */
struct FCameraActorAttachmentInfoReader
{
	FCameraActorAttachmentInfoReader() {}
	FCameraActorAttachmentInfoReader(const FCameraActorAttachmentInfo& InAttachmentInfo, FCameraContextDataID InDataID);

	void Initialize(const FCameraActorAttachmentInfo& InAttachmentInfo, FCameraContextDataID InDataID);

	bool GetAttachmentTransform(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FString RenderAttachmentInfo() const;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void CacheAttachmentInfo(const FCameraActorAttachmentInfo& InAttachmentInfo);

private:

	FCameraActorAttachmentInfo DefaultAttachmentInfo;
	FCameraContextDataID DataID;

	FCameraActorAttachmentInfo CachedAttachmentInfo;
	const USkeletalMeshComponent* CachedSkeletalMeshComponent = nullptr;
	FName CachedBoneName;

	friend struct FCameraActorAttachmentInfoArrayReader;
};

/** A special reader class for multiple attachment information. */
struct FCameraActorAttachmentInfoArrayReader
{
	FCameraActorAttachmentInfoArrayReader() {}
	FCameraActorAttachmentInfoArrayReader(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos, FCameraContextDataID InDataID);

	void Initialize(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos, FCameraContextDataID InDataID);

	bool GetAttachmentTransform(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform);

private:

	void CacheAttachmentInfos(TConstArrayView<FCameraActorAttachmentInfo> InAttachmentInfos);

private:

	TArray<FCameraActorAttachmentInfoReader> Readers;
	FCameraContextDataID DataID;
};

}  // namespace UE::Cameras

