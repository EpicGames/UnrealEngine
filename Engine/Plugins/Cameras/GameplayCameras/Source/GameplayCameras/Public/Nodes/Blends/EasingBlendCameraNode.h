// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "EasingBlendCameraNode.generated.h"

/**
 * The easing type.
 *
 * These types are copied from, and behave in the same way as the corresponding easing
 * types from Sequencer.
 */
UENUM(BlueprintType)
enum class EEasingCameraBlendType : uint8
{
	// Linear easing
	Linear UMETA(Grouping=Linear, DisplayName="Linear"),
	// Sinusoidal easing
	SinIn UMETA(Grouping=Sinusoidal, DisplayName="Sinusoidal In"),
	SinOut UMETA(Grouping=Sinusoidal, DisplayName="Sinusoidal Out"),
	SinInOut UMETA(Grouping=Sinusoidal, DisplayName="Sinusoidal InOut"),
	// Quadratic easing
	QuadIn UMETA(Grouping=Quadratic, DisplayName="Quadratic In"),
	QuadOut UMETA(Grouping=Quadratic, DisplayName="Quadratic Out"),
	QuadInOut UMETA(Grouping=Quadratic, DisplayName="Quadratic InOut"),
	// Cubic easing
	Cubic UMETA(Grouping=Cubic, DisplayName="Cubic"),
	CubicIn UMETA(Grouping=Cubic, DisplayName="Cubic In"),
	CubicOut UMETA(Grouping=Cubic, DisplayName="Cubic Out"),
	CubicInOut UMETA(Grouping=Cubic, DisplayName="Cubic InOut"),
	HermiteCubicInOut UMETA(Grouping=Cubic, DisplayName="Hermite-Cubic InOut"),
	// Quartic easing
	QuartIn UMETA(Grouping=Quartic, DisplayName="Quartic In"),
	QuartOut UMETA(Grouping=Quartic, DisplayName="Quartic Out"),
	QuartInOut UMETA(Grouping=Quartic, DisplayName="Quartic InOut"),
	// Quintic easing
	QuintIn UMETA(Grouping=Quintic, DisplayName="Quintic In"),
	QuintOut UMETA(Grouping=Quintic, DisplayName="Quintic Out"),
	QuintInOut UMETA(Grouping=Quintic, DisplayName="Quintic InOut"),
	// Exponential easing
	ExpoIn UMETA(Grouping=Exponential, DisplayName="Exponential In"),
	ExpoOut UMETA(Grouping=Exponential, DisplayName="Exponential Out"),
	ExpoInOut UMETA(Grouping=Exponential, DisplayName="Exponential InOut"),
	// Circular easing
	CircIn UMETA(Grouping=Circular, DisplayName="Circular In"),
	CircOut UMETA(Grouping=Circular, DisplayName="Circular Out"),
	CircInOut UMETA(Grouping=Circular, DisplayName="Circular InOut"),
};

/**
 * A blend camera node that implements basic easing curve types.
 */
UCLASS(MinimalAPI)
class UEasingBlendCameraNode : public USimpleFixedTimeBlendCameraNode
{
	GENERATED_BODY()

public:

	void SetCameraBlendType(EEasingCameraBlendType BlendTypeIn) { BlendType = BlendTypeIn; }

protected:

	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
	
public:

	/** The type of curve to use. */
	UPROPERTY(EditAnywhere, Category=Blending, meta=(CameraContextData=true))
	EEasingCameraBlendType BlendType = EEasingCameraBlendType::Linear;

	UPROPERTY()
	FCameraContextDataID BlendTypeDataID;
};

