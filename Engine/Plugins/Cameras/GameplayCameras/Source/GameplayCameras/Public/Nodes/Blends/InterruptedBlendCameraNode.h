// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"

namespace UE::Cameras
{

/**
 * A special blend meant to "freeze" an ongoing blend, and then start running another blend.
 * This is used for instance by the Camera Modifier Service to blend out a camera rig instance that had not
 * fully blended in yet.
 *
 * NOTE: this blend doesn't have any data, and can't be added by the user in the editor. It is meant to be
 * created programmatically only.
 */
class FInterruptedBlendCameraNodeEvaluator : public FBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FInterruptedBlendCameraNodeEvaluator)

public:

	FInterruptedBlendCameraNodeEvaluator();
	FInterruptedBlendCameraNodeEvaluator(FBlendCameraNodeEvaluator* InChildBlend, FBlendCameraNodeEvaluator* InFrozenBlend);

protected:

	// FBlendCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) override;
	virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) override;

public:

	FBlendCameraNodeEvaluator* ChildBlend = nullptr;
	FBlendCameraNodeEvaluator* FrozenBlend = nullptr;

	FCameraNodeEvaluationResult FrozenResult;
};

}  // namespace UE::Cameras

