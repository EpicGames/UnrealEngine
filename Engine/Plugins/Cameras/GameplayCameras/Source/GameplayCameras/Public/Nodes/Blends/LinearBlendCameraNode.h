// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "LinearBlendCameraNode.generated.h"

/**
 * Linear blend node.
 */
UCLASS(MinimalAPI)
class ULinearBlendCameraNode : public USimpleFixedTimeBlendCameraNode
{
	GENERATED_BODY()

protected:

	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

