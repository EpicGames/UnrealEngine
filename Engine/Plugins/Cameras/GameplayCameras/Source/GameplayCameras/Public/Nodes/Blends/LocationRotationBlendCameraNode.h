// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "LocationRotationBlendCameraNode.generated.h"

/**
 * A blend node that blends translation and rotation differently.
 */
UCLASS(MinimalAPI)
class ULocationRotationBlendCameraNode : public UBlendCameraNode
{
	GENERATED_BODY()

public:

	ULocationRotationBlendCameraNode(const FObjectInitializer& ObjInit);

public:

	/** The blend to use for the camera's translation. */
	UPROPERTY()
	TObjectPtr<USimpleBlendCameraNode> LocationBlend;

	/** The blend to use for the camera's rotation. */
	UPROPERTY()
	TObjectPtr<USimpleBlendCameraNode> RotationBlend;

	/** The blend to use for everything else. */
	UPROPERTY()
	TObjectPtr<USimpleBlendCameraNode> OtherBlend;

protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

