// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "OrbitBlendCameraNode.generated.h"

/**
 * Orbit blend node.
 */
UCLASS(MinimalAPI)
class UOrbitBlendCameraNode : public UBlendCameraNode
{
	GENERATED_BODY()

public:

	UOrbitBlendCameraNode(const FObjectInitializer& ObjInit);

	void SetDrivingBlend(USimpleBlendCameraNode* DrivingBlendIn) { DrivingBlend = DrivingBlendIn; }

public:

	UPROPERTY()
	TObjectPtr<USimpleBlendCameraNode> DrivingBlend;

protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

