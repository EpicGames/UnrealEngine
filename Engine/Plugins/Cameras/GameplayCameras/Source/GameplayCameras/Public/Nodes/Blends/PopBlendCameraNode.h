// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"

#include "PopBlendCameraNode.generated.h"

namespace UE::Cameras
{

struct FCameraNodeBlendParams;
struct FCameraNodeBlendResult;
struct FCameraNodePreBlendParams;
struct FCameraNodePreBlendResult;

/**
 * Utility function for cutting evaluation results without blending.
 */
struct FPopBlendCameraNodeHelper
{
	static void PopParameters(const UE::Cameras::FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult);
	static void PopResults(const UE::Cameras::FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult);
};

}  // namespace UE::Cameras

/**
 * A blend node that creates a camera cut (i.e. it doesn't blend at all).
 */
UCLASS(MinimalAPI)
class UPopBlendCameraNode : public UBlendCameraNode
{
	GENERATED_BODY()

protected:

	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

