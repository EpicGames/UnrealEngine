// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BlendCameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraParameterReader.h"
#include "SimpleBlendCameraNode.generated.h"

/**
 * Base class for a blend camera node that uses a simple scalar factor.
 */
UCLASS(MinimalAPI, Abstract)
class USimpleBlendCameraNode : public UBlendCameraNode
{
	GENERATED_BODY()
};

/**
 * Base class for a blend camera node that uses a simple scalar factor over a fixed time.
 */
UCLASS(MinimalAPI, Abstract)
class USimpleFixedTimeBlendCameraNode : public USimpleBlendCameraNode
{
	GENERATED_BODY()

public:
	/** Returns the volume of this element */
	UE_DEPRECATED(5.7, "Now sets the default time, use SetDefaultBlendTime instead")
	void SetBlendTime(float BlendTimeIn) { BlendTime.Value = BlendTimeIn; }

	void SetDefaultBlendTime(float BlendTimeIn) { BlendTime.Value = BlendTimeIn; }

	float GetDefaultBlendTime() { return BlendTime.Value; }
public:

	/** Duration of the blend. */
	UPROPERTY(EditAnywhere, Category=Blending)
	FFloatCameraParameter BlendTime = 1.f;
};

namespace UE::Cameras
{

/**
 * Result structure for defining a simple scalar-factor-based blend.
 */
struct FSimpleBlendCameraNodeEvaluationResult
{
	float BlendFactor = 0.f;
};

class FSimpleBlendCameraNodeEvaluator : public FBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FSimpleBlendCameraNodeEvaluator)

public:

	FSimpleBlendCameraNodeEvaluator();

	/** Gets the last evaluated blend factor. */
	float GetBlendFactor() const { return BlendFactor; }

	/** Gets whether the blend is currently at 100%. */
	bool IsBlendFull() const { return BlendFactor >= 1.f; }

	/** 
	 * Gets whether the blend was flagged as finished.
	 * A simple blend typically finishes when it reaches 100%, but in rare cases it may
	 * want to continue running after that, possibly going back below 100% before going
	 * back up.
	 */
	bool IsBlendFinished() const { return bIsBlendFinished; }

protected:

	// FBlendCameraNodeEvaluator interface.
	GAMEPLAYCAMERAS_API virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	GAMEPLAYCAMERAS_API virtual void OnBlendParameters(const FCameraNodePreBlendParams& Params, FCameraNodePreBlendResult& OutResult) override;
	GAMEPLAYCAMERAS_API virtual void OnBlendResults(const FCameraNodeBlendParams& Params, FCameraNodeBlendResult& OutResult) override;
	GAMEPLAYCAMERAS_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
	GAMEPLAYCAMERAS_API virtual bool OnSetReversed(bool bInReverse) override;

	virtual void OnComputeBlendFactor(const FCameraNodeEvaluationParams& Params, FSimpleBlendCameraNodeEvaluationResult& OutResult) {}

	void SetBlendFinished() { bIsBlendFinished = true; }

#if UE_GAMEPLAY_CAMERAS_DEBUG
	GAMEPLAYCAMERAS_API virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	TCameraParameterReader<float> BlendTimeReader;

private:

	float BlendFactor = 0.f;
	bool bIsBlendFinished = false;
	bool bReverse = false;
};

class FSimpleFixedTimeBlendCameraNodeEvaluator : public FSimpleBlendCameraNodeEvaluator
{
	UE_DECLARE_BLEND_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FSimpleFixedTimeBlendCameraNodeEvaluator, FSimpleBlendCameraNodeEvaluator)

protected:

	// FBlendCameraNodeEvaluator interface.
	GAMEPLAYCAMERAS_API virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	GAMEPLAYCAMERAS_API virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	GAMEPLAYCAMERAS_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;
	GAMEPLAYCAMERAS_API virtual bool OnInitializeFromInterruption(const FCameraNodeBlendInterruptionParams& Params) override;

	float GetTimeFactor() const;

private:

	float TotalTime = 0.f;
	float CurrentTime = 0.f;
};

}  // namespace UE::Cameras

