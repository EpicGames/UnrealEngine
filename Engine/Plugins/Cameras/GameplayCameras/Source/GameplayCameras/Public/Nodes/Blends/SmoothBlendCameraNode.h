// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include "SmoothBlendCameraNode.generated.h"

/**
 * The smoothstep type.
 */
UENUM()
enum class ESmoothCameraBlendType
{
	SmoothStep,
	SmootherStep
};

/**
 * A blend camera node that implements the smoothstep and smoothersteps algorithms.
 */
UCLASS(MinimalAPI)
class USmoothBlendCameraNode : public USimpleFixedTimeBlendCameraNode
{
	GENERATED_BODY()

public:

	void SetCameraBlendType(ESmoothCameraBlendType BlendTypeIn) { BlendType = BlendTypeIn; }

protected:

	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
	
public:

	/** The type of algorithm to use. */
	UPROPERTY(EditAnywhere, Category=Blending)
	ESmoothCameraBlendType BlendType;
};

