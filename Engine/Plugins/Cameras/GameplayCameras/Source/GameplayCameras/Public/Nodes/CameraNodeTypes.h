// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "CameraNodeTypes.generated.h"

/**
 * Defines an origin position for a camera node to operate at.
 */
UENUM()
enum class ECameraNodeOriginPosition : uint8
{
	/** The position of the current camera pose. */
	CameraPose,
	/** The origin of the active evaluation context on the main layer's blend stack. */
	ActiveContext,
	/** The origin of the evaluation context of the current camera node. */
	OwningContext,
	/** The location of the current pivot. If no pivot is found, fallback to ActiveContext. */
	Pivot,
	/** The location of the player's controlled pawn. */
	Pawn
};

/**
 * Defines what space a camera node, or one of its features, should operate in.
 */
UENUM()
enum class ECameraNodeSpace : uint8
{
	/** The local space of the current camera pose. */
	CameraPose,
	/** The space of the active evaluation context on the main layer's blend stack. */
	ActiveContext,
	/** The space of the evaluation context of the current camera node. */
	OwningContext,
	/** The space of the current pivot. If no pivot is found, fallback to ActiveContext. */
	Pivot,
	/** The space of the player's controlled pawn. */
	Pawn,
	/** The space of the world in which the camera rig evaluates. */
	World
};

