// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraValueInterpolator.h"
#include "Core/CameraVariableReferences.h"
#include "Core/CameraVariableReferenceReader.h"
#include "Engine/EngineTypes.h"
#include "WorldCollision.h"

#include "CollisionPushCameraNode.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

/**
 * Specifies how to compute the default safe position for the collision camera node
 * to push towards.
 */
UENUM()
enum class ECollisionSafePosition : uint8
{
	/**
	 * The initial result location of the active evaluation context on the main 
	 * layer's blend stack.
	 */
	ActiveContext,
	/**
	 * The initial result location of the evaluation context of the collision camera node.
	 */
	OwningContext,
	/**
	 * The current pivot. If no pivot is found, fallback to ActiveContext.
	 */
	Pivot,
	/**
	 * The location of the player's controlled pawn.
	 */
	Pawn
};

/**
 * Describes the coordinate system in which to offset the collision camera node's
 * safe position.
 */
UENUM()
enum class ECollisionSafePositionOffsetSpace : uint8
{
	/** The space of the active evaluation context on the main layer's blend stack. */
	ActiveContext,
	/** The space of the evaluation context of the collision camera node. */
	OwningContext,
	/** The space of the current pivot. If no pivot is found, fallback to ActiveContext. */
	Pivot,
	/** The local space of the current camera pose. */
	CameraPose,
	/** The space of the player's controlled pawn. */
	Pawn
};

namespace UE::Cameras
{

/**
 * The collision push node evaluator class.
 */
class FCollisionPushCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(UE_API, FCollisionPushCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	UE_API virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	UE_API virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	UE_API virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

protected:

	struct FCollisionTraceParams
	{
		UWorld* World = nullptr;
		APawn* Pawn = nullptr;
		FVector3d SafePosition = FVector3d::ZeroVector;
		FVector3d TraceStart = FVector3d::ZeroVector;
		FVector3d TraceEnd = FVector3d::ZeroVector;
		float CollisionSphereRadius = 1.f;
		ECollisionChannel CollisionChannel = ECollisionChannel::ECC_Camera;
		bool bRequestedAsyncCollision = false;
	};

	struct FCollisionTraceResult
	{
		TArray<FHitResult> HitResults;
		FTraceHandle AsyncTraceHandle;
	};

	/**
	 * Run the collision trace using the given parameters.
	 *
	 * @param Params  The parameters for the current node evaluation
	 * @param TraceParams  The parameters for the collision trace
	 * @param OutResult   The result for the current node evaluation
	 * @param OutTraceResult  The result with either the handle to the asynchronous trace, if bRequestedAsyncCollision 
	 *						  was true and it was possible to honor it, or the list of hit results to be processed 
	 *						  synchronously
	 */
	UE_API virtual void RunCollisionTrace(const FCameraNodeEvaluationParams& Params, const FCollisionTraceParams& TraceParams, FCameraNodeEvaluationResult& OutResult, FCollisionTraceResult& OutTraceResult);

private:

	TOptional<FVector3d> GetFinalSafePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult);
	TOptional<FVector3d> GetSafePosition(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult);

	void RunCollisionTrace(UWorld* World, APlayerController* PlayerController, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void HandleAsyncCollisionTraceResult(UWorld* World, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void HandleCollisionTraceResult(UWorld* World, TArrayView<const FHitResult> HitResults, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void HandleDisabledCollision(const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);
	void UpdatePushFactor(bool bFoundHit, float CurrentPushFactor, const FVector3d& SafePosition, const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

private:

	TCameraVariableReferenceReader<bool> EnableCollisionReader;
	TCameraVariableReferenceReader<FVector3d> CustomSafePositionReader;

	TCameraParameterReader<float> CollisionSphereRadiusReader;
	TCameraParameterReader<FVector3d> SafePositionOffsetReader;

	TUniquePtr<FCameraDoubleValueInterpolator> PushInterpolator;
	TUniquePtr<FCameraDoubleValueInterpolator> PullInterpolator;

	FTraceHandle CollisionTraceHandle;

	float LastPushFactor = 0.f;
	float LastDampedPushFactor = 0.f;

	enum class ECameraCollisionDirection { Pushing, Pulling };
	ECameraCollisionDirection LastDirection = ECameraCollisionDirection::Pushing;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	bool bDebugCollisionEnabled = false;
	bool bDebugFoundHit = false;
	bool bDebugGotSafePosition = false;
	bool bDebugGotSafePositionOffset = false;
	FString DebugHitObjectName;
	FVector3d DebugSafePosition;
#endif
};

} // namespace UE::Cameras

/**
 * A node that pushes the camera towards a "safe position" when it is colliding with 
 * the environment. By default, the "safe position" is the pivot of the camera (if any) 
 * or the position of the player pawn.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Collision"))
class UCollisionPushCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	/** How to compute the safe position. */
	UPROPERTY(EditAnywhere, Category="Safe Position")
	ECollisionSafePosition SafePosition = ECollisionSafePosition::Pivot;

	/**
	 * An optional camera variable to query for a safe position. If null, or if the variable
	 * isn't set, fallback to the value defined by SafePosition.
	 */
	UPROPERTY(EditAnywhere, Category="Safe Position")
	FVector3dCameraVariableReference CustomSafePosition;

	/** World-space offset from the target to the line trace's end. */
	UPROPERTY(EditAnywhere, Category="Safe Position")
	FVector3dCameraParameter SafePositionOffset;

	/** What space the safe position offset should be in. */
	UPROPERTY(EditAnywhere, Category="Safe Position")
	ECollisionSafePositionOffsetSpace SafePositionOffsetSpace = ECollisionSafePositionOffsetSpace::Pivot;

	/**
	 * An optional boolean camera variable that specifies whether collision should be enabled.
	 * When enabled/disabled, the collision push amount will interpolate as per the PushInterpolator
	 * and PullInterpolator.
	 */
	UPROPERTY(EditAnywhere, Category="Collision")
	FBooleanCameraVariableReference EnableCollision;

	/** Radius of the sphere used for collision testing. */
	UPROPERTY(EditAnywhere, Category="Collision")
	FFloatCameraParameter CollisionSphereRadius;

	/** Collision channel to use for the line trace. */
	UPROPERTY(EditAnywhere, Category="Collision")
	TEnumAsByte<ECollisionChannel> CollisionChannel = ECollisionChannel::ECC_Camera;

	/** The interpolation to use when pushing the camera towards the safe position. */
	UPROPERTY(EditAnywhere, Category="Collision")
	TObjectPtr<UCameraValueInterpolator> PushInterpolator;

	/** The interpolation to use when pulling the camera back to its ideal position. */
	UPROPERTY(EditAnywhere, Category="Collision")
	TObjectPtr<UCameraValueInterpolator> PullInterpolator;

	/**
	 * Whether to run the collision asynchrnously. 
	 * This is better for performance, but results in collision handling being one frame late.
	 */
	UPROPERTY(EditAnywhere, Category="Collision")
	bool bRunAsyncCollision = false;

public:

	UE_API UCollisionPushCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	UE_API virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

#undef UE_API

