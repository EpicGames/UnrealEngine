// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Engine/EngineTypes.h"
#include "Nodes/CameraNodeTypes.h"

#include "OcclusionMaterialCameraNode.generated.h"

class UMaterialInterface;

UCLASS(MinimalAPI, meta=(CameraNodeCategories="Collision"))
class UOcclusionMaterialCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	/** Material to apply on objects causing occlusion. */
	UPROPERTY(EditAnywhere, Category="Occlusion")
	TObjectPtr<UMaterialInterface> OcclusionTransparencyMaterial = nullptr;

	/** Radius of the sphere used for occlusion testing. */
	UPROPERTY(EditAnywhere, Category="Occlusion")
	FFloatCameraParameter OcclusionSphereRadius;

	/** Collision channel to use for the occlusion testing. */
	UPROPERTY(EditAnywhere, Category="Occlusion")
	TEnumAsByte<ECollisionChannel> OcclusionChannel = ECollisionChannel::ECC_Camera;

	/** 
	 * The position of the occlusion trace's target. Objects encountered between the current
	 * camera pose location and this target will have the transparency material applied to
	 * them until they move out of the way.
	 */
	UPROPERTY(EditAnywhere, Category="Occlusion Target")
	ECameraNodeOriginPosition OcclusionTargetPosition = ECameraNodeOriginPosition::Pawn;

	/** The space in which to apply the occlusion target offset. */
	UPROPERTY(EditAnywhere, Category="Occlusion Target")
	ECameraNodeSpace OcclusionTargetOffsetSpace = ECameraNodeSpace::World;

	/** Offset from the target to the occlusion trace's end. */
	UPROPERTY(EditAnywhere, Category="Occlusion Target")
	FVector3dCameraParameter OcclusionTargetOffset;

public:

	UOcclusionMaterialCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

