// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraNode.h"

#include "ArrayCameraNode.generated.h"

/**
 * A camera node that runs a list of other camera nodes.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Sequence", CameraNodeCategories="Common,Utility"))
class UArrayCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	UArrayCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The camera nodes to run. */
	UPROPERTY()
	TArray<TObjectPtr<UCameraNode>> Children;

public:

	// For unit tests.
	static GAMEPLAYCAMERAS_API TTuple<int32, int32> GetEvaluatorAllocationInfo();
};

