// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"

#include "AutoFocusCameraNode.generated.h"

/**
 * A camera node that sets the focus distance to the distance of the current target.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Rendering"))
class UAutoFocusCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	UAutoFocusCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Whether auto-focus should be enabled. */
	UPROPERTY(EditAnywhere, Category="Auto-Focus")
	FBooleanCameraVariableReference EnableAutoFocus;

	/**
	 * The damping factor for how fast the focus distance follows the target distance.
	 * When zero, damping is disabled and focus distance is always equal to target distance.
	 * Low factors are "laggy", high factors are "tight".
	 */
	UPROPERTY(EditAnywhere, Category="Auto-Focus")
	FFloatCameraParameter AutoFocusDampingFactor;
};

