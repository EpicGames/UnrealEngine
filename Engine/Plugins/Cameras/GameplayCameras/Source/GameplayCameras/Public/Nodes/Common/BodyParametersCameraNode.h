// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "BodyParametersCameraNode.generated.h"

/**
 * A camera node that configures parameters on the camera body.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Body"))
class UBodyParametersCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Current shutter speed, in 1/seconds */
	UPROPERTY(EditAnywhere, Category="Filmback", DisplayName="Shutter Speed (1/s)")
    FFloatCameraParameter ShutterSpeed;

	/** The camera sensor sensitivity in ISO. */
	UPROPERTY(EditAnywhere, Category="Filmback")
	FFloatCameraParameter ISO;
};

