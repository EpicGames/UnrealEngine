// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraParameterReader.h"

#include "BoomArmCameraNode.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraValueInterpolator;
class UInput2DCameraNode;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FInput2DCameraNodeEvaluator;
template<typename T> class TCameraValueInterpolator;

/**
 * The boom arm node evaluator class.
 */
class FBoomArmCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(UE_API, FBoomArmCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	UE_API virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	UE_API virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	UE_API virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	UE_API virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	UE_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	UE_API virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	// FBoomArmCameraNodeEvaluator interface.
	UE_API virtual FRotator ComputeBoomRotation(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

private:

	TCameraParameterReader<FVector3d> BoomOffsetReader;
	FInput2DCameraNodeEvaluator* InputSlotEvaluator = nullptr;

	TUniquePtr<TCameraValueInterpolator<double>> BoomLengthInterpolator;
	TCameraParameterReader<double> MaxForwardInterpolationFactorReader;
	TCameraParameterReader<double> MaxBackwardInterpolationFactorReader;
	FVector3d LastPivotLocation;
	double CumulativePull;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector2d DebugYawPitch;
	bool bDebugDidClampPull;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

}  // namespace UE::Cameras

/**
 * A camera node that can rotate the camera in yaw and pitch based on player input.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class UBoomArmCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	UE_API UBoomArmCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	UE_API virtual FCameraNodeChildrenView OnGetChildren() override;
	UE_API virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The offset of the boom. Rotation occurs at the base (i.e. before the offset). */
	UPROPERTY(EditAnywhere, Category=Common)
	FVector3dCameraParameter BoomOffset;

	/** The interpolator to use for changing the boom length based on its pivot's movements. */
	UPROPERTY(EditAnywhere, Category=Common)
	TObjectPtr<UCameraValueInterpolator> BoomLengthInterpolator;

	/** 
	 * The maximum amount of forward movement the interpolator can introduce, expressed
	 * as a factor of the default boom length.
	 */
	UPROPERTY(EditAnywhere, Category=Common)
	FDoubleCameraParameter MaxForwardInterpolationFactor = -1.0;

	/** 
	 * The maximum amount of backward movement the interpolator can introduce, expressed
	 * as a factor of the default boom length.
	 */
	UPROPERTY(EditAnywhere, Category=Common)
	FDoubleCameraParameter MaxBackwardInterpolationFactor = -1.0;

	/**
	 * The input slot for controlling the boom arm.
	 * If no input slot is specified, the boom arm will use the player controller view rotation.
	 */
	UPROPERTY(meta=(ObjectTreeGraphPinDirection=Input))
	TObjectPtr<UInput2DCameraNode> InputSlot;

	/* 
	* If true, the boom arm's rotation will add to the existing rotation instead of being absolute.
	*/
	UPROPERTY(EditAnywhere, Category=Common)
	bool bAdditiveRotation = false;

};

#undef UE_API

