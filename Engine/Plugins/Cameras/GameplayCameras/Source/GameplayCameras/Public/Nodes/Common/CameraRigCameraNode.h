// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"
#include "Core/IAssetReferenceCameraNode.h"
#include "Core/ICustomCameraNodeParameterProvider.h"

#include "CameraRigCameraNode.generated.h"

/**
 * A camera node that runs a camera rig's own node tree.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Camera Rig Prefab", CameraNodeCategories="Common,Utility"))
class UCameraRigCameraNode 
	: public UCameraNode
	, public IAssetReferenceCameraNode
	, public ICustomCameraNodeParameterProvider
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual void OnPreBuild(FCameraBuildContext& BuildContext) override;
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

	// IAssetReferenceCameraNode interface.
	virtual void GatherPackages(FCameraRigPackages& OutPackages) const override;

	// ICustomCameraNodeParameterProvider interface.
	virtual void GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos) override;

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override;
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const override;
#endif  // WITH_EDITOR

	// UObject interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif  // WITH_EDITOR

public:

	/** The camera rig to run. */
	UPROPERTY(EditAnywhere, Category=Common, meta=(ObjectTreeGraphHidden=true))
	FCameraRigAssetReference CameraRigReference;
};

namespace UE::Cameras
{

class FCameraRigCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FCameraRigCameraNodeEvaluator)

public:

	FCameraRigCameraNodeEvaluator();

	FCameraNodeEvaluator* GetCameraRigRootEvaluator() const { return CameraRigRootEvaluator; }

protected:

	// FCameraNodeEvaluator interface.
	virtual FCameraNodeEvaluatorChildrenView OnGetChildren() override;
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) override;
	virtual void OnBuild(const FCameraNodeEvaluatorBuildParams& Params) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;

private:

	FCameraNodeEvaluator* CameraRigRootEvaluator = nullptr;
};

}  // namesapce UE::Cameras

