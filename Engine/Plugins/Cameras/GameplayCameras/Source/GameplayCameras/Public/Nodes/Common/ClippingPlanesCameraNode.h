// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "ClippingPlanesCameraNode.generated.h"

/**
 * A camera node that sets the clipping planes for the evaluated camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Rendering"))
class UClippingPlanesCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	UPROPERTY(EditAnywhere, Category="Clipping", meta=(ClampMin="0.001"))
	FDoubleCameraParameter NearPlane;

	UPROPERTY(EditAnywhere, Category="Clipping", meta=(ClampMin="0.001"))
	FDoubleCameraParameter FarPlane;
};

