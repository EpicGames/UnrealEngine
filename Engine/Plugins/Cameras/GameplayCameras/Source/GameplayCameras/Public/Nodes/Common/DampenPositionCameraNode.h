// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "DampenPositionCameraNode.generated.h"

/**
 * A camera node that smoothes the movement of the camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class UDampenPositionCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Damping factor for forward/backward camera movement. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter ForwardDampingFactor = 0.f;

	/** Damping factor for left/right camera movement. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter LateralDampingFactor = 0.f;

	/** Damping factor for up/down camera movement. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter VerticalDampingFactor = 0.f;

	/** What space the damping should be in. */
	UPROPERTY(EditAnywhere, Category=Damping)
	ECameraNodeSpace DampenSpace = ECameraNodeSpace::CameraPose;
};

