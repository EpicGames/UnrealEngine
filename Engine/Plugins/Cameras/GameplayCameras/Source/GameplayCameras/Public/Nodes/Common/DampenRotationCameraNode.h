// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "DampenRotationCameraNode.generated.h"

/**
 * A camera node that smoothes the rotation of the camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class UDampenRotationCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Damping factor for the camera yaw. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter YawDampingFactor = 0.f;

	/** Damping factor for the camera pitch. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter PitchDampingFactor = 0.f;

	/** Damping factor for the camera roll. */
	UPROPERTY(EditAnywhere, Category=Damping)
	FFloatCameraParameter RollDampingFactor = 0.f;
};

