// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "FieldOfViewCameraNode.generated.h"

/**
 * A camera node that sets the field of view of the camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Lens"))
class UFieldOfViewCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	/**
	 * The horizontal field of view, in degrees.
	 *
	 * Note that the effective horizontal field of view may be different, depending on the viewport size and the way
	 * aspect ratio constraints have been setup on the camera.
	 */
	UPROPERTY(EditAnywhere, Category=Common, meta=(UIMin="5.0", UIMax="170", ClampMin="0.001", ClampMax="360.0", Units=deg))
	FFloatCameraParameter FieldOfView;

public:

	UFieldOfViewCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

