// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "FilmbackCameraNode.generated.h"

/**
 * A camera node that configures the camera filmback.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Body"))
class UFilmbackCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	UFilmbackCameraNode(const FObjectInitializer& ObjectInitializer);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Horizontal size of filmback or digital sensor, in mm. */
	UPROPERTY(EditAnywhere, Category="Filmback", meta=(ClampMin="0.001", ForceUnits=mm))
	FFloatCameraParameter SensorWidth;

	/** Vertical size of filmback or digital sensor, in mm. */
	UPROPERTY(EditAnywhere, Category="Filmback", meta=(ClampMin="0.001", ForceUnits=mm))
	FFloatCameraParameter SensorHeight;

	/** Horizontal offset of the sensor, in mm. */
	UPROPERTY(EditAnywhere, Category="Advanced", meta=(ForceUnits=mm))
	FFloatCameraParameter SensorHorizontalOffset;

	/** Vertical offset of the sensor, in mm. */
	UPROPERTY(EditAnywhere, Category="Advanced", meta=(ForceUnits=mm))
	FFloatCameraParameter SensorVerticalOffset;

	/** Sensor overscan, in percentages (0% for no increase). */
	UPROPERTY(EditAnywhere, Category="Advanced", meta=(ForceUnits=Percent))
	FFloatCameraParameter Overscan;

	/** Whether to constrain the aspect ratio of the evaluated camera. */
	UPROPERTY(EditAnywhere, Category="Filmback")
	FBooleanCameraParameter ConstrainAspectRatio = false;

	/**
	 * Whether to override the default aspect ratio axis constraint defined on the player controller.
	 */
	UPROPERTY(EditAnywhere, Category="Filmback")
	FBooleanCameraParameter OverrideAspectRatioAxisConstraint = false;

	/** 
	 * Defines the axis along which to constrain the aspect ratio of the evaluated camera.
	 * Only used when ConstrainAspectRatio is false and OverrideAspectRatioAxisConstraint is true.
	 */
	UPROPERTY(EditAnywhere, Category="Filmback")
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV;
};

