// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "FirstPersonCameraNode.generated.h"

/**
 * A camera node that sets and enables the first-person parameters on the resulting camera view.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common"))
class UFirstPersonCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:
	
	/** Whether to enable first-person view parameters on primitives tagged as "IsFirstPerson". */
	UPROPERTY(EditAnywhere, Category=Common)
	FBooleanCameraParameter EnableFirstPerson = true;

	/** 
	 * The horizontal field of view (in degrees) used for primitives tagged as "IsFirstPerson".
	 * If zero or negative, use the normal field of view.
	 */
	UPROPERTY(EditAnywhere, Category=Common, meta=(
				UIMin="5.0", UIMax="170", ClampMin="0.001", ClampMax="360.0", Units=deg, 
				EditCondition="bEnableFirstPersonFieldOfView"))
	FFloatCameraParameter FirstPersonFieldOfView = 90.f;

	/**
	 * The scale to apply to primitives tagged as "IsFirstPerson". 
	 *
	 * This is used to scale down these primitives and prevent them from intersecting with the scene.
	 */
	UPROPERTY(EditAnywhere, Category=Common, meta=(
				UIMin="0.001", UIMax="1.0", ClampMin="0.001", Units=%,
				EditCondition="bEnableFirstPersonScale"))
	FFloatCameraParameter FirstPersonScale = 1.f;

public:

	UPROPERTY(EditAnywhere, Category=Common, meta=(InlineEditConditionToggle))
	bool bEnableFirstPersonFieldOfView = true;

	UPROPERTY(EditAnywhere, Category=Common, meta=(InlineEditConditionToggle))
	bool bEnableFirstPersonScale = true;
};

