// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "LensParametersCameraNode.generated.h"

/**
 * A camera node that sets parameter values on the camera lens.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Lens"))
class ULensParametersCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	// UObject interface.
	virtual void Serialize(FArchive& Ar) override;

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Current focal length of the camera (i.e. controls FoV, zoom) */
	UPROPERTY(EditAnywhere, Category="Lens Parameters", meta=(
				UIMin=5.0, UIMax=200.0, ClampMin=0.001, ForceUnits=mm,
				EditCondition="bEnableFocalLength"))
	FFloatCameraParameter FocalLength = { 35.f };

	/** Current aperture, in terms of f-stop (e.g. 2.8 for f/2.8) */
	UPROPERTY(EditAnywhere, Category="Lens Parameters", meta=(
				UIMin=0.1, UIMax=20.0, ClampMin=0.001,
				EditCondition="bEnableAperture"))
	FFloatCameraParameter Aperture = { 16.f };

	/** Manually-controlled focus distance (manual focus mode only) */
	UPROPERTY(EditAnywhere, Category="Lens Parameters", meta=(
				UIMin=0.001, UIMax=100000.0, ClampMin=0.001,
				EditCondition="bEnableFocusDistance"))
	FFloatCameraParameter FocusDistance = { 1000.f };

	/** 
	 * Whether post-process settings should be automatically applied to reflect
	 * the lens settings.
	 */
	UPROPERTY(EditAnywhere, Category="Lens Parameters")
	FBooleanCameraParameter EnablePhysicalCamera = false;

public:

	/** Enable setting the focal length. When false, focal length is not set. */
	UPROPERTY(EditAnywhere, Category="Lens Features", meta=(InlineEditConditionToggle))
	bool bEnableFocalLength = true;

	/** Enable setting the aperture. When false, aperture is not set. */
	UPROPERTY(EditAnywhere, Category="Lens Features", meta=(InlineEditConditionToggle))
	bool bEnableAperture = true;

	/** Enable setting the focus distance. When false, focus distance is not set. */
	UPROPERTY(EditAnywhere, Category="Lens Features", meta=(InlineEditConditionToggle))
	bool bEnableFocusDistance = false;
};

