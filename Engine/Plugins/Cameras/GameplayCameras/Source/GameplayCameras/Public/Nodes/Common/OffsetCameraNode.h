// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "OffsetCameraNode.generated.h"

/**
 * A camera node that offsets the location of the camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class UOffsetCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The translation offset to apply to the camera. */
	UPROPERTY(EditAnywhere, Category=Common)
	FVector3dCameraParameter TranslationOffset;

	/** The rotation offset to apply to the camera. */
	UPROPERTY(EditAnywhere, Category=Common)
	FRotator3dCameraParameter RotationOffset;

	/** The space in which to apply the offset. */
	UPROPERTY(EditAnywhere, Category=Common)
	ECameraNodeSpace OffsetSpace = ECameraNodeSpace::CameraPose;

public:

	// For unit tests.
	static GAMEPLAYCAMERAS_API TTuple<int32, int32> GetEvaluatorAllocationInfo();
};

