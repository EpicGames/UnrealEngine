// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "EngineDefines.h"

#include "OrthographicCameraNode.generated.h"

/**
 * A camera node that sets the perspective mode to orthographic.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common"))
class UOrthographicCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Sets the perspective mode to orthographic. */
	UPROPERTY(EditAnywhere, Category="Orthographic")
	FBooleanCameraParameter EnableOrthographicMode = true;

	/** The width of the orthographic view. */
	UPROPERTY(EditAnywhere, Category="Orthographic", meta=(Units=cm))
	FFloatCameraParameter OrthographicWidth = DEFAULT_ORTHOWIDTH;
};

