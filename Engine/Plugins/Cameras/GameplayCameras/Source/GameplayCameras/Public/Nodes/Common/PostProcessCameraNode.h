// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Engine/Scene.h"

#include "PostProcessCameraNode.generated.h"

/**
 * A camera node that adds post-process settings on the evaluated camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Rendering"))
class UPostProcessCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The post-process settings to set on the output camera pose. */
	UPROPERTY(EditAnywhere, Category="Rendering", meta=(CameraContextData=true))
	FPostProcessSettings PostProcessSettings;

	UPROPERTY()
	FCameraContextDataID PostProcessSettingsDataID;
};

