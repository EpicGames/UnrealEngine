// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Nodes/CameraNodeTypes.h"

#include "SetRotationCameraNode.generated.h"

/**
 * A camera node that sets the rotation of the camera.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class USetRotationCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The rotation to set on the camera. */
	UPROPERTY(EditAnywhere, Category=Common)
	FRotator3dCameraParameter Rotation;

	/** The space in which to apply the transform. */
	UPROPERTY(EditAnywhere, Category=Common)
	ECameraNodeSpace OffsetSpace = ECameraNodeSpace::CameraPose;
};

