// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Curves/CameraSingleCurve.h"
#include "Nodes/CameraNodeTypes.h"

#include "SplineFieldOfViewCameraNode.generated.h"

/**
 * A camera node that sets the field of view of the camera by evaluating a curve.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Lens"))
class USplineFieldOfViewCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	/** The input to pass to the field of view spline. */
	UPROPERTY(EditAnywhere, Category=Common)
	FFloatCameraParameter SplineInput;

	/** The field of view, in degrees. */
	UPROPERTY(EditAnywhere, Category=Common)
	FCameraSingleCurve FieldOfViewSpline;

public:

	USplineFieldOfViewCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

