// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Curves/CameraRotatorCurve.h"
#include "Curves/CameraVectorCurve.h"
#include "Nodes/CameraNodeTypes.h"

#include "SplineOffsetCameraNode.generated.h"

/**
 * A camera node that offsets the location and rotation of the camera by evaluating curves.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class USplineOffsetCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The input to pass to the offset splines. */
	UPROPERTY(EditAnywhere, Category=Common)
	FFloatCameraParameter SplineInput;

	/** The spline that defines the translation offset to apply to the camera. */
	UPROPERTY(EditAnywhere, Category=Common)
	FCameraVectorCurve TranslationOffsetSpline;

	/** The rotation offset to apply to the camera. */
	UPROPERTY(EditAnywhere, Category=Common)
	FCameraRotatorCurve RotationOffsetSpline;

	/** The space in which to apply the offset. */
	UPROPERTY(EditAnywhere, Category=Common)
	ECameraNodeSpace OffsetSpace = ECameraNodeSpace::CameraPose;
};

