// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Curves/CameraRotatorCurve.h"
#include "Curves/CameraVectorCurve.h"
#include "Nodes/CameraNodeTypes.h"

#include "SplineOrbitCameraNode.generated.h"

class UCameraValueInterpolator;
class UInput2DCameraNode;

/**
 * Control point for the spline orbit camera node.
 */
USTRUCT()
struct FSplineOrbitControlPoint
{
	GENERATED_BODY()

	/**
	 * The offset of the camera position from the orbit pivot.
	 * This defines the radius of the orbit at the given control point, along with any lateral or 
	 * vertical position offsets.
	 */
	UPROPERTY(EditAnywhere, Category="Spline Orbit")
	FVector3d LocationOffset = FVector3d::ZeroVector;

	/**
	 * The offset of the camera target as defined by projecting the orbit pivot on the line of sight.
	 * This adds rotation to the camera by making it look higher/lower/etc at the given control
	 * point.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	FVector3d TargetOffset = FVector3d::ZeroVector;

	/**
	 * A rotation offset applied to the camera.
	 * This adds rotation to the camera, in local space, applied after TargetOffset.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	FRotator3d RotationOffset = FRotator3d::ZeroRotator;

	/**
	 * The pitch angle for this control point.
	 */
	UPROPERTY(EditAnywhere, Category="Spline Orbit")
	float PitchAngle = 0.f;
};

/**
 * A camera node that can orbit around a pivot point, and the shape of the orbit is defined
 * by pitch-based parameters.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class USplineOrbitCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	USplineOrbitCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** Spline that defines the camera location's offset for a given pitch angle. */
	UPROPERTY(EditAnywhere, Category="Spline Orbit", meta=(CurveEditorXLabelFormat="{0}deg"))
	FCameraVectorCurve LocationOffsetSpline;

	/** Spline that defines an additive camera target offset for a given pitch angle. */
	UPROPERTY(EditAnywhere, Category="Spline Orbit", meta=(CurveEditorXLabelFormat="{0}deg"))
	FCameraVectorCurve TargetOffsetSpline;

	/** Spline that defines an additive camera rotation offset for a given pitch angle. */
	UPROPERTY(EditAnywhere, Category="Spline Orbit", meta=(CurveEditorXLabelFormat="{0}deg"))
	FCameraRotatorCurve RotationOffsetSpline;

	UPROPERTY(EditAnywhere, Category="Spline Orbit")
	FFloatCameraParameter LocationOffsetMultiplier = 1.f;

	/**
	 * The space in which the control points' TargetOffset is applied.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	ECameraNodeSpace TargetOffsetSpace = ECameraNodeSpace::ActiveContext;

	/**
	 * The input slot for controlling the orbiting.
	 * If no input slot is specified, this node will use the player controller view rotation.
	 */
	UPROPERTY(meta=(ObjectTreeGraphPinDirection=Input))
	TObjectPtr<UInput2DCameraNode> InputSlot;
};

