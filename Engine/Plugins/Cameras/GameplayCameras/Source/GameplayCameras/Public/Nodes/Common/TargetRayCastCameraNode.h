// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"

#include "TargetRayCastCameraNode.generated.h"

/**
 * A camera node that determines and sets the camera's target by running a ray-cast
 * from the current camera position.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Target"))
class UTargetRayCastCameraNode : public UCameraNode
{
	GENERATED_BODY()

public:

	/** Trace channel to use for the ray-cast. */
	UPROPERTY(EditAnywhere, Category="Ray-Cast")
	TEnumAsByte<ECollisionChannel> TraceChannel = ECollisionChannel::ECC_Camera;

	/** Whether to set the focus distance to the ray-cast hit result. */
	UPROPERTY(EditAnywhere, Category="Auto-Focus")
	FBooleanCameraParameter AutoFocus = true;

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

