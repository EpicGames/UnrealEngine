// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "Nodes/Framing/CameraFramingZone.h"
#include "Nodes/Framing/CameraActorTargetInfo.h"
#include "Math/CameraFramingZoneMath.h"
#include "Math/CriticalDamper.h"

#include "BaseFramingCameraNode.generated.h"

class FArchive;
class UVector3dCameraVariable;

/**
 * The base class for a standard scren-space framing camera node.
 */
UCLASS(MinimalAPI, Abstract, meta=(CameraNodeCategories="Framing"))
class UBaseFramingCameraNode : public UCameraNode, public ICustomCameraNodeParameterProvider
{
	GENERATED_BODY()

public:

	/** 
	 * A variable whose value is the desired target's location in world space.
	 * If set, and if the variable has been set, the obtained value takes priority
	 * over the TargetInfos property.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	FVector3dCameraVariableReference TargetLocation;

	/** Specifies one or more target actors to frame. */
	UPROPERTY(EditAnywhere, Category="Target", meta=(CameraContextData=true))
	TArray<FCameraActorTargetInfo> TargetInfos;

	UPROPERTY()
	FCameraContextDataID TargetInfosDataID;

	/**
	 * Whether the camera pose's target distance should be set to the distance between
	 * its location and the effective target's location.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	FBooleanCameraParameter SetTargetDistance;

	/**
	 * Whether to frame the target with the ideal framing immediately on the first frame.
	 */
	UPROPERTY(EditAnywhere, Category="Target")
	FBooleanCameraParameter InitializeWithIdealFraming = true;

	/** The ideal horizontal and vertical screen-space position of the target. */
	UPROPERTY(EditAnywhere, Category="Framing Target")
	FVector2dCameraParameter IdealFramingLocation;

	/** The damping factor for how fast the framing recenters on the target. */
	UPROPERTY(EditAnywhere, Category="Framing Target")
	FFloatCameraParameter ReframeDampingFactor;

	/** 
	 * If valid, the recentering damping factor will interpolate between LowReframeDampingFactor 
	 * and ReframeDampingFactor as the target moves between the ideal target position and the
	 * boundaries of the hard-zone. If invalid, no interpolation occurs and the damping factor
	 * is always equal to ReframeDampingFactor. */
	UPROPERTY(EditAnywhere, Category="Framing Target")
	FFloatCameraParameter LowReframeDampingFactor;

	/**
	 * The time spent ramping up the reframing after exiting the dead zone.
	 * If set to zero or a negative value, reframing will immediately restart once the target
	 * has exited the dead zone. Otherwise the ReframeDampingFactor will interpolate from zero to
	 * its desired value over the specified amount of seconds.
	 */
	UPROPERTY(EditAnywhere, Category="Framing Target")
	FFloatCameraParameter ReengageTime;

	/**
	 * The time spent ramping down the reframing after entering the dead zone.
	 * If set to zero or a negative value, reframing will immediately stop once the target has 
	 * entered the dead zone. Otherwise, the ReframeDampingFactor will interpolate towards zero
	 * over the specified amount of seconds.
	 */
	UPROPERTY(EditAnywhere, Category="Framing Target")
	FFloatCameraParameter DisengageTime;

	UPROPERTY(EditAnywhere, Category="Framing Target")
	FFloatCameraParameter TargetMovementAnticipationTime;

	/** 
	 * The size of the dead zone, i.e. the zone inside which the target can freely move.
	 * Sizes are expressed screen percentages around the desired framing location.
	 */
	UPROPERTY(EditAnywhere, Category="Framing Zones")
	FCameraFramingZoneParameter DeadZone;

	/**
	 * The margins of the soft zone, i.e. the zone inside which the reframing will engage, in order
	 * to bring the target back towards the ideal framing position. If the target is outside of the
	 * soft zone, it will be forcibly and immedialy brought back to its edges, so this zone also 
	 * defines the "hard" or "safe" zone of framing.
	 * Sizes are expressed in screen percentages from the edges.
	 */
	UPROPERTY(EditAnywhere, Category="Framing Zones")
	FCameraFramingZoneParameter SoftZone;

private:

	UPROPERTY()
	FCameraActorTargetInfo TargetInfo_DEPRECATED;

public:

	UBaseFramingCameraNode(const FObjectInitializer& ObjectInit);

public:

	// UObject interface.
	virtual void PostLoad() override;

	// ICustomCameraNodeParameterProvider interface.
	virtual void GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos) override;
};

namespace UE::Cameras
{

class FCameraVariableTable;

/**
 * The base class for a framing camera node evaluator.
 *
 * This evaluator does nothing per se but provides utility functions to be called in 
 * a sub-class' OnRun method. Namely:
 *
 * - AcquireTargetLocation() : a default way to get the world location of the desired
 *			target.
 *
 * - UpdateFramingState() : computes the current state of the framing node. The result
 *			can be obtained from the State field.
 *			The, compute the desired framing state for the current tick, including the 
 *			desired framing correction. This can be obtained from the Desired field.
 *			It is up to the sub-class to implement the necessary logic to honor this 
 *			correction. For instance, a dolly shot would translate left/right (and maybe 
 *			up/down too) to try and reframe things accordingly, whereas a panning shot 
 *			would rotate the camera left/right/up/down to accomplish the same.
 *
 * - EndFramingUpdate() : the sub-class should call near the end of its OnRun method.
 *			This will for instance optionally set the target distance.
 */
class FBaseFramingCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FBaseFramingCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;

#if WITH_EDITOR
	virtual void OnDrawEditorPreview(const FCameraEditorPreviewDrawParams& Params, FCameraDebugRenderer& Renderer) override;
#endif  // WITH_EDITOR

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

protected:

	struct FState;
	struct FDesired;

	/** The first frame aiming direction we need for proper initialization. */
	TOptional<FVector3d> GetInitialDesiredWorldTarget(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult);
	/** Updates the framing state for the current tick, see State member field. */
	void UpdateFramingState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult, const FTransform3d& LastFraming);
	/** Wraps-up the update with optional operations. */
	void EndFramingUpdate(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

private:

#if UE_GAMEPLAY_CAMERAS_DEBUG
	static void DrawFramingState(const FState& State, const FDesired& Desired, FCameraDebugRenderer& Renderer);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

	FVector2d GetHardReframeCoords() const;

	void ComputeCurrentState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult, const FTransform3d& LastFraming);
	void ComputeDesiredState(const FCameraNodeEvaluationParams& Params, const FCameraNodeEvaluationResult& OutResult);

	bool AcquireTargetInfo(TSharedPtr<const FCameraEvaluationContext> EvaluationContext, const FCameraNodeEvaluationResult& InResult, TArray<FCameraActorComputedTargetInfo>& OutInfos);
	bool ComputeFinalTargetInfo(const FCameraPose& CameraPose, const FMatrix& ViewProjectionMatrix, FVector3d& OutWorldTarget, FVector2d& OutScreenTarget, FFramingZone& OutScreenBounds);
	float ComputeScreenTargetSpeed(float DeltaTime, const FMatrix& ViewProjectionMatrix, const FVector3d& OldWorldTarget, const FVector3d& NewWorldTarget);
	FVector2d ComputeAnticipatedScreenTarget(float DeltaTime, const FVector2d& InPreviousAnticipatedScreenTarget, const FVector2d& InScreenTarget);
	FFramingZone ComputeEffectiveDeadZone();

	static FFramingZone ComputeScreenTargetBounds(const FMatrix& ViewProjectionMatrix, const FTransform3d& TargetTransform, const FBoxSphereBounds3d& LocalBounds);

protected:

	/** The current location of the target. */
	enum class ETargetFramingState
	{
		/**
		 * The target is in the dead zone, i.e. it can roam freely unless we have an 
		 * active reframing to finish.
		 */
		InDeadZone,
		/**
		 * The target is in the soft zone, i.e. we will attempt to gently bring it back 
		 * to the ideal framing position.
		 */
		InSoftZone,
		/**
		 * The target is in the hard zone, i.e. it has exited the soft zone and we need
		 * to bring it back ASAP.
		 */
		InHardZone
	};

	/** Utility structure for all the parameter readers we need every frame. */
	struct FReaders
	{
		FCameraActorTargetInfoArrayReader TargetInfos;

		TCameraParameterReader<FVector2d> IdealFramingLocation;
		TCameraParameterReader<bool> InitializeWithIdealFraming;
		TCameraParameterReader<bool> SetTargetDistance;

		TCameraParameterReader<float> ReframeDampingFactor;
		TCameraParameterReader<float> LowReframeDampingFactor;
		TCameraParameterReader<float> ReengageTime;
		TCameraParameterReader<float> DisengageTime;
		TCameraParameterReader<float> TargetMovementAnticipationTime;

		TCameraParameterReader<FCameraFramingZone> DeadZone;
		TCameraParameterReader<FCameraFramingZone> SoftZone;
	};
	FReaders Readers;

	/** Utility struct for storing the current known state. */
	struct FState
	{
		/** Screen-space position of the ideal framing position. */
		FVector2d IdealTarget;
		/** Current reframing damping factor. */
		float ReframeDampingFactor;
		/** Current low reframing damping factor. */
		float LowReframeDampingFactor;
		/** Current alpha between reframing damping factors. */
		float ReframeDampingFactorAlpha;
		/** Current reengage time. */
		float ReengageTime;
		/** Current disengage time. */
		float DisengageTime;
		/** Current time spent disengaging or reengaging reframing */
		float ToggleEngageTimeLeft;
		/** Current reframing damping factor alpha due to engage toggle */
		float ToggleEngageAlpha;
		/** Current look-ahead time for anticipating target movement */
		float TargetMovementAnticipationTime;
		/** Current coordinates of the dead zone. */
		FFramingZone DeadZone;
		/** Current coordinates of the soft zone. */
		FFramingZone SoftZone;

		/** Current world-space position of the tracked target. */
		FVector3d WorldTarget;
		/** Current screen-space position of the tracked target. */
		FVector2d ScreenTarget;
		/** Current target bounds zone. */
		FFramingZone ScreenTargetBounds;
		/** Dead zone minus the screen target bounds. */
		FFramingZone EffectiveDeadZone;

		/** Current state of the tracked target. */
		ETargetFramingState TargetFramingState;
		/** Current screen-space speed of the tracked target. */
		float TargetScreenSpaceSpeed = 0.f;
		/** Whether we are actively trying to bring the target back to the ideal position. */
		bool bIsReframingTarget = false;

		/** The damper for reframing from the soft zone. */
		FCriticalDamper ReframeDamper;

#if UE_GAMEPLAY_CAMERAS_DEBUG
		/** Intersection of the reframing vector with the dead zone box. */
		FVector2d DebugDeadZoneEdgePoint;
		/** Intersection of the reframing vector with the hard zone box. */
		FVector2d DebugHardZoneEdgePoint;
		/** Screen bounds for all the targets. */
		TArray<FFramingZone, TInlineAllocator<4>> DebugAllScreenTargetBounds;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

		void Serialize(FArchive& Ar);
	};
	FState State;

	/** Utility struct for the desired reframing to be done in the current tick. */
	struct FDesired
	{
		/**
		 * The desired screen-space position of the tracked target. For instance, if the target
		 * is in the soft zone, this desired position will be the next step to get us closer to
		 * the ideal position.
		 */
		FVector2d ScreenTarget;
		/** 
		 * The screen-space correction we want this tick.
		 * This is effectively equal to: Desired.ScreenTarget - State.ScreenTarget
		 */
		FVector2d FramingCorrection;
		/**
		 * Whether we have any correction to do.
		 */
		bool bHasCorrection = false;

		void Serialize(FArchive& Ar);
	};
	FDesired Desired;

	struct FWorldTargetInfos
	{
		TArray<FCameraActorComputedTargetInfo, TInlineAllocator<4>> TargetInfos;

		void Serialize(FArchive& Ar);
	};
	FWorldTargetInfos WorldTargets;

	struct FScreenTargetHistory
	{
		FVector2d UnanticipatedScreenTarget;
		TArray<TTuple<FVector2d, float>, TInlineAllocator<10>> History;
	};
	FScreenTargetHistory ScreenTargetHistory;

	friend class FBaseFramingCameraDebugBlock;
	friend FArchive& operator <<(FArchive& Ar, FState& State);
	friend FArchive& operator <<(FArchive& Ar, FDesired& Desired);
	friend FArchive& operator <<(FArchive& Ar, FWorldTargetInfos& WorldTargets);
};

FArchive& operator <<(FArchive& Ar, FBaseFramingCameraNodeEvaluator::FState& State);
FArchive& operator <<(FArchive& Ar, FBaseFramingCameraNodeEvaluator::FDesired& Desired);

}  // namespace UE::Cameras

