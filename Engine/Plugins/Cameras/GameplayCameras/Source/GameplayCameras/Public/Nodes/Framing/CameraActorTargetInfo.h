// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Core/CameraContextDataTableFwd.h"
#include "Math/BoxSphereBounds.h"

#include "CameraActorTargetInfo.generated.h"

class AActor;
struct FCameraPose;

namespace UE::Cameras
{
	class FCameraContextDataTable;
}

UENUM(BlueprintType)
enum class ECameraTargetShape : uint8
{
	/** Use a single-point target. */
	Point,
	/** Use the target's computed bounds. */
	AutomaticBounds,
	/** Use custom bounds. */
	ManualBounds
};

/**
 * Targeting information for a camera rig.
 */
USTRUCT(BlueprintType)
struct FCameraActorTargetInfo
{
	GENERATED_BODY()

	/** The actor to target. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	TSoftObjectPtr<AActor> Actor;

	/** An optional socket to target on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	FName SocketName;

	/** An optional bone to target on the actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	FName BoneName;

	/** Specifies the shape of the target. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	ECameraTargetShape TargetShape = ECameraTargetShape::Point;

	/** Specifies the size of target. Only used if TargetShape is set to manual bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	float TargetSize = 10.f;

	/** The weight of this target. Unused when only one target is used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category="Target")
	float Weight = 1.f;

	bool IsValid() const { return Actor != nullptr; }

	bool operator== (const FCameraActorTargetInfo& Other) const = default;
};

namespace UE::Cameras
{

/** A special reader class for targeting information. */
struct FCameraActorTargetInfoReader
{
	FCameraActorTargetInfoReader() {}
	FCameraActorTargetInfoReader(const FCameraActorTargetInfo& InTargetInfo, FCameraContextDataID InDataID);

	void Initialize(const FCameraActorTargetInfo& InTargetInfo, FCameraContextDataID InDataID);

	bool GetTargetInfo(const FCameraContextDataTable& ContextDataTable, FTransform3d& OutTransform, FBoxSphereBounds3d& OutBounds);

private:

	void CacheTargetInfo(const FCameraActorTargetInfo& InTargetInfo);
	void ComputeTargetBounds(const FVector3d& TargetLocation, FBoxSphereBounds3d& OutBounds);

private:

	FCameraActorTargetInfo DefaultTargetInfo;
	FCameraContextDataID DataID;

	FCameraActorTargetInfo CachedTargetInfo;
	const USkeletalMeshComponent* CachedSkeletalMeshComponent = nullptr;
	FName CachedBoneName;
	FName CachedParentBoneName;

	friend struct FCameraActorTargetInfoArrayReader;
};

struct FCameraActorComputedTargetInfo
{
	FTransform3d Transform;
	FBoxSphereBounds3d LocalBounds;
	float NormalizedWeight = 1.f;
};

FArchive& operator <<(FArchive& Ar, FCameraActorComputedTargetInfo& TargetInfo);

/** A special reader class for multiple targeting information. */
struct FCameraActorTargetInfoArrayReader
{
	FCameraActorTargetInfoArrayReader() {}
	FCameraActorTargetInfoArrayReader(TConstArrayView<FCameraActorTargetInfo> InTargetInfos, FCameraContextDataID InDataID);

	void Initialize(TConstArrayView<FCameraActorTargetInfo> InTargetInfos, FCameraContextDataID InDataID);

	bool ComputeTargetInfos(const FCameraContextDataTable& ContextDataTable, TArray<FCameraActorComputedTargetInfo>& ComputedTargets);

#if WITH_EDITOR
	void Refresh(TConstArrayView<FCameraActorTargetInfo> InTargetInfos);
#endif

private:

	void CacheTargetInfos(TConstArrayView<FCameraActorTargetInfo> InTargetInfos);

private:

	TArray<FCameraActorTargetInfoReader> Readers;
	FCameraContextDataID DataID;
};

}  // namespace UE::Cameras

