// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTableFwd.h"
#include "CoreTypes.h"

#include "CameraFramingZone.generated.h"

/**
 * A structure that defines a zone for use in framing subjects in screen-space.
 *
 * Margins are generally expressed in screen size percentages (between 0 and 1), but don't have a standard meaning.
 * In some cases, they may be margins from the screen's edge, while in other cases they may be margins relative to
 * a given screen point.
 */
USTRUCT(BlueprintType)
struct FCameraFramingZone
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Framing")
	double Left;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Framing")
	double Top;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Framing")
	double Right;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Framing")
	double Bottom;

public:

	FCameraFramingZone()
	{
		Left = 0;
		Top = 0;
		Right = 0;
		Bottom = 0;
	}

	FCameraFramingZone(double UniformValue)
	{
		Left = UniformValue;
		Top = UniformValue;
		Right = UniformValue;
		Bottom = UniformValue;
	}

	FCameraFramingZone(double Horizontal, double Vertical)
	{
		Left = Horizontal;
		Top = Vertical;
		Right = Horizontal;
		Bottom = Vertical;
	}

	FCameraFramingZone(double InLeft, double InTop, double InRight, double InBottom)
	{
		Left = InLeft;
		Top = InTop;
		Right = InRight;
		Bottom = InBottom;
	}

public:

	FCameraFramingZone operator*(float Scale) const
	{
		return FCameraFramingZone(Left * Scale, Top * Scale, Right * Scale, Bottom * Scale);
	}

	FCameraFramingZone operator*(const FCameraFramingZone& Scale) const
	{
		return FCameraFramingZone(
				Left * Scale.Left, 
				Top * Scale.Top,
				Right * Scale.Right,
				Bottom * Scale.Bottom);
	}

	FCameraFramingZone operator+(const FCameraFramingZone& Other) const
	{
		return FCameraFramingZone(
				Left + Other.Left, 
				Top + Other.Top,
				Right + Other.Right,
				Bottom + Other.Bottom);
	}

	FCameraFramingZone operator-(const FCameraFramingZone& Other) const
	{
		return FCameraFramingZone(
				Left - Other.Left, 
				Top - Other.Top,
				Right - Other.Right,
				Bottom - Other.Bottom);
	}

	bool operator== (const FCameraFramingZone& Other) const = default;
	bool operator!= (const FCameraFramingZone& Other) const = default;

public:

	GAMEPLAYCAMERAS_API static void TypeErasedInterpolate(uint8* From, const uint8* To, float Alpha);

	GAMEPLAYCAMERAS_API FString ToString() const;
};

template <> struct TIsPODType<FCameraFramingZone>
{
	enum { Value = true };
};

/** Framing zone camera parameter. */
USTRUCT()
struct FCameraFramingZoneParameter
{
	GENERATED_BODY()

	using ValueType = FCameraFramingZone;

	UPROPERTY(EditAnywhere, Category=Common, meta=(ShowOnlyInnerProperties))
	FCameraFramingZone Value;

	UPROPERTY()
	FCameraVariableID VariableID;

	FCameraFramingZoneParameter() {}
	FCameraFramingZoneParameter(const FCameraFramingZone& InValue)
		: Value(InValue)
	{}
};

