// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Framing/BaseFramingCameraNode.h"

#include "DollyFramingCameraNode.generated.h"

/**
 * A camera node that frames a target by moving along a sideways rail (left/right relative to 
 * the camera transform) and optionally also up and down.
 */
UCLASS(MinimalAPI)
class UDollyFramingCameraNode : public UBaseFramingCameraNode
{
	GENERATED_BODY()

public:

	/** Whether the dolly can move laterally. */
	UPROPERTY(EditAnywhere, Category="Dolly")
	FBooleanCameraParameter CanMoveLaterally;

	/** Whether the dolly can move vertically. */
	UPROPERTY(EditAnywhere, Category="Dolly")
	FBooleanCameraParameter CanMoveVertically;

public:

	UDollyFramingCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

