// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Framing/BaseFramingCameraNode.h"

#include "PanningFramingCameraNode.generated.h"

/**
 * A camera node that frames a target by rotating in place.
 */
UCLASS(MinimalAPI)
class UPanningFramingCameraNode : public UBaseFramingCameraNode
{
	GENERATED_BODY()

public:

	/** Whether the camera can rotate laterally (yaw). */
	UPROPERTY(EditAnywhere, Category="Panning")
	FBooleanCameraParameter CanPanLaterally;

	/** Whether the camera can rotate vertically (pitch). */
	UPROPERTY(EditAnywhere, Category="Panning")
	FBooleanCameraParameter CanPanVertically;

public:

	UPanningFramingCameraNode(const FObjectInitializer& ObjectInit);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

