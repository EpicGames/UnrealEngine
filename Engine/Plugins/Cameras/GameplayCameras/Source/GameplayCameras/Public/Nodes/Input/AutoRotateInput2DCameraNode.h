// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Math/MathFwd.h"
#include "Nodes/Input/Input2DCameraNode.h"

#include "AutoRotateInput2DCameraNode.generated.h"

class UCameraValueInterpolator;

/** Describes the type of auto-rotate. */
UENUM()
enum class ECameraAutoRotateDirection
{
	/** Re-align towards the evaluation context's facing. */
	Facing,
	/** 
	 * Re-align towards the evaluation context's movement direction. 
	 * Doesn't do anything when there is no movement.
	 */
	Movement,
	/**
	 * Re-align towards the evaluation context's movement direction if there is movement,
	 * or towards its facing otherwise.
	 */
	MovementOrFacing,
	/**
	 * Re-align towards the zero direction.
	 */
	Zero,
};

/**
 * An input node that modifies a yaw/pitch input in order to re-align its
 * values to a given default direction.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Input"))
class UAutoRotateInput2DCameraNode : public UInput2DCameraNode
{
	GENERATED_BODY()

public:

	/** The direction to re-align towards. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	ECameraAutoRotateDirection Direction;

	/** An override for the direction to re-align towards. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	FVector3dCameraVariableReference DirectionVector;

	/** The time, in seconds, to wait before re-aligning. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	FFloatCameraParameter WaitTime = 1.f;

	/** The minimum player-induced/manual rotation, in degrees, to deactivate auto-rotation. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	FFloatCameraParameter DeactivationThreshold = 0.01f;

	/** The interpolation for re-alignment. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	TObjectPtr<UCameraValueInterpolator> Interpolator;

	/** Whether to suggest freezing the input control rotation. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate")
	FBooleanCameraParameter FreezeControlRotation = false;

	/** Whether to enable auto-rotation. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate Toggle")
	FBooleanCameraParameter EnableAutoRotate = true;

	/** Whether to auto-rotate yaw. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate Toggle")
	FBooleanCameraParameter AutoRotateYaw = true;

	/** Whether to auto-rotate pitch. */
	UPROPERTY(EditAnywhere, Category="Auto-Rotate Toggle")
	FBooleanCameraParameter AutoRotatePitch = true;

	/** The underlying input node. */
	UPROPERTY()
	TObjectPtr<UInput2DCameraNode> InputNode;

public:

	UAutoRotateInput2DCameraNode(const FObjectInitializer& ObjInit);
	
protected:

	// UCameraNode interface.
	virtual FCameraNodeChildrenView OnGetChildren() override;
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

