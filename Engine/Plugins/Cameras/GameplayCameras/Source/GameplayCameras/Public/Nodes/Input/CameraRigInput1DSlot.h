// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Nodes/Input/CameraRigInputSlotTypes.h"
#include "Nodes/Input/Input1DCameraNode.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigInput1DSlot.generated.h"

class UDoubleCameraVariable;

/**
 * The base class for a node that can handle and accumulate raw player input values.
 */
UCLASS(Abstract)
class UCameraRigInput1DSlot : public UInput1DCameraNode
{
	GENERATED_BODY()

public:

	/** Clamping of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterClamping Clamp;

	/** Normalization of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterNormalization Normalize;

	/** Whether to revert the axis. */
	UPROPERTY(EditAnywhere, Category="Input")
	FBooleanCameraParameter RevertAxis = false;

	/** A speed, in units/seconds, to use on the input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FDoubleCameraParameter Speed = 1.0;

	/** The variable to use to blend with other input slots. */
	UPROPERTY(EditAnywhere, Category="Input")
	EBuiltInDoubleCameraVariable BuiltInVariable = EBuiltInDoubleCameraVariable::Yaw;

	/** The variable to use to blend with other input slots. */
	UPROPERTY(EditAnywhere, Category="Input", meta=(EditCondition="BuiltInVariable == EBuiltInDoubleCameraVariable::None"))
	FDoubleCameraVariableReference CustomVariable;

	/** Whether the multiplier should be pre-blended. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsPreBlended = true;


	// Deprecated

	UPROPERTY()
	FCameraRigInputSlotParameters InputSlotParameters_DEPRECATED;

public:

	FCameraVariableID GetVariableID() const { return VariableID; }
	FCameraVariableID GetSpeedVariableID() const { return SpeedVariableID; }

public:

	// UObject interface.
	virtual void PostLoad() override;

protected:

	// UCameraNode interface.
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

private:

	UPROPERTY()
	FCameraVariableID SpeedVariableID;
	UPROPERTY()
	FCameraVariableID VariableID;
};

namespace UE::Cameras
{

class FInput1DCameraNodeEvaluator;

class FCameraRigInput1DSlotEvaluator : public FInput1DCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(GAMEPLAYCAMERAS_API, FCameraRigInput1DSlotEvaluator, FInput1DCameraNodeEvaluator)

public:

	FCameraRigInput1DSlotEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) override;
	virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

protected:

	TCameraParameterReader<bool> RevertAxisReader;
	TCameraParameterReader<double> SpeedReader;

	double DeltaInputValue = 0.f;
	bool bIsAccumulated = true;
	bool bIsLockedThisFrame = false;
};

}  // namespace UE::Cameras

