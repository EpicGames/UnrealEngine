// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameterReader.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Nodes/Input/CameraRigInputSlotTypes.h"
#include "Nodes/Input/Input2DCameraNode.h"
#include "UObject/ObjectPtr.h"

#include "CameraRigInput2DSlot.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

class UVector2dCameraVariable;

/**
 * The base class for a node that can handle and accumulate raw player input values.
 */
UCLASS(Abstract)
class UCameraRigInput2DSlot : public UInput2DCameraNode
{
	GENERATED_BODY()

public:

	/** Clamping of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterClamping ClampX;

	/** Clamping of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterClamping ClampY;

	/** Normalization of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterNormalization NormalizeX;

	/** Normalization of the final input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FCameraParameterNormalization NormalizeY;

	/** Whether to revert the X axis. */
	UPROPERTY(EditAnywhere, Category="Input")
	FBooleanCameraParameter RevertAxisX = false;

	/** Whether to revert the Y axis. */
	UPROPERTY(EditAnywhere, Category="Input")
	FBooleanCameraParameter RevertAxisY = false;

	/** A speed, in units/seconds, to use on the input value. */
	UPROPERTY(EditAnywhere, Category="Input")
	FVector2dCameraParameter Speed = FVector2d{ 1.0, 1.0 };

	/** The variable to use to blend with other input slots. */
	UPROPERTY(EditAnywhere, Category="Input")
	EBuiltInVector2dCameraVariable BuiltInVariable = EBuiltInVector2dCameraVariable::YawPitch;

	/** The variable to use to blend with other input slots. */
	UPROPERTY(EditAnywhere, Category="Input", meta=(EditCondition="BuiltInVariable == EBuiltInVector2dCameraVariable::None"))
	FVector2dCameraVariableReference CustomVariable;

	/** Whether parameters such as speed should be pre-blended. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsPreBlended = true;


	// Deprecated

	UPROPERTY()
	FCameraRigInputSlotParameters InputSlotParameters_DEPRECATED;

public:

	FCameraVariableID GetVariableID() const { return VariableID; }
	FCameraVariableID GetSpeedVariableID() const { return SpeedVariableID; }

public:

	// UObject interface.
	UE_API virtual void PostLoad() override;

protected:

	// UCameraNode interface.
	UE_API virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	UE_API virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

private:

	UPROPERTY()
	FCameraVariableID SpeedVariableID;
	UPROPERTY()
	FCameraVariableID VariableID;
};

namespace UE::Cameras
{

class FInput2DCameraNodeEvaluator;

class FCameraRigInput2DSlotEvaluator : public FInput2DCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR_EX(UE_API, FCameraRigInput2DSlotEvaluator, FInput2DCameraNodeEvaluator)

public:

	UE_API FCameraRigInput2DSlotEvaluator();

protected:

	// FCameraNodeEvaluator interface.
	UE_API virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	UE_API virtual void OnUpdateParameters(const FCameraBlendedParameterUpdateParams& Params, FCameraBlendedParameterUpdateResult& OutResult) override;
	UE_API virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	UE_API virtual void OnExecuteOperation(const FCameraOperationParams& Params, FCameraOperation& Operation) override;
	UE_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

protected:

	TCameraParameterReader<bool> RevertAxisXReader;
	TCameraParameterReader<bool> RevertAxisYReader;
	TCameraParameterReader<FVector2d> SpeedReader;

	FVector2d DeltaInputValue = FVector2d::ZeroVector;
	bool bIsAccumulated = true;
	bool bIsLockedThisFrame = false;
};

}  // namespace UE::Cameras

#undef UE_API

