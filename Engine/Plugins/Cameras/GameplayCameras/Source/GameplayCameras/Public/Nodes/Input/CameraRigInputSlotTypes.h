// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "CameraRigInputSlotTypes.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

/**
 * General options for an input slot.
 */
USTRUCT()
struct FCameraRigInputSlotParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsAccumulated = true;

	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsPreBlended = true;
};

/**
 * Value clamping parameters.
 */
USTRUCT()
struct FCameraParameterClamping
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Common", meta=(EditCondition="bClampMin"))
	double MinValue = 0;

	UPROPERTY(EditAnywhere, Category ="Common", meta=(EditCondition="bClampMax"))
	double MaxValue = 0;

	UPROPERTY(EditAnywhere, Category="Common")
	bool bClampMin = false;

	UPROPERTY(EditAnywhere, Category="Common")
	bool bClampMax = false;

public:

	/** Clamps the given value. */
	UE_API double ClampValue(double Value) const;

	/** 
	 * Gets the effective min/max values for this struct.
	 * If a bound is disabled, the effective value will be lowest or max
	 * double precision values.
	 */
	UE_API void GetEffectiveClamping(double& OutMinValue, double& OutMaxValue) const;
};

/**
 * Value normalization parameters.
 */
USTRUCT()
struct FCameraParameterNormalization
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Common", meta=(ClampMin=0, EditCondition="bNormalize"))
	double MaxValue = 360;

	UPROPERTY(EditAnywhere, Category="Common")
	bool bNormalize = false;

public:

	/** Normalizes the given value. */
	UE_API double NormalizeValue(double Value) const;
};

#undef UE_API

