// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/Input/Input2DCameraNode.h"

#include "DrivenControlRotationCameraNode.generated.h"

/**
 * An input camera node that provides the yaw and pitch components of the control rotation
 * when it is in the active camera rig, and then "detaches" from it while blending out,
 * applying delta-rotations on the last known active control rotation.
 */
UCLASS()
class UDrivenControlRotationCameraNode : public UInput2DCameraNode
{
	GENERATED_BODY()

protected:
	
	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

