// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"

#include "Input1DCameraNode.generated.h"

/**
 * A camera node that provides a floating-point value to an input slot.
 */
UCLASS(Abstract, MinimalAPI, meta=(
			CameraNodeCategories="Input",
			ObjectTreeGraphSelfPinDirection="Output",
			ObjectTreeGraphDefaultPropertyPinDirection="Input"))
class UInput1DCameraNode : public UCameraNode
{
	GENERATED_BODY()
};

namespace UE::Cameras
{

/**
 * Base class for a 1D input value node evaluator.
 */
class FInput1DCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FInput1DCameraNodeEvaluator)

public:

	/** Get the current input value. */
	double GetInputValue() const { return InputValue; }

protected:

	// FCameraNodeEvaluator interface.
	GAMEPLAYCAMERAS_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

protected:

	double InputValue;
};

}  // namespace UE::Cameras

