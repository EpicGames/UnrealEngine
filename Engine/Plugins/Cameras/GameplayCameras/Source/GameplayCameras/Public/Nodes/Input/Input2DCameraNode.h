// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraNodeEvaluator.h"
#include "Math/MathFwd.h"

#include "Input2DCameraNode.generated.h"

/**
 * A camera node that provides a two-dimensional value to an input slot.
 */
UCLASS(Abstract, MinimalAPI, meta=(
			CameraNodeCategories="Input",
			ObjectTreeGraphSelfPinDirection="Output",
			ObjectTreeGraphDefaultPropertyPinDirection="Input"))
class UInput2DCameraNode : public UCameraNode
{
	GENERATED_BODY()
};

namespace UE::Cameras
{

/**
 * Base class for a 2D input value node evaluator.
 */
class FInput2DCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(GAMEPLAYCAMERAS_API, FInput2DCameraNodeEvaluator)

public:

	/** Get the current input value. */
	FVector2d GetInputValue() const { return InputValue; }

protected:

	// FCameraNodeEvaluator interface.
	GAMEPLAYCAMERAS_API virtual void OnSerialize(const FCameraNodeEvaluatorSerializeParams& Params, FArchive& Ar) override;

protected:

	FVector2d InputValue;
};

}  // namespace UE::Cameras

