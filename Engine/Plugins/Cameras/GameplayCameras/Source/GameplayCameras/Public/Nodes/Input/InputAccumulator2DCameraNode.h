// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BuiltInCameraVariables.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Math/MathFwd.h"
#include "Nodes/Input/CameraRigInputSlotTypes.h"
#include "Nodes/Input/CameraRigInput2DSlot.h"

#include "InputAccumulator2DCameraNode.generated.h"

/**
 * A camera node that accumulates player input.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Input"))
class UInputAccumulator2DCameraNode : public UCameraRigInput2DSlot
{
	GENERATED_BODY()

public:

	/** Input node to accumulate values from. */
	UPROPERTY()
	TObjectPtr<UInput2DCameraNode> InputSlot;

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

