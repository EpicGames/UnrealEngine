// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "Math/MathFwd.h"
#include "Nodes/Input/CameraRigInput2DSlot.h"

#include "InputAxisBinding2DCameraNode.generated.h"

class UInputAction;

/**
 * An input node that reads player input from an input action and accumulates
 * it into a usable input value. Basically a Raw Input Axis Binding node combined
 * with an Input Accumulator node.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Input"))
class UInputAxisBinding2DCameraNode : public UCameraRigInput2DSlot
{
	GENERATED_BODY()

public:

	/** The axis input action(s) to read from. */
	UPROPERTY(EditAnywhere, Category="Input")
	TArray<TObjectPtr<UInputAction>> AxisActions;

	/** A multiplier to use on the input values. */
	UPROPERTY(EditAnywhere, Category="Input")
	FVector2dCameraParameter Multiplier = FVector2d(1.0, 1.0);

	/** Whether the player input is accumulated from frame to frame. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsAccumulated = true;

public:

	// UObject interface.
	virtual void PostLoad() override;
	
protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

