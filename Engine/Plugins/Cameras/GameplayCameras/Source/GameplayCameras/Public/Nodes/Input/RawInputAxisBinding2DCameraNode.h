// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNodeEvaluator.h"
#include "Core/CameraParameters.h"
#include "Math/MathFwd.h"
#include "Nodes/Input/Input2DCameraNode.h"

#include "RawInputAxisBinding2DCameraNode.generated.h"

class UInputAction;

/**
 * An input node that reads player input from an input action.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Input"))
class URawInputAxisBinding2DCameraNode : public UInput2DCameraNode
{
	GENERATED_BODY()

public:

	/** The axis input action(s) to read from. */
	UPROPERTY(EditAnywhere, Category="Input")
	TArray<TObjectPtr<UInputAction>> AxisActions;

	/** A multiplier to use on the input values. */
	UPROPERTY(EditAnywhere, Category="Input")
	FVector2dCameraParameter Multiplier = FVector2d(1.0, 1.0);

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

