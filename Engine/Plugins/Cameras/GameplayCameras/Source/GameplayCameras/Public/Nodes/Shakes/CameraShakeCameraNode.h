// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"
#include "Core/CameraShakeAsset.h"
#include "Core/CameraShakeAssetReference.h"
#include "Core/ICustomCameraNodeParameterProvider.h"

#include "CameraShakeCameraNode.generated.h"

UENUM(BlueprintType)
enum class ECameraShakeEvaluationMode : uint8
{
	Inline,
	VisualLayer
};

/**
 * A camera node that runs a camera shake asset.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Camera Shake Prefab", CameraNodeCategories="Shakes"))
class UCameraShakeCameraNode 
	: public UCameraNode
	, public ICustomCameraNodeParameterProvider
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual void OnPreBuild(FCameraBuildContext& BuildContext) override;
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

	// ICustomCameraNodeParameterProvider interface.
	virtual void GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos) override;

	// UObject interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif  // WITH_EDITOR

public:

	/** The camera shake to run. */
	UPROPERTY(EditAnywhere, Category=Common, meta=(ObjectTreeGraphHidden=true))
	FCameraShakeAssetReference CameraShakeReference;

	UPROPERTY(EditAnywhere, Category=Common)
	ECameraShakeEvaluationMode EvaluationMode = ECameraShakeEvaluationMode::VisualLayer;
};

