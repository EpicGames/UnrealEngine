// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/ShakeCameraNode.h"
#include "UObject/ObjectPtr.h"

#include "CompositeShakeCameraNode.generated.h"

UCLASS()
class UCompositeShakeCameraNode : public UShakeCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const;

public:

	/** The shakes to run simultaneously, in order. */
	UPROPERTY()
	TArray<TObjectPtr<UShakeCameraNode>> Shakes;
};

