// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "Core/ShakeCameraNode.h"
#include "UObject/ObjectPtr.h"

#include "EnvelopeShakeCameraNode.generated.h"

UCLASS()
class UEnvelopeShakeCameraNode : public UShakeCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const;

public:

	UPROPERTY(EditAnywhere, Category="Easing")
	FFloatCameraParameter EaseInTime = 0.2f;

	UPROPERTY(EditAnywhere, Category="Easing")
	FFloatCameraParameter EaseOutTime = 0.2f;

	UPROPERTY(EditAnywhere, Category="Time")
	FFloatCameraParameter TotalTime = 1.f;

	UPROPERTY(EditAnywhere, Category="Common")
	TObjectPtr<UShakeCameraNode> Shake;
};

