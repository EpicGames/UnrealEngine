// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "Core/ShakeCameraNode.h"
#include "Math/PerlinNoise.h"

#include "PerlinNoiseLocationShakeCameraNode.generated.h"

UCLASS()
class UPerlinNoiseLocationShakeCameraNode : public UShakeCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const;

public:

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FFloatCameraParameter AmplitudeMultiplier = 1.f;

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FFloatCameraParameter FrequencyMultiplier = 1.f;

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FInteger32CameraParameter Octaves = 1;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData X;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData Y;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData Z;
};

