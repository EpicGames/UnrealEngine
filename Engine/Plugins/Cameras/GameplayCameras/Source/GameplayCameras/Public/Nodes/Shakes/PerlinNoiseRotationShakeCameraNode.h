// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "Core/ShakeCameraNode.h"
#include "Math/PerlinNoise.h"

#include "PerlinNoiseRotationShakeCameraNode.generated.h"

UCLASS()
class UPerlinNoiseRotationShakeCameraNode : public UShakeCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const;

public:

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FFloatCameraParameter AmplitudeMultiplier = 1.f;

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FFloatCameraParameter FrequencyMultiplier = 1.f;

	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	FInteger32CameraParameter Octaves = 1;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData Yaw;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData Pitch;

	UPROPERTY(EditAnywhere, Category="Common")
	FPerlinNoiseData Roll;
};

