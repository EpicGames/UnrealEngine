// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationContext.h"
#include "Core/CameraNode.h"
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "Core/ObjectTreeGraphObject.h"
#include "GameFramework/BlueprintCameraEvaluationDataRef.h"

#include "BlueprintCameraNode.generated.h"

class UCameraVariableAsset;
class UPropertyBag;

namespace UE::Cameras
{
	class FBlueprintCameraNodeEvaluator;
	class FCameraVariableTable;
	struct FCameraNodeEvaluationParams;
	struct FCameraNodeEvaluationResult;
};

/**
 * The base class for Blueprint camera node evaluators.
 */
UCLASS(MinimalAPI, Blueprintable, Abstract, EditInlineNew, CollapseCategories)
class UBlueprintCameraNodeEvaluator : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintImplementableEvent, Category="Evaluation")
	void InitializeCameraNode();

	/** The main execution callback for the camera node. Call SetCameraPose to affect the result. */
	UFUNCTION(BlueprintImplementableEvent, Category="Evaluation")
	void TickCameraNode(float DeltaTime);

public:

	/**
	 * A utility function that tries to find if an actor owns the evaluation context.
	 * Handles the situation where the evaluation context is an actor component (like a
	 * UGameplayCameraComponent) or an actor itself.
	 */
	UFUNCTION(BlueprintPure, Category="Evaluation", meta=(DeterminesOutputType="ActorClass"))
	AActor* FindEvaluationContextOwnerActor(TSubclassOf<AActor> ActorClass = nullptr) const;

	/** A utility function to get the current camera pose from this node's camera data. */
	UFUNCTION(BlueprintPure, Category="Evaluation")
	FBlueprintCameraPose GetCurrentCameraPose() const;

	/** A utility function to set the current camera pose on this node's camera data. */
	UFUNCTION(BlueprintCallable, Category="Evaluation")
	void SetCurrentCameraPose(const FBlueprintCameraPose& CameraPose);

	/** Assigns the default parameter values of the owning camera rig to the given camera evaluation data. */
	UFUNCTION(BlueprintCallable, Category="Evaluation")
	void SetDefaultOwningCameraRigParameters(FBlueprintCameraEvaluationDataRef TargetCameraData) const;

	/* Gets the owning camera rig for this node. */
	UFUNCTION(BlueprintPure, Category="Evaluation")
	UCameraRigAsset* GetCameraRig() const;

	/** Gets the player controller that the node is running for, if any. */
	UFUNCTION(BlueprintPure, Category="Evaluation")
	APlayerController* GetPlayerController() const;

public:

	using FCameraNodeEvaluatorInitializeParams = UE::Cameras::FCameraNodeEvaluatorInitializeParams;
	using FCameraNodeEvaluationParams = UE::Cameras::FCameraNodeEvaluationParams;
	using FCameraNodeEvaluationResult = UE::Cameras::FCameraNodeEvaluationResult;
	using FCameraEvaluationContext = UE::Cameras::FCameraEvaluationContext;

	/** Initialize this camera node. */
	void NativeInitializeCameraNode(const UBlueprintCameraNode* InBlueprintNode, const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Runs this camera node. */
	void NativeRunCameraNode(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

public:

	// UObject interface.
	virtual UWorld* GetWorld() const override;
#if WITH_EDITOR
	virtual bool ImplementsGetWorld() const override { return true; }
#endif  // WITH_EDITOR

private:

	void SetupExecution(TSharedPtr<const FCameraEvaluationContext> EvaluationContext, FCameraNodeEvaluationResult& OutResult);
	void TeardownExecution();

protected:

	/** Whether this is the first frame of this camera node's lifetime. */
	UPROPERTY(BlueprintReadOnly, Category="Evaluation")
	bool bIsFirstFrame = false;

	/** Whether this camera node is running inside the active camera rig in this layer. */
	UPROPERTY(BlueprintReadOnly, Category="Evaluation")
	bool bIsActiveCameraRig = false;

	/** The owner object of this camera node's evaluation context. */
	UPROPERTY(BlueprintReadOnly, Category="Evaluation")
	TObjectPtr<UObject> EvaluationContextOwner;

	/** The input/output camera data for this frame. */
	UPROPERTY(BlueprintReadOnly, Category="Evaluation")
	FBlueprintCameraEvaluationDataRef CameraData;

public:

	// Deprecated methods.

	UFUNCTION(BlueprintPure, BlueprintGetter, meta=(DeprecatedFunction, DeprecationMessage="Please use GetCurrentCameraPose"))
	FBlueprintCameraPose GetCameraPose() const { return GetCurrentCameraPose(); }

	UFUNCTION(BlueprintSetter, meta=(DeprecatedFunction, DeprecationMessage="Please use SetCurrentCameraPose"))
	void SetCameraPose(UPARAM(DisplayName="Camera Pose") const FBlueprintCameraPose& InCameraPose) { SetCurrentCameraPose(InCameraPose); }

protected:

	// Deprecated fields.

	UPROPERTY(BlueprintGetter=GetCameraPose, BlueprintSetter=SetCameraPose, Category="Evaluation", meta=(DeprecatedProperty, DeprecationMessage="Please use CameraData, or GetCurrentCameraPose and SetCurrentCameraPose"))
	FBlueprintCameraPose CameraPose;

	UPROPERTY(BlueprintReadOnly, Category="Evaluation", meta=(DeprecatedProperty, DeprecationMessage="Please use CameraData"))
	FBlueprintCameraEvaluationDataRef VariableTable;

private:

	TSharedPtr<const UE::Cameras::FCameraEvaluationContext> CurrentContext;

	mutable TWeakObjectPtr<UWorld> WeakCachedWorld;

	const UBlueprintCameraNode* BlueprintNode = nullptr;
};

/**
 * A camera node that runs arbitrary Blueprint logic.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Transform"))
class UBlueprintCameraNode
	: public UCameraNode
	, public ICustomCameraNodeParameterProvider
{
	GENERATED_BODY()

public:

	UBlueprintCameraNode(const FObjectInitializer& ObjInit);

protected:

	// UCameraNode interface.
	virtual void OnPreBuild(FCameraBuildContext& BuildContext) override;
	virtual void OnBuild(FCameraObjectBuildContext& BuildContext) override;
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

	// IObjectTreeGraphObject interface.
#if WITH_EDITOR
	virtual EObjectTreeGraphObjectSupportFlags GetSupportFlags(FName InGraphName) const override;
	virtual void GetGraphNodeName(FName InGraphName, FText& OutName) const override;
#endif  // WITH_EDITOR

	// ICustomCameraNodeParameterProvider interface.
	virtual void GetCustomCameraNodeParameters(FCameraNodeParameterInfos& OutParameterInfos) override;

	// UObject interface.
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	const UBlueprintCameraNodeEvaluator* GetCameraNodeEvaluatorTemplate() const { return CameraNodeEvaluatorTemplate; }

private:

	void RebuildOverrides();

#if WITH_EDITOR
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
#endif

private:

	/** The camera node evaluator to instantiate and run. */
	UPROPERTY(Instanced, EditAnywhere, Category=Common)
	TObjectPtr<UBlueprintCameraNodeEvaluator> CameraNodeEvaluatorTemplate;

	/** Overrides for the evaluator instance. */
	UPROPERTY()
	FCustomCameraNodeParameters CameraNodeEvaluatorOverrides;

	/** Whether this camera node should run in the editor. */
	UPROPERTY(EditAnywhere, Category=Common)
	bool bRunInEditor = false;

	// Deprecated.
	
	UPROPERTY()
	TSubclassOf<UBlueprintCameraNodeEvaluator> CameraNodeEvaluatorClass_DEPRECATED;

	friend class UE::Cameras::FBlueprintCameraNodeEvaluator;
};

