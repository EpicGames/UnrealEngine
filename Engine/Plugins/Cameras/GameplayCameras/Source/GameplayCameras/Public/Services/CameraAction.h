// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CameraAction.generated.h"

#define UE_API GAMEPLAYCAMERAS_API

namespace UE::Cameras
{
	class FCameraActionEvaluator;
	struct FCameraActionEvaluatorBuilder;
}

using FCameraActionEvaluatorPtr = UE::Cameras::FCameraActionEvaluator*;

/**
 * The base class for a camera action. A camera action is a potentially long-running operation
 * that applies to camera rig instances updating in the main layer.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, DefaultToInstanced, EditInlineNew)
class UCameraAction : public UObject
{
	GENERATED_BODY()
	
public:

	/**
	 * Whether this action will be cloned onto new camera rigs as they get activated on the
	 * main layer.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	bool bPropagateToNewCameraRigs = false;

	/**
	 * The time, in seconds, after which the action should be automatically stopped. Zero or negative time means that
	 * this action never times out (it either completes on its own, or gets stopped by the user).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Common, meta=(ExposeOnSpawn=true))
	float TimeOut = 0.f;
	
public:

	using FCameraActionEvaluatorBuilder = UE::Cameras::FCameraActionEvaluatorBuilder;

	/** Builds the evaluator for this action. */
	UE_API FCameraActionEvaluatorPtr BuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const;

protected:

	/** Builds the evaluator for this action. */
	virtual FCameraActionEvaluatorPtr OnBuildEvaluator(FCameraActionEvaluatorBuilder& Builder) const { return nullptr; }
};

#undef UE_API

