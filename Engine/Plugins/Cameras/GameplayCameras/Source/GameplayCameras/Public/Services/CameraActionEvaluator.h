// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraObjectRtti.h"
#include "GameplayCameras.h"
#include "UObject/ObjectPtr.h"

#define UE_API GAMEPLAYCAMERAS_API

class FReferenceCollector;
class UCameraAction;

namespace UE::Cameras
{

class FCameraActionEvaluator;
class FCameraActionScope;
struct FCameraNodeEvaluationParams;
struct FCameraNodeEvaluationResult;

#if UE_GAMEPLAY_CAMERAS_DEBUG
struct FCameraDebugBlockBuilder;
struct FCameraDebugBlockBuildParams;
#endif

/** 
 * Flags for camera action evaluators.
 */
enum class ECameraActionEvaluatorFlags
{
	None = 0,
	CloneToNewActionScopes = 1,

	Default = None
};
ENUM_CLASS_FLAGS(ECameraActionEvaluatorFlags)

/**
 * Structure for initializing a new camera action evaluator.
 */
struct FCameraActionEvaluatorInitializeParams
{
	/** The scope in which the action will be running. */
	TSharedPtr<FCameraActionScope> Scope;
};

/**
 * Structure for tearing down a camera action evaluator.
 */
struct FCameraActionEvaluatorTeardownParams
{
	/** The scope in which the action was be running. */
	TSharedPtr<FCameraActionScope> Scope;
};

/**
 * Parameter structure for updating a camera action evaluator.
 */
struct FCameraActionEvaluationParams
{
	FCameraActionEvaluationParams(const FCameraNodeEvaluationParams& InParams)
		: EvaluationParams(InParams)
	{}

	/** Information about the evaluation pass that is currently happening. */
	const FCameraNodeEvaluationParams& EvaluationParams;
	/** The scope in which the action is running. */
	TSharedPtr<FCameraActionScope> Scope;
};

/**
 * Result of updating the camera action.
 */
struct FCameraActionEvaluationResult
{
	FCameraActionEvaluationResult(FCameraNodeEvaluationResult& InResult)
		: Result(InResult)
	{}

	/** The evaluation result that pass through the camera rig instance. */
	FCameraNodeEvaluationResult& Result;

	/** Whether this action has ended. */
	bool bIsActionFinished = false;
};

/**
 * Parameter structure for serializing a camera action evaluator.
 */
struct FCameraActionEvaluatorSerializeParams
{
};

/**
 * Structure for building new camera actions.
 */
struct FCameraActionEvaluatorBuilder
{
	FCameraActionEvaluatorBuilder()
	{}

	template<typename EvaluatorType, typename ...ArgTypes>
	EvaluatorType* BuildEvaluator(ArgTypes&&... InArgs);

private:

	TSharedPtr<FCameraActionEvaluator> BuiltEvaluator;

	friend class FCameraActionScope;
};

/**
 * Base class for a camera action evaluator.
 */
class FCameraActionEvaluator
{
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI_BASE(GAMEPLAYCAMERAS_API, FCameraActionEvaluator)

public:

	FCameraActionEvaluator() = default;
	virtual ~FCameraActionEvaluator() = default;

	/** Initialize this action. */
	void Initialize(const FCameraActionEvaluatorInitializeParams& Params, FCameraActionEvaluationResult& OutResult);

	/** Tear down this action. */
	void Teardown(const FCameraActionEvaluatorTeardownParams& Params);

	/** Collect referenced UObjects for this action. */
	void AddReferencedObjects(FReferenceCollector& Collector);

	/** Run this action before its scope evaluates. */
	void PreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult);

	/** Run this action after its scope has evaluated. */
	void PostScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult);

	/** Serializes the state of this action. */
	void Serialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar);

	/** Gets the flags for this evaluator. */
	ECameraActionEvaluatorFlags  GetActionEvaluatorFlags() const { return PrivateFlags; }

	/** Get the camera action. */
	const UCameraAction* GetCameraAction() const { return PrivateCameraAction; }

	/** Get the camera action. */
	template<typename CameraAction>
	const CameraAction* GetCameraActionAs() const
	{
		return Cast<CameraAction>(PrivateCameraAction);
	}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this action evaluator. */
	void BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

public:

	// Internal API.
	void SetPrivateCameraAction(TObjectPtr<const UCameraAction> InCameraAction);

protected:

	/** Initialize this action. */
	virtual void OnInitialize(const FCameraActionEvaluatorInitializeParams& Params, FCameraActionEvaluationResult& OutResult) {}

	/** Tear down this action. */
	virtual void OnTeardown(const FCameraActionEvaluatorTeardownParams& Params) {}

	/** Collect referenced UObjects for this action. */
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) {}

	/** Run this action before its scope evaluates. */
	virtual void OnPreScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult) {}

	/** Run this action after its scope has evaluated. */
	virtual void OnPostScopeRun(const FCameraActionEvaluationParams& Params, FCameraActionEvaluationResult& OutResult) {}

	/** Serializes the state of this evaluator. */
	virtual void OnSerialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar) {}

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this action evaluator. */
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) {}
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

protected:

	/**
	 * Sets the flags for this evaluator.
	 * Can be called from the constructor, or during OnInitialize().
	 */
	UE_API void SetActionEvaluatorFlags(ECameraActionEvaluatorFlags InFlags);

	/**
	 * Adds flags for this evaluator.
	 * Can be called from the constructor, or during OnInitialize().
	 */
	UE_API void AddActionEvaluatorFlags(ECameraActionEvaluatorFlags InFlags);

private:

	/** The camera action to run. */
	TObjectPtr<const UCameraAction> PrivateCameraAction;

	/** The flags for this evaluator. */
	ECameraActionEvaluatorFlags PrivateFlags = ECameraActionEvaluatorFlags::Default;
};

template<typename EvaluatorType, typename ...ArgTypes>
EvaluatorType* FCameraActionEvaluatorBuilder::BuildEvaluator(ArgTypes&&... InArgs)
{
	ensure(!BuiltEvaluator.IsValid());
	TSharedRef<FCameraActionEvaluator> Evaluator = MakeShared<EvaluatorType>(Forward<ArgTypes>(InArgs)...);
	BuiltEvaluator = Evaluator;
	return Evaluator->CastThisChecked<EvaluatorType>();
}

}  // namespace UE::Cameras

// Utility macros for declaring and defining camera actions.
//
#define UE_DECLARE_CAMERA_ACTION_EVALUATOR(ApiDeclSpec, ClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, FCameraActionEvaluator)

#define UE_DECLARE_CAMERA_ACTION_EVALUATOR_EX(ApiDeclSpec, ClassName, BaseClassName)\
	UE_GAMEPLAY_CAMERAS_DECLARE_RTTI(ApiDeclSpec, ClassName, BaseClassName)

#define UE_DEFINE_CAMERA_ACTION_EVALUATOR(ClassName)\
	UE_GAMEPLAY_CAMERAS_DEFINE_RTTI(ClassName)

#undef UE_API

