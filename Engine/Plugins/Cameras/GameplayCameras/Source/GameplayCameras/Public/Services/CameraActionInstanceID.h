// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "CameraActionInstanceID.generated.h"

namespace UE::Cameras
{
	class FCameraActionScope;
	class FCameraActionService;
}

/**
 * The ID of a camera action instance.
 */
USTRUCT(BlueprintType)
struct FCameraActionInstanceID
{
	GENERATED_BODY()

public:

	FCameraActionInstanceID() : Value(INVALID) {}

	bool IsValid() const { return Value != INVALID; }

	explicit operator bool() const { return IsValid(); }

public:

	friend bool operator==(FCameraActionInstanceID A, FCameraActionInstanceID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCameraActionInstanceID A, FCameraActionInstanceID B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(FCameraActionInstanceID In)
	{
		return In.Value;
	}

	friend FArchive& operator<< (FArchive& Ar, FCameraActionInstanceID& In)
	{
		Ar << In.Value;
		return Ar;
	}

private:

	FCameraActionInstanceID(uint32 InValue) 
		: Value(InValue)
	{}

	static const uint32 INVALID = uint32(-1);

	UPROPERTY()
	uint32 Value;

	friend class UE::Cameras::FCameraActionScope;
	friend class UE::Cameras::FCameraActionService;
};

/**
 * Blueprint functions for camera action instance IDs.
 */
UCLASS(MinimalAPI)
class UCameraActionInstanceFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** 
	 * Whether the given camera action instance ID is valid.
	 * A valid ID doesn't necessarily correspond to a camera action instance that is still running.
	 */
	UFUNCTION(BlueprintPure, Category="Camera")
	static bool IsValid(FCameraActionInstanceID InstanceID) { return InstanceID.IsValid(); }
};

