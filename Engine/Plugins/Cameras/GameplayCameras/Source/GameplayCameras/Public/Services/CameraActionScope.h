// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraRigInstanceID.h"
#include "GameplayCameras.h"
#include "Services/CameraActionInstanceID.h"
#include "Templates/SharedPointer.h"

#define UE_API GAMEPLAYCAMERAS_API

class FReferenceCollector;
class UCameraAction;
class UCameraRigAsset;

namespace UE::Cameras
{

class FCameraActionEvaluator;
class FCameraEvaluationContext;
class FCameraNodeEvaluator;
class FCameraNodeEvaluatorHierarchy;
struct FCameraActionEvaluationResult;
struct FCameraActionEvaluatorSerializeParams;
struct FCameraNodeEvaluationParams;
struct FCameraNodeEvaluationResult;
struct FCameraRigEvaluationInfo;

#if UE_GAMEPLAY_CAMERAS_DEBUG
struct FCameraDebugBlockBuildParams;
struct FCameraDebugBlockBuilder;
#endif

/**
 * A camera action scope, which is a logical container for zero or more camera actions
 * to be run for a given camera rig instance.
 */
class FCameraActionScope : public TSharedFromThis<FCameraActionScope>
{
public:

	/** Adds a new camera action. */
	FCameraActionInstanceID StartAction(const UCameraAction* InCameraAction);
	
	/** Returns whether the given action instance is running. */
	bool IsActionRunning(const FCameraActionInstanceID InInstanceID) const;

	/** Gets a running action. */
	FCameraActionEvaluator* GetAction(const FCameraActionInstanceID InInstanceID) const;

	/** Stops a running camera action. */
	bool StopAction(const FCameraActionInstanceID InInstanceID);

public:

	/** Initialize this scope and associate it with the given camera rig. */
	void Initialize(const FCameraRigEvaluationInfo& CameraRigInfo);

	/** Gets the evaluation context that the camera rig instance runs inside of. */
	TSharedPtr<const FCameraEvaluationContext> GetEvaluationContext() const { return EvaluationContext.Pin(); }

	/** Gets the camera rig associated with this action scope. */
	const UCameraRigAsset* GetCameraRig() const { return CameraRig; }

	/** Gets the root evaluator for the camera rig instance that this action acts upon. */
	FCameraNodeEvaluator* GetChildEvaluator() const { return ChildEvaluator; }

	/** Gets the evaluation info of the camera rig instance that this action acts upon. */
	FCameraRigEvaluationInfo GetCameraRigEvaluationInfo() const;

public:

	/** Collect referenced UObjects for all actions in the scope. */
	void AddReferencedObjects(FReferenceCollector& Collector);

	/** Runs the pre-scope evaluation for all actions in the scope. */
	void PreScopeRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Runs the post-scope evaluation for all actions in the scope. */
	void PostScopeRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult);

	/** Serializes the state of all actions in the scope. */
	void Serialize(const FCameraActionEvaluatorSerializeParams& Params, FArchive& Ar);

#if UE_GAMEPLAY_CAMERAS_DEBUG
	/** Called to create debug blocks for this action scope. */
	void BuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder);
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	struct FActionInfo;

	bool PrepareActionForRun(FActionInfo& ActionInfo, FCameraActionEvaluationResult& OutResult);
	bool HasActionTimedOut(float DeltaTime, FActionInfo& ActionInfo);
	void StopAction(FActionInfo& ActionInfo);

private:

	enum EActionState
	{
		Uninitialized,
		Running,
		Finished
	};

	struct FActionInfo
	{
		TSharedPtr<FCameraActionEvaluator> Evaluator;
		FCameraActionInstanceID InstanceID;
		float ElapsedTime = 0.f;
		EActionState State = EActionState::Uninitialized;
	};

	FCameraRigInstanceID CameraRigInstanceID;
	TWeakPtr<const FCameraEvaluationContext> EvaluationContext;
	TObjectPtr<const UCameraRigAsset> CameraRig;
	const FCameraNodeEvaluationResult* ChildResult = nullptr;
	FCameraNodeEvaluator* ChildEvaluator = nullptr;

	TArray<FActionInfo> Actions;
	uint32 NextActionID = 0;
};

}  // namespace UE::Cameras

#undef UE_API

