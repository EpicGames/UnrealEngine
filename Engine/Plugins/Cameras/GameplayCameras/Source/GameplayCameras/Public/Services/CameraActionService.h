// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Core/CameraEvaluationService.h"
#include "Services/CameraActionInstanceID.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/SubclassOf.h"

#define UE_API GAMEPLAYCAMERAS_API

class UCameraAction;
class UCameraRigAsset;

namespace UE::Cameras
{

class FCameraActionScope;
class FCameraEvaluationContext;
class FRootCameraNodeEvaluator;
struct FCameraNodeEvaluationParams;
struct FCameraNodeEvaluationResult;

/**
 * An evaluation service for running "camera actions", which allow manipuilating running camera
 * rig instances, such as making them turn towards a given point, freeze them, and so on.
 */
class FCameraActionService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(UE_API, FCameraActionService)

public:

	~FCameraActionService();

	/** Starts a new camera action. */
	FCameraActionInstanceID StartAction(const UCameraAction* CameraAction);

	/** Returns whether a given camera action instance is running. */
	bool IsActionRunning(const FCameraActionInstanceID InInstanceID) const;

	/** Stops a given camera action instance. */
	bool StopAction(const FCameraActionInstanceID InInstanceID);

	/** Stops all camera action instances of a given class. */
	bool StopAllActionsOfClass(TSubclassOf<UCameraAction> InActionClass);

public:

	/**
	 * Returns how many actual actions are running for the given instance ID.
	 *
	 * This returns 1 for a running non-propagating action.
	 * For a propagating action, this returns the number of running camera rig instances that the action is currently
	 * affecting, i.e. the number of running clones.
	 * If an action has been started but the camera system hasn't updated yet, it returns 0 since action evaluators
	 * are created only on the first update.
	 */
	int32 GetNumScopeActions(const FCameraActionInstanceID InInstanceID) const;

protected:

	// FCameraEvaluationService interface.
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) override;
	virtual void OnPostCameraDirectorUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;
	virtual void OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;
	virtual void OnTeardown(const FCameraEvaluationServiceTeardownParams& Params) override;
#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	struct FActionSetInfo;

	void OnRootNodeCameraRigEvent(const FRootCameraNodeCameraRigEvent& InEvent);

	void CloneAction(FActionSetInfo& ActionSet, TSharedRef<FCameraActionScope> NewActionScope);

	void CleanUpActions();

private:

	struct FActionInfo
	{
		TWeakPtr<FCameraActionScope> ActionScope = nullptr;
		FCameraActionInstanceID ActionID;
	};

	struct FActionSetInfo
	{
		TObjectPtr<const UCameraAction> Data;
		TArray<FActionInfo, TInlineAllocator<4>> Actions;
		FCameraActionInstanceID InstanceID;
		bool bIsPending = false;
	};

	FRootCameraNodeEvaluator* RootNodeEvaluator = nullptr;

	TArray<FActionSetInfo> ActionSets;

	uint32 NextActionSetID = 0;

	bool bHasAnyNewActiveCameraRigs = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	friend class FCameraActionServiceDebugBlock;
#endif
};

} // namespace UE::Cameras

#undef UE_API

