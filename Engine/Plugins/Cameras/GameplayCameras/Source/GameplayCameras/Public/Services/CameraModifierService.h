// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationService.h"
#include "Core/CameraRigInstanceID.h"

class UCameraRigAsset;

namespace UE::Cameras
{

class FCameraEvaluationContext;

/**
 * An evaluation service for running "camera modifiers", which are context-less camera rig instances
 * meant to run additively in the camera system.
 */
class FCameraModifierService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(GAMEPLAYCAMERAS_API, FCameraModifierService)

public:

	/** Blend stack order key for modifiers. */
	static int32 GetFirstBlendStackOrderKey() { return FirstBlendStackOrderKey; }

	/** Starts a new instance of the given camera rig, using a "null" context. */
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartCameraModifierRig(const UCameraRigAsset* CameraRig, ECameraRigLayer Layer, int32 OrderKey = 0);

	/** Starts a new instance of the given camera rig, using the given context. */
	GAMEPLAYCAMERAS_API FCameraRigInstanceID StartCameraModifierRig(const UCameraRigAsset* CameraRig, TSharedRef<FCameraEvaluationContext> EvaluationContext, ECameraRigLayer Layer, int32 OrderKey = 0);

	/** Stops a running instance of a camera rig. */
	GAMEPLAYCAMERAS_API void StopCameraModifierRig(FCameraRigInstanceID CameraRigID, bool bImmediately = false);

protected:

	// FCameraEvaluationService interface.
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) override;
	virtual void OnTeardown(const FCameraEvaluationServiceTeardownParams& Params) override;

private:

	void EnsureModifierContextCreated();

private:

	static const int32 FirstBlendStackOrderKey = 100;

	FCameraSystemEvaluator* Evaluator = nullptr;

	TSharedPtr<FCameraEvaluationContext> ModifierContext;
};

}  // namespace UE::Cameras

