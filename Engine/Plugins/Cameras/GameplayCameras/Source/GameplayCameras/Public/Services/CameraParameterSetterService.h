// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationService.h"
#include "Core/CameraVariableSetter.h"

namespace UE::Cameras
{

/**
 * A camera system service that handles running camera parameter setters.
 */
class FCameraParameterSetterService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(GAMEPLAYCAMERAS_API, FCameraParameterSetterService)

public:

	FCameraParameterSetterService();

	template<typename ValueType>
	FCameraVariableSetterHandle AddCameraVariableSetter(const TCameraVariableSetter<ValueType>& InSetter);

	void StopCameraVariableSetter(const FCameraVariableSetterHandle& InHandle, bool bImmediately = false);

	void ApplyCameraVariableSetters(FCameraVariableTable& OutVariableTable);


protected:

	// FCameraEvaluationService interface.
	virtual void OnPreUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;

private:

	void UpdateCameraVariableSetters(float DeltaTime);

private:

	struct FVariableSetterEntry
	{
		FCameraVariableSetterPtr Setter;
		FCameraVariableSetterHandle ThisHandle;
	};

	FCameraSystemEvaluator* Evaluator = nullptr;

	using FCameraVariableSetters = TSparseArray<FVariableSetterEntry>;
	FCameraVariableSetters VariableSetters;
	uint32 NextVariableSetterSerial = 0;
};

template<typename ValueType>
FCameraVariableSetterHandle FCameraParameterSetterService::AddCameraVariableSetter(const TCameraVariableSetter<ValueType>& InSetter)
{
	FSparseArrayAllocationInfo NewAllocation = VariableSetters.AddUninitialized();
	FCameraVariableSetterHandle NewHandle(NewAllocation.Index, NextVariableSetterSerial++);
	new(NewAllocation.Pointer) FVariableSetterEntry{ InSetter, NewHandle };
	return NewHandle;
}

}  // namespace UE::Cameras

