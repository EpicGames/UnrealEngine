// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraTypes.h"
#include "Core/CameraNode.h"
#include "Core/CameraEvaluationService.h"
#include "Core/CameraShakeInstanceID.h"

#include "CameraShakeService.generated.h"

class UCameraRigAsset;
class UCameraShakeAsset;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FCameraShakeServiceCameraNodeEvaluator;

/**
 * Parameters for starting a camera shake.
 */
struct FStartCameraShakeParams
{
	/** The camera shake to play. */
	const UCameraShakeAsset* CameraShake = nullptr;
	/** The intensity to use for the camera shake. */
	float ShakeScale = 1.f;
	/** The play space to modify the result by */
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;
	/** The custom space to use for the shake. Only used when PlaySpace is UserDefined. */
	FRotator UserPlaySpaceRotation;
};

/**
 * A camera system service that handles running camera shakes in the visual layer.
 */
class FCameraShakeService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(GAMEPLAYCAMERAS_API, FCameraShakeService)

public:

	/** Starts a new camera shake. */
	GAMEPLAYCAMERAS_API FCameraShakeInstanceID StartCameraShake(const FStartCameraShakeParams& Params);

	/** Checks if a camera shake is running. */
	GAMEPLAYCAMERAS_API bool IsCameraShakePlaying(FCameraShakeInstanceID InInstanceID) const;

	/** Stops a running camera shake. */
	GAMEPLAYCAMERAS_API bool StopCameraShake(FCameraShakeInstanceID InInstanceID, bool bImmediately = false);

	/**
	 * Requests that a given camera shake runs. Multiple requests for the same
	 * camera shake are combined, and the camera shake keeps running as long as
	 * there are requests active. Requests are cleared every frame.
	 */
	GAMEPLAYCAMERAS_API void RequestCameraShakeThisFrame(const FStartCameraShakeParams& Params);

protected:

	// FCameraEvaluationService interface.
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) override;
	virtual void OnTeardown(const FCameraEvaluationServiceTeardownParams& Params) override;

private:

	void EnsureShakeContextCreated();

private:

	/** The system running this service. */
	FCameraSystemEvaluator* Evaluator = nullptr;

	/** The evaluation context used for running the shakes. */
	TSharedPtr<FCameraEvaluationContext> ShakeContext;

	/** The camera rig used to create the shake container. */
	TObjectPtr<UCameraRigAsset> ShakeContainerRig = nullptr;

	/** The shake container evaluator. */
	FCameraShakeServiceCameraNodeEvaluator* ShakeEvaluator = nullptr;
};

}  // namespace UE::Cameras

UCLASS(MinimalAPI, Hidden)
class UCameraShakeServiceCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;
};

