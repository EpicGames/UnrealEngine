// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationService.h"
#include "Core/CameraIKAim.h"
#include "Core/CameraRigTransition.h"

namespace UE::Cameras
{

class FCameraEvaluationContext;
struct FCameraRigEvaluationInfo;

/**
 * A camera evaluation service responsible for initializing the orientation of newly activated camera rigs.
 */
class FOrientationInitializationService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(GAMEPLAYCAMERAS_API, FOrientationInitializationService)

public:

	/**
	 * Override the yaw/pitch to preserve on any camera rig activated this frame. This will affect camera rigs
	 * who have the "Preserve Yaw/Pitch" policy.
	 */
	GAMEPLAYCAMERAS_API void SetYawPitchPreservationOverride(const FRotator3d& InOrientation);

	/**
	 * Override the target location to preserve on any camera rig activated this frame. This will affect camera rigs
	 * who have the "Preserve Absolute/Relative Target" policies.
	 */
	GAMEPLAYCAMERAS_API void SetTargetPreservationOverride(const FVector3d& InTarget);

protected:

	// FCameraEvaluationService interface.
	virtual void OnInitialize(const FCameraEvaluationServiceInitializeParams& Params) override;
	virtual void OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;
	virtual void OnRootCameraNodeEvent(const FRootCameraNodeCameraRigEvent& InEvent) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void TryInitializeContextYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo);
	void TryPreserveYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo);
	void TryInitializeYawPitch(const FCameraRigEvaluationInfo& CameraRigInfo, TOptional<double> Yaw, TOptional<double> Pitch);
	void TryPreserveTarget(const FCameraRigEvaluationInfo& CameraRigInfo, bool bUseRelativeTarget);

private:

	FCameraSystemEvaluator* Evaluator = nullptr;

	TWeakPtr<FCameraEvaluationContext> PreviousEvaluationContext;
	FVector3d PreviousContextLocation;
	FRotator3d PreviousContextRotation;
	bool bHasPreviousContextTransform = false;

	TOptional<FRotator3d> YawPitchPreservationOverride;
	TOptional<FVector3d> TargetPreservationOverride;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FVector3d DebugLastEvaluatedTarget;
	FCameraIKAimDebugInfo LastAimDebugInfo;

	friend class FOrientationInitializationDebugBlock;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG
};

}  // namespace UE::Cameras

