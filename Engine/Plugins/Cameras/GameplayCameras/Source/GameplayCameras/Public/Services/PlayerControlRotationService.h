// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraEvaluationService.h"
#include "Debug/CameraDebugClock.h"
#include "Debug/CameraDebugGraph.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtr.h"

class APlayerController;
class UEnhancedInputComponent;
class UInputAction;
struct FEnhancedInputActionValueBinding;

namespace UE::Cameras
{

class FCameraEvaluationContext;
class FCameraSystemEvaluator;

/**
 * Parameter structure for the player control rotation service.
 */
struct FPlayerControlRotationParams
{
	/** The input magnitude below which we can change control rotation. */
	double AxisActionMagnitudeThreshold = 0;
	/** The input angular change speed over which we can change control rotation. */
	double AxisActionAngularSpeedThreshold = 0;
	/** Whether the service should set control rotation on the active context's player controller. */
	bool bApplyControlRotation = true;
	/** Input actions representing how the player can move their pawn. */
	TArray<TObjectPtr<UInputAction>> AxisActions;
};

/**
 * An evaluation service that manages the player's control rotation based on 
 * what's going on with cameras.
 */
class FPlayerControlRotationEvaluationService : public FCameraEvaluationService
{
	UE_DECLARE_CAMERA_EVALUATION_SERVICE(GAMEPLAYCAMERAS_API, FPlayerControlRotationEvaluationService)

public:

	/** Create a new player control rotation service. */
	GAMEPLAYCAMERAS_API FPlayerControlRotationEvaluationService();

	/** Create a new player control rotation service, setting its parameters immediately. */
	GAMEPLAYCAMERAS_API FPlayerControlRotationEvaluationService(const FPlayerControlRotationParams& InParams);

public:

	/** Gets the parameters for managing player control rotation. */
	const FPlayerControlRotationParams& GetParameters() const { return ServiceParams; }
	/** Sets the parameters for managing player control rotation. */
	void SetParameters(const FPlayerControlRotationParams& InParams) { ServiceParams = InParams; }

	/** Gets the last evaluated control rotation. */
	const FRotator3d& GetCurrentControlRotation() const { return CurrentControlRotation; }
	/** Gets whether the control rotation was last frozen. */
	bool IsControlRotationFrozen() const { return bIsFrozen; }

protected:

	virtual void OnPostUpdate(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	virtual void OnBuildDebugBlocks(const FCameraDebugBlockBuildParams& Params, FCameraDebugBlockBuilder& Builder) override;
#endif  // UE_GAMEPLAY_CAMERAS_DEBUG

private:

	void MonitorActiveContext(TSharedPtr<const FCameraEvaluationContext> ActiveContext);
	void BindActionValues(UEnhancedInputComponent* InputComponent);
	void UnbindActionValues();

	void UpdateControlRotation(const FCameraEvaluationServiceUpdateParams& Params, FCameraEvaluationServiceUpdateResult& OutResult);

private:

	FPlayerControlRotationParams ServiceParams;

	TWeakObjectPtr<UEnhancedInputComponent> WeakInputComponent;
	TArray<FEnhancedInputActionValueBinding*> AxisBindings;

	FVector2d PreviousAxisBindingValue;
	FRotator3d CameraRotation;
	FRotator3d FrozenControlRotation;
	FRotator3d CurrentControlRotation;
	bool bIsFrozen = false;

#if UE_GAMEPLAY_CAMERAS_DEBUG
	FTransform DebugPawnTransform;
	FString DebugFreezeReason;
	bool bDebugDidApplyControlRotation;
	TCameraDebugGraph<1> AxisActionAngularSpeedGraph;
	FCameraDebugClock AxisActionValueClock;
#endif
};

}  // namespace UE::Cameras

