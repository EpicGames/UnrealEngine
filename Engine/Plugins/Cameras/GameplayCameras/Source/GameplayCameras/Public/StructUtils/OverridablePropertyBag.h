// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Guid.h"
#include "StructUtils/PropertyBag.h"

#include "OverridablePropertyBag.generated.h"

/**
* Extended version of FInstancedPropertyBag to support "overriden properties" with an extra array of
* overriden parameter IDs.
*/
USTRUCT()
struct FInstancedOverridablePropertyBag : public FInstancedPropertyBag
{
	GENERATED_BODY()

public:

	/**
	 * Gets the IDs of overriden properties, i.e. properties that have override values.
	 */
	template<typename ContainerType>
	void GetOverridenPropertyIDs(ContainerType& OutPropertyIDs) const;

	/**
	 * Gets the IDs of overriden properties, i.e. properties that have override values.
	 */
	TConstArrayView<FGuid> GetOverridenPropertyIDs() const { return OverridenPropertyIDs; }

	/**
	 * Returns whether the given property is overriden.
	 */
	GAMEPLAYCAMERAS_API bool IsPropertyOverriden(const FGuid& InPropertyID) const;

	/**
	 * Sets whether the given property is overriden by setting the matching override property's value.
	 */
	GAMEPLAYCAMERAS_API void SetPropertyOverriden(const FGuid& InPropertyID, bool bIsOverriden);

	/**
	 * Returns whether the given property is overriden.
	 */
	GAMEPLAYCAMERAS_API bool IsPropertyOverriden(const FName InPropertyName) const;

	/**
	 * Sets whether the given property is overriden by setting the matching override property's value.
	 */
	GAMEPLAYCAMERAS_API void SetPropertyOverriden(const FName InPropertyName, bool bIsOverriden);

public:

	/**
	 * Migrate this property bag to the layout of the given one. Overriden values in this property bag
	 * will be preserved. Other values will adopt the values of the given new bag.
	 */
	GAMEPLAYCAMERAS_API void MigrateToNewBagInstanceWithOverrides(const FInstancedPropertyBag& NewBagInstance);

public:

	// Internal API.

	bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	bool Serialize(FArchive& Ar);

private:

	// List of properties that have override values.
	UPROPERTY()
	TArray<FGuid> OverridenPropertyIDs;
};

template<typename ContainerType>
void FInstancedOverridablePropertyBag::GetOverridenPropertyIDs(ContainerType& OutPropertyIDs) const
{
	OutPropertyIDs.Reserve(OutPropertyIDs.Num() + OverridenPropertyIDs.Num());

	for (const FGuid& PropertyID : OverridenPropertyIDs)
	{
		OutPropertyIDs.Add(PropertyID);
	}
}

template<>
struct TStructOpsTypeTraits<FInstancedOverridablePropertyBag> : public TStructOpsTypeTraits<FInstancedPropertyBag>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
};

namespace FOverridablePropertyBagCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		FixSerializer = 1,
		MissingPropertyFlags = 2,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	const FGuid GUID(0x5426C227, 0x4B3145B2, 0x9B9BED1F, 0x327FB126);
}

