// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigTransition.h"

#include "DefaultTransitionConditions.generated.h"

/**
 * A transition condition that matches the previous and/or next camera rigs against
 * one of the owning camera asset's rigs.
 */
UCLASS(MinimalAPI)
class UIsCameraRigTransitionCondition 
	: public UCameraRigTransitionCondition
{
	GENERATED_BODY()

public:

	/** Passes if null, or equal to the previous camera rig. */
	UPROPERTY(EditAnywhere, Category=Transition, meta=(UseSelfCameraRigPicker=true, ObjectTreeGraphHidden=true))
	TObjectPtr<UCameraRigAsset> PreviousCameraRig;

	/** Passes if null, or equal to the next camera rig. */
	UPROPERTY(EditAnywhere, Category=Transition, meta=(UseSelfCameraRigPicker=true, ObjectTreeGraphHidden=true))
	TObjectPtr<UCameraRigAsset> NextCameraRig;

protected:

	// UCameraRigTransitionCondition interface.
	virtual bool OnTransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const override;
};

