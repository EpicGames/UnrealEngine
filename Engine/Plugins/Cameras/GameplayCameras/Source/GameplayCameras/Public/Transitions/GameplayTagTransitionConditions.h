// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigTransition.h"
#include "GameplayTagContainer.h"

#include "GameplayTagTransitionConditions.generated.h"

/**
 * A transition condition that matches the gameplay tags on the previous and next
 * camera rigs and assets. Both queries need to pass. Empty queries pass by default.
 */
UCLASS(MinimalAPI)
class UGameplayTagTransitionCondition
	: public UCameraRigTransitionCondition
{
	GENERATED_BODY()

public:

	/** The gameplay tags to look for on the previous camera rig/asset. */
	UPROPERTY(EditAnywhere, Category=Transition)
	FGameplayTagQuery PreviousGameplayTagQuery;

	/** The gameplay tags to look for on the next camera rig/asset. */
	UPROPERTY(EditAnywhere, Category=Transition)
	FGameplayTagQuery NextGameplayTagQuery;

protected:

	// UCameraRigTransitionCondition interface.
	virtual bool OnTransitionMatches(const FCameraRigTransitionConditionMatchParams& Params) const override;
};

