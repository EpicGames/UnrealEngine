// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraValueInterpolator.h"

#include "AccelerationDecelerationValueInterpolator.generated.h"

/**
 * A value interpolator that first accelerates up to a maximum speed, and then decelerates
 * before reaching the target value.
 */
UCLASS(meta=(DisplayName="Acceleration-Deceleration"))
class UAccelerationDecelerationValueInterpolator : public UCameraValueInterpolator
{
	GENERATED_BODY()

	UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()

public:

	/** The acceleration rate at the start of interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Speed", meta=(ExposeOnSpawn=true))
	float Acceleration = 1.f;

	/** The maximum speed reachable during interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Speed", meta=(ExposeOnSpawn=true))
	float MaxSpeed = 10.f;

	/** The deceleration rate at the end of interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Speed", meta=(ExposeOnSpawn=true))
	float Deceleration = 1.f;
};

