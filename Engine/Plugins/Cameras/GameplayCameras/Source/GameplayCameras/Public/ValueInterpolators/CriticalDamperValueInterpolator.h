// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraValueInterpolator.h"
#include "Math/CriticalDamper.h"

#include "CriticalDamperValueInterpolator.generated.h"

/**
 * A value interpolator that uses a spring-mass system to converge towards the target value.
 */
UCLASS(meta=(DisplayName="Critical Damper"))
class UCriticalDamperValueInterpolator : public UCameraValueInterpolator
{
	GENERATED_BODY()

	UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()

public:

	/** The damping factor of the spring-mass system. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Damping", meta=(ExposeOnSpawn=true))
	float DampingFactor = 1.f;
};

