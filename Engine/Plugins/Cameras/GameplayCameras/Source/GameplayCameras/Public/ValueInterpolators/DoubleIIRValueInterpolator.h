// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraValueInterpolator.h"

#include "DoubleIIRValueInterpolator.generated.h"

/**
 * Double infinite impulse response filter interpolator.
 */
UCLASS(meta=(DisplayName="Double-IIR"))
class UDoubleIIRValueInterpolator : public UCameraValueInterpolator
{
	GENERATED_BODY()

	UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()

public:

	/** The primary speed of interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Speed", meta=(ExposeOnSpawn=true))
	float PrimarySpeed = 1.f;

	/** The intermediate speed of interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Speed", meta=(ExposeOnSpawn=true))
	float IntermediateSpeed = 1.f;

	/** Whether to use fixed-step evaluation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interpolation", meta=(ExposeOnSpawn=true))
	bool bUseFixedStep = true;
};

