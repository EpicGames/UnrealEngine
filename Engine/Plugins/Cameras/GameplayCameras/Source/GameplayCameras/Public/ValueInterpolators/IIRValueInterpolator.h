// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraValueInterpolator.h"

#include <type_traits>

#include "IIRValueInterpolator.generated.h"

/**
 * Infinite impulse response filter interpolator.
 */
UCLASS(meta=(DisplayName="IIR"))
class UIIRValueInterpolator : public UCameraValueInterpolator
{
	GENERATED_BODY()

	UE_DECLARE_CAMERA_VALUE_INTERPOLATOR()

public:

	/** The speed of interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interpolation", meta=(ExposeOnSpawn=true))
	float Speed = 1.f;

	/** Whether to use fixed-step evaluation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interpolation", meta=(ExposeOnSpawn=true))
	bool bUseFixedStep = true;
};

namespace UE::Cameras
{

template<typename ValueType>
struct TIIRValueInterpolatorTraits
{};

/**
 * The actual evaluation code for the IIR interpolator. Exposed here in this header
 * because it's also re-used in the double-IIR interpolator.
 * Most of the code here courtesy of Jeff Farris.
 */
template<typename ValueType>
class TIIRValueInterpolator : public TCameraValueInterpolator<ValueType>
{
public:

	TIIRValueInterpolator(const UIIRValueInterpolator* InParameters)
		: TCameraValueInterpolator<ValueType>(nullptr)
		, Speed(InParameters->Speed)
		, bUseFixedStep(InParameters->bUseFixedStep)
	{}

	TIIRValueInterpolator(float InSpeed, bool bInUseFixedStep)
		: TCameraValueInterpolator<ValueType>(nullptr)
		, Speed(InSpeed)
		, bUseFixedStep(bInUseFixedStep)
	{}

protected:

	using ValueTypeParam = typename TCameraValueInterpolator<ValueType>::ValueTypeParam;

	virtual void OnReset(ValueTypeParam OldCurrentValue, ValueTypeParam OldTargetValue) override
	{
		LastTargetValue = this->TargetValue;
		// Clear out any leftovers for rewind.
		LastUpdateLeftoverTime = 0.f; 
	}

	virtual void OnRun(const FCameraValueInterpolationParams& Params, FCameraValueInterpolationResult& OutResult) override
	{
		if (bUseFixedStep)
		{
			float RemainingTime = Params.DeltaTime;

			// Handle any leftover rewind.
			if (bDoLeftoverRewind && (LastUpdateLeftoverTime > 0.f))
			{
				// Rewind back to the state at end of the last full-step update.
				RemainingTime += LastUpdateLeftoverTime;
				this->CurrentValue = ValueAfterLastFullStep;
				LastUpdateLeftoverTime = 0.f;
			}

			// Move the substep target value linearly toward real target value while we evaluate the substeps.
			ValueType LastToTargetValue(this->TargetValue - LastTargetValue);
			if constexpr (std::is_same_v<FRotator3d, ValueType>)
			{
				LastToTargetValue = LastToTargetValue.GetNormalized();
			}
			const ValueType EquilibriumStepRate = LastToTargetValue * (1.f / RemainingTime);

			ValueType LerpedTargetValue = LastTargetValue;

			while (RemainingTime > KINDA_SMALL_NUMBER)
			{
				const float StepTime = FMath::Min(MaxSubstepTime, RemainingTime);

				if (bDoLeftoverRewind && (StepTime < MaxSubstepTime))
				{
					// If this is the last substep, let's cache where we were after the last full-step 
					// so we can resume from there on the next evaluation.
					LastUpdateLeftoverTime = StepTime;
					ValueAfterLastFullStep = this->CurrentValue;
				}

				LerpedTargetValue += EquilibriumStepRate * StepTime;
				RemainingTime -= StepTime;

				this->CurrentValue = RunSubstep(LerpedTargetValue, StepTime);
			}

			LastTargetValue = this->TargetValue;
		}
		else
		{
			this->CurrentValue = RunSubstep(this->TargetValue, Params.DeltaTime);
			LastUpdateLeftoverTime = 0.f;
		}
	}

	virtual void OnSerialize(const FCameraValueInterpolatorSerializeParams& Params, FArchive& Ar) override
	{
		Ar << ValueAfterLastFullStep;
		Ar << LastTargetValue;
		Ar << LastUpdateLeftoverTime;
		Ar << bDoLeftoverRewind;
	}

private:

	ValueType RunSubstep(ValueType SubstepTargetValue, float SubstepDeltaTime)
	{
		return TIIRValueInterpolatorTraits<ValueType>::InterpTo(this->CurrentValue, SubstepTargetValue, SubstepDeltaTime, Speed);
	}

private:

	static constexpr float MaxSubstepTime = 1.f / 120.f;

	ValueType ValueAfterLastFullStep;
	ValueType LastTargetValue;
	float LastUpdateLeftoverTime = 0.f;
	float Speed = 1.0;
	bool bDoLeftoverRewind = true;
	bool bUseFixedStep = false;
};

template<>
struct TIIRValueInterpolatorTraits<double>
{
	static double InterpTo(double CurrentValue, double TargetValue, float DeltaTime, double Speed)
	{
		return FMath::FInterpTo(CurrentValue, TargetValue, DeltaTime, Speed);
	}
};

template<>
struct TIIRValueInterpolatorTraits<FVector2d>
{
	static FVector2d InterpTo(FVector2d CurrentValue, FVector2d TargetValue, float DeltaTime, double Speed)
	{
		return FMath::Vector2DInterpTo(CurrentValue, TargetValue, DeltaTime, Speed);
	}
};

template<>
struct TIIRValueInterpolatorTraits<FVector3d>
{
	static FVector3d InterpTo(FVector3d CurrentValue, FVector3d TargetValue, float DeltaTime, double Speed)
	{
		return FMath::VInterpTo(CurrentValue, TargetValue, DeltaTime, Speed);
	}
};

template<>
struct TIIRValueInterpolatorTraits<FRotator>
{
	static FRotator InterpTo(FRotator CurrentValue, FRotator TargetValue, float DeltaTime, double Speed)
	{
		return FMath::RInterpTo(CurrentValue, TargetValue, DeltaTime, Speed);
	}
};

template<>
struct TIIRValueInterpolatorTraits<FQuat>
{
	static FQuat InterpTo(FQuat CurrentValue, FQuat TargetValue, float DeltaTime, double Speed)
	{
		return FMath::QInterpTo(CurrentValue, TargetValue, DeltaTime, Speed);
	}
};

}  // namespace UE::Cameras

