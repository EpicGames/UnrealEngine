// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameplayCamerasEditor : ModuleRules
{
	public GameplayCamerasEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] 
			{
				"Kismet",
				"EditorWidgets",
				"MessageLog",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"BlueprintGraph",
				"CinematicCamera",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"CurveEditor",
				"DeveloperSettings",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
				"GameplayCameras",
				"GameplayCamerasUncookedOnly",
				"GraphEditor",
				"InputCore",
				"InteractiveToolsFramework",
				"Kismet",
				"KismetWidgets",
				"LevelEditor",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Projects",
				"RenderCore",
				"RewindDebuggerInterface",
				"RHI",
				"Sequencer",
				"Slate",
				"SlateCore",
				"StructUtilsEditor",
				"StructViewer",
				"TimeManagement",
				"ToolMenus",
				"ToolWidgets",
				"TraceAnalysis",
				"TraceInsights",
				"TraceLog",
				"TraceServices",
				"UnrealEd",
			}
		);

		var DynamicModuleNames = new string[] {
			"PropertyEditor",
			"WorkspaceMenuStructure",
		};

		foreach (var Name in DynamicModuleNames)
		{
			PrivateIncludePathModuleNames.Add(Name);
			DynamicallyLoadedModuleNames.Add(Name);
		}
	}
}

