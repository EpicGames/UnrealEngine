// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactories/GameplayCameraActorFactory.h"

#include "Core/CameraAsset.h"
#include "GameFramework/GameplayCameraActor.h"
#include "GameFramework/GameplayCameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraActorFactory)

#define LOCTEXT_NAMESPACE "GameplayCameraActorFactory"

UGameplayCameraActorFactory::UGameplayCameraActorFactory(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DisplayName = LOCTEXT("DisplayName", "Gameplay Camera Actor");
	NewActorClass = AGameplayCameraActor::StaticClass();
}

AActor* UGameplayCameraActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AGameplayCameraActor* NewActor = Cast<AGameplayCameraActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));
	if (NewActor)
	{
		if (UCameraAsset* CameraAsset = Cast<UCameraAsset>(InAsset))
		{
			UGameplayCameraComponent* NewComponent = NewActor->GetCameraComponent();
			NewComponent->CameraReference.SetCameraAsset(CameraAsset);
		}
	}
	return NewActor;
}

bool UGameplayCameraActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (Super::CanCreateActorFrom(AssetData, OutErrorMsg))
	{
		return true;
	}

	if (AssetData.IsValid() && AssetData.IsInstanceOf(UCameraAsset::StaticClass()))
	{
		return true;
	}

	OutErrorMsg = LOCTEXT("NoCameraAsset", "A valid Gameplay Camera asset must be specified.");
	return false;
}

UObject* UGameplayCameraActorFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if (AGameplayCameraActor* CameraActor = Cast<AGameplayCameraActor>(ActorInstance))
	{
		return CameraActor->GetCameraComponent()->CameraReference.GetCameraAsset();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

