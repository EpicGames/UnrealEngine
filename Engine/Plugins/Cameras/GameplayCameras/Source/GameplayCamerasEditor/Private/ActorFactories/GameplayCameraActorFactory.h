// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"

#include "GameplayCameraActorFactory.generated.h"

UCLASS()
class UGameplayCameraActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:

	UGameplayCameraActorFactory(const FObjectInitializer& ObjInit);

public:

	// UActorFactory interface.
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
};

