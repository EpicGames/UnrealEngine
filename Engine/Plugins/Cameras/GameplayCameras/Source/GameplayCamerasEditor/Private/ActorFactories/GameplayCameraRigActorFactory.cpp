// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactories/GameplayCameraRigActorFactory.h"

#include "Core/CameraRigAsset.h"
#include "GameFramework/GameplayCameraRigActor.h"
#include "GameFramework/GameplayCameraRigComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCameraRigActorFactory)

#define LOCTEXT_NAMESPACE "GameplayCameraRigActorFactory"

UGameplayCameraRigActorFactory::UGameplayCameraRigActorFactory(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DisplayName = LOCTEXT("DisplayName", "Gameplay Camera Rig Actor");
	NewActorClass = AGameplayCameraRigActor::StaticClass();
}

AActor* UGameplayCameraRigActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AGameplayCameraRigActor* NewActor = Cast<AGameplayCameraRigActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));
	if (NewActor)
	{
		if (UCameraRigAsset* CameraRig = Cast<UCameraRigAsset>(InAsset))
		{
			UGameplayCameraRigComponent* NewComponent = NewActor->GetCameraRigComponent();
			NewComponent->CameraRigReference.SetCameraRig(CameraRig);
		}
	}
	return NewActor;
}

bool UGameplayCameraRigActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (Super::CanCreateActorFrom(AssetData, OutErrorMsg))
	{
		return true;
	}

	if (AssetData.IsValid() && AssetData.IsInstanceOf(UCameraRigAsset::StaticClass()))
	{
		return true;
	}

	OutErrorMsg = LOCTEXT("NoCameraRigAsset", "A valid Gameplay Camera Rig asset must be specified.");
	return false;
}

UObject* UGameplayCameraRigActorFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if (AGameplayCameraRigActor* CameraRigActor = Cast<AGameplayCameraRigActor>(ActorInstance))
	{
		return CameraRigActor->GetCameraRigComponent()->CameraRigReference.GetCameraRig();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

