// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_CameraAsset.h"

#include "IGameplayCamerasEditorModule.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Toolkits/CameraAssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_CameraAsset"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraAsset)

FText UAssetDefinition_CameraAsset::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Camera Asset");
}

FLinearColor UAssetDefinition_CameraAsset::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> UAssetDefinition_CameraAsset::GetAssetClass() const
{
	return UCameraAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraAsset::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("CameraAsset_SubMenu", "Camera"), ECategoryMenuType::Section) };
	return Categories;
}

FAssetOpenSupport UAssetDefinition_CameraAsset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(
			OpenSupportArgs.OpenMethod,
			OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit,
			EToolkitMode::Standalone); 
}

EAssetCommandResult UAssetDefinition_CameraAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
	for (UCameraAsset* CameraAsset : OpenArgs.LoadObjects<UCameraAsset>())
	{
		GameplayCamerasEditorModule.CreateCameraAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, CameraAsset);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE

