// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_CameraRigAsset.h"

#include "IGameplayCamerasEditorModule.h"
#include "Toolkits/CameraRigAssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_CameraRigAsset"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraRigAsset)

FText UAssetDefinition_CameraRigAsset::GetAssetDisplayName() const
{
	return LOCTEXT("CameraRigAsset_DisplayName", "Camera Rig Prefab");
}

FLinearColor UAssetDefinition_CameraRigAsset::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> UAssetDefinition_CameraRigAsset::GetAssetClass() const
{
	return UCameraRigAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraRigAsset::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("CameraRigAsset_SubMenu", "Camera"), ECategoryMenuType::Section) };
	return Categories;
}

FAssetOpenSupport UAssetDefinition_CameraRigAsset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(
			OpenSupportArgs.OpenMethod,
			OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit,
			EToolkitMode::Standalone); 
}

EAssetCommandResult UAssetDefinition_CameraRigAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
	for (UCameraRigAsset* CameraRig : OpenArgs.LoadObjects<UCameraRigAsset>())
	{
		GameplayCamerasEditorModule.CreateCameraRigEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, CameraRig);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE

