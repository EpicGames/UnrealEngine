// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_CameraRigProxyAsset.h"

#include "IGameplayCamerasEditorModule.h"
#include "Toolkits/IToolkit.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_CameraRigProxyAsset"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraRigProxyAsset)

FText UAssetDefinition_CameraRigProxyAsset::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Camera Rig Proxy");
}

FLinearColor UAssetDefinition_CameraRigProxyAsset::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> UAssetDefinition_CameraRigProxyAsset::GetAssetClass() const
{
	return UCameraRigProxyAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraRigProxyAsset::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("CameraRigProxyAsset_SubMenu", "Camera"), ECategoryMenuType::Section) };
	return Categories;
}

FAssetOpenSupport UAssetDefinition_CameraRigProxyAsset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(
			OpenSupportArgs.OpenMethod,
			OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit,
			EToolkitMode::Standalone); 
}

EAssetCommandResult UAssetDefinition_CameraRigProxyAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
	for (UCameraRigProxyAsset* CameraRigProxyAsset : OpenArgs.LoadObjects<UCameraRigProxyAsset>())
	{
		GameplayCamerasEditorModule.CreateCameraRigProxyEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, CameraRigProxyAsset);
	}	
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE

