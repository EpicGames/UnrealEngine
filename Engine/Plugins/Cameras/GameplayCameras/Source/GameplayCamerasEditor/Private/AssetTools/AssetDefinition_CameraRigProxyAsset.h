// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Core/CameraRigProxyAsset.h"

#include "AssetDefinition_CameraRigProxyAsset.generated.h"

UCLASS()
class UAssetDefinition_CameraRigProxyAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

