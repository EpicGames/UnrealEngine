// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_CameraShakeAsset.h"

#include "IGameplayCamerasEditorModule.h"
#include "Toolkits/CameraShakeAssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_CameraShakeAsset"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraShakeAsset)

FText UAssetDefinition_CameraShakeAsset::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Camera Shake");
}

FLinearColor UAssetDefinition_CameraShakeAsset::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> UAssetDefinition_CameraShakeAsset::GetAssetClass() const
{
	return UCameraShakeAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraShakeAsset::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("CameraAsset_SubMenu", "Camera"), ECategoryMenuType::Section) };
	return Categories;
}

FAssetOpenSupport UAssetDefinition_CameraShakeAsset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(
			OpenSupportArgs.OpenMethod,
			OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit,
			EToolkitMode::Standalone); 
}

EAssetCommandResult UAssetDefinition_CameraShakeAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
	for (UCameraShakeAsset* CameraShake : OpenArgs.LoadObjects<UCameraShakeAsset>())
	{
		GameplayCamerasEditorModule.CreateCameraShakeEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, CameraShake);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE

