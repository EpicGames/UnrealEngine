// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_CameraVariableCollection.h"

#include "Core/CameraVariableCollection.h"
#include "IGameplayCamerasEditorModule.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_CameraVariableCollection"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraVariableCollection)

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraVariableCollection::StaticMenuCategories()
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("CameraVariableCollection_SubMenu", "Camera"), ECategoryMenuType::Section) };
	return Categories;
}

FText UAssetDefinition_CameraVariableCollection::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Camera Variable Collection");
}

FLinearColor UAssetDefinition_CameraVariableCollection::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> UAssetDefinition_CameraVariableCollection::GetAssetClass() const
{
	return UCameraVariableCollection::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraVariableCollection::GetAssetCategories() const
{
	return StaticMenuCategories();
}

FAssetOpenSupport UAssetDefinition_CameraVariableCollection::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(
			OpenSupportArgs.OpenMethod,
			OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit,
			EToolkitMode::Standalone); 
}

EAssetCommandResult UAssetDefinition_CameraVariableCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
	for (UCameraVariableCollection* VariableCollection : OpenArgs.LoadObjects<UCameraVariableCollection>())
	{
		GameplayCamerasEditorModule.CreateCameraVariableCollectionEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, VariableCollection);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE

