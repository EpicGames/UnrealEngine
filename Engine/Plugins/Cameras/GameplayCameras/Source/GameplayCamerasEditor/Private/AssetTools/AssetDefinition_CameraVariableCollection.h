// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CameraVariableCollection.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_CameraVariableCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	static TConstArrayView<FAssetCategoryPath> StaticMenuCategories();

	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

