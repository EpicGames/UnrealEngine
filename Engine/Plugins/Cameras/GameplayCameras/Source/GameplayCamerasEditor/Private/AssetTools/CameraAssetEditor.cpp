// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraAssetEditor.h"

#include "Core/CameraAsset.h"
#include "EditorModeManager.h"
#include "Toolkits/CameraAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAssetEditor)

void UCameraAssetEditor::Initialize(TObjectPtr<UCameraAsset> InCameraAsset)
{
	CameraAsset = InCameraAsset;

	Super::Initialize();
}

void UCameraAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(CameraAsset.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraAssetEditor::CreateToolkit()
{
	return MakeShared<UE::Cameras::FCameraAssetEditorToolkit>(this);
}

