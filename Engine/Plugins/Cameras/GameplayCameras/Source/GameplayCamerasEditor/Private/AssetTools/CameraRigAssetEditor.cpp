// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraRigAssetEditor.h"

#include "Core/CameraRigAsset.h"
#include "EditorModeManager.h"
#include "Toolkits/CameraRigAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigAssetEditor)

void UCameraRigAssetEditor::Initialize(TObjectPtr<UCameraRigAsset> InCameraRigAsset)
{
	CameraRigAsset = InCameraRigAsset;

	Super::Initialize();
}

void UCameraRigAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(CameraRigAsset.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraRigAssetEditor::CreateToolkit()
{
	using namespace UE::Cameras;
	TSharedPtr<FCameraRigAssetEditorToolkit> Toolkit = MakeShared<FCameraRigAssetEditorToolkit>(this);
	Toolkit->SetCameraRigAsset(CameraRigAsset);
	return Toolkit;
}

