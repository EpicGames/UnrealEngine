// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraRigProxyAssetEditor.h"

#include "Core/CameraRigProxyAsset.h"
#include "Toolkits/CameraRigProxyAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigProxyAssetEditor)

void UCameraRigProxyAssetEditor::Initialize(TObjectPtr<UCameraRigProxyAsset> InCameraRigProxyAsset)
{
	CameraRigProxyAsset = InCameraRigProxyAsset;

	Super::Initialize();
}

void UCameraRigProxyAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(CameraRigProxyAsset.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraRigProxyAssetEditor::CreateToolkit()
{
	using namespace UE::Cameras;
	TSharedPtr<FCameraRigProxyAssetEditorToolkit> Toolkit = MakeShared<FCameraRigProxyAssetEditorToolkit>(this);
	Toolkit->SetCameraRigProxyAsset(CameraRigProxyAsset);
	return Toolkit;
}

