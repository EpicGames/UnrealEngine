// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraRigTransitionEditor.h"

#include "Core/CameraRigTransition.h"
#include "EditorModeManager.h"
#include "Toolkits/CameraRigTransitionEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigTransitionEditor)

void UCameraRigTransitionEditor::Initialize(TObjectPtr<UObject> InTransitionOwner)
{
	TransitionOwner = InTransitionOwner;

	Super::Initialize();
}

void UCameraRigTransitionEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(TransitionOwner.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraRigTransitionEditor::CreateToolkit()
{
	return MakeShared<UE::Cameras::FCameraRigTransitionEditorToolkit>(this);
}

