// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraShakeAssetEditor.h"

#include "Core/CameraShakeAsset.h"
#include "Toolkits/CameraShakeAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeAssetEditor)

void UCameraShakeAssetEditor::Initialize(TObjectPtr<UCameraShakeAsset> InCameraShakeAsset)
{
	CameraShakeAsset = InCameraShakeAsset;

	Super::Initialize();
}

void UCameraShakeAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(CameraShakeAsset.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraShakeAssetEditor::CreateToolkit()
{
	using namespace UE::Cameras;
	TSharedPtr<FCameraShakeAssetEditorToolkit> Toolkit = MakeShared<FCameraShakeAssetEditorToolkit>(this);
	Toolkit->SetCameraShakeAsset(CameraShakeAsset);
	return Toolkit;
}

