// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/CameraVariableCollectionEditor.h"

#include "Core/CameraVariableCollection.h"
#include "EditorModeManager.h"
#include "Toolkits/CameraVariableCollectionEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableCollectionEditor)

void UCameraVariableCollectionEditor::Initialize(TObjectPtr<UCameraVariableCollection> InVariableCollection)
{
	VariableCollection = InVariableCollection;

	Super::Initialize();
}

void UCameraVariableCollectionEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Add(VariableCollection.Get());
}

TSharedPtr<FBaseAssetToolkit> UCameraVariableCollectionEditor::CreateToolkit()
{
	return MakeShared<UE::Cameras::FCameraVariableCollectionEditorToolkit>(this);
}

