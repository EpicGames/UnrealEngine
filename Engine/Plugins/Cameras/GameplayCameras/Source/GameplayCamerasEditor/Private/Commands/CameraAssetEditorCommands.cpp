// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/CameraAssetEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CameraAssetEditorCommands"

namespace UE::Cameras
{

FCameraAssetEditorCommands::FCameraAssetEditorCommands()
	: TCommands<FCameraAssetEditorCommands>(
			"CameraAssetEditor",
			LOCTEXT("CameraAssetEditor", "Camera Asset Editor"),
			NAME_None,
			FGameplayCamerasEditorStyle::Get()->GetStyleSetName()
		)
{
}

void FCameraAssetEditorCommands::RegisterCommands()
{
	UI_COMMAND(Build, "Build", "Builds the asset and refreshes it in PIE",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F7));

	UI_COMMAND(ShowCameraDirector, "Director", "Shows the camera director editor",
			EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowCameraRigs, "Rigs", "Shows the camera rigs",
			EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowSharedTransitions, "Shared Transitions", "Shows the shared transitions",
			EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ChangeCameraDirector, "Change Camera Director", "Opens a dialog for changing the type of camera director",
			EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ShowMessages, "Messages", "Shows the message log for this camera asset",
			EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FindInCamera, "Search", "Searches for nodes in this camera asset",
			EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(RenameInterfaceParameter, "Rename Parameter", "Renames the selected interface parameter",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(DeleteInterfaceParameter, "Delete Parameter", "Deletes the selected interface parameter(s)",
			EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

