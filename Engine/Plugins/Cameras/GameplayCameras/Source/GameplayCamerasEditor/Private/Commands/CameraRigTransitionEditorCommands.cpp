// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/CameraRigTransitionEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CameraRigTransitionEditorCommands"

namespace UE::Cameras
{

FCameraRigTransitionEditorCommands::FCameraRigTransitionEditorCommands()
	: TCommands<FCameraRigTransitionEditorCommands>(
			"CameraRigTransitionEditor",
			LOCTEXT("CameraRigTransitionEditor", "Camera Rig Transition Editor"),
			NAME_None,
			FGameplayCamerasEditorStyle::Get()->GetStyleSetName()
		)
{
}

void FCameraRigTransitionEditorCommands::RegisterCommands()
{
	UI_COMMAND(FindInTransitions, "Search", "Searches for nodes in these transitions",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
	UI_COMMAND(FocusHome, "Home", "Focuses the graph canvas back on the root node",
			EUserInterfaceActionType::Button, FInputChord());
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

