// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/CameraShakeAssetEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CameraShakeAssetEditorCommands"

namespace UE::Cameras
{

FCameraShakeAssetEditorCommands::FCameraShakeAssetEditorCommands()
	: TCommands<FCameraShakeAssetEditorCommands>(
			"CameraShakeAssetEditor",
			LOCTEXT("CameraShakeAssetEditor", "Camera Shake Asset Editor"),
			NAME_None,
			FGameplayCamerasEditorStyle::Get()->GetStyleSetName()
		)
{
}

void FCameraShakeAssetEditorCommands::RegisterCommands()
{
	UI_COMMAND(Build, "Build", "Builds the asset and refreshes it in PIE",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F7));

	UI_COMMAND(ShowMessages, "Messages", "Shows the message log for this camera rig",
			EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FindInCameraShake, "Search", "Searches for nodes in this camera rig",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
	UI_COMMAND(FocusHome, "Home", "Focuses the graph canvas back on the root node",
			EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(RenameInterfaceParameter, "Rename Interface Parameter", "Renames the selected interface parameter",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(DeleteInterfaceParameter, "Delete Interface Parameter", "Deletes the selected interface parameter(s)",
			EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

