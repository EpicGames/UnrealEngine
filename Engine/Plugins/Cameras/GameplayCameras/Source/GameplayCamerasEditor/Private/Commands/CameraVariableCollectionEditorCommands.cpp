// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/CameraVariableCollectionEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CameraVariableCollectionEditorCommands"

namespace UE::Cameras
{

FCameraVariableCollectionEditorCommands::FCameraVariableCollectionEditorCommands()
	: TCommands<FCameraVariableCollectionEditorCommands>(
			"CameraVariableCollectionEditor",
			LOCTEXT("CameraVariableCollectionEditor", "Camera Variable Collection Editor"),
			NAME_None,
			FGameplayCamerasEditorStyle::Get()->GetStyleSetName()
		)
{
}

void FCameraVariableCollectionEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateVariable, "Create Variable", "Adds a new camera variable to the collection",
			EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameVariable, "Rename Variable", "Renames a camera variable",
			EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(DeleteVariable, "Delete Variable", "Removes a camera variable from the collection",
			EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

