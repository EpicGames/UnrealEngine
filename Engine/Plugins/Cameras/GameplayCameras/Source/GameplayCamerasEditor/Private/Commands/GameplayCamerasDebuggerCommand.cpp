// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/GameplayCamerasDebuggerCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "GameplayCamerasDebuggerCommands"

namespace UE::Cameras
{

FGameplayCamerasDebuggerCommands::FGameplayCamerasDebuggerCommands()
	: TCommands<FGameplayCamerasDebuggerCommands>(
			"GameplayCameras_Debugger",
			NSLOCTEXT("Contexts", "GameplayCameras_Debugger", "Gameplay Cameras Debugger"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
{
}

void FGameplayCamerasDebuggerCommands::RegisterCommands()
{
	UI_COMMAND(
			EnableDebugInfo, "Enable Debug Info", 
			"Shows Gameplay Cameras debugging information in the game",
			EUserInterfaceActionType::ToggleButton,
			FInputChord());
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

