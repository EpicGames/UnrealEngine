// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/ObjectTreeGraphEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styles/GameplayCamerasEditorStyle.h"

#define LOCTEXT_NAMESPACE "ObjectTreeGraphEditorCommands"

namespace UE::Cameras
{

FObjectTreeGraphEditorCommands::FObjectTreeGraphEditorCommands()
	: TCommands<FObjectTreeGraphEditorCommands>(
			"ObjectTreeGraphEditor",
			LOCTEXT("ObjectTreeGraphEditor", "Object Tree Graph Editor"),
			NAME_None,
			FGameplayCamerasEditorStyle::Get()->GetStyleSetName()
		)
{
}

void FObjectTreeGraphEditorCommands::RegisterCommands()
{
	UI_COMMAND(InsertArrayItemPinBefore, "Insert Item Pin Before", "Insert array item pin before this pin",
			EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InsertArrayItemPinAfter, "Insert Item Pin After", "Insert array item pin after this pin",
			EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveArrayItemPin, "Remove Item Pin", "Remove array item pin",
			EUserInterfaceActionType::Button, FInputChord());
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

