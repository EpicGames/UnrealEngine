// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Misc/EngineVersionComparison.h"

// Useful aliases for changes in editor APIs.

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

using FSlateCompatVector2f = FVector2f;

using FPerformGraphActionLocation = const FVector2f&;

#else

using FSlateCompatVector2f = FVector2D;

using FPerformGraphActionLocation = const FVector2D;

using FDelegateUserObjectConst = const void*;

#endif  // 5.6.0

