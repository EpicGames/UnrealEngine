// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compat/SResizeBox.h"

#if !UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

#include "Brushes/SlateColorBrush.h"
#include "Widgets/Layout/SBox.h"

void SVerticalResizeBox::Construct(const FArguments& InArgs)
{
	ContentHeight = InArgs._ContentHeight;
	HandleHeight = InArgs._HandleHeight;
	HandleColor = InArgs._HandleColor;
	HandleHighlightColor = InArgs._HandleHighlightColor;
	HandleBrush = FSlateColorBrush(FLinearColor::White);
	ContentHeightChanged = InArgs._ContentHeightChanged;

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(this, &SVerticalResizeBox::GetHeightOverride)
		.Padding(FMargin(0, 0, 0, HandleHeight))
		[
			InArgs._Content.Widget
		]
	];
}

FReply SVerticalResizeBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2f MouseLocation = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		if (MyGeometry.GetLocalSize().Y - MouseLocation.Y < HandleHeight)
		{
			DragStartLocation = MouseLocation.Y;
			DragStartContentHeight = ContentHeight.Get();
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}
	return FReply::Unhandled();
}

FReply SVerticalResizeBox::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (this->HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SVerticalResizeBox::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2f MouseLocation = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	LastMouseLocation = MouseLocation.Y;
	if (this->HasMouseCapture())
	{
		float NewContentHeight = DragStartContentHeight + (MouseLocation.Y - DragStartLocation);
		if (ContentHeight.IsBound() && ContentHeightChanged.IsBound())
		{
			ContentHeightChanged.Execute(NewContentHeight);
		}
		else
		{
			ContentHeight = NewContentHeight;
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FCursorReply SVerticalResizeBox::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	FVector2f CursorLocation = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition());
	if (MyGeometry.GetLocalSize().Y - CursorLocation.Y < HandleHeight)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
	}
	return FCursorReply::Unhandled();
}

int32 SVerticalResizeBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FLinearColor HandleBoxColor;
	int32 HandleLayerId = LayerId + 1;
	FVector2f LocalSize = AllottedGeometry.GetLocalSize();
	if (IsHovered() && LastMouseLocation.IsSet() && LastMouseLocation.GetValue() >= LocalSize.Y - HandleHeight && LastMouseLocation.GetValue() <= LocalSize.Y)
	{
		HandleBoxColor = HandleHighlightColor.Get();
	}
	else
	{
		HandleBoxColor = HandleColor.Get();
	}

	FVector2f HandleLocation(0, AllottedGeometry.GetLocalSize().Y - HandleHeight);
	FVector2f HandleSize(AllottedGeometry.GetLocalSize().X, HandleHeight);
	FSlateDrawElement::MakeBox
	(
		OutDrawElements,
		HandleLayerId,
		AllottedGeometry.ToPaintGeometry(HandleSize, FSlateLayoutTransform(HandleLocation)),
		&HandleBrush,
		ESlateDrawEffect::None,
		HandleBoxColor
	);

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, HandleLayerId, InWidgetStyle, bParentEnabled);
}

FOptionalSize SVerticalResizeBox::GetHeightOverride() const
{
	return ContentHeight.Get() + HandleHeight;
}

#endif  // <5.6.0

