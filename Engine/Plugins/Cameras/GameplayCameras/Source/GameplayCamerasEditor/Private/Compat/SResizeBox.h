// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EngineVersionComparison.h"

#if !UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateStructs.h"

class SVerticalResizeBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnContentHeightChanged, float);

public:
	SLATE_BEGIN_ARGS(SVerticalResizeBox)
		: _HandleHeight(5)
		, _ContentHeight(50)
		, _HandleColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
		, _HandleHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.5f))
	{}
		SLATE_ARGUMENT(float, HandleHeight)
		SLATE_ATTRIBUTE(float, ContentHeight)
		SLATE_ATTRIBUTE(FLinearColor, HandleColor)
		SLATE_ATTRIBUTE(FLinearColor, HandleHighlightColor)
		SLATE_EVENT(FOnContentHeightChanged, ContentHeightChanged)
		SLATE_DEFAULT_SLOT(FArguments, Content);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	FOptionalSize GetHeightOverride() const;

private:
	TOptional<float> LastMouseLocation;

	TAttribute<float> ContentHeight;
	float HandleHeight;

	float DragStartLocation;
	float DragStartContentHeight;

	TAttribute<FLinearColor> HandleColor;
	TAttribute<FLinearColor> HandleHighlightColor;
	FSlateBrush HandleBrush;

	FOnContentHeightChanged ContentHeightChanged;
};

#endif  // <5.6.0

