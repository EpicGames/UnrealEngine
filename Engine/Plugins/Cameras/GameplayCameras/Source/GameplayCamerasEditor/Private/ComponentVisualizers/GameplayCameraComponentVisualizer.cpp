// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentVisualizers/GameplayCameraComponentVisualizer.h"

#include "GameFramework/GameplayCameraComponentBase.h"

namespace UE::Cameras
{

void FGameplayCameraComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
}

void FGameplayCameraComponentVisualizer::DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	if (const UGameplayCameraComponentBase* GameplayCameraComponent = Cast<const UGameplayCameraComponentBase>(Component))
	{
		GameplayCameraComponent->OnDrawVisualizationHUD(Viewport, View, Canvas);
	}
}

}  // namespace UE::Cameras

