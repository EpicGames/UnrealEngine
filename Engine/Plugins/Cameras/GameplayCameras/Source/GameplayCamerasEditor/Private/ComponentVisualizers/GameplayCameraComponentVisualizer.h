// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

class FPrimitiveDrawInterface;
class FSceneView;
class UActorComponent;

namespace UE::Cameras
{

class FGameplayCameraComponentVisualizer : public FComponentVisualizer
{
public:

	// FComponentVisualizer interface.
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
};

}  // namespace UE::Cameras

