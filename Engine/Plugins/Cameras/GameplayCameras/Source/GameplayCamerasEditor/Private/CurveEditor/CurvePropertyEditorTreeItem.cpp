// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditor/CurvePropertyEditorTreeItem.h"

#include "RichCurveEditorModel.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Cameras
{

FCurvePropertyEditorTreeItem::FCurvePropertyEditorTreeItem()
{
}

FCurvePropertyEditorTreeItem::FCurvePropertyEditorTreeItem(FCurvePropertyInfo&& InInfo)
	: Info(MoveTemp(InInfo))
{
}

FCurvePropertyEditorTreeItem::FCurvePropertyEditorTreeItem(const FText& InDisplayName, TWeakObjectPtr<> InWeakOwner)
{
	Info.DisplayName = InDisplayName;
	Info.WeakOwner = InWeakOwner;
}

FCurvePropertyEditorTreeItem::FCurvePropertyEditorTreeItem(FRichCurve* InRichCurve, const FText& InCurveName, const FLinearColor& InCurveColor, TWeakObjectPtr<> InWeakOwner)
{
	Info.Curve = InRichCurve;
	Info.DisplayName = InCurveName;
	Info.Color = InCurveColor;
	Info.WeakOwner = InWeakOwner;
}

TSharedPtr<SWidget> FCurvePropertyEditorTreeItem::GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow)
{
	if (InColumnName == ColumnNames.Label)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(4.f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(Info.DisplayName)
				.ColorAndOpacity(FSlateColor(Info.Color))
			];
	}
	else if (InColumnName == ColumnNames.SelectHeader)
	{
		return SNew(SCurveEditorTreeSelect, InCurveEditor, InTreeItemID, TableRow);
	}
	else if (InColumnName == ColumnNames.PinHeader)
	{
		return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
	}

	return nullptr;
}

void FCurvePropertyEditorTreeItem::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	if (Info.Curve)
	{
		TUniquePtr<FRichCurveEditorModelRaw> NewCurve = MakeUnique<FRichCurveEditorModelRaw>(Info.Curve, Info.WeakOwner.Get());
		NewCurve->SetShortDisplayName(Info.DisplayName);
		NewCurve->SetColor(Info.Color);
		OutCurveModels.Add(MoveTemp(NewCurve));
	}
}

}  // namespace UE::Cameras

