// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "UObject/WeakObjectPtr.h"

struct FRichCurve;

namespace UE::Cameras
{

/**
 * Structure for providing information about a curve property, to be shown in
 * a curve editor tree item (see FCurvePropertyEditorTreeItem).
 */
struct FCurvePropertyInfo
{
	/** Display name of the tree item. */
	FText DisplayName;
	/** Color of the tree item and any associated curve. */
	FLinearColor Color = FLinearColor::White;

	/** Name of the property on the owning object. */
	FName PropertyName;
	/** The object on which the curve property resides. */
	TWeakObjectPtr<> WeakOwner;

	/** The curve associated with the tree item. */
	FRichCurve* Curve = nullptr;
};

/**
 * Curve editor tree view item for a curve tied to an object's property.
 */
struct FCurvePropertyEditorTreeItem : public ICurveEditorTreeItem
{
	FCurvePropertyInfo Info;

public:

	FCurvePropertyEditorTreeItem();
	FCurvePropertyEditorTreeItem(FCurvePropertyInfo&& InInfo);
	FCurvePropertyEditorTreeItem(const FText& InDisplayName, TWeakObjectPtr<> InWeakOwner);
	FCurvePropertyEditorTreeItem(FRichCurve* InRichCurve, const FText& InCurveName, const FLinearColor& InCurveColor, TWeakObjectPtr<> InWeakOwner);

	UObject* GetOwner() const { return Info.WeakOwner.Get(); }

	// ICurveEditorTreeItem interface.
	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow) override;
	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;
};

}  // namespace UE::Cameras

