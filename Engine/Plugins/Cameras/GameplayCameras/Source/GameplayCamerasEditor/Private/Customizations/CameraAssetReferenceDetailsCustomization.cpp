// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraAssetReferenceDetailsCustomization.h"

#include "Core/CameraAsset.h"
#include "Core/CameraAssetReference.h"
#include "GameplayCamerasDelegates.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "CameraAssetReferenceDetailsCustomization"

namespace UE::Cameras
{

class FCameraAssetParameterOverrideDataDetails : public FPropertyBagInstanceDataDetails
{
public:

	FCameraAssetParameterOverrideDataDetails(
			TSharedPtr<IPropertyHandle> InStructPropertyHandle,
			TSharedPtr<IPropertyHandle> InParametersPropertyHandle,
			TSharedPtr<IPropertyUtilities>& InPropertyUtilities)
		: FPropertyBagInstanceDataDetails(InParametersPropertyHandle, InPropertyUtilities, true)
		, StructPropertyHandle(InStructPropertyHandle)
	{
	}

protected:

	struct FCameraAssetReferenceOverrideProvider : public IPropertyBagOverrideProvider
	{
		FCameraAssetReferenceOverrideProvider(FCameraAssetReference& InCameraAssetReference)
			: CameraAssetReference(InCameraAssetReference)
		{
		}
		
		virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
		{
			return CameraAssetReference.IsParameterOverridden(PropertyID);
		}
		
		virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
		{
			CameraAssetReference.SetParameterOverridden(PropertyID, bIsOverridden);
		}

	private:
		FCameraAssetReference& CameraAssetReference;
	};

	virtual bool HasPropertyOverrides() const override
	{
		return true;
	}

	virtual void PreChangeOverrides() override
	{
		StructPropertyHandle->NotifyPreChange();
	}

	virtual void PostChangeOverrides() override
	{
		StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructPropertyHandle->NotifyFinishedChangingProperties();
	}

	virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override
	{
		StructPropertyHandle->EnumerateRawData([Func](void* RawData, const int32 DataIndex, const int32 NumDatas)
		{
			if (FCameraAssetReference* CameraAssetReference = static_cast<FCameraAssetReference*>(RawData))
			{
				if (const UCameraAsset* CameraAsset = CameraAssetReference->GetCameraAsset())
				{
					const FInstancedPropertyBag& DefaultParameters = CameraAsset->GetDefaultParameters();
					FInstancedPropertyBag& Parameters = CameraAssetReference->GetParameters();
					FCameraAssetReferenceOverrideProvider OverrideProvider(*CameraAssetReference);
					if (!Func(DefaultParameters, Parameters, OverrideProvider))
					{
						return false;
					}
				}
			}
			return true;
		});
	}

private:
	
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};

TSharedRef<IPropertyTypeCustomization> FCameraAssetReferenceDetailsCustomization::MakeInstance()
{
	return MakeShared<FCameraAssetReferenceDetailsCustomization>();
}

FCameraAssetReferenceDetailsCustomization::~FCameraAssetReferenceDetailsCustomization()
{
	FGameplayCamerasDelegates::OnCameraAssetBuilt().RemoveAll(this);
}

void FCameraAssetReferenceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();

	CameraAssetPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraAssetReference, CameraAsset));
	ParametersPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraAssetReference, Parameters));

	CameraAssetPropertyHandle->SetOnPropertyValueChanged(
			FSimpleDelegate::CreateSP(this, &FCameraAssetReferenceDetailsCustomization::RebuildParametersIfNeeded));

	if (!GIsTransacting)
	{
		RebuildParametersIfNeeded();
	}

	InHeaderRow
	.ShouldAutoExpand(true)
	.NameContent()
	[
		CameraAssetPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		CameraAssetPropertyHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
	];

	FGameplayCamerasDelegates::OnCameraAssetBuilt().AddSP(this, &FCameraAssetReferenceDetailsCustomization::OnCameraAssetBuilt);
}

void FCameraAssetReferenceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<FCameraAssetParameterOverrideDataDetails> ParameterOverrideDataDetails = 
		MakeShared<FCameraAssetParameterOverrideDataDetails>(StructPropertyHandle, ParametersPropertyHandle, PropertyUtilities);
	InChildrenBuilder.AddCustomBuilder(ParameterOverrideDataDetails);
}

void FCameraAssetReferenceDetailsCustomization::OnCameraAssetBuilt(const UCameraAsset* InCameraAsset)
{
	RebuildParametersIfNeeded();
}

void FCameraAssetReferenceDetailsCustomization::RebuildParametersIfNeeded()
{
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	bool bRebuiltAny = false;

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		FCameraAssetReference* CameraAssetReference = static_cast<FCameraAssetReference*>(RawData[Index]);
		if (CameraAssetReference)
		{
			const bool bRebuiltOne = CameraAssetReference->RebuildParametersIfNeeded();
			bRebuiltAny |= bRebuiltOne;
		}
	}

	if (bRebuiltAny && PropertyUtilities)
	{
		PropertyUtilities->RequestRefresh();
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

