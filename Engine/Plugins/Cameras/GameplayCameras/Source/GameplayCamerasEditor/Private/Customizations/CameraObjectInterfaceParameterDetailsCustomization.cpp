// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraObjectInterfaceParameterDetailsCustomization.h"

#include "Core/BaseCameraObject.h"
#include "Core/CameraObjectInterface.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CameraObjectInterfaceParameterDetailsCustomization"

namespace UE::Cameras
{

TSharedRef<IDetailCustomization> FCameraObjectInterfaceParameterDetailsCustomization::MakeInstance()
{
	return MakeShared<FCameraObjectInterfaceParameterDetailsCustomization>();
}

void FCameraObjectInterfaceParameterDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	bool bFoundValidOuter = false;
	UBaseCameraObject* CommonCameraObject = nullptr;
	TSet<FGuid> ParameterGuids;

	TArray<TWeakObjectPtr<UObject>> WeakObjects;
	DetailBuilder.GetObjectsBeingCustomized(WeakObjects);
	for (TWeakObjectPtr<UObject> WeakObj : WeakObjects)
	{
		if (UCameraObjectInterfaceParameterBase* Obj = Cast<UCameraObjectInterfaceParameterBase>(WeakObj.Get()))
		{
			ParameterGuids.Add(Obj->GetGuid());

			UBaseCameraObject* CameraObject = Obj->GetTypedOuter<UBaseCameraObject>();
			if (!bFoundValidOuter && CameraObject)
			{
				CommonCameraObject = CameraObject;
				bFoundValidOuter = true;
			}
			else if (bFoundValidOuter && CommonCameraObject && CameraObject != CommonCameraObject)
			{
				CommonCameraObject = nullptr;
			}
		}
	}
	if (!CommonCameraObject)
	{
		return;
	}

	TArray<FName> PropertyNames;
	FInstancedPropertyBag& DefaultParameters = CommonCameraObject->GetDefaultParameters();
	const UPropertyBag* DefaultParametersStruct = DefaultParameters.GetPropertyBagStruct();
	if (DefaultParametersStruct)
	{
		for (const FPropertyBagPropertyDesc& Item : DefaultParametersStruct->GetPropertyDescs())
		{
			if (ParameterGuids.Contains(Item.ID))
			{
				PropertyNames.Add(Item.Name);
			}
		}
	}

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(
			TEXT("DefaultValue"), 
			LOCTEXT("DefaultValueCategory", "Default Value"));

	// WARNING: here we put a Struct-On-Scope inside the details view, and it has a direct access to the default 
	// parameters' memory. However, that memory may move when we build the current asset if we have added enough 
	// new parameters to require re-allocation. The Details View would then suddenly start hitting freed memory.
	// We fix this by forcing a Details View refresh on build in all the toolkits that do it.
	if (DefaultParametersStruct)
	{
		auto ModifyCommonCameraObject = [CommonCameraObject]() { CommonCameraObject->Modify(); };
		TSharedRef<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(DefaultParametersStruct, DefaultParameters.GetMutableValue().GetMemory());
		for (FName PropertyName : PropertyNames)
		{
			FAddPropertyParams AddParams;
			AddParams.ForceShowProperty();  // The property may not have CPF_Edit if the parameter is set to not-visible,
											// so force it to be shown.
			IDetailPropertyRow* PropertyRow = CategoryBuilder.AddExternalStructureProperty(StructOnScope, PropertyName, EPropertyLocation::Default, AddParams);

			TSharedPtr<IPropertyHandle> DefaultValuePropertyHandle = PropertyRow->GetPropertyHandle();
			DefaultValuePropertyHandle->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda(ModifyCommonCameraObject));
			DefaultValuePropertyHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda(ModifyCommonCameraObject));
			DefaultValuePropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda(ModifyCommonCameraObject));

		}
	}
	if (PropertyNames.Num() < ParameterGuids.Num())
	{
		FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(LOCTEXT("UnbuiltDataRow", "Needs Building"));
		CustomRow.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NeedsBuildWarning", "Please build the asset to edit default parameter values"))
			];
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

