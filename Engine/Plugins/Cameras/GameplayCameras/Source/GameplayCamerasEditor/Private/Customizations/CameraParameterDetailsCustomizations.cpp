// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraParameterDetailsCustomizations.h"

#include "ContentBrowserModule.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableCollection.h"
#include "Customizations/MathStructCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "Editors/CameraVariablePickerConfig.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IGameplayCamerasEditorModule.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/CameraVariableCollectionEditorToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "UObject/Object.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CameraParameterDetailsCustomization"

namespace UE::Cameras
{

static TArrayView<FName> GetCameraParameterMetaDataToCopy()
{
	static TArray<FName> MetaDataKeys{ 
		TEXT("UIMin"), TEXT("UIMax"),
		TEXT("ClampMin"), TEXT("ClampMax"),
		TEXT("Units"), TEXT("ForceUnits")
	};
	return MetaDataKeys;
}

void FCameraParameterDetailsCustomization::Register(FPropertyEditorModule& PropertyEditorModule)
{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(\
			F##ValueName##CameraParameter::StaticStruct()->GetFName(),\
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(\
				[]{ return MakeShared<F##ValueName##CameraParameterDetailsCustomization>(); }));
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
}

void FCameraParameterDetailsCustomization::Unregister(FPropertyEditorModule& PropertyEditorModule)
{
	if (UObjectInitialized())
	{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(\
				F##ValueName##CameraParameter::StaticStruct()->GetFName());
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
	}
}

void FCameraParameterDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Gather up the things we need.
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();
	StructProperty = PropertyHandle;

	// All camera parameters should have a "Value" and "Variable" property.
	ValueProperty = PropertyHandle->GetChildHandle("Value");
	VariableProperty = PropertyHandle->GetChildHandle("Variable");
	ensure(ValueProperty && VariableProperty);

	// Copy select details-view-related metadata from the camera parameter struct onto the property handle.
	for (const FName Key : GetCameraParameterMetaDataToCopy())
	{
		const FString& MetaDataValue = StructProperty->GetMetaData(Key);
		if (!MetaDataValue.IsEmpty())
		{
			ValueProperty->SetInstanceMetaData(Key, MetaDataValue);
		}
	}

	// Get the type of camera variable we need for this camera parameter (bool variable, float variable, etc.)
	VariableClass = nullptr;
	if (FObjectProperty* VariableObjectProperty = CastField<FObjectProperty>(VariableProperty->GetProperty()))
	{
		VariableClass = VariableObjectProperty->PropertyClass;
	}
	ensure(VariableClass);

	// Update our variable info once now. We will then update it every tick, since the UI needs it
	// for various things.
	UpdateVariableInfo();

	// Create the parameter value editor (float editor, vector editor, etc.)
	TSharedRef<SWidget> ValueWidget = ValueProperty->CreatePropertyValueWidgetWithCustomization(nullptr);
	ValueWidget->SetEnabled(TAttribute<bool>::CreateSP(this, &FCameraParameterDetailsCustomization::IsValueEditorEnabled));

	// Create the whole UI layout.
	TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasStyle = FGameplayCamerasEditorStyle::Get();

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(100.f)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(LayoutBox, SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0)
		.FillWidth(1.f)
		[
			ValueWidget
		]
		+SHorizontalBox::Slot()
		.Padding(0)
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			ValueProperty->CreateDefaultPropertyButtonWidgets()
		]
		+SHorizontalBox::Slot()
		.Padding(2.f)
		.AutoWidth()
		[
			SAssignNew(VariableBrowserButton, SComboButton)
			.HasDownArrow(true)
			.ContentPadding(1.f)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.IsEnabled(this, &FCameraParameterDetailsCustomization::IsCameraVariableBrowserEnabled)
			.ToolTipText(this, &FCameraParameterDetailsCustomization::GetCameraVariableBrowserToolTip)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(GameplayCamerasStyle->GetBrush("CameraParameter.VariableBrowser"))
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.Visibility(this, &FCameraParameterDetailsCustomization::GetVariableInfoTextVisibility)
					.MaxDesiredWidth(this, &FCameraParameterDetailsCustomization::GetVariableInfoTextMaxWidth)
					[
						SNew(STextBlock)
						.Text(this, &FCameraParameterDetailsCustomization::GetVariableInfoText)
						.MinDesiredWidth(20.f)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.Visibility(this, &FCameraParameterDetailsCustomization::GetVariableErrorTextVisibility)
					.MaxDesiredWidth(this, &FCameraParameterDetailsCustomization::GetVariableErrorTextMaxWidth)
					[
						SNew(STextBlock)
						.Text(this, &FCameraParameterDetailsCustomization::GetVariableErrorText)
						.MinDesiredWidth(20.f)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.ColorAndOpacity(FStyleColors::Error)
					]
				]
			]
			.OnGetMenuContent(this, &FCameraParameterDetailsCustomization::BuildCameraVariableBrowser)
		]
	];

	HeaderRow.OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FCameraParameterDetailsCustomization::IsResetToDefaultVisible),
				FResetToDefaultHandler::CreateSP(this, &FCameraParameterDetailsCustomization::OnResetToDefault)));
}

void FCameraParameterDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	FPropertyAccess::Result Result = ValueProperty->GetNumChildren(NumChildren);
	if (Result == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = ValueProperty->GetChildHandle(Index);
			if (ChildProperty)
			{
				ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

void FCameraParameterDetailsCustomization::Tick(float DeltaTime)
{
	// Use the editor tick to query the property values only once per frame.
	UpdateVariableInfo();
}

void FCameraParameterDetailsCustomization::UpdateVariableInfo()
{
	VariableInfo = FCameraVariableInfo();

	if (StructProperty->IsValidHandle())
	{
		StructProperty->EnumerateRawData(
				[this](void* RawData, const int32 ValueIndex, const int32 NumValues)
				{
					if (RawData)
					{
						VariableInfo.bHasNonUserOverride |= HasNonUserOverride(RawData);
						VariableInfo.bHasOverride |= HasOverride(RawData);
					}
					return true;
				});
	}

	UObject* VariableObject = nullptr;
	FPropertyAccess::Result PropertyAccessResult = VariableProperty->GetValue(VariableObject);
	if (PropertyAccessResult == FPropertyAccess::Success)
	{
		if (VariableObject)
		{
			if (UCameraVariableAsset* Variable = Cast<UCameraVariableAsset>(VariableObject))
			{
				VariableInfo.VariableValue = ECameraVariableValue::Set;
				VariableInfo.CommonVariable = Variable;
				VariableInfo.InfoText = Variable->DisplayName.IsEmpty() ?
					FText::FromName(Variable->GetFName()) :
					FText::FromString(Variable->DisplayName);
			}
			else
			{
				VariableInfo.VariableValue = ECameraVariableValue::Invalid;
				VariableInfo.ErrorText = LOCTEXT("InvalidVariableObject", "Invalid Variable");
			}
		}
		// else: variable is not set, leave info/error texts empty.
	}
	else if (PropertyAccessResult == FPropertyAccess::MultipleValues)
	{
		VariableInfo.VariableValue = ECameraVariableValue::MultipleSet;
		VariableInfo.InfoText = LOCTEXT("MultipleVariableValues", "Multiple Variables");
	}
	else
	{
		VariableInfo.VariableValue = ECameraVariableValue::Invalid;
		VariableInfo.ErrorText = LOCTEXT("ErrorReadingVariable", "Error Reading Variable");
	}
}

TSharedRef<SWidget> FCameraParameterDetailsCustomization::BuildCameraVariableBrowser()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	const bool bSearchable = false;
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CameraVariableOperations", "Current Parameter"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("GoToVariable", "Go to variable"),
			LOCTEXT("GoToVariable_ToolTip", "Open the referenced camera variable collection asset"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCameraParameterDetailsCustomization::OnGoToVariable),
				FCanExecuteAction::CreateSP(this, &FCameraParameterDetailsCustomization::CanGoToVariable))
			);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearVariable", "Clear"),
			LOCTEXT("ClearVariable_ToolTip", "Clears the variable from the camera parameter"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCameraParameterDetailsCustomization::OnClearVariable),
				FCanExecuteAction::CreateSP(this, &FCameraParameterDetailsCustomization::CanClearVariable))
			);
	}
	MenuBuilder.EndSection();

	FCameraVariablePickerConfig PickerConfig;
	PickerConfig.CameraVariableClass = VariableClass;
	PickerConfig.InitialCameraVariableSelection = VariableInfo.CommonVariable;
	PickerConfig.CameraVariableCollectionSaveSettingsName = TEXT("CameraParameterVariablePropertyPicker");
	PickerConfig.OnCameraVariableSelected = FOnCameraVariableSelected::CreateSP(
			this, &FCameraParameterDetailsCustomization::OnSetVariable);
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = IGameplayCamerasEditorModule::Get();
	TSharedRef<SWidget> PickerWidget = GameplayCamerasEditorModule.CreateCameraVariablePicker(PickerConfig);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CameraVariableBrowser", "Browse"));
	{
		TSharedRef<SWidget> VariableBrowser = SNew(SBox)
			.MinDesiredWidth(300.f)
			.MinDesiredHeight(300.f)
			[
				PickerWidget
			];
		MenuBuilder.AddWidget(VariableBrowser, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool FCameraParameterDetailsCustomization::IsValueEditorEnabled() const
{
	// The value widget is enabled (i.e. the user can change the value) if the parameter isn't driven by
	// a variable or a rig parameter;
	return !VariableInfo.bHasOverride;
}

bool FCameraParameterDetailsCustomization::IsCameraVariableBrowserEnabled() const
{
	// The variable picker is enabled if the parameter isn't exposed to the rig interface via a private variable,
	// since we can't drive a value with both an interface parameter and a user-defined variable.
	return !VariableInfo.bHasNonUserOverride;
}

FText FCameraParameterDetailsCustomization::GetCameraVariableBrowserToolTip() const
{
	if (!VariableInfo.bHasNonUserOverride)
	{
		return LOCTEXT(
				"SetVariable_ToolTip", 
				"Selects a camera variable to drive this parameter");
	}
	else
	{
		return LOCTEXT(
				"SetVariableDisabled_ToolTip",
				"This parameter is exposed by the camera rig and cannot be also driven by a camera variable");
	}
}

FText FCameraParameterDetailsCustomization::GetVariableInfoText() const
{
	return VariableInfo.InfoText;
}

EVisibility FCameraParameterDetailsCustomization::GetVariableInfoTextVisibility() const
{
	const bool bShowVariableInfoText = !VariableInfo.InfoText.IsEmpty();
	return bShowVariableInfoText ? EVisibility::Visible : EVisibility::Collapsed;
}

FOptionalSize FCameraParameterDetailsCustomization::GetVariableInfoTextMaxWidth() const
{
	// We want this text to take at most 30% of the free-standing space of the combo button.
	// Free-standing space excludes fixed things like the combo button icon, the dropdown icon, paddings, etc.
	// IMPORTANT: update this if the main layout changes inside Construct()
	const float FixedSpace = 1.f + (2.f+ 16.f + 2.f) + (2.f + 16.f + 2.f) + 1.f;

	const bool bShowVariableInfoText = !VariableInfo.InfoText.IsEmpty();
	const float LayoutBoxWidth = LayoutBox ? LayoutBox->GetPaintSpaceGeometry().GetLocalSize().X : 0.f;
	return bShowVariableInfoText ? FOptionalSize((LayoutBoxWidth - FixedSpace) / 3.f) : FOptionalSize(0);
}

FText FCameraParameterDetailsCustomization::GetVariableErrorText() const
{
	return VariableInfo.ErrorText;
}

EVisibility FCameraParameterDetailsCustomization::GetVariableErrorTextVisibility() const
{
	const bool bShowVariableErrorText = !VariableInfo.ErrorText.IsEmpty();
	return bShowVariableErrorText ? EVisibility::Visible : EVisibility::Collapsed;
}

FOptionalSize FCameraParameterDetailsCustomization::GetVariableErrorTextMaxWidth() const
{
	// See comments in GetVariableInfoTextMaxWidth.
	const float FixedSpace = 1.f + (2.f+ 16.f + 2.f) + (2.f + 16.f + 2.f) + 1.f;

	const bool bShowVariableErrorText = !VariableInfo.ErrorText.IsEmpty();
	const float LayoutBoxWidth = LayoutBox ? LayoutBox->GetPaintSpaceGeometry().GetLocalSize().X : 0.f;
	return bShowVariableErrorText ? FOptionalSize((LayoutBoxWidth - FixedSpace) / 3.f) : FOptionalSize(0);
}

bool FCameraParameterDetailsCustomization::CanGoToVariable() const
{
	UObject* VariableObject = nullptr;
	FPropertyAccess::Result PropertyAccessResult = VariableProperty->GetValue(VariableObject);
	return PropertyAccessResult == FPropertyAccess::Success;
}

void FCameraParameterDetailsCustomization::OnGoToVariable()
{
	UObject* VariableObject = nullptr;
	FPropertyAccess::Result PropertyAccessResult = VariableProperty->GetValue(VariableObject);
	if (VariableObject && PropertyAccessResult == FPropertyAccess::Success)
	{
		UCameraVariableCollection* VariableCollection = VariableObject->GetTypedOuter<UCameraVariableCollection>();
		if (VariableCollection)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(VariableCollection);

			TSharedPtr<IToolkit> FoundToolkit = FToolkitManager::Get().FindEditorForAsset(VariableCollection);
			if (FoundToolkit)
			{
				TSharedPtr<FCameraVariableCollectionEditorToolkit> VariableCollectionToolkit = 
					StaticCastSharedPtr<FCameraVariableCollectionEditorToolkit>(FoundToolkit);
				VariableCollectionToolkit->FocusWindow(VariableObject);
			}
		}
	}
}

bool FCameraParameterDetailsCustomization::CanClearVariable() const
{
	return VariableProperty->CanResetToDefault();
}

void FCameraParameterDetailsCustomization::OnClearVariable()
{
	OnSetVariable(nullptr);
}

void FCameraParameterDetailsCustomization::OnSetVariable(UCameraVariableAsset* InVariable)
{
	TArray<void*> RawData;
	StructProperty->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);

	check(!OuterObjects.Num() || OuterObjects.Num() == RawData.Num());
	
	{
		FScopedTransaction Transaction(FText::Format(
					LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		StructProperty->NotifyPreChange();

		for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ++ValueIndex)
		{
			SetParameterVariable(RawData[ValueIndex], InVariable);
		}

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	}

	FPropertyChangedEvent ChangeEvent(StructProperty->GetProperty(), EPropertyChangeType::ValueSet, OuterObjects);
	PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);

	PropertyUtilities->RequestForceRefresh();
	VariableBrowserButton->SetIsOpen(false);
}

bool FCameraParameterDetailsCustomization::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	// The user can reset the camera parameter to its default if the value is non-default, and/or the
	// variable is a user-defined variable. In other words, the VariableID property should not play a role
	// in this.
	return ValueProperty->CanResetToDefault() || VariableProperty->CanResetToDefault();
}

void FCameraParameterDetailsCustomization::OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	// As mentioned above, we only reset the value and the variable, not the VariableID.
	ValueProperty->ResetToDefault();
	VariableProperty->ResetToDefault();

	PropertyUtilities->RequestForceRefresh();
}

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
bool F##ValueName##CameraParameterDetailsCustomization::HasOverride(void* InRawData)\
{\
	F##ValueName##CameraParameter* TypedData = reinterpret_cast<F##ValueName##CameraParameter*>(InRawData);\
	return TypedData->HasOverride();\
}\
bool F##ValueName##CameraParameterDetailsCustomization::HasNonUserOverride(void* InRawData)\
{\
	F##ValueName##CameraParameter* TypedData = reinterpret_cast<F##ValueName##CameraParameter*>(InRawData);\
	return TypedData->HasNonUserOverride();\
}\
void F##ValueName##CameraParameterDetailsCustomization::SetParameterVariable(void* InRawData, UCameraVariableAsset* InVariable)\
{\
	F##ValueName##CameraParameter* TypedData = reinterpret_cast<F##ValueName##CameraParameter*>(InRawData);\
	TypedData->Variable = CastChecked<U##ValueName##CameraVariable>(InVariable, ECastCheckedType::NullAllowed);\
	TypedData->VariableID = InVariable ? InVariable->GetVariableID() : FCameraVariableID();\
}
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

