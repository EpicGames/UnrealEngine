// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraRigAssetReferenceDetailsCustomization.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraRigAssetReference.h"
#include "Customizations/CameraObjectInterfaceParameterOverrideDataDetails.h"
#include "GameplayCamerasDelegates.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "CameraRigAssetReferenceDetailsCustomization"

namespace UE::Cameras
{

using FCameraRigAssetParameterOverrideDataDetails = TCameraObjectInterfaceParameterOverrideDataDetails<FCameraRigAssetReference>;

TSharedRef<IPropertyTypeCustomization> FCameraRigAssetReferenceDetailsCustomization::MakeInstance()
{
	return MakeShared<FCameraRigAssetReferenceDetailsCustomization>();
}

FCameraRigAssetReferenceDetailsCustomization::~FCameraRigAssetReferenceDetailsCustomization()
{
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().RemoveAll(this);
}

void FCameraRigAssetReferenceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();

	CameraRigAssetPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraRigAssetReference, CameraRig));
	ParametersPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraRigAssetReference, Parameters));

	CameraRigAssetPropertyHandle->SetOnPropertyValueChanged(
			FSimpleDelegate::CreateSP(this, &FCameraRigAssetReferenceDetailsCustomization::RebuildParametersIfNeeded));

	if (!GIsTransacting)
	{
		RebuildParametersIfNeeded();
	}

	InHeaderRow
	.ShouldAutoExpand(true)
	.NameContent()
	[
		CameraRigAssetPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		CameraRigAssetPropertyHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
	];
	
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().AddSP(this, &FCameraRigAssetReferenceDetailsCustomization::OnCameraRigAssetBuilt);
}

void FCameraRigAssetReferenceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<FCameraRigAssetParameterOverrideDataDetails> ParameterOverrideDataDetails = 
		MakeShared<FCameraRigAssetParameterOverrideDataDetails>(StructPropertyHandle, ParametersPropertyHandle, PropertyUtilities);
	InChildrenBuilder.AddCustomBuilder(ParameterOverrideDataDetails);
}

void FCameraRigAssetReferenceDetailsCustomization::OnCameraRigAssetBuilt(const UCameraRigAsset* CameraRig)
{
	RebuildParametersIfNeeded();
}

void FCameraRigAssetReferenceDetailsCustomization::RebuildParametersIfNeeded()
{
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	bool bRebuiltAny = false;

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		FCameraRigAssetReference* CameraRigAssetReference = static_cast<FCameraRigAssetReference*>(RawData[Index]);
		if (CameraRigAssetReference)
		{
			const bool bRebuiltOne = CameraRigAssetReference->RebuildParametersIfNeeded();
			bRebuiltAny |= bRebuiltOne;
		}
	}

	if (bRebuiltAny && PropertyUtilities)
	{
		PropertyUtilities->RequestRefresh();
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

