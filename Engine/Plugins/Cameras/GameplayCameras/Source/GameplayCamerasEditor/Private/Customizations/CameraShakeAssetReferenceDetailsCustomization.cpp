// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraShakeAssetReferenceDetailsCustomization.h"

#include "Core/CameraShakeAsset.h"
#include "Core/CameraShakeAssetReference.h"
#include "Customizations/CameraObjectInterfaceParameterOverrideDataDetails.h"
#include "GameplayCamerasDelegates.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "CameraShakeAssetReferenceDetailsCustomization"

namespace UE::Cameras
{

using FCameraShakeAssetParameterOverrideDataDetails = TCameraObjectInterfaceParameterOverrideDataDetails<FCameraShakeAssetReference>;

TSharedRef<IPropertyTypeCustomization> FCameraShakeAssetReferenceDetailsCustomization::MakeInstance()
{
	return MakeShared<FCameraShakeAssetReferenceDetailsCustomization>();
}

FCameraShakeAssetReferenceDetailsCustomization::~FCameraShakeAssetReferenceDetailsCustomization()
{
	FGameplayCamerasDelegates::OnCameraShakeAssetBuilt().RemoveAll(this);
}

void FCameraShakeAssetReferenceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();

	CameraShakeAssetPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraShakeAssetReference, CameraShake));
	ParametersPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCameraShakeAssetReference, Parameters));

	CameraShakeAssetPropertyHandle->SetOnPropertyValueChanged(
			FSimpleDelegate::CreateSP(this, &FCameraShakeAssetReferenceDetailsCustomization::RebuildParametersIfNeeded));

	if (!GIsTransacting)
	{
		RebuildParametersIfNeeded();
	}

	InHeaderRow
	.ShouldAutoExpand(true)
	.NameContent()
	[
		CameraShakeAssetPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		CameraShakeAssetPropertyHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
	];
	
	FGameplayCamerasDelegates::OnCameraShakeAssetBuilt().AddSP(this, &FCameraShakeAssetReferenceDetailsCustomization::OnCameraShakeAssetBuilt);
}

void FCameraShakeAssetReferenceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<FCameraShakeAssetParameterOverrideDataDetails> ParameterOverrideDataDetails = 
		MakeShared<FCameraShakeAssetParameterOverrideDataDetails>(StructPropertyHandle, ParametersPropertyHandle, PropertyUtilities);
	InChildrenBuilder.AddCustomBuilder(ParameterOverrideDataDetails);
}

void FCameraShakeAssetReferenceDetailsCustomization::OnCameraShakeAssetBuilt(const UCameraShakeAsset* CameraShake)
{
	RebuildParametersIfNeeded();
}

void FCameraShakeAssetReferenceDetailsCustomization::RebuildParametersIfNeeded()
{
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	bool bRebuiltAny = false;

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		FCameraShakeAssetReference* CameraShakeAssetReference = static_cast<FCameraShakeAssetReference*>(RawData[Index]);
		if (CameraShakeAssetReference)
		{
			const bool bRebuiltOne = CameraShakeAssetReference->RebuildParametersIfNeeded();
			bRebuiltAny |= bRebuiltOne;
		}
	}

	if (bRebuiltAny && PropertyUtilities)
	{
		PropertyUtilities->RequestRefresh();
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

