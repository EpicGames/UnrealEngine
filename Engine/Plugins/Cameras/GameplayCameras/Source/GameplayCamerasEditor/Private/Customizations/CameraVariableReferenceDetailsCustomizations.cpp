// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CameraVariableReferenceDetailsCustomizations.h"

#include "ContentBrowserModule.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableCollection.h"
#include "Core/CameraVariableReferences.h"
#include "Editors/CameraVariablePickerConfig.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IGameplayCamerasEditorModule.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CameraVariableReferenceDetailsCustomization"

namespace UE::Cameras
{

void FCameraVariableReferenceDetailsCustomization::Register(FPropertyEditorModule& PropertyEditorModule)
{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(\
			F##ValueName##CameraVariableReference::StaticStruct()->GetFName(),\
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(\
				[]{ return MakeShared<F##ValueName##CameraVariableReferenceDetailsCustomization>(); }));
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
}

void FCameraVariableReferenceDetailsCustomization::Unregister(FPropertyEditorModule& PropertyEditorModule)
{
	if (UObjectInitialized())
	{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(\
				F##ValueName##CameraVariableReference::StaticStruct()->GetFName());
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
	}
}

void FCameraVariableReferenceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	StructProperty = PropertyHandle;
	StructProperty->SetOnPropertyResetToDefault(
			FSimpleDelegate::CreateSP(this, &FCameraVariableReferenceDetailsCustomization::OnResetToDefault));

	// All references should have a "Variable" property.
	VariableProperty = PropertyHandle->GetChildHandle("Variable");
	ensure(VariableProperty);

	VariableClass = nullptr;
	if (FObjectProperty* VariableObjectProperty = CastField<FObjectProperty>(VariableProperty->GetProperty()))
	{
		VariableClass = VariableObjectProperty->PropertyClass;
	}
	ensure(VariableClass);

	TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasStyle = FGameplayCamerasEditorStyle::Get();

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(100.f)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(VariableBrowserButton, SComboButton)
		.HasDownArrow(true)
		.ContentPadding(1.f)
		.ToolTipText(LOCTEXT("SetVariable_ToolTip", "Selects the camera variable"))
		.IsEnabled(this, &FCameraVariableReferenceDetailsCustomization::IsCameraVariableBrowserEnabled)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(GameplayCamerasStyle->GetBrush("CameraParameter.VariableBrowser"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FCameraVariableReferenceDetailsCustomization::GetVariableName)
				]
			]
		]
		.OnGetMenuContent(this, &FCameraVariableReferenceDetailsCustomization::BuildCameraVariableBrowser)
	];
}

void FCameraVariableReferenceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

TSharedRef<SWidget> FCameraVariableReferenceDetailsCustomization::BuildCameraVariableBrowser()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	const bool bSearchable = false;
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CameraVariableOperations", "Current Variable Reference"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearVariable", "Clear"),
			LOCTEXT("ClearVariable_ToolTip", "Clears the variable"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCameraVariableReferenceDetailsCustomization::OnClearVariable),
				FCanExecuteAction::CreateSP(this, &FCameraVariableReferenceDetailsCustomization::CanClearVariable))
			);
	}
	MenuBuilder.EndSection();

	UCameraVariableAsset* CommonVariable = nullptr;
	{
		UObject* VariableObject = nullptr;
		FPropertyAccess::Result PropertyAccessResult = VariableProperty->GetValue(VariableObject);
		if (PropertyAccessResult == FPropertyAccess::Success)
		{
			CommonVariable = Cast<UCameraVariableAsset>(VariableObject);
		}
	}

	FCameraVariablePickerConfig PickerConfig;
	PickerConfig.CameraVariableClass = VariableClass;
	PickerConfig.InitialCameraVariableSelection = CommonVariable;
	PickerConfig.CameraVariableCollectionSaveSettingsName = TEXT("CameraVariableReferencePicker");
	PickerConfig.OnCameraVariableSelected = FOnCameraVariableSelected::CreateSP(
			this, &FCameraVariableReferenceDetailsCustomization::OnSetVariable);
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = IGameplayCamerasEditorModule::Get();
	TSharedRef<SWidget> PickerWidget = GameplayCamerasEditorModule.CreateCameraVariablePicker(PickerConfig);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CameraVariableBrowser", "Browse"));
	{
		TSharedRef<SWidget> VariableBrowser = SNew(SBox)
			.MinDesiredWidth(300.f)
			.MinDesiredHeight(300.f)
			[
				PickerWidget
			];
		MenuBuilder.AddWidget(VariableBrowser, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool FCameraVariableReferenceDetailsCustomization::IsCameraVariableBrowserEnabled() const
{
	TArray<void*> RawData;
	StructProperty->AccessRawData(RawData);

	for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ++ValueIndex)
	{
		if (HasNonUserOverride(RawData[ValueIndex]))
		{
			return false;
		}
	}

	return true;
}

FText FCameraVariableReferenceDetailsCustomization::GetVariableName() const
{
	UObject* VariableObject = nullptr;
	FPropertyAccess::Result PropertyAccessResult = VariableProperty->GetValue(VariableObject);
	if (PropertyAccessResult == FPropertyAccess::Success)
	{
		if (VariableObject)
		{
			if (UCameraVariableAsset* Variable = Cast<UCameraVariableAsset>(VariableObject))
			{
				return Variable->DisplayName.IsEmpty() ?
					FText::FromName(Variable->GetFName()) :
					FText::FromString(Variable->DisplayName);
			}
			else
			{
				return LOCTEXT("InvalidVariableObject", "Invalid Variable");
			}
		}
		return LOCTEXT("NullVariable", "None");
	}
	else if (PropertyAccessResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleVariableValues", "Multiple Variables");
	}
	else
	{
		return LOCTEXT("ErrorReadingVariable", "Error Reading Variable");
	}
}

bool FCameraVariableReferenceDetailsCustomization::CanClearVariable() const
{
	return VariableProperty->CanResetToDefault();
}

void FCameraVariableReferenceDetailsCustomization::OnClearVariable()
{
	OnSetVariable(nullptr);
}

void FCameraVariableReferenceDetailsCustomization::OnSetVariable(UCameraVariableAsset* InVariable)
{
	TArray<void*> RawData;
	StructProperty->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);

	check(!OuterObjects.Num() || OuterObjects.Num() == RawData.Num());

	{
		FScopedTransaction Transaction(FText::Format(
					LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		StructProperty->NotifyPreChange();

		for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ++ValueIndex)
		{
			SetReferenceVariable(RawData[ValueIndex], InVariable);
		}

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);

		FPropertyChangedEvent ChangeEvent(StructProperty->GetProperty(), EPropertyChangeType::ValueSet, OuterObjects);
		PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	PropertyUtilities->RequestForceRefresh();
	VariableBrowserButton->SetIsOpen(false);
}

void FCameraVariableReferenceDetailsCustomization::OnResetToDefault()
{
	PropertyUtilities->RequestForceRefresh();
}

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
bool F##ValueName##CameraVariableReferenceDetailsCustomization::HasNonUserOverride(void* InRawData) const\
{\
	F##ValueName##CameraVariableReference* TypedData = reinterpret_cast<F##ValueName##CameraVariableReference*>(InRawData);\
	return TypedData->HasNonUserOverride();\
}\
void F##ValueName##CameraVariableReferenceDetailsCustomization::SetReferenceVariable(void* InRawData, UCameraVariableAsset* InVariable)\
{\
	F##ValueName##CameraVariableReference* TypedData = reinterpret_cast<F##ValueName##CameraVariableReference*>(InRawData);\
	TypedData->Variable = CastChecked<U##ValueName##CameraVariable>(InVariable, ECastCheckedType::NullAllowed);\
	TypedData->VariableID = InVariable ? InVariable->GetVariableID() : FCameraVariableID();\
}
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

