// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/FilmbackCameraNodeDetailsCustomization.h"

#include "CineCameraSettings.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Nodes/Common/FilmbackCameraNode.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FilmbackCameraNodeDetailsCustomization"

namespace UE::Cameras
{

TSharedRef<IDetailCustomization> FFilmbackCameraNodeDetailsCustomization::MakeInstance()
{
	return MakeShared<FFilmbackCameraNodeDetailsCustomization>();
}

void FFilmbackCameraNodeDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName SensorWidthPropertyName = GET_MEMBER_NAME_CHECKED(UFilmbackCameraNode, SensorWidth);
	const FName SensorHeightPropertyName = GET_MEMBER_NAME_CHECKED(UFilmbackCameraNode, SensorHeight);

	SensorWidthProperty = DetailBuilder.GetProperty(SensorWidthPropertyName);
	SensorHeightProperty = DetailBuilder.GetProperty(SensorHeightPropertyName);

	DetailBuilder.HideProperty(SensorWidthProperty);
	DetailBuilder.HideProperty(SensorHeightProperty);

	IDetailCategoryBuilder& FilmbackCategory = DetailBuilder.EditCategory(TEXT("Filmback"));

	IDetailGroup& SensorSizeGroup = FilmbackCategory.AddGroup(
			TEXT("SensorSize"), 
			LOCTEXT("SensorSizeRow", "Sensor Size"),
			false,
			true);

	FDetailWidgetRow& FilmbackPresetsRow = SensorSizeGroup.AddWidgetRow();
	FilmbackPresetsRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 1.0f))
			.FillWidth(1)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("PropertyWindow.NoOverlayColor")))
				.Padding(FMargin(0.0f, 2.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("SensorSizePresets", "Sensor Size Presets"))
				]
			]
		]
		.ValueContent()
		.MaxDesiredWidth(0.f)
		[
			SAssignNew(PresetComboBox, SComboBox<TSharedPtr<FText>>)
			.OptionsSource(&PresetComboList)
			.IsEnabled(this, &FFilmbackCameraNodeDetailsCustomization::IsPresetEnabled)
			.OnGenerateWidget(this, &FFilmbackCameraNodeDetailsCustomization::MakePresetComboWidget)
			.OnSelectionChanged(this, &FFilmbackCameraNodeDetailsCustomization::OnPresetChanged)
			.ContentPadding(2)
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FFilmbackCameraNodeDetailsCustomization::GetPresetComboBoxContent)
				.ToolTipText(this, &FFilmbackCameraNodeDetailsCustomization::GetPresetComboBoxContent)
			]
		];

	SensorSizeGroup.AddPropertyRow(SensorWidthProperty.ToSharedRef());
	SensorSizeGroup.AddPropertyRow(SensorHeightProperty.ToSharedRef());

	BuildPresetComboList();
}

void FFilmbackCameraNodeDetailsCustomization::BuildPresetComboList()
{
	const TArray<FNamedFilmbackPreset>& Presets = UCineCameraSettings::GetFilmbackPresets();

	const int32 NumPresets = Presets.Num();
	PresetComboList.Reserve(NumPresets + 1);

	PresetComboList.Add(MakeShared<FText>(LOCTEXT("CustomPreset", "Custom...")));
	for (const FNamedFilmbackPreset& Preset : Presets)
	{
		PresetComboList.Add(MakeShared<FText>(FText::FromString(Preset.Name)));
	}
}

bool FFilmbackCameraNodeDetailsCustomization::IsPresetEnabled() const
{
	if (SensorHeightProperty && SensorWidthProperty)
	{
		return 
			SensorHeightProperty->IsEditable() &&
			SensorWidthProperty->IsEditable() &&
			FSlateApplication::Get().GetNormalExecutionAttribute().Get();
	}
	return false;
}

TSharedRef<SWidget> FFilmbackCameraNodeDetailsCustomization::MakePresetComboWidget(TSharedPtr<FText> InItem)
{
	return
		SNew(STextBlock)
		.Text(*InItem)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FFilmbackCameraNodeDetailsCustomization::OnPresetChanged(TSharedPtr<FText> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	const FString NewPresetName = NewSelection->ToString();

	const TArray<FNamedFilmbackPreset>& Presets = UCineCameraSettings::GetFilmbackPresets();
	for (const FNamedFilmbackPreset& Preset : Presets)
	{
		if (Preset.Name == NewPresetName)
		{
			const FName ParameterValueName = GET_MEMBER_NAME_CHECKED(FFloatCameraParameter, Value);

			const FScopedTransaction Transaction(LOCTEXT("ChangeFilmbackPreset", "Change Filmback Preset"));
			
			TSharedPtr<IPropertyHandle> SensorWidthValueProperty = SensorWidthProperty->GetChildHandle(ParameterValueName);
			TSharedPtr<IPropertyHandle> SensorHeightValueProperty = SensorHeightProperty->GetChildHandle(ParameterValueName);

			SensorWidthValueProperty->SetValue(Preset.FilmbackSettings.SensorWidth);
			SensorHeightValueProperty->SetValue(Preset.FilmbackSettings.SensorHeight);
			
			break;
		}
	}
}

FText FFilmbackCameraNodeDetailsCustomization::GetPresetComboBoxContent() const
{
	const FName ParameterValueName = GET_MEMBER_NAME_CHECKED(FFloatCameraParameter, Value);

	TSharedPtr<IPropertyHandle> SensorWidthValueProperty = SensorWidthProperty->GetChildHandle(ParameterValueName);
	TSharedPtr<IPropertyHandle> SensorHeightValueProperty = SensorHeightProperty->GetChildHandle(ParameterValueName);

	float CurSensorWidth;
	float CurSensorHeight;
	if (SensorWidthValueProperty->GetValue(CurSensorWidth) == FPropertyAccess::MultipleValues ||
			SensorHeightValueProperty->GetValue(CurSensorHeight) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	const TArray<FNamedFilmbackPreset>& Presets = UCineCameraSettings::GetFilmbackPresets();
	for (int32 PresetIndex = 0, NumPresets = Presets.Num(); PresetIndex < NumPresets; ++PresetIndex)
	{
		const FNamedFilmbackPreset& Preset(Presets[PresetIndex]);
		if ((Preset.FilmbackSettings.SensorWidth == CurSensorWidth) && 
				(Preset.FilmbackSettings.SensorHeight == CurSensorHeight))
		{
			if (PresetComboList.IsValidIndex(PresetIndex + 1))
			{
				return *PresetComboList[PresetIndex + 1];
			}
		}
	}

	return *PresetComboList[0];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

