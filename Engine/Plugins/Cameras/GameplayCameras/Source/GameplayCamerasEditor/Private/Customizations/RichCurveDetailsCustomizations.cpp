// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/RichCurveDetailsCustomizations.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "CurveEditor.h"
#include "CurveEditor/CurvePropertyEditorTreeItem.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/CameraRotatorCurve.h"
#include "Curves/CameraSingleCurve.h"
#include "Curves/CameraVectorCurve.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "ICurveEditorBounds.h"
#include "IDetailChildrenBuilder.h"
#include "IGameplayCamerasEditorModule.h"
#include "Misc/EngineVersionComparison.h"
#include "PropertyEditorModule.h"
#include "RichCurveEditorModel.h"
#include "SCurveEditorPanel.h"
#include "SCurveKeyDetailPanel.h"
#include "Slate/SlateTextures.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Toolkits/ToolkitManager.h"
#include "Tree/CurveEditorTree.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
#include "SResizeBox.h"
#else
#include "Compat/SResizeBox.h"
#endif

#define LOCTEXT_NAMESPACE "RichCurveDetailsCustomizations"

namespace UE::Cameras
{

float GGameplayCamerasRichCurvePreviewOpacity = 0.9f;
static FAutoConsoleVariableRef CVarGameplayCamerasRichCurvePreviewOpacity(
	TEXT("GameplayCameras.RichCurvePreview.Opacity"),
	GGameplayCamerasRichCurvePreviewOpacity,
	TEXT("(Default: 0.9. The opacity of curve colors in the Details View preview."));

float GGameplayCamerasRichCurvePreviewDesaturation = 0.1f;
static FAutoConsoleVariableRef CVarGameplayCamerasRichCurvePreviewDesaturation(
	TEXT("GameplayCameras.RichCurvePreview.Desaturation"),
	GGameplayCamerasRichCurvePreviewDesaturation,
	TEXT("(Default: 0.1. The desaturation of curve colors in the Details View preview."));

/** Viewport interface for SRichCurveViewport. */
class FRichCurveViewportInterface
	: public TSharedFromThis<FRichCurveViewportInterface>
	, public ISlateViewport
{

public:

	FRichCurveViewportInterface();
	~FRichCurveViewportInterface();

	bool UpdateDesiredSize(const FIntPoint& InDesiredSize);

	FRenderTarget* GetRenderTarget() const;

public:

	// ISlateViewport interface.
	virtual FIntPoint GetSize() const override;
	virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	virtual bool RequiresVsync() const override;

private:

	void CreateTexture();
	void DestroyTexture(bool bImmediately);

private:

	FIntPoint DesiredTextureSize = FIntPoint::ZeroValue;
	FSlateTexture2DRHIRef* Texture = nullptr;
	FSlateTextureRenderTarget2DResource* RenderTarget = nullptr;
};

FRichCurveViewportInterface::FRichCurveViewportInterface()
{
	CreateTexture();
}

FRichCurveViewportInterface::~FRichCurveViewportInterface()
{
	DestroyTexture(false);
}

bool FRichCurveViewportInterface::UpdateDesiredSize(const FIntPoint& InDesiredSize)
{
	if (DesiredTextureSize == InDesiredSize)
	{
		return false;
	}

	DestroyTexture(true);

	DesiredTextureSize = InDesiredSize;

	CreateTexture();

	return true;
}

FRenderTarget* FRichCurveViewportInterface::GetRenderTarget() const
{
	return RenderTarget;
}

void FRichCurveViewportInterface::CreateTexture()
{
	if (DesiredTextureSize.X == 0 || DesiredTextureSize.Y == 0)
	{
		ensure(Texture == nullptr && RenderTarget == nullptr);
		return;
	}

	Texture = new FSlateTexture2DRHIRef(DesiredTextureSize.X, DesiredTextureSize.Y, PF_B8G8R8A8, NULL, TexCreate_None);
	RenderTarget = new FSlateTextureRenderTarget2DResource(FLinearColor::Black, DesiredTextureSize.X, DesiredTextureSize.Y, PF_B8G8R8A8, SF_Point, TA_Wrap, TA_Wrap, 0.0f);

	FSlateTexture2DRHIRef* TexturePtr = Texture;
	FSlateTextureRenderTarget2DResource* RenderTargetPtr = RenderTarget;

	ENQUEUE_RENDER_COMMAND(AssignRenderTarget)(
		[TexturePtr, RenderTargetPtr](FRHICommandList& RHICmdList)
		{
			if (TexturePtr && RenderTargetPtr)
			{
				TexturePtr->InitResource(RHICmdList);
				RenderTargetPtr->InitResource(RHICmdList);
				TexturePtr->SetRHIRef(RenderTargetPtr->GetTextureRHI(), RenderTargetPtr->GetSizeX(), RenderTargetPtr->GetSizeY());
			}
		}
	);
}

void FRichCurveViewportInterface::DestroyTexture(bool bImmediately)
{
	if (Texture)
	{
		FSlateTexture2DRHIRef* TexturePtr = Texture;
		FSlateTextureRenderTarget2DResource* RenderTargetPtr = RenderTarget;
		
		auto DestroyTextureImpl = [TexturePtr, RenderTargetPtr]()
		{
			ENQUEUE_RENDER_COMMAND(DestroyTexture)(
				[TexturePtr, RenderTargetPtr](FRHICommandList& RHICmdList)
				{
					if (TexturePtr)
					{
						TexturePtr->ReleaseResource();
						delete TexturePtr;
					}
					if (RenderTargetPtr)
					{
						RenderTargetPtr->ReleaseResource();
						delete RenderTargetPtr;
					}
				}
			);
		};

		if (bImmediately)
		{
			DestroyTextureImpl();
		}
		else
		{
			GEditor->GetTimerManager()->SetTimerForNextTick(DestroyTextureImpl);
		}

		Texture = nullptr;
		RenderTarget = nullptr;
	}
}

FIntPoint FRichCurveViewportInterface::GetSize() const
{
	if (Texture)
	{
		return FIntPoint(Texture->GetWidth(), Texture->GetHeight());
	}
	return FIntPoint::ZeroValue;
}

FSlateShaderResource* FRichCurveViewportInterface::GetViewportRenderTargetTexture() const
{
	return Texture;
}

bool FRichCurveViewportInterface::RequiresVsync() const
{
	return false;
}

/** A viewport that renders a preview of one or more rich curves. */
class SRichCurveViewport : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRichCurveViewport)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void AddCurve(FRichCurve* InCurve, const FText& InCurveName, const FLinearColor& InCurveColor, TWeakObjectPtr<> InWeakOwner);
	int32 NumCurves() const { return Curves.Num(); }

	void InitializeCurveEditor(TSharedRef<FCurveEditor> InCurveEditor);

	void InvalidateCurves();

protected:

	// SWidget interface.
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	FVector2D GetViewportSize() const;

	void DrawCurves();
	void DrawCurve(FCanvas* Canvas, const FRichCurve* Curve, const FLinearColor& Color, float MinTime, float MaxTime, float MinValue, float MaxValue);

private:

	TSharedPtr<SViewport> Viewport;
	TSharedPtr<FRichCurveViewportInterface> ViewportInterface;
	
	struct FCurveInfo
	{
		FRichCurve* Curve = nullptr;
		FText CurveName;
		FLinearColor CurveColor;
		TWeakObjectPtr<> WeakOwner;
		FName PropertyName;
	};
	TArray<FCurveInfo> Curves;

	FVector2D DesiredViewportSize = FVector2D(300, 48);
	bool bNeedsRedraw = false;
};

void SRichCurveViewport::Construct(const FArguments& InArgs)
{
	ViewportInterface = MakeShared<FRichCurveViewportInterface>();

	ChildSlot
	[
		SAssignNew(Viewport, SViewport)
		.ViewportInterface(ViewportInterface)
		.ViewportSize(this, &SRichCurveViewport::GetViewportSize)
	];
}

void SRichCurveViewport::AddCurve(FRichCurve* InCurve, const FText& InCurveName, const FLinearColor& InCurveColor, TWeakObjectPtr<> InWeakOwner)
{
	Curves.Add(FCurveInfo{ InCurve, InCurveName, InCurveColor, InWeakOwner });
	bNeedsRedraw = true;
}

void SRichCurveViewport::InitializeCurveEditor(TSharedRef<FCurveEditor> InCurveEditor)
{
	for (const FCurveInfo& CurveInfo : Curves)
	{
		FCurveEditorTreeItem* TreeItem = InCurveEditor->AddTreeItem(FCurveEditorTreeItemID::Invalid());

		TSharedPtr<FCurvePropertyEditorTreeItem> TreeItemModel = MakeShared<FCurvePropertyEditorTreeItem>(
				CurveInfo.Curve, CurveInfo.CurveName, CurveInfo.CurveColor, CurveInfo.WeakOwner);
		TreeItemModel->Info.PropertyName = CurveInfo.PropertyName;
		TreeItem->SetStrongItem(TreeItemModel);

		for (const FCurveModelID& CurveModelID : TreeItem->GetOrCreateCurves(&InCurveEditor.Get()))
		{
			InCurveEditor->PinCurve(CurveModelID);
		}
	}
}

void SRichCurveViewport::InvalidateCurves()
{
	bNeedsRedraw = true;
}

void SRichCurveViewport::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

void SRichCurveViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();
	DesiredViewportSize = FVector2D(LocalSize.X, 48);

	const bool bSizeChanged = ViewportInterface->UpdateDesiredSize(FIntPoint((int32)LocalSize.X, (int32)LocalSize.Y));
	if (bSizeChanged || bNeedsRedraw)
	{
		DrawCurves();

		bNeedsRedraw = false;
	}
}

FVector2D SRichCurveViewport::GetViewportSize() const
{
	return DesiredViewportSize;
}

void SRichCurveViewport::DrawCurves()
{
	if (Curves.IsEmpty())
	{
		return;
	}

	float MinTime, MaxTime;
	Curves[0].Curve->GetTimeRange(MinTime, MaxTime);

	float MinValue, MaxValue;
	Curves[0].Curve->GetValueRange(MinValue, MaxValue);

	for (int32 CurveIndex = 1; CurveIndex < Curves.Num(); ++CurveIndex)
	{
		float CurMinTime, CurMaxTime;
		Curves[CurveIndex].Curve->GetTimeRange(CurMinTime, CurMaxTime);

		MinTime = FMath::Min(MinTime, CurMinTime);
		MaxTime = FMath::Max(MaxTime, CurMaxTime);

		float CurMinValue, CurMaxValue;
		Curves[CurveIndex].Curve->GetValueRange(CurMinValue, CurMaxValue);

		MinValue = FMath::Min(MinValue, CurMinValue);
		MaxValue = FMath::Max(MaxValue, CurMaxValue);
	}

	FCanvas Canvas(ViewportInterface->GetRenderTarget(), NULL, FGameTime::GetTimeSinceAppStart(), GMaxRHIFeatureLevel);
	Canvas.Clear(FLinearColor::Black);

	for (const FCurveInfo& CurveInfo : Curves)
	{
		DrawCurve(&Canvas, CurveInfo.Curve, CurveInfo.CurveColor, MinTime, MaxTime, MinValue, MaxValue);
	}

	Canvas.Flush_GameThread();
}

void SRichCurveViewport::DrawCurve(FCanvas* Canvas, const FRichCurve* Curve, const FLinearColor& Color, float MinTime, float MaxTime, float MinValue, float MaxValue)
{
	const FVector2D TextureSize = Canvas->GetRenderTarget()->GetSizeXY();
	ensure(TextureSize.X > 0 && TextureSize.Y > 0);

	const float TimeRange = (MaxTime - MinTime);
	const float ValueRange = (MaxValue - MinValue);

	if (Curve->GetNumKeys() < 1 || TimeRange <= 0 || ValueRange <= 0)
	{
		FCanvasLineItem Line(FVector2D(0, TextureSize.Y / 2), FVector2D(TextureSize.X, TextureSize.Y / 2));
		Line.LineThickness = 1.5;
		Line.SetColor(FLinearColor::Gray);
		Line.Draw(Canvas);
		return;
	}
	
	// Add 10% horizontal padding (in curve time space) to draw a bit of the curve before and after
	// the first and last control points.
	const float PaddingTime = 0.1f * TimeRange;
	const float FullTimeRange = TimeRange + 2.f * PaddingTime;
	const int32 NumSamples = TextureSize.X / 2.5;
	const float TimeStep = FullTimeRange / NumSamples;

	// Add vertical padding (in pixels) to have some breathing room above/below the curve.
	const float VerticalPadding = 5.f;
	const float TimeToPixel = TextureSize.X / FullTimeRange;
	const float ValueToPixel = (TextureSize.Y - 2.f * VerticalPadding) / ValueRange;

	// Make colors muted in the preview.
	const FLinearColor PreviewColor = Color
		.CopyWithNewOpacity(GGameplayCamerasRichCurvePreviewOpacity)
		.Desaturate(GGameplayCamerasRichCurvePreviewDesaturation);

	FVector2D PrevPos;
	const float FirstTime = MinTime - PaddingTime;
	for (int32 Index = 0; Index < NumSamples; Index++)
	{
		const float Time = FirstTime + Index * TimeStep;
		const float Value = Curve->Eval(Time);
		const float NormalizedValue = (Value - MinValue) / ValueRange;

		FVector2D Pos;
		Pos.X = (Time - FirstTime) * TimeToPixel;
		Pos.Y = (TextureSize.Y - VerticalPadding) - (Value - MinValue) * ValueToPixel;

		if (Index > 0)
		{
			FCanvasLineItem Line(PrevPos, Pos);
			Line.LineThickness = 1.5;
			Line.SetColor(PreviewColor);
			Line.BlendMode = SE_BLEND_Translucent;
			Line.Draw(Canvas);
		}

		PrevPos = Pos;
	}
}

FOnInvokeCurveEditor FRichCurveDetailsCustomization::OnInvokeCurveEditorDelegate;

void FRichCurveDetailsCustomization::Register(FPropertyEditorModule& PropertyEditorModule)
{
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			FCameraSingleCurve::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(
				[]{ return MakeShared<FRichSingleCurveDetailsCustomization>(); }));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			FCameraVectorCurve::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(
				[]{ return MakeShared<FRichVectorCurveDetailsCustomization>(); }));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			FCameraRotatorCurve::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(
				[]{ return MakeShared<FRichRotatorCurveDetailsCustomization>(); }));
}

void FRichCurveDetailsCustomization::Unregister(FPropertyEditorModule& PropertyEditorModule)
{
	if (UObjectInitialized())
	{
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FCameraSingleCurve::StaticStruct()->GetFName());
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FCameraRotatorCurve::StaticStruct()->GetFName());
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FCameraVectorCurve::StaticStruct()->GetFName());
	}
}

FRichCurveDetailsCustomization::~FRichCurveDetailsCustomization()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FRichCurveDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PrivatePropertyHandle = PropertyHandle;

	PropertyHandle->SetOnPropertyValueChanged(
			FSimpleDelegate::CreateSP(this, &FRichCurveDetailsCustomization::OnPropertyValueChanged));
	PropertyHandle->SetOnPropertyResetToDefault(
			FSimpleDelegate::CreateSP(this, &FRichCurveDetailsCustomization::OnPropertyValueChanged));

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FRichCurveDetailsCustomization::OnObjectPropertyChanged);

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(800.f)
		[
			SAssignNew(HeaderLayout, SHorizontalBox)
		];

	TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

	HeaderLayout->AddSlot()
	.Padding(0.0f, 3.0f, 5.0f, 0.0f)
	.FillWidth(1.f)
	.FillContentWidth(1.f)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SBorder)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
		.BorderImage(FAppStyle::Get().GetBrush("PropertyEditor.AssetTileItem.DropShadow"))
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.Padding(1.f)
			[
				SNew(SBorder)
				.Padding(0.0f)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.ToolTipText(LOCTEXT("CurvePreviewToolTip", "Preview of the curves"))
				[
					SAssignNew(RichCurveViewport, SRichCurveViewport)
				]
			]
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("PropertyEditor.AssetThumbnailBorder"))
			]
		]
	];

	HeaderLayout->AddSlot()
	.Padding(0.0f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.Padding(FMargin(0.0f, 2.0f, 4.0f, 2.0f))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.WidthOverride(22.f)
			.HeightOverride(22.f)
			.ToolTipText(LOCTEXT("EditCurves", "Edit Curves"))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(0.0f)
				.IsEnabled(!PropertyHandle->IsEditConst())
				.OnClicked(this, &FRichCurveDetailsCustomization::OnFocusInCurvesTab)
				[
					SNew(SImage)
					.Image(GameplayCamerasEditorStyle->GetBrush("CurveEditor.ShowInCurvesTab"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	];

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	ensure(RawData.Num() == OuterObjects.Num());

	const FText PropertyDisplayName = FText::FromName(PropertyHandle->GetProperty()->GetFName());
	TSharedRef<SRichCurveViewport> RichCurveViewportRef = RichCurveViewport.ToSharedRef();

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		AddCurves(PropertyHandle, RichCurveViewportRef, PropertyDisplayName, OuterObjects[Index], RawData[Index]);
	}
}

void FRichCurveDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedRef<FCurveEditor> CurveEditor = CreateCurveEditor();
	TSharedRef<SWidget> CurveEditorPanel = CreateCurveEditorPanel(CurveEditor);

	// We need to do this after the curve editor panel has been created because the curve editor tree view
	// doesn't initially read any existing items... we need to add these items afterwards.
	RichCurveViewport->InitializeCurveEditor(CurveEditor);

	ChildBuilder.AddCustomRow(LOCTEXT("InlineCurveEditorSearchString", "Curve Editor"))
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		.MinDesiredWidth(300.f)
		[
			SNew(SVerticalResizeBox)
			.ContentHeight(300.f)
			[
				CurveEditorPanel
			]
		];
}

void FRichCurveDetailsCustomization::OnPropertyValueChanged()
{
	if (RichCurveViewport)
	{
		RichCurveViewport->InvalidateCurves();
	}
}

void FRichCurveDetailsCustomization::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (RichCurveViewport && PropertyChangedEvent.Property == nullptr)
	{
		TArray<UObject*> OuterObjects;
		PrivatePropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Contains(Object))
		{
			RichCurveViewport->InvalidateCurves();
		}
	}
}

FReply FRichCurveDetailsCustomization::OnFocusInCurvesTab()
{
	TArray<UObject*> OuterObjects;
	PrivatePropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() > 0)
	{
		// Close the inline curve editor when moving over to the curves tab.
		PrivatePropertyHandle->SetExpanded(false);

		OnInvokeCurveEditorDelegate.Broadcast(OuterObjects[0], PrivatePropertyHandle->GetProperty()->GetFName());
	}

	return FReply::Handled();
}

TSharedRef<FCurveEditor> FRichCurveDetailsCustomization::CreateCurveEditor()
{
	TSharedRef<FCurveEditor> CurveEditor = MakeShared<FCurveEditor>();

	FCurveEditorInitParams InitParams;
	CurveEditor->InitCurveEditor(InitParams);

	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

	TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FStaticCurveEditorBounds>();
	CurveEditor->SetBounds(MoveTemp(EditorBounds));

	return CurveEditor;
}

TSharedRef<SWidget> FRichCurveDetailsCustomization::CreateCurveEditorPanel(TSharedRef<FCurveEditor> CurveEditor)
{
	// Build the curve editor panel.
	TSharedRef<SCurveEditorPanel> CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor)
		.MinimumViewPanelHeight(50.0f);
	// No tree view in the inline curve editor.

	// Build the toolbar.
	TSharedPtr<FUICommandList> Commands = CurveEditorPanel->GetCommands();
	TSharedPtr<FExtender> ToolbarExtender = CurveEditorPanel->GetToolbarExtender();

	FSlimHorizontalToolBarBuilder ToolBarBuilder(Commands, FMultiBoxCustomization::None, ToolbarExtender, true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.EndSection();
	TSharedRef<SWidget> ToolBarWidget = ToolBarBuilder.MakeWidget();

	// Assemble everything.
	TSharedRef<SBorder> CurveEditorPanelWrapper = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16.0f, 16.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				ToolBarWidget
				//SNew(SHorizontalBox)
				//+SHorizontalBox::Slot()
				//.AutoWidth()
				//[
				//	SNew(STextBlock)
				//	.Text(LOCTEXT("KeyDetails", "Key Details"))
				//]
				//+SHorizontalBox::Slot()
				//.FillWidth(1.f)
				//.HAlign(HAlign_Left)
				//[
				//	CurveEditorPanel->GetKeyDetailsView().ToSharedRef()
				//]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				CurveEditorPanel
			]
		];

	return CurveEditorPanelWrapper;
}

void FRichSingleCurveDetailsCustomization::AddCurves(
		TSharedRef<IPropertyHandle> PropertyHandle, 
		TSharedRef<SRichCurveViewport> InRichCurveViewport,
		const FText& PropertyDisplayName,
		UObject* OuterObject, 
		void* RawData)
{
	FCameraSingleCurve* SingleCurve = reinterpret_cast<FCameraSingleCurve*>(RawData);
	InRichCurveViewport->AddCurve(&SingleCurve->Curve, PropertyDisplayName, FLinearColor::Red, OuterObject);
}

void FRichRotatorCurveDetailsCustomization::AddCurves(
		TSharedRef<IPropertyHandle> PropertyHandle, 
		TSharedRef<SRichCurveViewport> InRichCurveViewport,
		const FText& PropertyDisplayName,
		UObject* OuterObject, 
		void* RawData)
{
	static const FText CurvePropertyName = LOCTEXT("CurvePropertyNameFmt", "{0}.{1}");

	FCameraRotatorCurve* RotatorCurve = reinterpret_cast<FCameraRotatorCurve*>(RawData);
	InRichCurveViewport->AddCurve(
			&RotatorCurve->Curves[0], 
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("Yaw", "Yaw") }), FLinearColor::Red, OuterObject);
	InRichCurveViewport->AddCurve(
			&RotatorCurve->Curves[1], 
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("Pitch", "Pitch") }), FLinearColor::Green, OuterObject);
	InRichCurveViewport->AddCurve(
			&RotatorCurve->Curves[2], 
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("Roll", "Roll") }), FLinearColor::Blue, OuterObject);
}

void FRichVectorCurveDetailsCustomization::AddCurves(
		TSharedRef<IPropertyHandle> PropertyHandle, 
		TSharedRef<SRichCurveViewport> InRichCurveViewport,
		const FText& PropertyDisplayName,
		UObject* OuterObject, 
		void* RawData)
{
	static const FText CurvePropertyName = LOCTEXT("CurvePropertyNameFmt", "{0}.{1}");

	FCameraVectorCurve* VectorCurve = reinterpret_cast<FCameraVectorCurve*>(RawData);
	InRichCurveViewport->AddCurve(
			&VectorCurve->Curves[0],
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("X", "X") }), FLinearColor::Red, OuterObject);
	InRichCurveViewport->AddCurve(
			&VectorCurve->Curves[1], 
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("Y", "Y") }), FLinearColor::Green, OuterObject);
	InRichCurveViewport->AddCurve(
			&VectorCurve->Curves[2], 
			FText::Format(CurvePropertyName, { PropertyDisplayName, LOCTEXT("Z", "Z") }), FLinearColor::Blue, OuterObject);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

