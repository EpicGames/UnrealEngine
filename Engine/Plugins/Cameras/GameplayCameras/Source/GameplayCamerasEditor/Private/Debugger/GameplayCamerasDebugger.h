// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Cameras
{

class FGameplayCamerasDebugger
{
};

}  // namespace UE::Cameras

