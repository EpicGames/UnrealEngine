// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SBlendStacksDebugPanel.h"

#include "Debugger/SDebugWidgetUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBlendStacksDebugPanel"

namespace UE::Cameras
{

void SBlendStacksDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
					.Margin(4.f)
					.Text(LOCTEXT("FilterBlendStackNames", "Filter blend stack names:"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableTextBox(
						TEXT("GameplayCameras.Debug.BlendStacks.Filter"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowUnchanged", "Show unchanged properties"),
						TEXT("GameplayCameras.Debug.BlendStack.ShowUnchanged"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowVariableIDs", "Show variable IDs"),
						TEXT("GameplayCameras.Debug.BlendStack.ShowVariableIDs"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowDataIDs", "Show data IDs"),
						TEXT("GameplayCameras.Debug.BlendStack.ShowDataIDs"))
			]
	];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

