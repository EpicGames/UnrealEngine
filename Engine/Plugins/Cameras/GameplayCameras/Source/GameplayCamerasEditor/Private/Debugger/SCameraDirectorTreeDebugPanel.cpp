// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SCameraDirectorTreeDebugPanel.h"

#include "Debugger/SDebugWidgetUtils.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SCameraDirectorTreeDebugPanel"

namespace UE::Cameras
{

void SCameraDirectorTreeDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowUnchanged", "Show unchanged properties"),
						TEXT("GameplayCameras.Debug.ContextInitialResult.ShowUnchanged"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowVariableIDs", "Show variable IDs"),
						TEXT("GameplayCameras.Debug.ContextInitialResult.ShowVariableIDs"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowDataIDs", "Show data IDs"),
						TEXT("GameplayCameras.Debug.ContextInitialResult.ShowDataIDs"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowCoordinateSystem", "Show context coordinate system"),
						TEXT("GameplayCameras.Debug.ContextInitialResult.ShowCoordinateSystem"))
			]
	];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

