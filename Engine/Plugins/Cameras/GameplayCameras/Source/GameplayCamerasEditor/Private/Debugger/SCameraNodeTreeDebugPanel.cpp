// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SCameraNodeTreeDebugPanel.h"

#include "Debugger/SDebugWidgetUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCameraNodeTreeDebugPanel"

namespace UE::Cameras
{

void SCameraNodeTreeDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
					.Margin(4.f)
					.Text(LOCTEXT("FilterNodeNames", "Filter node names:"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableTextBox(
						TEXT("GameplayCameras.Debug.NodeTree.Filter"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowDampingLocalSpace", "Show damping local spaces"),
						TEXT("GameplayCameras.Debug.Damping.ShowLocalSpace"))
			]
	];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

