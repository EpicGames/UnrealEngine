// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SCameraPoseStatsDebugPanel.h"

#include "Debugger/SDebugWidgetUtils.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SCameraPoseStatsDebugPanel"

namespace UE::Cameras
{

void SCameraPoseStatsDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowUnchanged", "Show unchanged properties"),
						TEXT("GameplayCameras.Debug.PoseStats.ShowUnchanged"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowVariableIDs", "Show variable IDs"),
						TEXT("GameplayCameras.Debug.PoseStats.ShowVariableIDs"))
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowDataIDs", "Show data IDs"),
						TEXT("GameplayCameras.Debug.PoseStats.ShowDataIDs"))
			]
	];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

