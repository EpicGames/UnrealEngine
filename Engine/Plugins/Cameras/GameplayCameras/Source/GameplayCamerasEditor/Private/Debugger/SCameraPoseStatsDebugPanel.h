// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"

class FTabManager;

namespace UE::Cameras
{

class SCameraPoseStatsDebugPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraPoseStatsDebugPanel) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);
};

}  // namespace UE::Cameras

