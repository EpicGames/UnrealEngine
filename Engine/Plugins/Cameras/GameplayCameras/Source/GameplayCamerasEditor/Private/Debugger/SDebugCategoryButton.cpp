// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SDebugCategoryButton.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Cameras
{

void SDebugCategoryButton::Construct(const FArguments& InArgs)
{
	DebugCategoryName = InArgs._DebugCategoryName.Get();
	IsDebugCategoryActive = InArgs._IsDebugCategoryActive;
	RequestDebugCategoryChange = InArgs._RequestDebugCategoryChange;

	InactiveModeBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Normal");
	ActiveModeBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Pressed");
	HoverBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Hovered");

	const FMargin IconPadding(8.0f, 8.0f, 4.0f, 8.0f);
	const FMargin TextPadding(4.0f, 8.0f, 8.0f, 8.0f);

	TSharedRef<SHorizontalBox> ButtonContents = SNew(SHorizontalBox);

	if (InArgs._IconImage.IsSet())
	{
		ButtonContents->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(IconPadding)
			[
				SNew(SImage)
					.Image(InArgs._IconImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	ButtonContents->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(TextPadding)
		[
			SNew(STextBlock)
				.Text(InArgs._DisplayText)
		];

	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SNew(SCheckBox)
			.ToolTipText(InArgs._ToolTipText)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.IsChecked(this, &SDebugCategoryButton::GetDebugCategoryCheckState)
			.OnCheckStateChanged(this, &SDebugCategoryButton::OnDebugCategoryCheckStateChanged)
			[
				ButtonContents
			]
	];
}

ECheckBoxState SDebugCategoryButton::GetDebugCategoryCheckState() const
{
	return IsDebugCategoryActive.Get() ?
		ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDebugCategoryButton::OnDebugCategoryCheckStateChanged(ECheckBoxState CheckBoxState)
{
	RequestDebugCategoryChange.ExecuteIfBound(DebugCategoryName);
}

}  // namespace UE::Cameras

