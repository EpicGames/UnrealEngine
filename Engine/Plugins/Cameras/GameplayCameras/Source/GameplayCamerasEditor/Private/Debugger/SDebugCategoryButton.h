// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;

namespace UE::Cameras
{

DECLARE_DELEGATE_OneParam(FOnDebugCategoryChangeRequested, const FString&);

class SDebugCategoryButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDebugCategoryButton) {}
		SLATE_ATTRIBUTE(const FSlateBrush*, IconImage)
		SLATE_ATTRIBUTE(FText, DisplayText)
		SLATE_ATTRIBUTE(FText, ToolTipText)
		SLATE_ATTRIBUTE(FString, DebugCategoryName)
		SLATE_ATTRIBUTE(bool, IsDebugCategoryActive)
		SLATE_EVENT(FOnDebugCategoryChangeRequested, RequestDebugCategoryChange)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	ECheckBoxState GetDebugCategoryCheckState() const;
	void OnDebugCategoryCheckStateChanged(ECheckBoxState CheckBoxState);

private:

	FString DebugCategoryName;
	TAttribute<bool> IsDebugCategoryActive;
	FOnDebugCategoryChangeRequested RequestDebugCategoryChange;

	const FSlateBrush* ActiveModeBorderImage;
	const FSlateBrush* InactiveModeBorderImage;
	const FSlateBrush* HoverBorderImage;
};

}  // namespace UE::Cameras

