// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SDebugWidgetUtils.h"

#include "HAL/IConsoleManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDebugWidgetUtils"

namespace UE::Cameras
{

TSharedRef<SWidget> SDebugWidgetUtils::CreateConsoleVariableCheckBox(const FText& Text, const FString& ConsoleVariableName)
{
	IConsoleVariable* ConsoleVariable = SDebugWidgetUtils::GetConsoleVariable(ConsoleVariableName);

	return SNew(SCheckBox)
		.Padding(4.f)
		.IsChecked_Lambda([ConsoleVariable]() 
				{
					if (ConsoleVariable)
					{
						return ConsoleVariable->GetBool() ?
							ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Undetermined;
				})
		.OnCheckStateChanged_Lambda([ConsoleVariable](ECheckBoxState NewState)
				{
					if (ConsoleVariable)
					{
						ConsoleVariable->Set(NewState == ECheckBoxState::Checked);
					}
				})
		[
			SNew(STextBlock)
				.Text(Text)
		];
}

TSharedRef<SWidget> SDebugWidgetUtils::CreateConsoleVariableSpinBox(const FString& ConsoleVariableName)
{
	IConsoleVariable* ConsoleVariable = SDebugWidgetUtils::GetConsoleVariable(ConsoleVariableName);

	return SNew(SSpinBox<float>)
		.Value_Lambda([ConsoleVariable]()
				{
					return ConsoleVariable ? ConsoleVariable->GetFloat() : 0.f;
				})
		.OnValueChanged_Lambda([ConsoleVariable](float NewValue)
				{
					if (ConsoleVariable)
					{
						ConsoleVariable->Set(NewValue);
					}
				})
		.OnValueCommitted_Lambda([ConsoleVariable](float NewValue, ETextCommit::Type CommitType)
				{
					if (ConsoleVariable)
					{
						ConsoleVariable->Set(NewValue);
					}
				});
}

TSharedRef<SWidget> SDebugWidgetUtils::CreateConsoleVariableSpinBox(const FText& Text, const FString& ConsoleVariableName)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(4.f)
			[
				SNew(STextBlock)
					.Text(Text)
			]
		+ SHorizontalBox::Slot()
			.Padding(4.f)
			[
				CreateConsoleVariableSpinBox(ConsoleVariableName)
			];
}

TSharedRef<SWidget> SDebugWidgetUtils::CreateConsoleVariableComboBox(const FString& ConsoleVariableName, TArray<TSharedPtr<FString>>* OptionsSource)
{
	IConsoleVariable* ConsoleVariable = SDebugWidgetUtils::GetConsoleVariable(ConsoleVariableName);

	TSharedPtr<FString> InitialSelection;
	if (ConsoleVariable)
	{
		const FString CurrentValue = ConsoleVariable->GetString();
		TSharedPtr<FString>* FoundItem = OptionsSource->FindByPredicate([&CurrentValue](TSharedPtr<FString> Item)
				{
					return Item && CurrentValue == *Item.Get();
				});
		if (FoundItem)
		{
			InitialSelection = *FoundItem;
		}
	}

	return SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(OptionsSource)
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> InOption)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*InOption));
				})
		.OnSelectionChanged_Lambda([ConsoleVariable](TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
				{
					if (ConsoleVariable)
					{
						ConsoleVariable->Set(**NewSelection.Get());
					}
				})
		.InitiallySelectedItem(InitialSelection)
		.ContentPadding(2.f)
		.Content()
		[
			SNew(STextBlock)
				.Text_Lambda([ConsoleVariable]() -> FText
						{
							if (ConsoleVariable)
							{
								return FText::FromString(ConsoleVariable->GetString());
							}
							return LOCTEXT("NoSuchVariable", "<no such variable>");
						})
		];
}

TSharedRef<SWidget> SDebugWidgetUtils::CreateConsoleVariableTextBox(const FString& ConsoleVariableName)
{
	IConsoleVariable* ConsoleVariable = SDebugWidgetUtils::GetConsoleVariable(ConsoleVariableName);

	return SNew(SEditableTextBox)
		.Padding(4.f)
		.Text_Lambda([ConsoleVariable]()
				{
					if (ConsoleVariable)
					{
						return FText::FromString(ConsoleVariable->GetString());
					}
					return FText();
				})
		.OnTextChanged_Lambda([ConsoleVariable](const FText& NewText)
				{
					if (ConsoleVariable)
					{
						ConsoleVariable->Set(*NewText.ToString());
					}
				});
}

IConsoleVariable* SDebugWidgetUtils::GetConsoleVariable(const FString& ConsoleVariableName)
{
	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
	ensureMsgf(ConsoleVariable, TEXT("No such console variable: %s"), *ConsoleVariableName);
	return ConsoleVariable;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

