// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointerFwd.h"

class FString;
class IConsoleVariable;
class SWidget;

namespace UE::Cameras
{

/**
 * Utility class for creating Slate widgets that drive console variables.
 */
class SDebugWidgetUtils
{
public:

	/** Creates a checkbox that sets a boolean console variable. */
	static TSharedRef<SWidget> CreateConsoleVariableCheckBox(const FText& Text, const FString& ConsoleVariableName);
	/** Creates a spinbox that sets a float console variable. */
	static TSharedRef<SWidget> CreateConsoleVariableSpinBox(const FString& ConsoleVariableName);
	/** Creates a spinbox that sets a float console variable, with a text label. */
	static TSharedRef<SWidget> CreateConsoleVariableSpinBox(const FText& Text, const FString& ConsoleVariableName);
	/** Creates a combox box that sets a string console variable. */
	static TSharedRef<SWidget> CreateConsoleVariableComboBox(const FString& ConsoleVariableName, TArray<TSharedPtr<FString>>* OptionsSource);
	/** Creates a text box that sets a string console variable. */
	static TSharedRef<SWidget> CreateConsoleVariableTextBox(const FString& ConsoleVariableName);

private:

	static IConsoleVariable* GetConsoleVariable(const FString& ConsoleVariableName);
};

}  // namespace UE::Cameras

