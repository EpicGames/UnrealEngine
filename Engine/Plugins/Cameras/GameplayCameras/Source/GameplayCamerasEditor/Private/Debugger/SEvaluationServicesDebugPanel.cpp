// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SEvaluationServicesDebugPanel.h"

#include "Debugger/SDebugWidgetUtils.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SEvaluationServicesDebugPanel"

namespace UE::Cameras
{

void SEvaluationServicesDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SDebugWidgetUtils::CreateConsoleVariableCheckBox(
						LOCTEXT("ShowLastTargetPreservation", "Show last target preservation"),
						TEXT("GameplayCameras.Debug.OrientationInitialization.ShowLastTargetPreservation"))
			]
	];
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

