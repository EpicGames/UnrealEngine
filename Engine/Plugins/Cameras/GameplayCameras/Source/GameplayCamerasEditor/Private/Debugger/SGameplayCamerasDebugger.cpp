// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SGameplayCamerasDebugger.h"

#include "Commands/GameplayCamerasDebuggerCommands.h"
#include "Core/CameraSystemEvaluator.h"
#include "Debug/CameraDebugColors.h"
#include "Debug/CameraSystemDebugRegistry.h"
#include "Debug/RootCameraDebugBlock.h"
#include "Debugger/SDebugCategoryButton.h"
#include "Debugger/SDebugWidgetUtils.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCamerasEditorSettings.h"
#include "IGameplayCamerasEditorModule.h"
#include "Modules/ModuleManager.h"
#include "String/ParseTokens.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Styling/SlateTypes.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SGameplayCamerasDebugger)

#define LOCTEXT_NAMESPACE "GameplayCamerasDebugger"

namespace UE::Cameras
{

const FName SGameplayCamerasDebugger::WindowName(TEXT("GameplayCamerasDebugger"));
const FName SGameplayCamerasDebugger::MenubarName(TEXT("GameplayCamerasDebugger.Menubar"));
const FName SGameplayCamerasDebugger::ToolbarName(TEXT("GameplayCamerasDebugger.Toolbar"));

FGameplayCamerasDebuggerContext::FGameplayCamerasDebuggerContext()
{
	FEditorDelegates::MapChange.AddRaw(this, &FGameplayCamerasDebuggerContext::OnMapChange);
	FEditorDelegates::BeginPIE.AddRaw(this, &FGameplayCamerasDebuggerContext::OnPieEvent);
	FEditorDelegates::EndPIE.AddRaw(this, &FGameplayCamerasDebuggerContext::OnPieEvent);

	if (GEngine)
	{
		GEngine->OnWorldAdded().AddRaw(this, &FGameplayCamerasDebuggerContext::OnWorldListChanged);
		GEngine->OnWorldDestroyed().AddRaw(this, &FGameplayCamerasDebuggerContext::OnWorldListChanged);
	}
}

FGameplayCamerasDebuggerContext::~FGameplayCamerasDebuggerContext()
{
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnWorldAdded().RemoveAll(this);
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}
}

UWorld* FGameplayCamerasDebuggerContext::GetContext()
{
	UpdateContext();
	return WeakContext.Get();
}

void FGameplayCamerasDebuggerContext::UpdateContext()
{
	if (WeakContext.IsValid())
	{
		return;
	}

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	// Pick the first editor world we find, but if there's any PIE/SIE world, prefer those.
	UWorld* NewContext = nullptr;
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::PIE)
		{
			NewContext = WorldContext.World();
			break;
		}
		else if (WorldContext.WorldType == EWorldType::Editor)
		{
			if (NewContext == nullptr)
			{
				NewContext = WorldContext.World();
			}
		}
	}
	ensure(NewContext);
	WeakContext = NewContext;
}

void FGameplayCamerasDebuggerContext::InvalidateContext()
{
	WeakContext = nullptr;
	OnContextChangedEvent.Broadcast();
}

void FGameplayCamerasDebuggerContext::OnPieEvent(bool bIsSimulating)
{
	InvalidateContext();
}

void FGameplayCamerasDebuggerContext::OnMapChange(uint32 MapChangeFlags)
{
	InvalidateContext();
}

void FGameplayCamerasDebuggerContext::OnWorldListChanged(UWorld* InWorld)
{
	InvalidateContext();
}

void SGameplayCamerasDebugger::RegisterTabSpawners()
{
	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		SGameplayCamerasDebugger::WindowName,
		FOnSpawnTab::CreateStatic(&SGameplayCamerasDebugger::SpawnGameplayCamerasDebugger)
	)
	.SetDisplayName(LOCTEXT("TabDisplayName", "Camera Debugger"))
	.SetTooltipText(LOCTEXT("TabTooltipText", "Open the Gameplay Cameras Debugger tab."))
	.SetIcon(FSlateIcon(CamerasEditorStyle->GetStyleSetName(), "Debugger.TabIcon"))
	.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
	.SetCanSidebarTab(false);
}

void SGameplayCamerasDebugger::UnregisterTabSpawners()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SGameplayCamerasDebugger::WindowName);
	}
}

TSharedRef<SDockTab> SGameplayCamerasDebugger::SpawnGameplayCamerasDebugger(const FSpawnTabArgs& Args)
{
	auto NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(LOCTEXT("TabTitle", "Camera Debugger"));

	TSharedRef<SWidget> MainWidget = SNew(SGameplayCamerasDebugger);
	NomadTab->SetContent(MainWidget);
	return NomadTab;
}

SGameplayCamerasDebugger::SGameplayCamerasDebugger()
{
	DebugContext.OnContextChanged().AddRaw(this, &SGameplayCamerasDebugger::OnDebugContextChanged);
}

SGameplayCamerasDebugger::~SGameplayCamerasDebugger()
{
	DebugContext.OnContextChanged().RemoveAll(this);
}

void SGameplayCamerasDebugger::Construct(const FArguments& InArgs)
{
	TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasEditorStyle = FGameplayCamerasEditorStyle::Get();
	GameplayCamerasEditorStyleName = GameplayCamerasEditorStyle->GetStyleSetName();

	InitializeColorSchemeNames();

	// Setup commands.
	const FGameplayCamerasDebuggerCommands& Commands = FGameplayCamerasDebuggerCommands::Get();
	TSharedRef<FUICommandList> CommandList = MakeShareable(new FUICommandList);
	CommandList->MapAction(
			Commands.EnableDebugInfo,
			FExecuteAction::CreateSP(this, &SGameplayCamerasDebugger::ToggleDebugDraw),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SGameplayCamerasDebugger::IsDebugDrawing));

	// Build all UI elements.
	TSharedRef<SWidget> MenubarContents = ConstructMenubar();
	TSharedRef<SWidget> ToolbarContents = ConstructToolbar(CommandList);
	TSharedRef<SWidget> GeneralOptionsContents = ConstructGeneralOptions(CommandList);
	ConstructDebugPanels();

	// Main layout.
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				MenubarContents
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				ToolbarContents
			]
		+ SVerticalBox::Slot()
			.Padding(2.0)
			[
				SAssignNew(PanelHost, SBox)
					.Padding(8.f)
					[
						EmptyPanel.ToSharedRef()
					]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0)
			[
				GeneralOptionsContents
			]
	];

	// Set initial debug panel.
	TArray<FStringView, TInlineAllocator<4>> ActiveCategories;
	UE::String::ParseTokens(GGameplayCamerasDebugCategories, ',', ActiveCategories);
	if (!ActiveCategories.IsEmpty())
	{
		SetActiveDebugCategoryPanel(FString(ActiveCategories[0]));
	}

	bRefreshDebugID = true;
}

void SGameplayCamerasDebugger::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (bRefreshDebugID)
	{
		// Auto-set the camera system debug ID when PIE starts/ends, and for other similar events.
		FCameraSystemDebugID DebugID;
		UWorld* DebugWorld = DebugContext.GetContext();
		if (DebugWorld)
		{
			const bool bIsEditorWorld = (
					DebugWorld->WorldType == EWorldType::Editor || 
					DebugWorld->WorldType == EWorldType::EditorPreview);

			if (bIsEditorWorld)
			{
				DebugID = FCameraSystemDebugID::Any();
			}
			else
			{
				DebugID = FCameraSystemDebugID::Auto();
			}
		}

		GGameplayCamerasDebugSystemID = DebugID.GetValue();
		bRefreshDebugID = false;
	}
}

void SGameplayCamerasDebugger::InitializeColorSchemeNames()
{
	TArray<FString> RawNames;
	FCameraDebugColors::GetColorSchemeNames(RawNames);
	for (const FString& RawName :RawNames)
	{
		ColorSchemeNames.Add(MakeShared<FString>(RawName));
	}
}

SGameplayCamerasDebugger* SGameplayCamerasDebugger::FromContext(UToolMenu* InMenu)
{
	UGameplayCamerasDebuggerMenuContext* Context = InMenu->FindContext<UGameplayCamerasDebuggerMenuContext>();
	if (ensure(Context))
	{
		TSharedPtr<SGameplayCamerasDebugger> This = Context->CamerasDebugger.Pin();
		return This.Get();
	}
	return nullptr;
}

TSharedRef<SWidget> SGameplayCamerasDebugger::ConstructMenubar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(SGameplayCamerasDebugger::MenubarName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* Menubar = ToolMenus->RegisterMenu(
				SGameplayCamerasDebugger::MenubarName, NAME_None, EMultiBoxType::MenuBar);
	}

	FToolMenuContext MenubarContext;
	return ToolMenus->GenerateWidget(SGameplayCamerasDebugger::MenubarName, MenubarContext);
}

TSharedRef<SWidget> SGameplayCamerasDebugger::ConstructToolbar(TSharedRef<FUICommandList> InCommandList)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(SGameplayCamerasDebugger::ToolbarName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* Toolbar = ToolMenus->RegisterMenu(
				SGameplayCamerasDebugger::ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

		Toolbar->AddDynamicSection(TEXT("Main"), FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu)
			{
				const FGameplayCamerasDebuggerCommands& Commands = FGameplayCamerasDebuggerCommands::Get();
				SGameplayCamerasDebugger* This = SGameplayCamerasDebugger::FromContext(InMenu);

				FToolMenuSection& MainSection = InMenu->AddSection(TEXT("Main"));

				FToolMenuEntry ToggleDebugInfo = FToolMenuEntry::InitToolBarButton(
						Commands.EnableDebugInfo,
						TAttribute<FText>::CreateSP(This, &SGameplayCamerasDebugger::GetToggleDebugDrawText),
						TAttribute<FText>(),
						TAttribute<FSlateIcon>::CreateSP(This, &SGameplayCamerasDebugger::GetToggleDebugDrawIcon));
				MainSection.AddEntry(ToggleDebugInfo);

				FToolMenuEntry BindComboEntry = FToolMenuEntry::InitComboButton(
						"BindToCameraSystemsMenu",
						FUIAction(),
						FNewToolMenuDelegate::CreateSP(This, &SGameplayCamerasDebugger::GetCameraSystemPickerContent),
						LOCTEXT("BindToCameraSystemsMenu", "Bind to..."),
						LOCTEXT("BindToCameraSystemsMenuToolTip", "Pick a camera system instance to bind to"),
						FSlateIcon(This->GameplayCamerasEditorStyleName, "Debugger.BindToCameraSystem"),
						true);
				MainSection.AddEntry(BindComboEntry);
			}));
	
		Toolbar->AddDynamicSection(TEXT("DebugCategories"), FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu)
			{
				IGameplayCamerasEditorModule& ThisModule = FModuleManager::GetModuleChecked<IGameplayCamerasEditorModule>(
						TEXT("GameplayCamerasEditor"));
				TArray<FCameraDebugCategoryInfo> RegisteredDebugCategories;
				ThisModule.GetRegisteredDebugCategories(RegisteredDebugCategories);

				SGameplayCamerasDebugger* This = SGameplayCamerasDebugger::FromContext(InMenu);

				FToolMenuSection& DebugCategoriesSection = InMenu->AddSection(TEXT("DebugCategories"));

				for (const FCameraDebugCategoryInfo& DebugCategory : RegisteredDebugCategories)
				{
					FToolMenuEntry ToggleDebugCategory = FToolMenuEntry::InitToolBarButton(
						FName(DebugCategory.Name),
						FUIAction(
							FExecuteAction::CreateSP(This, &SGameplayCamerasDebugger::SetActiveDebugCategoryPanel, DebugCategory.Name),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&SGameplayCamerasDebugger::IsDebugCategoryActive, DebugCategory.Name)),
						DebugCategory.DisplayText,
						DebugCategory.ToolTipText,
						DebugCategory.IconImage,
						EUserInterfaceActionType::ToggleButton);
					DebugCategoriesSection.AddEntry(ToggleDebugCategory);
				}
			}));
	}

	UGameplayCamerasDebuggerMenuContext* ThisContextWrapper = NewObject<UGameplayCamerasDebuggerMenuContext>();
	ThisContextWrapper->CamerasDebugger = SharedThis(this);
	FToolMenuContext ToolbarContext(InCommandList, TSharedPtr<FExtender>());
	ToolbarContext.AddObject(ThisContextWrapper);

	return ToolMenus->GenerateWidget(SGameplayCamerasDebugger::ToolbarName, ToolbarContext);
}

TSharedRef<SWidget> SGameplayCamerasDebugger::ConstructGeneralOptions(TSharedRef<FUICommandList> InCommandList)
{
	const ISlateStyle& AppStyle = FAppStyle::Get();
	const FMargin GridCellPadding(4.f);

	return SNew(SExpandableArea)
		.BorderImage(AppStyle.GetBrush("Brushes.Header"))
		.BodyBorderImage(AppStyle.GetBrush("Brushes.Recessed"))
		.HeaderPadding(FMargin(4.0f))
		.Padding(FMargin(0, 1, 0, 0))
		.InitiallyCollapsed(true)
		.AllowAnimatedTransition(false)
		.HeaderContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("GeneralOptions", "General Options"))
						.TextStyle(AppStyle, "ButtonText")
						.Font(AppStyle.GetFontStyle("NormalFontBold"))
				]
		]
		.BodyContent()
		[
			SNew(SBorder)
				.BorderImage(AppStyle.GetBrush("Brushes.Header"))
				.Padding(2.f)
				[
					SNew(SGridPanel)
						.FillColumn(0, 1.f)
						.FillColumn(2, 1.f)
					+ SGridPanel::Slot(0, 0)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("TopMargin", "Top margin"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SDebugWidgetUtils::CreateConsoleVariableSpinBox(TEXT("GameplayCameras.Debug.TopMargin"))
					]
					+ SGridPanel::Slot(0, 1)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("LeftMargin", "Left margin"))
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SDebugWidgetUtils::CreateConsoleVariableSpinBox(TEXT("GameplayCameras.Debug.LeftMargin"))
					]
					+ SGridPanel::Slot(0, 2)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("InnerMargin", "Inner margin"))
					]
					+ SGridPanel::Slot(1, 2)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SDebugWidgetUtils::CreateConsoleVariableSpinBox(TEXT("GameplayCameras.Debug.InnerMargin"))
					]
					+ SGridPanel::Slot(0, 3)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("IndentSize", "Indent size"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SDebugWidgetUtils::CreateConsoleVariableSpinBox(TEXT("GameplayCameras.Debug.Indent"))
					]
					+ SGridPanel::Slot(2, 0)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("ColorScheme", "Color scheme"))
					]
					+ SGridPanel::Slot(3, 0)
					.Padding(GridCellPadding)
					.VAlign(VAlign_Center)
					[
						SDebugWidgetUtils::CreateConsoleVariableComboBox(TEXT("GameplayCameras.Debug.ColorScheme"), &ColorSchemeNames)
					]
				]
		];
}

void SGameplayCamerasDebugger::ConstructDebugPanels()
{
	// Empty panel.
	EmptyPanel = SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(LOCTEXT("EmptyPanelWarning", "No custom controls for this debug category."))
		];

	// Register custom panels.
	IGameplayCamerasEditorModule& ThisModule = FModuleManager::GetModuleChecked<IGameplayCamerasEditorModule>(
			TEXT("GameplayCamerasEditor"));
	TArray<FCameraDebugCategoryInfo> RegisteredDebugCategories;
	ThisModule.GetRegisteredDebugCategories(RegisteredDebugCategories);

	for (const FCameraDebugCategoryInfo& DebugCategory : RegisteredDebugCategories)
	{
		TSharedPtr<SWidget> DebugCategoryPanel = ThisModule.CreateDebugCategoryPanel(DebugCategory.Name);
		if (DebugCategoryPanel.IsValid())
		{
			DebugPanels.Add(DebugCategory.Name, DebugCategoryPanel);
		}
		else
		{
			// If there aren't any special UI controls for this category, use an empty panel.
			DebugPanels.Add(DebugCategory.Name, EmptyPanel);
		}
	}
}

void SGameplayCamerasDebugger::ToggleDebugDraw()
{
	GGameplayCamerasDebugEnable = !GGameplayCamerasDebugEnable;
}

bool SGameplayCamerasDebugger::IsDebugDrawing() const
{
	return GGameplayCamerasDebugEnable;
}

FText SGameplayCamerasDebugger::GetToggleDebugDrawText() const
{
	if (GGameplayCamerasDebugEnable)
	{
		return LOCTEXT("DebugInfoEnabled", "Debug Info Enabled");
	}
	else
	{
		return LOCTEXT("DebugInfoDisabled", "Debug Info Disabled");
	}
}

FSlateIcon SGameplayCamerasDebugger::GetToggleDebugDrawIcon() const
{
	if (GGameplayCamerasDebugEnable)
	{
		return FSlateIcon(GameplayCamerasEditorStyleName, "Debugger.DebugInfoEnabled.Icon");
	}
	else
	{
		return FSlateIcon(GameplayCamerasEditorStyleName, "Debugger.DebugInfoDisabled.Icon");
	}
}

void SGameplayCamerasDebugger::GetCameraSystemPickerContent(UToolMenu* ToolMenu)
{
	FCameraSystemDebugRegistry::FRegisteredCameraSystems CameraSystems;
	FCameraSystemDebugRegistry::Get().GetRegisteredCameraSystemEvaluators(CameraSystems);

	UWorld* DebugWorld = DebugContext.GetContext();
	if (DebugWorld)
	{
		const bool bIsEditorWorld = (
				DebugWorld->WorldType == EWorldType::Editor || 
				DebugWorld->WorldType == EWorldType::EditorPreview);

		FToolMenuSection& CameraSystemsSection = ToolMenu->AddSection(
				"CameraSystems",
				FText::Format(
					LOCTEXT("BoundToWorldName", "Camera Systems in {0}"),
					FText::FromName(DebugWorld->GetFName()))
				);

		if (bIsEditorWorld)
		{
			CameraSystemsSection.AddMenuEntry(
					TEXT("SelectToBindInEditorWorld"),
					LOCTEXT("SelectToBindInEditorWorld", "Select actor to show debug info"),
					LOCTEXT("SelectToBindInEditorWorldToolTip", "In editor worlds, debug info is shown for the selected camera actor."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction::CreateLambda([]() { return false; }))
					);
		}
		else if (CameraSystems.Num() > 0)
		{
			CameraSystemsSection.AddMenuEntry(
					TEXT("AutoBindToViewTarget"),
					LOCTEXT("AutoBindToViewTarget", "Auto-bind to the current view target"),
					LOCTEXT("AutoBindToViewTargetToolTip", "Show the debug info for the view target of the local player."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(
							this, &SGameplayCamerasDebugger::BindToCameraSystem, FCameraSystemDebugID::Auto()),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(
							this, &SGameplayCamerasDebugger::IsBoundToCameraSystem, FCameraSystemDebugID::Auto())),
						EUserInterfaceActionType::Check);

			CameraSystemsSection.AddSeparator(NAME_None);

			for (TSharedPtr<FCameraSystemEvaluator> CameraSystem : CameraSystems)
			{
				UObject* CameraSystemOwner = CameraSystem->GetOwner();
				if (CameraSystemOwner && CameraSystemOwner->GetWorld() == DebugWorld)
				{
					AActor* OwnerActor = Cast<AActor>(CameraSystemOwner);
					if (!OwnerActor)
					{
						OwnerActor = CameraSystemOwner->GetTypedOuter<AActor>();
					}
					const FName OwnerName = (OwnerActor ? OwnerActor->GetFName() : CameraSystemOwner->GetFName());

					CameraSystemsSection.AddMenuEntry(
							NAME_None,
							FText::Format(
								LOCTEXT("BindToCameraSystem", "{0} (ID={1})"),
								FText::FromName(OwnerName),
								FText::FromString(LexToString(CameraSystem->GetDebugID()))),
							LOCTEXT("BindToCameraSystemToolTip", "Bind to this camera system instance"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(
									this, &SGameplayCamerasDebugger::BindToCameraSystem, CameraSystem->GetDebugID()),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(
									this, &SGameplayCamerasDebugger::IsBoundToCameraSystem, CameraSystem->GetDebugID())),
							EUserInterfaceActionType::Check);
				}
			}
		}
		else
		{
			CameraSystemsSection.AddMenuEntry(
					TEXT("NoCameraSystem"),
					LOCTEXT("NoCameraSystem", "None"),
					LOCTEXT("NoCameraSystemToolTip", "No camera systems found"),
					FSlateIcon(),
					FUIAction());
		}
	}
}

void SGameplayCamerasDebugger::BindToCameraSystem(FCameraSystemDebugID InDebugID)
{
	GGameplayCamerasDebugSystemID = InDebugID.GetValue();
}

bool SGameplayCamerasDebugger::IsBoundToCameraSystem(FCameraSystemDebugID InDebugID)
{
	return GGameplayCamerasDebugSystemID == InDebugID.GetValue();
}

void SGameplayCamerasDebugger::OnDebugContextChanged()
{
	bRefreshDebugID = true;
}

bool SGameplayCamerasDebugger::IsDebugCategoryActive(FString InCategoryName)
{
	TArray<FStringView, TInlineAllocator<4>> ActiveCategories;
	UE::String::ParseTokens(GGameplayCamerasDebugCategories, ',', ActiveCategories);
	return ActiveCategories.Contains(InCategoryName);
}

void SGameplayCamerasDebugger::SetActiveDebugCategoryPanel(FString InCategoryName)
{
	if (ensureMsgf(
				DebugPanels.Contains(InCategoryName), 
				TEXT("Debug category was not registered with IGameplayCamerasEditorModule: %s"), *InCategoryName))
	{
		TSharedPtr<SWidget> DebugPanel = DebugPanels.FindChecked(InCategoryName);
		check(DebugPanel.IsValid());
		PanelHost->SetContent(DebugPanel.ToSharedRef());

		GGameplayCamerasDebugCategories = InCategoryName;
	}
	else
	{
		PanelHost->SetContent(SNullWidget::NullWidget);
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

