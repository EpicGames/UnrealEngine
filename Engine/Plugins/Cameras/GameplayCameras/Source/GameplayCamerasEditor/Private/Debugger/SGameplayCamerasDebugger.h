// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Debug/CameraSystemDebugRegistry.h"
#include "Widgets/SCompoundWidget.h"

#include "SGameplayCamerasDebugger.generated.h"

class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class SBox;
class SDockTab;
class SWidget;
class UToolMenu;
struct FSlateIcon;

namespace UE::Cameras
{

class FGameplayCamerasDebuggerContext
{
public:

	FGameplayCamerasDebuggerContext();
	~FGameplayCamerasDebuggerContext();

	UWorld* GetContext();

	FSimpleMulticastDelegate& OnContextChanged() { return OnContextChangedEvent; }

private:

	void UpdateContext();
	void InvalidateContext();

	void OnPieEvent(bool bIsSimulating);
	void OnMapChange(uint32 MapChangeFlags);
	void OnWorldListChanged(UWorld* InWorld);

private:

	TWeakObjectPtr<UWorld> WeakContext;
	FSimpleMulticastDelegate OnContextChangedEvent;
};

class SGameplayCamerasDebugger : public SCompoundWidget
{
public:

	static const FName WindowName;
	static const FName MenubarName;
	static const FName ToolbarName;

	static void RegisterTabSpawners();
	static TSharedRef<SDockTab> SpawnGameplayCamerasDebugger(const FSpawnTabArgs& Args);
	static void UnregisterTabSpawners();

public:

	SLATE_BEGIN_ARGS(SGameplayCamerasDebugger) {}
	SLATE_END_ARGS();

	SGameplayCamerasDebugger();
	virtual ~SGameplayCamerasDebugger();

	void Construct(const FArguments& InArgs);

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	static SGameplayCamerasDebugger* FromContext(UToolMenu* InMenu);
	TSharedRef<SWidget> ConstructMenubar();
	TSharedRef<SWidget> ConstructToolbar(TSharedRef<FUICommandList> InCommandList);
	TSharedRef<SWidget> ConstructGeneralOptions(TSharedRef<FUICommandList> InCommandList);
	void ConstructDebugPanels();

	void InitializeColorSchemeNames();

	static bool IsDebugCategoryActive(FString InCategoryName);
	void SetActiveDebugCategoryPanel(FString InCategoryName);

	void ToggleDebugDraw();
	bool IsDebugDrawing() const;
	FText GetToggleDebugDrawText() const;
	FSlateIcon GetToggleDebugDrawIcon() const;
	
	void GetCameraSystemPickerContent(UToolMenu* ToolMenu);

	void BindToCameraSystem(FCameraSystemDebugID InDebugID);
	bool IsBoundToCameraSystem(FCameraSystemDebugID InDebugID);

	void OnDebugContextChanged();

private:

	FName GameplayCamerasEditorStyleName;

	FGameplayCamerasDebuggerContext DebugContext;
	bool bRefreshDebugID = false;

	TSharedPtr<SBox> PanelHost;

	TSharedPtr<SWidget> EmptyPanel;
	TMap<FString, TSharedPtr<SWidget>> DebugPanels;

	TArray<TSharedPtr<FString>> ColorSchemeNames;
};

}  // namespace UE::Cameras

UCLASS()
class UGameplayCamerasDebuggerMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Cameras::SGameplayCamerasDebugger> CamerasDebugger;
};

