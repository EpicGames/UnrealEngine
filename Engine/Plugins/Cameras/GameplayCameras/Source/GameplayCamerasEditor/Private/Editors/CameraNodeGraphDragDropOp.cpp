// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraNodeGraphDragDropOp.h"

#include "Core/CameraRigAsset.h"
#include "Editors/CameraObjectGraphSchemaBase.h"
#include "GraphEditor.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "CameraNodeGraphDragDropOp"

TSharedRef<FCameraNodeGraphInterfaceParameterDragDropOp> FCameraNodeGraphInterfaceParameterDragDropOp::New(UCameraObjectInterfaceParameterBase* InInterfaceParameter)
{
	TSharedRef<FCameraNodeGraphInterfaceParameterDragDropOp> Operation = MakeShared<FCameraNodeGraphInterfaceParameterDragDropOp>();
	Operation->InterfaceParameter = InInterfaceParameter;
	Operation->Construct();
	return Operation;
}

FReply FCameraNodeGraphInterfaceParameterDragDropOp::ExecuteDragOver(TSharedPtr<SGraphEditor> GraphEditor)
{
	if (InterfaceParameter)
	{
		const FSlateBrush* OKIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		SetToolTip(LOCTEXT("OnDragOver_Success", "Add interface parameter getter"), OKIcon);
	}

	return FReply::Handled();
}

FReply FCameraNodeGraphInterfaceParameterDragDropOp::ExecuteDrop(TSharedPtr<SGraphEditor> GraphEditor, const FSlateCompatVector2f& NewLocation)
{
	if (!InterfaceParameter)
	{
		return FReply::Handled();
	}

	const FScopedTransaction Transaction(LOCTEXT("DropObjectClasses", "Drop New Nodes"));

	UEdGraph* Graph = GraphEditor->GetCurrentGraph();

	GraphEditor->ClearSelectionSet();

	FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode Action;
	Action.InterfaceParameter = InterfaceParameter;
	UEdGraphNode* NewNode = Action.PerformAction(Graph, nullptr, NewLocation, false);
	GraphEditor->SetNodeSelection(NewNode, true);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

