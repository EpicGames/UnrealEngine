// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraNodeGraphNode.h"

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "EdGraph/EdGraphPin.h"
#include "Editors/CameraObjectGraphSchemaBase.h"
#include "Editors/SCameraNodeGraphNode.h"
#include "GameplayCamerasDelegates.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraNodeGraphNode)

UCameraNodeGraphNode::UCameraNodeGraphNode(const FObjectInitializer& ObjInit)
	: UObjectTreeGraphNode(ObjInit)
{
}

void UCameraNodeGraphNode::OnInitialize()
{
	using namespace UE::Cameras;

	const bool bIsCustomParameterProvider = GetObject()->Implements<UCustomCameraNodeParameterProvider>();
	if (bIsCustomParameterProvider)
	{
		FGameplayCamerasDelegates::OnCustomCameraNodeParametersChanged().AddUObject(
				this, &UCameraNodeGraphNode::OnCustomCameraNodeParametersChanged);
	}
}

void UCameraNodeGraphNode::BeginDestroy()
{
	using namespace UE::Cameras;

	FGameplayCamerasDelegates::OnCustomCameraNodeParametersChanged().RemoveAll(this);

	Super::BeginDestroy();
}

void UCameraNodeGraphNode::OnCustomCameraNodeParametersChanged(const UCameraNode* CameraNode)
{
	if (CameraNode == GetObject())
	{
		ReconstructNode();
	}
}

void UCameraNodeGraphNode::AllocateDefaultPins()
{
	using namespace UE::Cameras;

	Super::AllocateDefaultPins();

	// Add extra input pins for any camera parameter, variable reference, and context data.
	UObject* Object = GetObject();
	UClass* CameraNodeClass = Object->GetClass();
	const FName ContextDataMetaData = TEXT("CameraContextData");
	for (TFieldIterator<FProperty> PropertyIt(CameraNodeClass); PropertyIt; ++PropertyIt)
	{
		const FName PropertyName = PropertyIt->GetFName();

		FEdGraphPinType PinType;
		const FText PinFriendlyName = FText::FromName(PropertyName);
		
		if (FStructProperty* StructProperty = CastField<FStructProperty>(*PropertyIt))
		{
			const FString PinToolTip = StructProperty->Struct->GetDisplayNameText().ToString();

#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
			if (StructProperty->Struct == F##ValueName##CameraParameter::StaticStruct())\
			{\
				PinType.PinCategory = UCameraObjectGraphSchemaBase::PC_CameraParameter;\
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraVariableType::ValueName);\
				PinType.PinSubCategoryObject = F##ValueName##CameraParameter::StaticStruct();\
				UEdGraphPin* ParameterPin = CreatePin(EGPD_Input, PinType, PropertyName);\
				ParameterPin->PinFriendlyName = PinFriendlyName;\
				ParameterPin->PinToolTip = PinToolTip;\
				continue;\
			}\
			if (StructProperty->Struct == F##ValueName##CameraVariableReference::StaticStruct())\
			{\
				PinType.PinCategory = UCameraObjectGraphSchemaBase::PC_CameraVariableReference;\
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraVariableType::ValueName);\
				PinType.PinSubCategoryObject = F##ValueName##CameraVariableReference::StaticStruct();\
				UEdGraphPin* VariableReferencePin = CreatePin(EGPD_Input, PinType, PropertyName);\
				VariableReferencePin->PinFriendlyName = PinFriendlyName;\
				VariableReferencePin->PinToolTip = PinToolTip;\
				continue;\
			}
			UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
		}

		if (PropertyIt->HasMetaData(ContextDataMetaData))
		{
			bool bGotValidDataProperty = true;
			FProperty* DataProperty = *PropertyIt;
			FString PinToolTip;

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(DataProperty))
			{
				PinType.ContainerType = EPinContainerType::Array;
				DataProperty = ArrayProperty->Inner;
			}

			if (FNameProperty* NameProperty = CastField<FNameProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::Name);
			}
			else if (FStrProperty* StringProperty = CastField<FStrProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::String);
			}
			else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::Enum);
				PinType.PinSubCategoryObject = EnumProperty->GetEnum();
				PinToolTip = EnumProperty->GetEnum()->GetDisplayNameText().ToString();
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::Struct);
				PinType.PinSubCategoryObject = StructProperty->Struct;
				PinToolTip = StructProperty->Struct->GetDisplayNameText().ToString();
			}
			else if (FClassProperty* ClassProperty = CastField<FClassProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::Class);
				PinType.PinSubCategoryObject = ClassProperty->MetaClass;
			}
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(DataProperty))
			{
				PinType.PinSubCategory = UEnum::GetValueAsName(ECameraContextDataType::Object);
				PinType.PinSubCategoryObject = ObjectProperty->PropertyClass;
			}
			else
			{
				bGotValidDataProperty = false;
			}

			if (bGotValidDataProperty)
			{
				PinType.PinCategory = UCameraObjectGraphSchemaBase::PC_CameraContextData;
				UEdGraphPin* ContextDataPin = CreatePin(EGPD_Input, PinType, PropertyName);
				ContextDataPin->PinFriendlyName = PinFriendlyName;
				ContextDataPin->PinToolTip = PinToolTip;
				continue;
			}
		}
	}

	ICustomCameraNodeParameterProvider* CustomParameterProvider = Cast<ICustomCameraNodeParameterProvider>(Object);
	if (CustomParameterProvider)
	{
		FCameraNodeParameterInfos ObjectParameters;
		CustomParameterProvider->GetCustomCameraNodeParameters(ObjectParameters);

		// Add pins for blendable parameters.
		const UEnum* VariableTypeEnum = StaticEnum<ECameraVariableType>();
		for (const FCameraNodeBlendableParameterInfo& BlendableParameter : ObjectParameters.GetBlendableParameters())
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = UCameraObjectGraphSchemaBase::PC_CameraParameter;
			PinType.PinSubCategory = VariableTypeEnum->GetValueAsName(BlendableParameter.VariableType);

			switch (BlendableParameter.VariableType)
			{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
				case ECameraVariableType::ValueName:\
					PinType.PinSubCategoryObject = F##ValueName##CameraVariableReference::StaticStruct();\
					break;
				UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
				case ECameraVariableType::BlendableStruct:
					PinType.PinSubCategoryObject = const_cast<UScriptStruct*>(BlendableParameter.BlendableStructType);
					break;
			}

			UEdGraphPin* ParameterPin = CreatePin(EGPD_Input, PinType, BlendableParameter.ParameterName);
			ParameterPin->PinFriendlyName = FText::FromName(BlendableParameter.ParameterName);
			ParameterPin->PinToolTip = VariableTypeEnum->GetNameStringByValue((int64)BlendableParameter.VariableType);
		}

		// Add pins for data parameters.
		const UEnum* DataTypeEnum = StaticEnum<ECameraContextDataType>();
		for (const FCameraNodeDataParameterInfo& DataParameter : ObjectParameters.GetDataParameters())
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = UCameraObjectGraphSchemaBase::PC_CameraContextData;
			PinType.PinSubCategory = DataTypeEnum->GetNameByValue((int64)DataParameter.DataType);
			PinType.PinSubCategoryObject = const_cast<UObject*>(DataParameter.DataTypeObject);

			if (DataParameter.DataContainerType == ECameraContextDataContainerType::Array)
			{
				PinType.ContainerType = EPinContainerType::Array;
			}

			UEdGraphPin* ContextDataPin = CreatePin(EGPD_Input, PinType, DataParameter.ParameterName);
			ContextDataPin->PinFriendlyName = FText::FromName(DataParameter.ParameterName);

			FString PinToolTip = DataTypeEnum->GetNameStringByValue((int64)DataParameter.DataType);
			if (DataParameter.DataTypeObject)
			{
				PinToolTip = DataParameter.DataTypeObject->GetName();
			}
			ContextDataPin->PinToolTip = PinToolTip;
		}
	}
}

TSharedPtr<SGraphNode> UCameraNodeGraphNode::CreateVisualWidget()
{
	return SNew(SCameraNodeGraphNode).GraphNode(this);
}

