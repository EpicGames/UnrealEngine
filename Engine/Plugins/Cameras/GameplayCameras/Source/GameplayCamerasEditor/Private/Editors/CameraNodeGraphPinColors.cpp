// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraNodeGraphPinColors.h"

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/CameraVariableTableFwd.h"
#include "GraphEditorSettings.h"

namespace UE::Cameras
{

void FCameraNodeGraphPinColors::Initialize()
{
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

	PinColors.Reset();

	const UEnum* TypeEnum = StaticEnum<ECameraVariableType>();
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Boolean), Settings->BooleanPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Integer32), Settings->IntPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Float), Settings->FloatPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Double), Settings->DoublePinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector2f), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector2d), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector3f), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector3d), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector4f), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Vector4d), Settings->VectorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Rotator3f), Settings->RotatorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Rotator3d), Settings->RotatorPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Transform3f), Settings->TransformPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::Transform3d), Settings->TransformPinTypeColor);
	PinColors.Add(TypeEnum->GetNameByValue((int64)ECameraVariableType::BlendableStruct), Settings->StructPinTypeColor);

	PinColors.Add(FBooleanCameraParameter::StaticStruct()->GetFName(), Settings->BooleanPinTypeColor);
	PinColors.Add(FInteger32CameraParameter::StaticStruct()->GetFName(), Settings->IntPinTypeColor);
	PinColors.Add(FFloatCameraParameter::StaticStruct()->GetFName(), Settings->FloatPinTypeColor);
	PinColors.Add(FDoubleCameraParameter::StaticStruct()->GetFName(), Settings->DoublePinTypeColor);
	PinColors.Add(FVector2fCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector2dCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector3fCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector3dCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector4fCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector4dCameraParameter::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FRotator3fCameraParameter::StaticStruct()->GetFName(), Settings->RotatorPinTypeColor);
	PinColors.Add(FRotator3dCameraParameter::StaticStruct()->GetFName(), Settings->RotatorPinTypeColor);
	PinColors.Add(FTransform3fCameraParameter::StaticStruct()->GetFName(), Settings->TransformPinTypeColor);
	PinColors.Add(FTransform3dCameraParameter::StaticStruct()->GetFName(), Settings->TransformPinTypeColor);

	PinColors.Add(FBooleanCameraVariableReference::StaticStruct()->GetFName(), Settings->BooleanPinTypeColor);
	PinColors.Add(FInteger32CameraVariableReference::StaticStruct()->GetFName(), Settings->IntPinTypeColor);
	PinColors.Add(FFloatCameraVariableReference::StaticStruct()->GetFName(), Settings->FloatPinTypeColor);
	PinColors.Add(FDoubleCameraVariableReference::StaticStruct()->GetFName(), Settings->DoublePinTypeColor);
	PinColors.Add(FVector2fCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector2dCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector3fCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector3dCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector4fCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FVector4dCameraVariableReference::StaticStruct()->GetFName(), Settings->VectorPinTypeColor);
	PinColors.Add(FRotator3fCameraVariableReference::StaticStruct()->GetFName(), Settings->RotatorPinTypeColor);
	PinColors.Add(FRotator3dCameraVariableReference::StaticStruct()->GetFName(), Settings->RotatorPinTypeColor);
	PinColors.Add(FTransform3fCameraVariableReference::StaticStruct()->GetFName(), Settings->TransformPinTypeColor);
	PinColors.Add(FTransform3dCameraVariableReference::StaticStruct()->GetFName(), Settings->TransformPinTypeColor);

	DefaultPinColor = Settings->DefaultPinTypeColor;
	NamePinColor = Settings->NamePinTypeColor;
	StringPinColor = Settings->StringPinTypeColor;
	EnumPinColor = Settings->Int64PinTypeColor;
	StructPinColor = Settings->StructPinTypeColor;
	ObjectPinColor = Settings->ObjectPinTypeColor;
	ClassPinColor = Settings->ClassPinTypeColor;
}

FLinearColor FCameraNodeGraphPinColors::GetPinColor(const FName& TypeName) const
{
	return PinColors.FindRef(TypeName, DefaultPinColor);
}

FLinearColor FCameraNodeGraphPinColors::GetContextDataPinColor(const FName& DataTypeName) const
{
	const UEnum* DataTypeEnum = StaticEnum<ECameraContextDataType>();
	const ECameraContextDataType DataType = (ECameraContextDataType)DataTypeEnum->GetValueByName(DataTypeName);
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			return NamePinColor;
		case ECameraContextDataType::String:
			return StringPinColor;
		case ECameraContextDataType::Enum:
			return EnumPinColor;
		case ECameraContextDataType::Struct:
			return StructPinColor;
		case ECameraContextDataType::Object:
			return ObjectPinColor;
		case ECameraContextDataType::Class:
			return ClassPinColor;
		default:
			return DefaultPinColor;
	}
}

}  // namespace UE::Cameras

