// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"

namespace UE::Cameras
{

struct FCameraNodeGraphPinColors
{
	void Initialize();

	FLinearColor GetPinColor(const FName& TypeName) const;
	FLinearColor GetContextDataPinColor(const FName& DataTypeName) const;

private:

	TMap<FName, FLinearColor> PinColors;
	FLinearColor DefaultPinColor;
	FLinearColor NamePinColor;
	FLinearColor StringPinColor;
	FLinearColor EnumPinColor;
	FLinearColor StructPinColor;
	FLinearColor ObjectPinColor;
	FLinearColor ClassPinColor;
};

}  // namespace UE::Cameras

