// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraObjectGraphSchemaBase.h"

#include "Algo/Reverse.h"
#include "Build/CameraObjectInterfaceParameterBuilder.h"
#include "Core/BaseCameraObject.h"
#include "Core/CameraNode.h"
#include "Core/CameraParameters.h"
#include "Core/CameraVariableReferences.h"
#include "Core/ICustomCameraNodeParameterProvider.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Editors/CameraNodeGraphNode.h"
#include "Editors/CameraObjectInterfaceParameterGraphNode.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "GameplayCamerasEditorSettings.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraObjectGraphSchemaBase)

#define LOCTEXT_NAMESPACE "CameraObjectGraphSchemaBase"

const FName UCameraObjectGraphSchemaBase::PC_CameraParameter("CameraParameter");
const FName UCameraObjectGraphSchemaBase::PC_CameraVariableReference("CameraVariableReference");
const FName UCameraObjectGraphSchemaBase::PC_CameraContextData("CameraContextData");

UCameraObjectGraphSchemaBase::UCameraObjectGraphSchemaBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	PinColors.Initialize();
}

FObjectTreeGraphConfig UCameraObjectGraphSchemaBase::BuildGraphConfig() const
{
	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	FObjectTreeGraphConfig GraphConfig;

	GraphConfig.DefaultSelfPinName = NAME_None;
	GraphConfig.ConnectableObjectClasses.Add(UCameraObjectInterfaceParameterGetter::StaticClass());
	GraphConfig.ObjectClassConfigs.Emplace(UCameraObjectInterfaceParameterGetter::StaticClass())
		.GraphNodeClass(UCameraObjectInterfaceParameterGraphNode::StaticClass());

	OnBuildGraphConfig(GraphConfig);

	return GraphConfig;
}

void UCameraObjectGraphSchemaBase::OnCreateAllNodes(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const
{
	Super::OnCreateAllNodes(InGraph, InCreatedNodes);

	// Once all nodes are created, add connections between them.
	CreateValueFlowConnections(InGraph, InCreatedNodes);
}

void UCameraObjectGraphSchemaBase::CreateValueFlowConnections(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const
{
	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(InGraph->GetRootObject());
	if (!CameraObject)
	{
		return;
	}

	const FName SelfPinName = UObjectTreeGraphSchema::PC_Self;

	// Add connections for connected objects that are in this graph.
	for (const FCameraObjectConnection& Connection : CameraObject->Connections.Connections)
	{
		if (!Connection.Source || !Connection.Target)
		{
			continue;
		}

		UEdGraphNode* SourceNode = InCreatedNodes.CreatedNodes.FindRef(Connection.Source);
		UEdGraphNode* TargetNode = InCreatedNodes.CreatedNodes.FindRef(Connection.Target);
		// Either both nodes in the connection are in this graph, or none of them are. We shouldn't have
		// one node from one graph connected to a node from another graph, or nodes belonging to two graphs.
		ensure((SourceNode && TargetNode) || (!SourceNode && !TargetNode));
		if (!SourceNode || !TargetNode)
		{
			continue;
		}

		UEdGraphPin* SourcePin = Connection.SourcePropertyName.IsNone() ?
			FindPinByType(SourceNode, PC_Self) : 
			SourceNode->FindPin(Connection.SourcePropertyName);
		if (!SourcePin)
		{
			SourcePin = SourceNode->CreatePin(EGPD_Output, NAME_None, Connection.SourcePropertyName);
			SourcePin->bOrphanedPin = true;
		}

		UEdGraphPin* TargetPin = Connection.TargetPropertyName.IsNone() ? 
			FindPinByType(TargetNode, PC_Self) : 
			TargetNode->FindPin(Connection.TargetPropertyName);
		if (!TargetPin)
		{
			TargetPin = TargetNode->CreatePin(EGPD_Input, NAME_None, Connection.TargetPropertyName);
			TargetPin->bOrphanedPin = true;
		}

		SourcePin->MakeLinkTo(TargetPin);
	}
}

UEdGraphPin* UCameraObjectGraphSchemaBase::FindPinByType(UEdGraphNode* InNode, const FName& InPinCategory) const
{
	UEdGraphPin* const* FoundItem = InNode->Pins.FindByPredicate(
			[InPinCategory](UEdGraphPin* Item)
			{ 
				return Item->PinType.PinCategory == InPinCategory;
			});
	if (FoundItem)
	{
		return *FoundItem;
	}
	return nullptr;
}

void UCameraObjectGraphSchemaBase::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	using namespace UE::Cameras;

	// See if we were dragging a camera parameter pin or camera variable reference pin.
	if (const UEdGraphPin* DraggedPin = ContextMenuBuilder.FromPin)
	{
		UCameraNodeGraphNode* CameraNodeNode = Cast<UCameraNodeGraphNode>(DraggedPin->GetOwningNode());

		if (DraggedPin->PinType.PinCategory == PC_CameraParameter ||
				DraggedPin->PinType.PinCategory == PC_CameraVariableReference ||
				DraggedPin->PinType.PinCategory == PC_CameraContextData)
		{
			ensure(DraggedPin->PinName != NAME_None);

			// If this is an invalid parameter/data pin, don't show any actions.
			if (DraggedPin->bOrphanedPin)
			{
				return;
			}

			// Find the property being dragged, so we know what kind of parameter to create.
			const UClass* CameraNodeClass = CameraNodeNode->GetObject()->GetClass();
			FProperty* Property = CameraNodeClass->FindPropertyByName(DraggedPin->PinName);

			FCameraNodeParameterInfos CameraNodeParameters;
			ICustomCameraNodeParameterProvider* CustomParameterProvider = Cast<ICustomCameraNodeParameterProvider>(CameraNodeNode->GetObject());
			if (CustomParameterProvider)
			{
				CustomParameterProvider->GetCustomCameraNodeParameters(CameraNodeParameters);
			}

			TSharedRef<FCameraObjectGraphSchemaAction_NewInterfaceParameterNode> Action = 
				MakeShared<FCameraObjectGraphSchemaAction_NewInterfaceParameterNode>(
						FText::GetEmpty(),
						LOCTEXT("NewInterfaceParameterAction", "Camera Interface Parameter"),
						LOCTEXT("NewInterfaceParameterActionToolTip", "Exposes this parameter on the camera object"));

			if (DraggedPin->PinType.PinCategory == PC_CameraParameter ||
				DraggedPin->PinType.PinCategory == PC_CameraVariableReference)
			{
				ECameraVariableType VariableType;
				const UScriptStruct* BlendableStructType = nullptr;

				if (const FCameraNodeBlendableParameterInfo* BlendableParameter = CameraNodeParameters.FindBlendableParameter(DraggedPin->PinName))
				{
					VariableType = BlendableParameter->VariableType;
					BlendableStructType = BlendableParameter->BlendableStructType;
				}
				else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
					if ((StructProperty->Struct == F##ValueName##CameraParameter::StaticStruct()) ||\
						(StructProperty->Struct == F##ValueName##CameraVariableReference::StaticStruct()))\
					{\
						VariableType = ECameraVariableType::ValueName;\
					}\
					else
					UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
					{
						// Unexpected: if there was a camera parameter pin or a variable reference pin, we should
						// have had a camera parameter property or variable reference property!
						ensure(false);
						return;
					}
				}
				else
				{
					// Unexpected as per previous comments.
					ensure(false);
					return;
				}

				FCameraObjectInterfaceParameterDefinition NewParameterDefinition;
				NewParameterDefinition.ParameterType = ECameraObjectInterfaceParameterType::Blendable;
				NewParameterDefinition.VariableType = VariableType;
				NewParameterDefinition.BlendableStructType = BlendableStructType;
				Action->ParameterDefinition = NewParameterDefinition;
			}
			else if (DraggedPin->PinType.PinCategory == PC_CameraContextData)
			{
				ECameraContextDataType DataType;
				ECameraContextDataContainerType DataContainerType = ECameraContextDataContainerType::None;
				const UObject* DataTypeObject = nullptr;
				
				if (const FCameraNodeDataParameterInfo* DataParameter = CameraNodeParameters.FindDataParameter(DraggedPin->PinName))
				{
					DataType = DataParameter->DataType;
					DataContainerType = DataParameter->DataContainerType;
					DataTypeObject = DataParameter->DataTypeObject;
				}
				else if (Property)
				{
					if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						DataContainerType = ECameraContextDataContainerType::Array;
						Property = ArrayProperty->Inner;
					}

					if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
					{
						DataType = ECameraContextDataType::Name;
					}
					else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property))
					{
						DataType = ECameraContextDataType::String;
					}
					else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
					{
						DataType = ECameraContextDataType::Enum;
						DataTypeObject = EnumProperty->GetEnum();
					}
					else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						DataType = ECameraContextDataType::Struct;
						DataTypeObject = StructProperty->Struct;
					}
					else if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
					{
						DataType = ECameraContextDataType::Class;
						DataTypeObject = ClassProperty->PropertyClass;
					}
					else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
					{
						DataType = ECameraContextDataType::Object;
						DataTypeObject = ObjectProperty->PropertyClass;
					}
					else
					{
						// Unexpected as per previous comments.
						ensure(false);
						return;
					}
				}
				else
				{
					// Unexpected as per previous comments.
					ensure(false);
					return;
				}

				FCameraObjectInterfaceParameterDefinition NewParameterDefinition;
				NewParameterDefinition.ParameterType = ECameraObjectInterfaceParameterType::Data;
				NewParameterDefinition.DataType = DataType;
				NewParameterDefinition.DataContainerType = DataContainerType;
				NewParameterDefinition.DataTypeObject = DataTypeObject;
				Action->ParameterDefinition = NewParameterDefinition;
			}

			ContextMenuBuilder.AddAction(StaticCastSharedPtr<FEdGraphSchemaAction>(Action.ToSharedPtr()));

			return;
		}
	}

	Super::GetGraphContextActions(ContextMenuBuilder);
}

const FPinConnectionResponse UCameraObjectGraphSchemaBase::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	// Check if we are connecting parameter pins of compatible types.
	if ((A->PinType.PinCategory == PC_CameraParameter || 
				A->PinType.PinCategory == PC_CameraVariableReference ||
				A->PinType.PinCategory == PC_CameraContextData) && 
			B->PinType.PinCategory == PC_Self &&
			!A->bOrphanedPin)
	{
		UCameraObjectInterfaceParameterGraphNode* NodeB = Cast<UCameraObjectInterfaceParameterGraphNode>(B->GetOwningNode());
		if (NodeB)
		{
			UCameraObjectInterfaceParameterGetter* ParameterGetter = NodeB->CastObject<UCameraObjectInterfaceParameterGetter>();
			UCameraObjectInterfaceParameterBase* Parameter = ParameterGetter->GetInterfaceParameter();
			UCameraObjectInterfaceBlendableParameter* BlendableParameter = Cast<UCameraObjectInterfaceBlendableParameter>(Parameter);
			if (BlendableParameter && 
					A->PinType.PinSubCategory == UEnum::GetValueAsName(BlendableParameter->VariableType))
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT("Compatible pin types"));
			}
			UCameraObjectInterfaceDataParameter* DataParameter = Cast<UCameraObjectInterfaceDataParameter>(Parameter);
			if (DataParameter && 
					A->PinType.PinSubCategory == UEnum::GetValueAsName(DataParameter->DataType) &&
					A->PinType.PinSubCategoryObject == DataParameter->DataTypeObject)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT("Compatible pin types"));
			}
		}
	}
	else if (A->PinType.PinCategory == PC_Self && 
			(B->PinType.PinCategory == PC_CameraParameter || 
			 B->PinType.PinCategory == PC_CameraVariableReference ||
			 B->PinType.PinCategory == PC_CameraContextData) &&
			!B->bOrphanedPin)
	{
		UCameraObjectInterfaceParameterGraphNode* NodeA = Cast<UCameraObjectInterfaceParameterGraphNode>(A->GetOwningNode());
		if (NodeA)
		{
			UCameraObjectInterfaceParameterGetter* ParameterGetter = NodeA->CastObject<UCameraObjectInterfaceParameterGetter>();
			UCameraObjectInterfaceParameterBase* Parameter = ParameterGetter->GetInterfaceParameter();
			UCameraObjectInterfaceBlendableParameter* BlendableParameter = Cast<UCameraObjectInterfaceBlendableParameter>(Parameter);
			if (BlendableParameter && 
					B->PinType.PinSubCategory == UEnum::GetValueAsName(BlendableParameter->VariableType))
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT("Compatible pin types"));
			}
			UCameraObjectInterfaceDataParameter* DataParameter = Cast<UCameraObjectInterfaceDataParameter>(Parameter);
			if (DataParameter && 
					B->PinType.PinSubCategory == UEnum::GetValueAsName(DataParameter->DataType) &&
					B->PinType.PinSubCategoryObject == DataParameter->DataTypeObject)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT("Compatible pin types"));
			}
		}
	}

	return Super::CanCreateConnection(A, B);
}

bool UCameraObjectGraphSchemaBase::OnTryCreateCustomConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	UEdGraphPin* SourcePin = nullptr;
	UEdGraphPin* TargetPin = nullptr;
	if (A->Direction == EGPD_Output && B->Direction == EGPD_Input)
	{
		SourcePin = A;
		TargetPin = B;
	}
	else if (B->Direction == EGPD_Output && A->Direction == EGPD_Input)
	{
		SourcePin = B;
		TargetPin = A;
	}
	if (!ensure(SourcePin && TargetPin))
	{
		return false;
	}

	UObjectTreeGraphNode* SourceNode = Cast<UObjectTreeGraphNode>(SourcePin->GetOwningNode());
	UObjectTreeGraphNode* TargetNode = Cast<UObjectTreeGraphNode>(TargetPin->GetOwningNode());
	if (!SourceNode || !TargetNode)
	{
		return false;
	}

	ensure(SourceNode->GetGraph() == TargetNode->GetGraph());
	UObjectTreeGraph* Graph = Cast<UObjectTreeGraph>(SourceNode->GetGraph());
	if (!ensure(Graph))
	{
		return false;
	}

	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(Graph->GetRootObject());
	if (!CameraObject)
	{
		return false;
	}

	// See if we are connecting a parameter getter to a camera node's input property.
	if (UCameraObjectInterfaceParameterGraphNode* ParameterNode = Cast<UCameraObjectInterfaceParameterGraphNode>(SourceNode))
	{
		if (SourcePin->PinType.PinCategory == PC_Self && 
			(TargetPin->PinType.PinCategory == PC_CameraParameter || 
			 TargetPin->PinType.PinCategory == PC_CameraVariableReference ||
			 TargetPin->PinType.PinCategory == PC_CameraContextData))
		{
			CameraObject->Modify();

			FCameraObjectConnection NewConnection;
			NewConnection.Source = SourceNode->GetObject();
			NewConnection.Target = TargetNode->GetObject();
			NewConnection.TargetPropertyName = TargetPin->PinName;
			CameraObject->Connections.Connections.Add(NewConnection);

			return true;
		}
	}

	// See if we are connecting one node's output to another node's input.
	bool bDoFlowConnection = false;
	if (SourcePin->PinType.PinCategory == PC_CameraVariableReference &&
			(TargetPin->PinType.PinCategory == PC_CameraParameter ||
			 TargetPin->PinType.PinCategory == PC_CameraVariableReference))
	{
		bDoFlowConnection = true;
	}
	else if (SourcePin->PinType.PinCategory == PC_CameraContextData &&
			TargetPin->PinType.PinCategory == PC_CameraContextData)
	{
		bDoFlowConnection = true;
	}
	if (bDoFlowConnection)
	{
		CameraObject->Modify();

		FCameraObjectConnection NewConnection;
		NewConnection.Source = SourceNode->GetObject();
		NewConnection.SourcePropertyName = SourcePin->PinName;
		NewConnection.Target = TargetNode->GetObject();
		NewConnection.TargetPropertyName = TargetPin->PinName;
		CameraObject->Connections.Connections.Add(NewConnection);

		return true;
	}

	return false;
}

bool UCameraObjectGraphSchemaBase::OnBreakCustomPinLinks(UEdGraphPin& TargetPin) const
{
	UObjectTreeGraphNode* Node = Cast<UObjectTreeGraphNode>(TargetPin.GetOwningNode());
	if (!Node)
	{
		return false;
	}

	UObjectTreeGraph* Graph = Cast<UObjectTreeGraph>(Node->GetGraph());
	if (!ensure(Graph))
	{
		return false;
	}

	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(Graph->GetRootObject());
	if (!CameraObject)
	{
		return false;
	}

	UObject* Object = Node->GetObject();
	if (!Object)
	{
		return false;
	}

	bool bFoundAny = false;
	for (auto It = CameraObject->Connections.Connections.CreateIterator(); It; ++It)
	{
		FCameraObjectConnection& Connection(*It);
		if (
				(Connection.Source == Object && TargetPin.PinName == Connection.SourcePropertyName) ||
				(Connection.Target == Object && TargetPin.PinName == Connection.TargetPropertyName)
		   )
		{
			if (!bFoundAny)
			{
				CameraObject->Modify();
				bFoundAny = true;
			}

			It.RemoveCurrent();
		}
	}
	return bFoundAny;
}

bool UCameraObjectGraphSchemaBase::OnBreakSingleCustomPinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	UObjectTreeGraphNode* SourceNode = Cast<UObjectTreeGraphNode>(SourcePin->GetOwningNode());
	UObjectTreeGraphNode* TargetNode = Cast<UObjectTreeGraphNode>(TargetPin->GetOwningNode());
	if (!SourceNode || !TargetNode)
	{
		return false;
	}

	UObjectTreeGraph* Graph = Cast<UObjectTreeGraph>(SourceNode->GetGraph());
	if (!ensure(Graph))
	{
		return false;
	}

	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(Graph->GetRootObject());
	if (!CameraObject)
	{
		return false;
	}

	UObject* SourceObject = SourceNode->GetObject();
	UObject* TargetObject = TargetNode->GetObject();

	bool bFoundAny = false;
	for (auto It = CameraObject->Connections.Connections.CreateIterator(); It; ++It)
	{
		FCameraObjectConnection& Connection(*It);
		if (
				(Connection.Source == SourceObject && SourcePin->PinName == Connection.SourcePropertyName) &&
				(Connection.Target == TargetObject && TargetPin->PinName == Connection.TargetPropertyName)
		   )
		{
			if (!bFoundAny)
			{
				CameraObject->Modify();
				bFoundAny = true;
			}

			It.RemoveCurrent();
		}
	}
	return bFoundAny;
}

FLinearColor UCameraObjectGraphSchemaBase::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == PC_CameraParameter || PinType.PinCategory == PC_CameraVariableReference)
	{
		const FName TypeName = PinType.PinSubCategory;
		return PinColors.GetPinColor(TypeName);
	}
	if (PinType.PinCategory == PC_CameraContextData)
	{
		const FName DataTypeName = PinType.PinSubCategory;
		return PinColors.GetContextDataPinColor(DataTypeName);
	}

	return UObjectTreeGraphSchema::GetPinTypeColor(PinType);
}

FCameraObjectGraphSchemaAction_NewInterfaceParameterNode::FCameraObjectGraphSchemaAction_NewInterfaceParameterNode()
{
}

FCameraObjectGraphSchemaAction_NewInterfaceParameterNode::FCameraObjectGraphSchemaAction_NewInterfaceParameterNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping, InKeywords)
{
}

UEdGraphNode* FCameraObjectGraphSchemaAction_NewInterfaceParameterNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode)
{
	using namespace UE::Cameras;

	UObjectTreeGraph* ObjectTreeGraph = Cast<UObjectTreeGraph>(ParentGraph);
	if (!ensure(ObjectTreeGraph))
	{
		return nullptr;
	}

	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(ObjectTreeGraph->GetRootObject());
	if (!ensure(CameraObject))
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateNewNodeAction", "Create New Node"));

	const UCameraObjectGraphSchemaBase* Schema = CastChecked<UCameraObjectGraphSchemaBase>(ParentGraph->GetSchema());

	CameraObject->Modify();

	// Create a new interface parameter and set it up based on the pin we're creating it from, if any.
	UCameraObjectInterfaceParameterBase* NewInterfaceParameter = nullptr;
	if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
	{
		UCameraObjectInterfaceBlendableParameter* NewBlendableParameter = NewObject<UCameraObjectInterfaceBlendableParameter>(CameraObject, NAME_None, RF_Transactional);
		NewBlendableParameter->VariableType = ParameterDefinition.VariableType;
		NewBlendableParameter->BlendableStructType = ParameterDefinition.BlendableStructType;
		CameraObject->Interface.BlendableParameters.Add(NewBlendableParameter);
		NewInterfaceParameter = NewBlendableParameter;
	}
	else if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Data)
	{
		UCameraObjectInterfaceDataParameter* NewDataParameter = NewObject<UCameraObjectInterfaceDataParameter>(CameraObject, NAME_None, RF_Transactional);
		NewDataParameter->DataType = ParameterDefinition.DataType;
		NewDataParameter->DataContainerType = ParameterDefinition.DataContainerType;
		NewDataParameter->DataTypeObject = ParameterDefinition.DataTypeObject;
		CameraObject->Interface.DataParameters.Add(NewDataParameter);
		NewInterfaceParameter = NewDataParameter;
	}

	if (!ensure(NewInterfaceParameter))
	{
		return nullptr;
	}

	NewInterfaceParameter->InterfaceParameterName = FromPin ? FromPin->GetName() : NewInterfaceParameter->GetName();

	// Set the value on the default parameters property bag to be the same as the value from which
	// we pulled a connection.
	if (FromPin)
	{
		if (UObjectTreeGraphNode* FromGraphNode = Cast<UObjectTreeGraphNode>(FromPin->GetOwningNode()))
		{
			if (UCameraNode* FromCameraNode = FromGraphNode->CastObject<UCameraNode>())
			{
				const FName TargetPropertyName = FromPin->GetFName();
				FCameraObjectInterfaceParameterDefinition NewParameterDefinition;
				NewInterfaceParameter->GetParameterDefinition(NewParameterDefinition);
				FCameraObjectInterfaceParameterBuilder::SetDefaultParameterValue(
						CameraObject, NewParameterDefinition, FromCameraNode, TargetPropertyName, true);
			}
		}
	}

	// Create a getter node for this parameter. It will then get connected inside AutowireNewNode.
	UCameraObjectInterfaceParameterGetter* NewGetter = NewObject<UCameraObjectInterfaceParameterGetter>(CameraObject, NAME_None, RF_Transactional);
	NewGetter->ParameterGuid = NewInterfaceParameter->GetGuid();
	ensure(NewGetter->ParameterGuid.IsValid());

	ObjectTreeGraph->Modify();

	UObjectTreeGraphNode* NewGetterNode = CastChecked<UObjectTreeGraphNode>(Schema->CreateObjectNode(ObjectTreeGraph, NewGetter));

	Schema->AddConnectableObject(ObjectTreeGraph, NewGetter);

	NewGetterNode->NodePosX = Location.X;
	NewGetterNode->NodePosY = Location.Y;
	NewGetterNode->OnGraphNodeMoved(false);

	NewGetterNode->AutowireNewNode(FromPin);

	CameraObject->EventHandlers.Notify(&UE::Cameras::ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	return NewGetterNode;
}

FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode::FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode()
{
}

FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode::FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping, InKeywords)
{
}

UEdGraphNode* FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode)
{
	if (!ensure(InterfaceParameter))
	{
		return nullptr;
	}

	UObjectTreeGraph* ObjectTreeGraph = Cast<UObjectTreeGraph>(ParentGraph);
	if (!ensure(ObjectTreeGraph))
	{
		return nullptr;
	}

	UBaseCameraObject* CameraObject = Cast<UBaseCameraObject>(ObjectTreeGraph->GetRootObject());
	if (!ensure(CameraObject))
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateNewNodeAction", "Create New Node"));

	const UCameraObjectGraphSchemaBase* Schema = CastChecked<UCameraObjectGraphSchemaBase>(ParentGraph->GetSchema());

	CameraObject->Modify();

	// Create a new getter for the given parameter.
	UCameraObjectInterfaceParameterGetter* NewGetter = NewObject<UCameraObjectInterfaceParameterGetter>(CameraObject, NAME_None, RF_Transactional);
	NewGetter->ParameterGuid = InterfaceParameter->GetGuid();
	ensure(NewGetter->ParameterGuid.IsValid());

	ParentGraph->Modify();
	
	UObjectTreeGraphNode* NewGetterNode = CastChecked<UObjectTreeGraphNode>(Schema->CreateObjectNode(ObjectTreeGraph, NewGetter));

	Schema->AddConnectableObject(ObjectTreeGraph, NewGetter);

	NewGetterNode->NodePosX = Location.X;
	NewGetterNode->NodePosY = Location.Y;
	NewGetterNode->OnGraphNodeMoved(false);

	NewGetterNode->AutowireNewNode(FromPin);

	CameraObject->EventHandlers.Notify(&UE::Cameras::ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	return NewGetterNode;
}

#undef LOCTEXT_NAMESPACE

