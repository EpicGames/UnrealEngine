// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraObjectInterfaceParameterGraphNode.h"

#include "Core/CameraRigAsset.h"
#include "Editors/SCameraObjectInterfaceParameterGraphNode.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraObjectInterfaceParameterGraphNode)

UCameraObjectInterfaceParameterGraphNode::UCameraObjectInterfaceParameterGraphNode(const FObjectInitializer& ObjInit)
	: UObjectTreeGraphNode(ObjInit)
{
}

UCameraObjectInterfaceParameterBase* UCameraObjectInterfaceParameterGraphNode::GetInterfaceParameter() const
{
	if (UCameraObjectInterfaceParameterGetter* GetterNode = CastChecked<UCameraObjectInterfaceParameterGetter>(GetObject(), ECastCheckedType::NullAllowed))
	{
		return GetterNode->GetInterfaceParameter();
	}
	return nullptr;
}

FText UCameraObjectInterfaceParameterGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (UCameraObjectInterfaceParameterGetter* GetterNode = CastChecked<UCameraObjectInterfaceParameterGetter>(GetObject(), ECastCheckedType::NullAllowed))
	{
		return FText::FromString(GetterNode->GetInterfaceParameterName());
	}
	return FText();
}

TSharedPtr<SGraphNode> UCameraObjectInterfaceParameterGraphNode::CreateVisualWidget()
{
	return SNew(SCameraObjectInterfaceParameterGraphNode).GraphNode(this);
}

