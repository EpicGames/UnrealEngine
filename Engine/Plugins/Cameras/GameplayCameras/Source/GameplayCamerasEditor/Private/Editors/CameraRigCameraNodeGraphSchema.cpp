// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraRigCameraNodeGraphSchema.h"

#include "Containers/Set.h"
#include "Core/BlendCameraNode.h"
#include "Core/CameraNodeHierarchy.h"
#include "Core/CameraRigAsset.h"
#include "Editors/CameraNodeGraphNode.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameplayCamerasEditorSettings.h"
#include "Nodes/Common/ArrayCameraNode.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigCameraNodeGraphSchema)

#define LOCTEXT_NAMESPACE "CameraRigCameraNodeGraphSchema"

UCameraRigCameraNodeGraphSchema::UCameraRigCameraNodeGraphSchema(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UCameraRigCameraNodeGraphSchema::OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const
{
	Super::OnBuildGraphConfig(InOutGraphConfig);

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	InOutGraphConfig.GraphName = UCameraRigAsset::NodeTreeGraphName;
	InOutGraphConfig.GraphDisplayInfo.PlainName = LOCTEXT("NodeGraphPlainName", "CameraNodes");
	InOutGraphConfig.GraphDisplayInfo.DisplayName = LOCTEXT("NodeGraphDisplayName", "Camera Nodes");
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraNode::StaticClass());
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraRigAsset::StaticClass());
	InOutGraphConfig.NonConnectableObjectClasses.Add(UBlendCameraNode::StaticClass());
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraNode::StaticClass())
		.StripDisplayNameSuffix(TEXT("Camera Node"))
		.CreateCategoryMetaData(TEXT("CameraNodeCategories"))
		.NodeTitleColor(Settings->CameraNodeTitleColor)
		.GraphNodeClass(UCameraNodeGraphNode::StaticClass());
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraRigAsset::StaticClass())
		.OnlyAsRoot()
		.HasSelfPin(false)
		.NodeTitleUsesObjectName(true)
		.NodeTitleColor(Settings->CameraRigAssetTitleColor);
	InOutGraphConfig.ObjectClassConfigs.Emplace(UArrayCameraNode::StaticClass())
		.OnSetupNewObject(FOnSetupNewObject::CreateLambda([](UObject* NewObject)
				{
					// Add two new pins by default.
					UArrayCameraNode* ArrayNode = CastChecked<UArrayCameraNode>(NewObject);
					ArrayNode->Children.AddDefaulted();
					ArrayNode->Children.AddDefaulted();
				}));
}

void UCameraRigCameraNodeGraphSchema::CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const
{
	using namespace UE::Cameras;

	// Start with the graph objects from the root interface.
	CollectAllConnectableObjectsFromRootInterface(InGraph, OutAllObjects, false);

	// See if we are missing objects from AllNodeTreeObjects... if so, add them and notify the user.
	UCameraRigAsset* CameraRig = Cast<UCameraRigAsset>(InGraph->GetRootObject());
	if (CameraRig)
	{
		FCameraNodeHierarchy Hierarchy(CameraRig);

		TSet<UObject*> AllNodeTreeObjects;
		((IObjectTreeGraphRootObject*)CameraRig)->GetConnectableObjects(UCameraRigAsset::NodeTreeGraphName, AllNodeTreeObjects);

		TSet<UObject*> MissingNodeTreeObjects;
		if (Hierarchy.FindMissingConnectableObjects(AllNodeTreeObjects, MissingNodeTreeObjects))
		{
			FNotificationInfo NotificationInfo(
					FText::Format(
						LOCTEXT("AllNodeTreeObjectsMismatch", 
							"Found {0} nodes missing from the internal list. Please re-save the asset."),
						MissingNodeTreeObjects.Num()));
			NotificationInfo.ExpireDuration = 4.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);

			for (UObject* MissingObject : MissingNodeTreeObjects)
			{
				((IObjectTreeGraphRootObject*)CameraRig)->AddConnectableObject(UCameraRigAsset::NodeTreeGraphName, MissingObject);
				OutAllObjects.Add(MissingObject);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

