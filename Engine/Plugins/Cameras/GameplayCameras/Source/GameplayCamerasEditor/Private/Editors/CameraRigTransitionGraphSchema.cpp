// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraRigTransitionGraphSchema.h"

#include "Core/CameraRigAsset.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "GameplayCamerasEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigTransitionGraphSchema)

#define LOCTEXT_NAMESPACE "CameraRigTransitionGraphSchema"

void UCameraRigTransitionGraphSchema::OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const
{
	Super::OnBuildGraphConfig(InOutGraphConfig);

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	InOutGraphConfig.GraphName = UCameraRigAsset::TransitionsGraphName;
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraRigAsset::StaticClass());
	InOutGraphConfig.GraphDisplayInfo.PlainName = LOCTEXT("NodeGraphPlainName", "Transitions");
	InOutGraphConfig.GraphDisplayInfo.DisplayName = LOCTEXT("NodeGraphDisplayName", "Transitions");
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraRigAsset::StaticClass())
		.HasSelfPin(false)
		.OnlyAsRoot()
		.NodeTitleUsesObjectName(true)
		.NodeTitleColor(Settings->CameraRigAssetTitleColor);
}

#undef LOCTEXT_NAMESPACE

