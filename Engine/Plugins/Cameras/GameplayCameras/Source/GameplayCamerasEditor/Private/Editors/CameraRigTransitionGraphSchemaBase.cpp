// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraRigTransitionGraphSchemaBase.h"

#include "Core/BlendCameraNode.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigTransition.h"
#include "EdGraph/EdGraphPin.h"
#include "Editors/CameraNodeGraphNode.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameplayCamerasEditorSettings.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigTransitionGraphSchemaBase)

#define LOCTEXT_NAMESPACE "CameraRigTransitionGraphSchemaBase"

namespace UE::Cameras
{

struct FCameraRigTransitionObjectCollector
{
	static bool FindMissingConnectableObjects(UCameraRigAsset* InCameraRig, const TSet<UObject*>& InTransitionObjects, TSet<UObject*>& OutMissingObjects)
	{
		TSet<UObject*> CollectedObjects;
		CollectObjects(InCameraRig, CollectedObjects);

		OutMissingObjects = CollectedObjects.Difference(InTransitionObjects);
		return !OutMissingObjects.IsEmpty();
	}

private:

	static void CollectObjects(UCameraRigAsset* InCameraRig, TSet<UObject*>& OutObjects)
	{
		if (!InCameraRig)
		{
			return;
		}

		CollectTransitions(InCameraRig->EnterTransitions, OutObjects);
		CollectTransitions(InCameraRig->ExitTransitions, OutObjects);
	}

	static void CollectTransitions(TArrayView<TObjectPtr<UCameraRigTransition>> Transitions, TSet<UObject*>& OutObjects)
	{
		for (TObjectPtr<UCameraRigTransition> Transition : Transitions)
		{
			if (Transition)
			{
				OutObjects.Add(Transition);

				// We don't support nested blends and nested conditions here... but most of those
				// were added after AllTransitionsObjects was added so it should be fine.
				if (Transition->Blend)
				{
					OutObjects.Add(Transition->Blend);
				}

				for (UCameraRigTransitionCondition* Condition : Transition->Conditions)
				{
					if (Condition)
					{
						OutObjects.Add(Condition);
					}
				}
			}
		}
	}
};

}  // namespace UE::Cameras

void UCameraRigTransitionGraphSchemaBase::OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const
{
	using namespace UE::Cameras;

	Super::OnBuildGraphConfig(InOutGraphConfig);

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraRigTransition::StaticClass());
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraRigTransitionCondition::StaticClass());
	InOutGraphConfig.ConnectableObjectClasses.Add(UBlendCameraNode::StaticClass());
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraRigTransition::StaticClass())
		.NodeTitleColor(Settings->CameraRigTransitionTitleColor);
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraRigTransitionCondition::StaticClass())
		.StripDisplayNameSuffix(TEXT("Transition Condition"))
		.NodeTitleColor(Settings->CameraRigTransitionConditionTitleColor);
	InOutGraphConfig.ObjectClassConfigs.Emplace(UBlendCameraNode::StaticClass())
		.StripDisplayNameSuffix(TEXT("Camera Node"))
		.CreateCategoryMetaData(TEXT("CameraNodeCategories"))
		.NodeTitleColor(Settings->CameraBlendNodeTitleColor)
		.GraphNodeClass(UCameraNodeGraphNode::StaticClass());
}

void UCameraRigTransitionGraphSchemaBase::CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const
{
	using namespace UE::Cameras;

	// Start with the graph objects from the root interface.
	CollectAllConnectableObjectsFromRootInterface(InGraph, OutAllObjects, false);

	// See if we are missing objects from AllTransitionsObjects... if so, add them and notify the user.
	UCameraRigAsset* CameraRig = Cast<UCameraRigAsset>(InGraph->GetRootObject());
	if (CameraRig)
	{
		TSet<UObject*> AllTransitionObjects;
		((IObjectTreeGraphRootObject*)CameraRig)->GetConnectableObjects(UCameraRigAsset::TransitionsGraphName, AllTransitionObjects);

		TSet<UObject*> MissingTransitionObjects;
		if (FCameraRigTransitionObjectCollector::FindMissingConnectableObjects(
					CameraRig, AllTransitionObjects, MissingTransitionObjects))
		{
			FNotificationInfo NotificationInfo(
					FText::Format(
						LOCTEXT("AllTransitionObjectsMismatch", 
							"Found {0} nodes missing from the internal list. Please re-save the asset."),
						MissingTransitionObjects.Num()));
			NotificationInfo.ExpireDuration = 4.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);

			for (UObject* MissingObject : MissingTransitionObjects)
			{
				((IObjectTreeGraphRootObject*)CameraRig)->AddConnectableObject(UCameraRigAsset::TransitionsGraphName, MissingObject);
				OutAllObjects.Add(MissingObject);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

