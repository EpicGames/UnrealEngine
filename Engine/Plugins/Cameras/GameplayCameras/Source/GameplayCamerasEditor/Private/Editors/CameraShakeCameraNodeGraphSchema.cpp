// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraShakeCameraNodeGraphSchema.h"

#include "Core/CameraShakeAsset.h"
#include "Core/ShakeCameraNode.h"
#include "Editors/CameraNodeGraphNode.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "GameplayCamerasEditorSettings.h"
#include "Nodes/Blends/SimpleBlendCameraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeCameraNodeGraphSchema)

#define LOCTEXT_NAMESPACE "CameraShakeCameraNodeGraphSchema"

UCameraShakeCameraNodeGraphSchema::UCameraShakeCameraNodeGraphSchema(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UCameraShakeCameraNodeGraphSchema::CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const
{
	using namespace UE::Cameras;

	// Only get the graph objects from the root interface.
	CollectAllConnectableObjectsFromRootInterface(InGraph, OutAllObjects, false);
}

void UCameraShakeCameraNodeGraphSchema::OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const
{
	Super::OnBuildGraphConfig(InOutGraphConfig);

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	InOutGraphConfig.GraphDisplayInfo.PlainName = LOCTEXT("NodeGraphPlainName", "CameraShakeNodes");
	InOutGraphConfig.GraphDisplayInfo.DisplayName = LOCTEXT("NodeGraphDisplayName", "Camera Shake Nodes");
	InOutGraphConfig.ConnectableObjectClasses.Add(UShakeCameraNode::StaticClass());
	InOutGraphConfig.ConnectableObjectClasses.Add(USimpleFixedTimeBlendCameraNode::StaticClass());
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraShakeAsset::StaticClass());
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraShakeAsset::StaticClass())
		.OnlyAsRoot()
		.HasSelfPin(false)
		.NodeTitleUsesObjectName(true)
		.NodeTitleColor(Settings->CameraShakeAssetTitleColor);
	InOutGraphConfig.ObjectClassConfigs.Emplace(UShakeCameraNode::StaticClass())
		.StripDisplayNameSuffix(TEXT("Camera Node"))
		.CreateCategoryMetaData(TEXT("CameraNodeCategories"))
		.NodeTitleColor(Settings->CameraNodeTitleColor)
		.GraphNodeClass(UCameraNodeGraphNode::StaticClass());
}

#undef LOCTEXT_NAMESPACE

