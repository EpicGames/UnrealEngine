// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/CameraSharedTransitionGraphSchema.h"

#include "Core/CameraAsset.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "GameplayCamerasEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraSharedTransitionGraphSchema)

#define LOCTEXT_NAMESPACE "CameraSharedTransitionGraphSchema"

void UCameraSharedTransitionGraphSchema::OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const
{
	Super::OnBuildGraphConfig(InOutGraphConfig);

	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();

	InOutGraphConfig.GraphName = UCameraAsset::SharedTransitionsGraphName;
	InOutGraphConfig.ConnectableObjectClasses.Add(UCameraAsset::StaticClass());
	InOutGraphConfig.GraphDisplayInfo.PlainName = LOCTEXT("NodeGraphPlainName", "SharedTransitions");
	InOutGraphConfig.GraphDisplayInfo.DisplayName = LOCTEXT("NodeGraphDisplayName", "Shared Transitions");
	InOutGraphConfig.ObjectClassConfigs.Emplace(UCameraAsset::StaticClass())
		.HasSelfPin(false)
		.OnlyAsRoot()
		.NodeTitleUsesObjectName(true)
		.NodeTitleColor(Settings->CameraAssetTitleColor);
}

#undef LOCTEXT_NAMESPACE

