// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/GameplayCamerasGraphPanelPinFactory.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraVariableAssets.h"
#include "EdGraphSchema_K2.h"
#include "Editors/SCameraVariableNameGraphPin.h"
#include "K2Node_CallFunction.h"

namespace UE::Cameras
{

TSharedPtr<SGraphPin> FGameplayCamerasGraphPanelPinFactory::CreatePin(UEdGraphPin* Pin) const
{
	if (!Pin)
	{
		return nullptr;
	}

	const FEdGraphPinType& PinType = Pin->PinType;
	const UClass* PinPropertyClass = Cast<const UClass>(PinType.PinSubCategoryObject);
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object && PinPropertyClass)
	{
		if (PinPropertyClass->IsChildOf<UCameraVariableAsset>())
		{
			return SNew(SCameraVariableNameGraphPin, Pin);
		}
	}

	return nullptr;
}

}  // namespace UE::Cameras

