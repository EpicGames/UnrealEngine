// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeConnectionDrawingPolicy.h"

FObjectTreeConnectionDrawingPolicy::FObjectTreeConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
{
	// Don't draw arrowheads.
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
}

