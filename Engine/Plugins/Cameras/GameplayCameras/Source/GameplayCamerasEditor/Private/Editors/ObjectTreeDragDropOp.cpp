// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeDragDropOp.h"

#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphSchema.h"
#include "GraphEditor.h"
#include "SGraphPanel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "ObjectTreeGraphDragDropOp"

TSharedRef<FObjectTreeClassDragDropOp> FObjectTreeClassDragDropOp::New(UClass* ObjectClass)
{
	TArray<UClass*> ObjectClasses;
	ObjectClasses.Add(ObjectClass);
	return New(ObjectClasses);
}

TSharedRef<FObjectTreeClassDragDropOp> FObjectTreeClassDragDropOp::New(TArrayView<UClass*> ObjectClasses)
{
	TSharedRef<FObjectTreeClassDragDropOp> Operation = MakeShared<FObjectTreeClassDragDropOp>();
	Operation->ObjectClasses = ObjectClasses;
	Operation->Construct();
	return Operation;
}

FReply FObjectTreeClassDragDropOp::ExecuteDragOver(TSharedPtr<SGraphEditor> GraphEditor)
{
	UObjectTreeGraph* Graph = CastChecked<UObjectTreeGraph>(GraphEditor->GetCurrentGraph());
	TArray<UClass*> PlaceableClasses = FilterPlaceableObjectClasses(Graph);

	if (PlaceableClasses.Num() == ObjectClasses.Num())
	{
		const FSlateBrush* OKIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		SetToolTip(
				FText::Format(
					LOCTEXT("OnDragOver_Success", "Create {0} node(s) from the dragged object classes"),
					ObjectClasses.Num()),
				OKIcon);
	}
	else if (PlaceableClasses.Num() > 0)
	{
		const FSlateBrush* WarnIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OKWarn"));
		SetToolTip(
				FText::Format(
					LOCTEXT("OnDragOver_Warning", "Create {0} node(s) from the dragged object classes, ignoring {1} that can't be created in this graph"),
					PlaceableClasses.Num(), (ObjectClasses.Num() - PlaceableClasses.Num())),
				WarnIcon);
	}
	else
	{
		const FSlateBrush* ErrorIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		SetToolTip(
				LOCTEXT("OnDragOver_Error", "The dragged object classes can't be created in this graph"),
				ErrorIcon);
	}

	return FReply::Handled();
}

FReply FObjectTreeClassDragDropOp::ExecuteDrop(TSharedPtr<SGraphEditor> GraphEditor, const FSlateCompatVector2f& NewLocation)
{
	const FScopedTransaction Transaction(LOCTEXT("DropObjectClasses", "Drop New Nodes"));

	UObjectTreeGraph* Graph = CastChecked<UObjectTreeGraph>(GraphEditor->GetCurrentGraph());
	TArray<UClass*> PlaceableClasses = FilterPlaceableObjectClasses(Graph);

	GraphEditor->ClearSelectionSet();

	FSlateCompatVector2f CurLocation = NewLocation;
	for (UClass* PlaceableClass : PlaceableClasses)
	{
		FObjectTreeGraphSchemaAction_NewNode Action;
		Action.ObjectClass = PlaceableClass;
		UEdGraphNode* NewNode = Action.PerformAction(Graph, nullptr, CurLocation, false);
		GraphEditor->SetNodeSelection(NewNode, true);

		CurLocation += FSlateCompatVector2f(20.0f, 20.0f);
	}

	return FReply::Handled();
}

TArray<UClass*> FObjectTreeClassDragDropOp::FilterPlaceableObjectClasses(UObjectTreeGraph* InGraph)
{
	const FObjectTreeGraphConfig& GraphConfig = InGraph->GetConfig();
	TArray<UClass*> PlaceableClasses = ObjectClasses.FilterByPredicate(
			[&GraphConfig](UClass* ObjectClass)
			{
				return GraphConfig.IsConnectable(ObjectClass);
			});
	return PlaceableClasses;
}

#undef LOCTEXT_NAMESPACE

