// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraph.h"

#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/ObjectTreeGraphSchema.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTreeGraph)

UObjectTreeGraph::UObjectTreeGraph(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Schema = UObjectTreeGraphSchema::StaticClass();
}

void UObjectTreeGraph::Reset(TObjectPtr<UObject> InRootObject, const FObjectTreeGraphConfig& InConfig)
{
	WeakRootObject = InRootObject;

	Config = InConfig;
	if (!Config.DefaultGraphNodeClass)
	{
		Config.DefaultGraphNodeClass = UObjectTreeGraphNode::StaticClass();
	}

	RebuildGraph();
}

UObjectTreeGraphNode* UObjectTreeGraph::FindObjectNode(UObject* InObject) const
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UObjectTreeGraphNode* ObjectNode = Cast<UObjectTreeGraphNode>(Node))
		{
			if (ObjectNode->GetObject() == InObject)
			{
				return ObjectNode;
			}
		}
	}
	return nullptr;
}

const FObjectTreeGraphConfig& UObjectTreeGraph::GetConfig() const
{
	return Config;
}

void UObjectTreeGraph::RebuildGraph()
{
	const UObjectTreeGraphSchema* GraphSchema = Cast<UObjectTreeGraphSchema>(GetSchema());
	if (ensure(GraphSchema))
	{
		GraphSchema->RebuildGraph(this);
	}
}

