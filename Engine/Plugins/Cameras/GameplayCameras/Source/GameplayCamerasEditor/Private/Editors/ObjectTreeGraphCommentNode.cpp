// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraphCommentNode.h"

#include "Core/ObjectTreeGraphComment.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/SObjectTreeGraphCommentNode.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "ToolMenuDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTreeGraphCommentNode)

#define LOCTEXT_NAMESPACE "ObjectTreeGraphCommentNode"

void UObjectTreeGraphCommentNode::Initialize(UObjectTreeGraphComment* InObject)
{
	WeakObject = InObject;
}

TSharedPtr<SGraphNode> UObjectTreeGraphCommentNode::CreateVisualWidget()
{
	return SNew(SObjectTreeGraphCommentNode).GraphNode(this);
}

void UObjectTreeGraphCommentNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	const FGraphEditorCommandsImpl& GraphEditorCommands = FGraphEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	// General actions.
	{
		FToolMenuSection& Section = Menu->AddSection(
				"ObjectTreeGraphNodeGenericActions", LOCTEXT("GenericActionsMenuHeader", "General"));

		Section.AddMenuEntry(GenericCommands.Delete);
		Section.AddMenuEntry(GenericCommands.Cut);
		Section.AddMenuEntry(GenericCommands.Copy);
		Section.AddMenuEntry(GenericCommands.Duplicate);
	}

	// Graph organization.
	{
		FToolMenuSection& Section = Menu->AddSection(
				"ObjectTreeGraphOrganizationActions", LOCTEXT("OrganizationActionsMenuHeader", "Organization"));

		Section.AddSubMenu(
				"Alignment",
				LOCTEXT("AlignmentHeader", "Alignment"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([&GraphEditorCommands](UToolMenu* InMenu)
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection(
								"ObjectTreeGraphAlignmentActions", LOCTEXT("AlignmentHeader", "Alignment"));
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesTop);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesBottom);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesLeft);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesCenter);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesRight);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.StraightenConnections);
					}));

		Section.AddSubMenu(
				"Distribution",
				LOCTEXT("DistributionHeader", "Distribution"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([&GraphEditorCommands](UToolMenu* InMenu)
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection(
								"ObjectTreeGraphDistributionActions", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(GraphEditorCommands.DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.DistributeNodesVertically);
					}));
	}
}

void UObjectTreeGraphCommentNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	MoveMode = ECommentBoxMode::GroupMovement;

	UObjectTreeGraphComment* CommentObject = WeakObject.Get();
	if (CommentObject)
	{
		UObjectTreeGraph* OuterGraph = CastChecked<UObjectTreeGraph>(GetGraph());
		const FObjectTreeGraphConfig& OuterGraphConfig = OuterGraph->GetConfig();

		CommentObject->GetGraphNodePosition(OuterGraphConfig.GraphName, NodePosX, NodePosY);

		NodeWidth = CommentObject->GraphNodeSize.X;
		NodeHeight = CommentObject->GraphNodeSize.Y;

		NodeComment = CommentObject->CommentText;
		CommentColor = CommentObject->CommentColor;
	}
}

void UObjectTreeGraphCommentNode::ResizeNode(const FSlateCompatVector2f& NewSize)
{
	Super::ResizeNode(NewSize);

	if (UObjectTreeGraphComment* Object = WeakObject.Get())
	{
		Object->Modify();
		Object->GraphNodeSize = FIntVector2((int32)NewSize.X, (int32)NewSize.Y);
		// Set position as well since may have been resized from the top corner.
		Object->GraphNodePos = FIntVector2(NodePosX, NodePosY);
	}
}

void UObjectTreeGraphCommentNode::OnRenameNode(const FString& NewName)
{
	Super::OnRenameNode(NewName);

	UObjectTreeGraphComment* CommentObject = WeakObject.Get();
	if (CommentObject)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));

		UObjectTreeGraph* OuterGraph = CastChecked<UObjectTreeGraph>(GetGraph());
		const FObjectTreeGraphConfig& OuterGraphConfig = OuterGraph->GetConfig();

		CommentObject->OnRenameGraphNode(OuterGraphConfig.GraphName, NewName);
	}
}

void UObjectTreeGraphCommentNode::OnGraphNodeMoved(bool bMarkDirty)
{
	UObject* Object = WeakObject.Get();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object);
	if (GraphObject)
	{
		UObjectTreeGraph* OuterGraph = CastChecked<UObjectTreeGraph>(GetGraph());
		const FObjectTreeGraphConfig& OuterGraphConfig = OuterGraph->GetConfig();

		GraphObject->OnGraphNodeMoved(OuterGraphConfig.GraphName, NodePosX, NodePosY, bMarkDirty);
	}
}

#undef LOCTEXT_NAMESPACE

