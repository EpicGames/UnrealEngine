// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraphConfig.h"

#include "Algo/AnyOf.h"
#include "Core/ObjectTreeGraphObject.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "ObjectTreeGraphConfig"

FObjectTreeGraphClassConfig::FObjectTreeGraphClassConfig()
	: _SelfPinName(NAME_None)
	, _HasSelfPin(true)
	, _NodeTitleUsesObjectName(false)
	, _CanCreateNew(true)
	, _CanDelete(true)
	, _CreateCategoryMetaData(TEXT("ObjectTreeGraphCategory"))
{
}

FObjectTreeGraphClassConfig& FObjectTreeGraphClassConfig::OnlyAsRoot()
{
	CanCreateNew(false);
	CanDelete(false);
	return *this;
}

FObjectTreeGraphClassConfig FObjectTreeGraphClassConfigs::DefaultConfig;

FObjectTreeGraphClassConfigs::FObjectTreeGraphClassConfigs()
{
}

FObjectTreeGraphClassConfigs::FObjectTreeGraphClassConfigs(TArrayView<const FObjectTreeGraphClassConfig*> InClassConfigs)
	: InnerConfigs(InClassConfigs)
{
}

void FObjectTreeGraphClassConfigs::GetStripDisplayNameSuffixes(TArray<FString>& OutSuffixes) const
{
	for (const FObjectTreeGraphClassConfig* InnerConfig : InnerConfigs)
	{
		OutSuffixes.Append(InnerConfig->StripDisplayNameSuffixes());
	}
}

TOptional<EEdGraphPinDirection> FObjectTreeGraphClassConfigs::GetPropertyPinDirectionOverride(const FName& InPropertyName) const
{
	for (const FObjectTreeGraphClassConfig* InnerConfig : InnerConfigs)
	{
		TOptional<EEdGraphPinDirection> DirectionOverride = InnerConfig->GetPropertyPinDirectionOverride(InPropertyName);
		if (DirectionOverride.IsSet())
		{
			return DirectionOverride;
		}
	}
	return TOptional<EEdGraphPinDirection>();
}

FObjectTreeGraphConfig::FObjectTreeGraphConfig()
	: DefaultSelfPinName(NAME_Self)
	, DefaultGraphNodeTitleColor(FLinearColor(0.549f, 0.745f, 0.698f))
	, DefaultGraphNodeTitleTextColor(FLinearColor::White)
	, DefaultGraphNodeBodyTintColor(FLinearColor::White)
{
}

bool FObjectTreeGraphConfig::IsConnectable(UClass* InObjectClass) const
{
	if (!ensure(InObjectClass))
	{
		return false;
	}

	const bool bIsConnectable = Algo::AnyOf(ConnectableObjectClasses, [InObjectClass](UClass* Item)
			{
				return InObjectClass->IsChildOf(Item);
			});
	if (!bIsConnectable)
	{
		return false;
	}

	const bool bIsExcluded = Algo::AnyOf(NonConnectableObjectClasses, [InObjectClass](UClass* Item)
			{
				return InObjectClass->IsChildOf(Item);
			});
	if (bIsExcluded)
	{
		return false;
	}

	return true;
}

bool FObjectTreeGraphConfig::IsConnectable(FObjectProperty* InObjectProperty) const
{
	if (!ensure(InObjectProperty))
	{
		return false;
	}

	if (InObjectProperty->GetBoolMetaData(TEXT("ObjectTreeGraphHidden")))
	{
		return false;
	}

	if (!IsConnectable(InObjectProperty->PropertyClass))
	{
		return false;
	}

	return true;
}

bool FObjectTreeGraphConfig::IsConnectable(FArrayProperty* InArrayProperty) const
{
	if (!ensure(InArrayProperty))
	{
		return false;
	}

	if (InArrayProperty->GetBoolMetaData(TEXT("ObjectTreeGraphHidden")))
	{
		return false;
	}

	FObjectProperty* InnerProperty = CastField<FObjectProperty>(InArrayProperty->Inner);
	if (!InnerProperty)
	{
		return false;
	}

	if (!IsConnectable(InnerProperty->PropertyClass))
	{
		return false;
	}

	return true;
}

void FObjectTreeGraphConfig::GetConnectableClasses(TArray<UClass*>& OutClasses, bool bPlaceableOnly)
{
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		if (!IsConnectable(*ClassIt))
		{
			continue;
		}

		if (bPlaceableOnly)
		{
			if (ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden))
			{
				continue;
			}

			const FObjectTreeGraphClassConfigs ClassConfig = GetObjectClassConfigs(*ClassIt);
			if (!ClassConfig.CanCreateNew())
			{
				continue;
			}
		}

		OutClasses.Add(*ClassIt);
	}
}

FObjectTreeGraphClassConfigs FObjectTreeGraphConfig::GetObjectClassConfigs(const UClass* InObjectClass) const
{
	TArray<const FObjectTreeGraphClassConfig*, TInlineAllocator<2>> ClassConfigs;
	
	while (InObjectClass)
	{
		const FObjectTreeGraphClassConfig* ClassConfig = ObjectClassConfigs.Find(InObjectClass);
		if (ClassConfig)
		{
			ClassConfigs.Add(ClassConfig);
		}

		InObjectClass = InObjectClass->GetSuperClass();
	}

	return FObjectTreeGraphClassConfigs(ClassConfigs);
}

FText FObjectTreeGraphConfig::GetDisplayNameText(const UObject* InObject) const
{
	if (InObject)
	{
		FText DisplayNameText;
		const FObjectTreeGraphClassConfigs ClassConfig = GetObjectClassConfigs(InObject->GetClass());

		const IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(InObject);
		if (GraphObject && GraphObject->HasAnySupportFlags(
					GraphName, EObjectTreeGraphObjectSupportFlags::CustomTitle | EObjectTreeGraphObjectSupportFlags::CustomRename))
		{
			GraphObject->GetGraphNodeName(GraphName, DisplayNameText);
		}

		if (DisplayNameText.IsEmpty() && ClassConfig.NodeTitleUsesObjectName())
		{
			DisplayNameText = FText::FromString(InObject->GetName());
		}

		if (!DisplayNameText.IsEmpty())
		{
			FormatDisplayNameText(InObject, ClassConfig, DisplayNameText);
			return DisplayNameText;
		}

		return GetDisplayNameText(InObject->GetClass(), ClassConfig);
	}
	return FText::GetEmpty();
}

FText FObjectTreeGraphConfig::GetDisplayNameText(const UClass* InClass) const
{
	if (InClass)
	{
		const FObjectTreeGraphClassConfigs ClassConfig = GetObjectClassConfigs(InClass);
		return GetDisplayNameText(InClass, ClassConfig);
	}
	return FText::GetEmpty();
}

FText FObjectTreeGraphConfig::GetDisplayNameText(const UClass* InClass, const FObjectTreeGraphClassConfigs& InClassConfig) const
{
	check(InClass);
	
	if (InClassConfig.OnGetObjectClassDisplayName().IsBound())
	{
		return InClassConfig.OnGetObjectClassDisplayName().Execute(InClass);
	}

	FText DisplayNameText = InClass->GetDisplayNameText();
	FormatDisplayNameText(InClass, InClassConfig, DisplayNameText);
	return DisplayNameText;
}

void FObjectTreeGraphConfig::FormatDisplayNameText(const UObject* InObject, const FObjectTreeGraphClassConfigs& InClassConfig, FText& InOutDisplayNameText) const
{
	TArray<FString> StripSuffixes;
	InClassConfig.GetStripDisplayNameSuffixes(StripSuffixes);
	if (StripSuffixes.Num() > 0)
	{
		FString DisplayName = InOutDisplayNameText.ToString();
		for (const FString& StripSuffix : StripSuffixes)
		{
			if (DisplayName.RemoveFromEnd(StripSuffix))
			{
				DisplayName.TrimEndInline();
				break;
			}
		}

		InOutDisplayNameText = FText::FromString(DisplayName);
	}

	OnFormatObjectDisplayName.ExecuteIfBound(InObject, InOutDisplayNameText);
}

EEdGraphPinDirection FObjectTreeGraphConfig::GetSelfPinDirection(const UClass* InObjectClass) const
{
	const FObjectTreeGraphClassConfigs ClassConfig = GetObjectClassConfigs(InObjectClass);
	TOptional<EEdGraphPinDirection> PinDirectionOverride = ClassConfig.SelfPinDirectionOverride();
	if (PinDirectionOverride.IsSet())
	{
		return PinDirectionOverride.GetValue();
	}

	while (InObjectClass)
	{
		const FString& CustomDefaultDirection = InObjectClass->GetMetaData(TEXT("ObjectTreeGraphSelfPinDirection"));
		if (CustomDefaultDirection == TEXT("Input"))
		{
			return EGPD_Input;
		}
		else if (CustomDefaultDirection == TEXT("Output"))
		{
			return EGPD_Output;
		}

		InObjectClass = InObjectClass->GetSuperClass();
	}

	return EGPD_Input;
}

EEdGraphPinDirection FObjectTreeGraphConfig::GetPropertyPinDirection(const UClass* InObjectClass, const FName& InPropertyName) const
{
	const FObjectTreeGraphClassConfigs ClassConfig = GetObjectClassConfigs(InObjectClass);
	TOptional<EEdGraphPinDirection> PinDirectionOverride = ClassConfig.GetPropertyPinDirectionOverride(InPropertyName);
	if (PinDirectionOverride.IsSet())
	{
		return PinDirectionOverride.GetValue();
	}

	const FProperty* Property = InObjectClass->FindPropertyByName(InPropertyName);
	const FString& CustomDirection = Property->GetMetaData(TEXT("ObjectTreeGraphPinDirection"));
	if (CustomDirection == TEXT("Input"))
	{
		return EGPD_Input;
	}
	else if (CustomDirection == TEXT("Output"))
	{
		return EGPD_Output;
	}

	TOptional<EEdGraphPinDirection> DefaultPinDirectionOverride = ClassConfig.DefaultPropertyPinDirectionOverride();
	if (DefaultPinDirectionOverride.IsSet())
	{
		return DefaultPinDirectionOverride.GetValue();
	}

	while (InObjectClass)
	{
		const FString& CustomDefaultDirection = InObjectClass->GetMetaData(TEXT("ObjectTreeGraphDefaultPropertyPinDirection"));
		if (CustomDefaultDirection == TEXT("Input"))
		{
			return EGPD_Input;
		}
		else if (CustomDefaultDirection == TEXT("Output"))
		{
			return EGPD_Output;
		}

		InObjectClass = InObjectClass->GetSuperClass();
	}

	return EGPD_Output;
}

#undef LOCTEXT_NAMESPACE

