// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraphNode.h"

#include "Core/ObjectTreeGraphObject.h"
#include "EdGraph/EdGraphPin.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphSchema.h"
#include "Editors/SObjectTreeGraphNode.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTreeGraphNode)

#define LOCTEXT_NAMESPACE "ObjectTreeGraphNode"

UObjectTreeGraphNode::UObjectTreeGraphNode(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bCanRenameNode = true;
}

void UObjectTreeGraphNode::Initialize(UObject* InObject)
{
	ensure(InObject);

	WeakObject = InObject;

	const FNodeContext NodeContext = GetNodeContext();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(InObject);
	if (GraphObject && GraphObject->HasAnySupportFlags(NodeContext.GraphConfig.GraphName, EObjectTreeGraphObjectSupportFlags::CommentText))
	{
		NodeComment = GraphObject->GetGraphNodeCommentText(NodeContext.GraphConfig.GraphName);
	}

	OnInitialize();
}

FText UObjectTreeGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UObject* Object = WeakObject.Get();
	if (Object)
	{
		const FNodeContext NodeContext = GetNodeContext();
		return NodeContext.GraphConfig.GetDisplayNameText(Object);
	}
	return FText::GetEmpty();
}

TSharedPtr<SGraphNode> UObjectTreeGraphNode::CreateVisualWidget()
{
	return SNew(SObjectTreeGraphNode).GraphNode(this);
}

FLinearColor UObjectTreeGraphNode::GetNodeTitleColor() const
{
	const FNodeContext NodeContext = GetNodeContext();
	return NodeContext.ObjectClassConfigs.NodeTitleColor().Get(NodeContext.GraphConfig.DefaultGraphNodeTitleColor);
}

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
FLinearColor UObjectTreeGraphNode::GetNodeTitleTextColor() const
{
	const FNodeContext NodeContext = GetNodeContext();
	return NodeContext.ObjectClassConfigs.NodeTitleTextColor().Get(NodeContext.GraphConfig.DefaultGraphNodeTitleTextColor);
}
#endif

FLinearColor UObjectTreeGraphNode::GetNodeBodyTintColor() const
{
	const FNodeContext NodeContext = GetNodeContext();
	return NodeContext.ObjectClassConfigs.NodeBodyTintColor().Get(NodeContext.GraphConfig.DefaultGraphNodeBodyTintColor);
}

FText UObjectTreeGraphNode::GetTooltipText() const
{
	UObject* Object = WeakObject.Get();
	if (Object)
	{
		return Object->GetClass()->GetToolTipText();
	}
	return FText::GetEmpty();
}

void UObjectTreeGraphNode::AllocateDefaultPins()
{
	UObject* Object = WeakObject.Get();
	if (!ensure(Object))
	{
		return;
	}

	const FNodeContext NodeContext = GetNodeContext();
	const FObjectTreeGraphConfig& OuterGraphConfig = NodeContext.GraphConfig;
	const FObjectTreeGraphClassConfigs ObjectClassConfigs = NodeContext.ObjectClassConfigs;

	if (ObjectClassConfigs.HasSelfPin())
	{
		FEdGraphPinType SelfPinType;
		SelfPinType.PinCategory = UObjectTreeGraphSchema::PC_Self;
		const FName& SelfPinName = ObjectClassConfigs.SelfPinName(OuterGraphConfig.DefaultSelfPinName);
		UEdGraphPin* SelfPin = CreatePin(OuterGraphConfig.GetSelfPinDirection(NodeContext.ObjectClass), SelfPinType, SelfPinName);
		SelfPin->PinFriendlyName = ObjectClassConfigs.SelfPinFriendlyName(OuterGraphConfig.DefaultSelfPinFriendlyName);
	}

	for (TFieldIterator<FProperty> PropertyIt(NodeContext.ObjectClass); PropertyIt; ++PropertyIt)
	{
		const FName PropertyName = PropertyIt->GetFName();

		const EEdGraphPinDirection PinDirection = OuterGraphConfig.GetPropertyPinDirection(NodeContext.ObjectClass, PropertyName);

		FEdGraphPinType ChildPinType;
		ChildPinType.PinCategory = UObjectTreeGraphSchema::PC_Property;

		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(*PropertyIt))
		{
			if (!OuterGraphConfig.IsConnectable(ObjectProperty))
			{
				continue;
			}

			// Make a new pin for connecting this property to another object node.
			ChildPinType.PinSubCategory = UObjectTreeGraphSchema::PSC_ObjectProperty;
			UEdGraphPin* PropertyPin = CreatePin(PinDirection, ChildPinType, PropertyName);

			PropertyPin->PinFriendlyName = FText::FromName(PropertyName);
			PropertyPin->PinToolTip = ObjectProperty->PropertyClass->GetName();
		}
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*PropertyIt))
		{
			if (!OuterGraphConfig.IsConnectable(ArrayProperty))
			{
				continue;
			}

			// Make a new invisible pin that will be the parent pin to each array item's pin.
			FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);

			ChildPinType.PinSubCategory = UObjectTreeGraphSchema::PSC_ArrayProperty;
			ChildPinType.ContainerType = EPinContainerType::Array;
			UEdGraphPin* ArrayPin = CreatePin(PinDirection, ChildPinType, PropertyName);

			ArrayPin->PinFriendlyName = FText::FromName(PropertyName);
			ArrayPin->PinToolTip = InnerProperty->PropertyClass->GetName();
			ArrayPin->bHidden = true;  // Always hidden, we only ever show the sub-pins.

			// Create pins for each array item.
			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));
			const int32 ArrayNum = ArrayHelper.Num();
			CreateNewItemPins(ArrayPin, ArrayNum);
		}
	}
}

void UObjectTreeGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	UEdGraphPin* SelfPin = GetSelfPin();
	if (FromPin && SelfPin)
	{
		const UObjectTreeGraphSchema* GraphSchema = CastChecked<UObjectTreeGraphSchema>(GetSchema());
		GraphSchema->TryCreateConnection(FromPin, SelfPin);
	}

	Super::AutowireNewNode(FromPin);
}

void UObjectTreeGraphNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	// Auto-remove orphaned pins when they are disconnected.
	if (Pin->bOrphanedPin && Pin->LinkedTo.IsEmpty())
	{
		if (Pin->ParentPin)
		{
			Pin->ParentPin->SubPins.Remove(Pin);
		}

		RemovePin(Pin);

		UEdGraph* OuterGraph = GetGraph();
		if (OuterGraph)
		{
			OuterGraph->NotifyNodeChanged(this);
		}
	}

	Super::PinConnectionListChanged(Pin);
}

void UObjectTreeGraphNode::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();
}

void UObjectTreeGraphNode::OnPinRemoved(UEdGraphPin* InRemovedPin)
{
	Super::OnPinRemoved(InRemovedPin);

	if (InRemovedPin && 
			InRemovedPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
			InRemovedPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem)
	{
		RefreshArrayPropertyPinNames();
	}
}

void UObjectTreeGraphNode::ReconstructNode()
{
	Modify(false);

	ErrorMsg.Reset();

	// Save old pins.
	TMap<FName, UEdGraphPin*> OldPins;
	for (auto NodeIt = Pins.CreateIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphPin* Pin(*NodeIt);
		OldPins.Add(Pin->PinName, Pin);
	}

	// Reconstruct all pins from scratch.
	Pins.Reset();
	AllocateDefaultPins();

	// Rewire existing connections to new pins, matched by name, direction, and type.
	TArray<UEdGraphPin*> ErrorPins;
	for (UEdGraphPin* NewPin : Pins)
	{
		UEdGraphPin* OldPin = nullptr;
		OldPins.RemoveAndCopyValue(NewPin->PinName, OldPin);
		if (OldPin)
		{
			const bool bOldMatchesNew = (
				OldPin->Direction == NewPin->Direction &&
				OldPin->PinType == NewPin->PinType);
			if (bOldMatchesNew)
			{
				NewPin->MovePersistentDataFromOldPin(*OldPin);
			}
			else if (OldPin->LinkedTo.Num() > 0)
			{
				ErrorPins.Add(OldPin);
			}
		}
	}

	// Old pins that had connections must be preserved, but made into orphans.
	for (TPair<FName, UEdGraphPin*> Pair : OldPins)
	{
		if (Pair.Value->LinkedTo.Num() > 0)
		{
			ErrorPins.Add(Pair.Value);
		}
	}
	for (UEdGraphPin* ErrorPin : ErrorPins)
	{
		Pins.Add(ErrorPin);
		ErrorPin->bOrphanedPin = true;
	}

	GetGraph()->NotifyNodeChanged(this);

	Super::ReconstructNode();
}

void UObjectTreeGraphNode::GetArrayProperties(TArray<FArrayProperty*>& OutArrayProperties, EEdGraphPinDirection Direction) const
{
	const FNodeContext NodeContext = GetNodeContext();

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == Direction &&
				Pin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
				Pin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty &&
				Pin->ParentPin == nullptr)
		{
			FProperty* Property = NodeContext.ObjectClass->FindPropertyByName(Pin->GetFName());
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				OutArrayProperties.Add(ArrayProperty);
			}
		}
	}
}

void UObjectTreeGraphNode::CreateNewItemPins(FArrayProperty& InArrayProperty, int32 NumExtraPins)
{
	UEdGraphPin** ParentArrayPinPtr = Pins.FindByPredicate([&InArrayProperty](UEdGraphPin* Item)
			{
				return Item->GetFName() == InArrayProperty.GetFName();
			});
	if (ensure(ParentArrayPinPtr))
	{
		CreateNewItemPins(*ParentArrayPinPtr, NumExtraPins);
	}
}

void UObjectTreeGraphNode::CreateNewItemPins(UEdGraphPin* InParentArrayPin, int32 NumExtraPins)
{
	if (!ensure(InParentArrayPin && NumExtraPins >= 0))
	{
		return;
	}

	if (NumExtraPins == 0)
	{
		return;
	}

	const FNodeContext NodeContext = GetNodeContext();

	const FName PropertyName = InParentArrayPin->GetFName();
	const int32 StartIndex = InParentArrayPin->SubPins.Num();

	const int32 ParentPinIndex = Pins.Find(InParentArrayPin);
	ensure(ParentPinIndex >= 0);

	FEdGraphPinType ChildPinType;
	ChildPinType.PinCategory = UObjectTreeGraphSchema::PC_Property;
	ChildPinType.PinSubCategory = UObjectTreeGraphSchema::PSC_ArrayPropertyItem;

	const EEdGraphPinDirection PinDirection = NodeContext.GraphConfig.GetPropertyPinDirection(NodeContext.ObjectClass, PropertyName);

	InParentArrayPin->Modify();

	for (int32 Index = 0; Index < NumExtraPins; ++Index)
	{
		const int32 NewIndex = StartIndex + Index;

		FName ChildPinName = PropertyName;
		ChildPinName.SetNumber(NewIndex);
		UEdGraphPin* ChildPin = CreatePin(PinDirection, ChildPinType, ChildPinName);
		if (NewIndex == 0)
		{
			ChildPin->PinFriendlyName = FText::Format(
					LOCTEXT("ArrayPinFriendlyNameFmt", "{0} {1}"), FText::FromName(PropertyName), NewIndex);
		}
		else
		{
			ChildPin->PinFriendlyName = FText::AsNumber(NewIndex);
		}

		ChildPin->ParentPin = InParentArrayPin;
		InParentArrayPin->SubPins.Add(ChildPin);

		// Always re-insert the child pin so that all child pins are just after
		// the parent array pin.
		const int32 ChildPinIndex = ParentPinIndex + InParentArrayPin->SubPins.Num();
		Pins.Pop(EAllowShrinking::No);
		Pins.Insert(ChildPin, ChildPinIndex);
	}
}

void UObjectTreeGraphNode::InsertNewItemPin(UEdGraphPin* InParentArrayPin, int32 Index)
{
	if (!ensure(InParentArrayPin))
	{
		return;
	}
	if (!ensure(Index >= 0 && Index < InParentArrayPin->SubPins.Num()))
	{
		return;
	}

	const FNodeContext NodeContext = GetNodeContext();

	const FName PropertyName = InParentArrayPin->GetFName();
	const int32 ParentPinIndex = Pins.Find(InParentArrayPin);
	ensure(ParentPinIndex >= 0);

	FEdGraphPinType ChildPinType;
	ChildPinType.PinCategory = UObjectTreeGraphSchema::PC_Property;
	ChildPinType.PinSubCategory = UObjectTreeGraphSchema::PSC_ArrayPropertyItem;

	const EEdGraphPinDirection PinDirection = NodeContext.GraphConfig.GetPropertyPinDirection(NodeContext.ObjectClass, PropertyName);

	InParentArrayPin->Modify();

	FName ChildPinName = PropertyName;
	ChildPinName.SetNumber(Index);
	UEdGraphPin* ChildPin = CreatePin(PinDirection, ChildPinType, ChildPinName);
	if (Index == 0)
	{
		ChildPin->PinFriendlyName = FText::Format(
				LOCTEXT("ArrayPinFriendlyNameFmt", "{0} {1}"), FText::FromName(PropertyName), Index);
	}
	else
	{
		ChildPin->PinFriendlyName = FText::AsNumber(Index);
	}

	ChildPin->ParentPin = InParentArrayPin;
	InParentArrayPin->SubPins.Insert(ChildPin, Index);

	// Always re-insert the child pin so that all child pins are just after
	// the parent array pin.
	const int32 ChildPinIndex = ParentPinIndex + Index + 1;
	Pins.Pop(EAllowShrinking::No);
	Pins.Insert(ChildPin, ChildPinIndex);

	// Rename all subsequent pins so they display the correct index.
	// NOTE: this will actually rename *all* array property pins, which is a bit heavy handed.
	RefreshArrayPropertyPinNames();
}

void UObjectTreeGraphNode::RemoveItemPin(UEdGraphPin* InItemPin)
{
	if (ensure(InItemPin && 
				InItemPin->ParentPin &&
				InItemPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
				InItemPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem))
	{
		InItemPin->ParentPin->Modify();

		// Don't call RemovePin() because that also removes the parent pin.
		// We just want to remove the child pin.
		const int32 NumPinRemoved = Pins.Remove(InItemPin);
		ensure(NumPinRemoved == 1);
		const int32 NumSubPinRemoved = InItemPin->ParentPin->SubPins.Remove(InItemPin);
		ensure(NumSubPinRemoved == 1);

		OnPinRemoved(InItemPin);

		InItemPin->MarkAsGarbage();
	}
}

void UObjectTreeGraphNode::RefreshArrayPropertyPinNames()
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && 
				Pin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
				Pin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty)
		{
			const FName PropertyName = Pin->GetFName();
			for (int32 PinIndex = 0; PinIndex < Pin->SubPins.Num(); ++PinIndex)
			{
				UEdGraphPin* ChildPin = Pin->SubPins[PinIndex];
				ChildPin->PinName.SetNumber(PinIndex);

				if (PinIndex == 0)
				{
					ChildPin->PinFriendlyName = FText::Format(
							LOCTEXT("ArrayPinFriendlyNameFmt", "{0} {1}"), FText::FromName(PropertyName), PinIndex);
				}
				else
				{
					ChildPin->PinFriendlyName = FText::AsNumber(PinIndex);
				}
			}
		}
	}
}

void UObjectTreeGraphNode::GetAllConnectableProperties(TArray<FProperty*>& OutProperties) const
{
	UObject* Object = WeakObject.Get();
	if (!ensure(Object))
	{
		return;
	}

	UClass* ObjectClass = Object->GetClass();
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin 
				&& Pin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property
				&& (Pin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ObjectProperty || Pin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty))
		{
			FProperty* Property = ObjectClass->FindPropertyByName(Pin->GetFName());
			if (ensure(Property))
			{
				OutProperties.Add(Property);
			}
		}
	}
}

UEdGraphPin* UObjectTreeGraphNode::GetSelfPin() const
{
	UEdGraphPin* const* SelfPinPtr = Pins.FindByPredicate(
			[](UEdGraphPin* Item) { return Item->PinType.PinCategory == UObjectTreeGraphSchema::PC_Self; });
	if (SelfPinPtr)
	{
		return *SelfPinPtr;
	}
	return nullptr;
}

void UObjectTreeGraphNode::OverrideSelfPinDirection(EEdGraphPinDirection Direction)
{
	Modify();

	bOverrideSelfPinDirection = true;
	SelfPinDirectionOverride = Direction;

	if (UEdGraphPin* SelfPin = GetSelfPin())
	{
		SelfPin->Direction = Direction;
		GetGraph()->NotifyNodeChanged(this);
	}
}

UEdGraphPin* UObjectTreeGraphNode::GetPinForProperty(FObjectProperty* InProperty) const
{
	UEdGraphPin* const* FoundItem = Pins.FindByPredicate(
			[InProperty](UEdGraphPin* Item)
			{ 
				return Item->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
					Item->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ObjectProperty &&
					Item->GetFName() == InProperty->GetFName(); 
			});
	if (FoundItem)
	{
		return *FoundItem;
	}
	return nullptr;
}

UEdGraphPin* UObjectTreeGraphNode::GetPinForProperty(FArrayProperty* InProperty) const
{
	UEdGraphPin* const* FoundItem = Pins.FindByPredicate(
			[InProperty](UEdGraphPin* Item)
			{ 
				return Item->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
					Item->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty &&
					Item->GetFName() == InProperty->GetFName(); 
			});
	if (FoundItem)
	{
		return *FoundItem;
	}
	return nullptr;
}

UEdGraphPin* UObjectTreeGraphNode::GetPinForProperty(FArrayProperty* InProperty, int32 Index) const
{
	UEdGraphPin* ArrayPin = GetPinForProperty(InProperty);
	if (ArrayPin && ensure(ArrayPin->SubPins.IsValidIndex(Index)))
	{
		return ArrayPin->SubPins[Index];
	}
	return nullptr;
}

FProperty* UObjectTreeGraphNode::GetPropertyForPin(const UEdGraphPin* InPin) const
{
	UObject* Object = WeakObject.Get();
	if (!ensure(Object))
	{
		return nullptr;
	}
	if (InPin->PinType.PinCategory != UObjectTreeGraphSchema::PC_Property)
	{
		return nullptr;
	}

	UClass* ObjectClass = Object->GetClass();

	if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ObjectProperty)
	{
		return ObjectClass->FindPropertyByName(InPin->GetFName());
	}
	else if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty)
	{
		return ObjectClass->FindPropertyByName(InPin->GetFName());
	}
	else if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem)
	{
		UEdGraphPin* ParentArrayPin = InPin->ParentPin;
		check(ParentArrayPin);
		return ObjectClass->FindPropertyByName(ParentArrayPin->GetFName());
	}

	return nullptr;
}

UClass* UObjectTreeGraphNode::GetConnectedObjectClassForPin(const UEdGraphPin* InPin) const
{
	UObject* Object = WeakObject.Get();
	if (!ensure(Object))
	{
		return nullptr;
	}
	if (InPin->PinType.PinCategory != UObjectTreeGraphSchema::PC_Property)
	{
		return nullptr;
	}

	UClass* ObjectClass = Object->GetClass();

	if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ObjectProperty)
	{
		FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(ObjectClass->FindPropertyByName(InPin->GetFName()));
		return ObjectProperty->PropertyClass;
	}
	else if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayProperty)
	{
		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(ObjectClass->FindPropertyByName(InPin->GetFName()));
		FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
		return InnerProperty->PropertyClass;
	}
	else if (InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem)
	{
		UEdGraphPin* ParentArrayPin = InPin->ParentPin;
		check(ParentArrayPin);
		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(ObjectClass->FindPropertyByName(ParentArrayPin->GetFName()));
		FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
		return InnerProperty->PropertyClass;
	}

	return nullptr;
}

int32 UObjectTreeGraphNode::GetIndexOfArrayPin(const UEdGraphPin* InPin) const
{
	if (!ensure(InPin &&
				InPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
				InPin->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem))
	{
		return INDEX_NONE;
	}

	UEdGraphPin* ParentArrayPin = InPin->ParentPin;
	check(ParentArrayPin);
	return ParentArrayPin->SubPins.Find(const_cast<UEdGraphPin*>(InPin));
}

void UObjectTreeGraphNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(GetObject());
	if (GraphObject)
	{
		const FNodeContext NodeContext = GetNodeContext();
		GraphObject->GetGraphNodePosition(NodeContext.GraphConfig.GraphName, NodePosX, NodePosY);
	}
}

void UObjectTreeGraphNode::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	FToolMenuInsert MenuPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);

	const FGraphEditorCommandsImpl& GraphEditorCommands = FGraphEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	// Common actions.
	{
		FToolMenuSection& NodeSection = Menu->AddSection(
				"ObjectTreeGraphNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"), MenuPosition);

		NodeSection.AddMenuEntry(GraphEditorCommands.BreakNodeLinks);
	}

	// General actions.
	{
		FToolMenuSection& Section = Menu->AddSection(
				"ObjectTreeGraphNodeGenericActions", LOCTEXT("GenericActionsMenuHeader", "General"));

		Section.AddMenuEntry(GenericCommands.Delete);
		Section.AddMenuEntry(GenericCommands.Cut);
		Section.AddMenuEntry(GenericCommands.Copy);
		Section.AddMenuEntry(GenericCommands.Duplicate);
	}

	// Graph organization.
	{
		FToolMenuSection& Section = Menu->AddSection(
				"ObjectTreeGraphOrganizationActions", LOCTEXT("OrganizationActionsMenuHeader", "Organization"));

		Section.AddSubMenu(
				"Alignment",
				LOCTEXT("AlignmentHeader", "Alignment"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([&GraphEditorCommands](UToolMenu* InMenu)
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection(
								"ObjectTreeGraphAlignmentActions", LOCTEXT("AlignmentHeader", "Alignment"));
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesTop);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesBottom);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesLeft);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesCenter);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.AlignNodesRight);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.StraightenConnections);
					}));

		Section.AddSubMenu(
				"Distribution",
				LOCTEXT("DistributionHeader", "Distribution"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([&GraphEditorCommands](UToolMenu* InMenu)
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection(
								"ObjectTreeGraphDistributionActions", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(GraphEditorCommands.DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(GraphEditorCommands.DistributeNodesVertically);
					}));
	}
}

bool UObjectTreeGraphNode::GetCanRenameNode() const
{
	UObject* Object = WeakObject.Get();
	const FNodeContext NodeContext = GetNodeContext();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object);
	return GraphObject && GraphObject->HasAnySupportFlags(NodeContext.GraphConfig.GraphName, EObjectTreeGraphObjectSupportFlags::CustomRename);
}

void UObjectTreeGraphNode::OnRenameNode(const FString& NewName)
{
	Super::OnRenameNode(NewName);

	UObject* Object = WeakObject.Get();
	if (IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object))
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));

		const FNodeContext NodeContext = GetNodeContext();
		GraphObject->OnRenameGraphNode(NodeContext.GraphConfig.GraphName, NewName);
	}
}

bool UObjectTreeGraphNode::CanDuplicateNode() const
{
	const FObjectTreeGraphClassConfigs ObjectClassConfigs = GetObjectClassConfigs();
	if (!ObjectClassConfigs.CanCreateNew())  // If it can't be created, it shouldn't be worked around by copy/pasting
	{
		return false;
	}

	return Super::CanUserDeleteNode();
}

bool UObjectTreeGraphNode::CanUserDeleteNode() const
{
	const FObjectTreeGraphClassConfigs ObjectClassConfigs = GetObjectClassConfigs();
	if (!ObjectClassConfigs.CanDelete())
	{
		return false;
	}

	return Super::CanUserDeleteNode();
}

bool UObjectTreeGraphNode::SupportsCommentBubble() const
{
	UObject* Object = WeakObject.Get();
	const FNodeContext NodeContext = GetNodeContext();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object);
	return GraphObject && GraphObject->HasAnySupportFlags(NodeContext.GraphConfig.GraphName, EObjectTreeGraphObjectSupportFlags::CommentText);
}

void UObjectTreeGraphNode::OnUpdateCommentText(const FString& NewComment)
{
	Super::OnUpdateCommentText(NewComment);

	UObject* Object = WeakObject.Get();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object);
	if (GraphObject)
	{
		const FScopedTransaction Transaction(LOCTEXT("UpdateNodeComment", "Update Node Comment"));

		const FNodeContext NodeContext = GetNodeContext();
		GraphObject->OnUpdateGraphNodeCommentText(NodeContext.GraphConfig.GraphName, NewComment);
	}
}

void UObjectTreeGraphNode::OnGraphNodeMoved(bool bMarkDirty)
{
	UObject* Object = WeakObject.Get();
	IObjectTreeGraphObject* GraphObject = Cast<IObjectTreeGraphObject>(Object);
	if (GraphObject)
	{
		const FNodeContext NodeContext = GetNodeContext();
		GraphObject->OnGraphNodeMoved(NodeContext.GraphConfig.GraphName, NodePosX, NodePosY, bMarkDirty);
	}
}

UObjectTreeGraphNode::FNodeContext UObjectTreeGraphNode::GetNodeContext() const
{
	UObjectTreeGraph* OuterGraph = CastChecked<UObjectTreeGraph>(GetGraph());
	const FObjectTreeGraphConfig& OuterGraphConfig = OuterGraph->GetConfig();

	if (UObject* Object = WeakObject.Get())
	{
		UClass* ObjectClass = Object->GetClass();
		const FObjectTreeGraphClassConfigs ObjectClassConfigs = OuterGraphConfig.GetObjectClassConfigs(ObjectClass);

		return FNodeContext{ ObjectClass, OuterGraph, OuterGraphConfig, ObjectClassConfigs };
	}
	else
	{
		const FObjectTreeGraphClassConfigs ObjectClassConfigs = OuterGraphConfig.GetObjectClassConfigs(nullptr);
		return FNodeContext{ nullptr, OuterGraph, OuterGraphConfig, ObjectClassConfigs };
	}
}

const FObjectTreeGraphClassConfigs UObjectTreeGraphNode::GetObjectClassConfigs() const
{
	return GetNodeContext().ObjectClassConfigs;
}

#undef LOCTEXT_NAMESPACE

