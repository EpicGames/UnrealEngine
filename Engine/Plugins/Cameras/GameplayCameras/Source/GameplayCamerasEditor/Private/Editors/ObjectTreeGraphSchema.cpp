// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraphSchema.h"

#include "Commands/ObjectTreeGraphEditorCommands.h"
#include "Core/ObjectTreeGraphComment.h"
#include "Core/ObjectTreeGraphRootObject.h"
#include "Editors/ObjectTreeConnectionDrawingPolicy.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphCommentNode.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/SObjectTreeGraphEditor.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "IGameplayCamerasEditorModule.h"
#include "ScopedTransaction.h"
#include "Serialization/ArchiveUObject.h"
#include "ToolMenu.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "UnrealExporter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTreeGraphSchema)

#define LOCTEXT_NAMESPACE "ObjectTreeGraphSchema"

const FName UObjectTreeGraphSchema::PC_Self("Self");
const FName UObjectTreeGraphSchema::PC_Property("Property");

const FName UObjectTreeGraphSchema::PSC_ObjectProperty("ObjectProperty");
const FName UObjectTreeGraphSchema::PSC_ArrayProperty("ArrayProperty");
const FName UObjectTreeGraphSchema::PSC_ArrayPropertyItem("ArrayPropertyItem");

namespace UE::ObjectTreeGraph
{

struct FPackageReferenceCollector : public FArchiveUObject
{
	FPackageReferenceCollector(UObject* InRootObject, TArray<UObject*>& InOutReferencedObjects)
		: FArchiveUObject()
		, RootObject(InRootObject)
		, PackageScope(InRootObject->GetOutermost())
		, ReferencedObjects(InOutReferencedObjects)
	{
		SetIsPersistent(true);
		SetIsSaving(true);
		SetFilterEditorOnly(false);

		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
	}

	void StopAtObjectClasses(TArray<UClass*> InStopAtClasses)
	{
		StopAtClasses = TSet<UClass*>(InStopAtClasses);
	}

	void CollectReferences()
	{
		ObjectsToVisit.Reset();
		VisitedObjects.Reset();

		ObjectsToVisit.Add(RootObject);
		while (ObjectsToVisit.Num() > 0)
		{
			UObject* CurObj = ObjectsToVisit.Pop(EAllowShrinking::No);
			VisitedObjects.Add(CurObj);
			CurObj->Serialize(*this);
		}
	}

private:

	bool ShouldStopAt(UObject* Obj)
	{
		UClass* ObjClass = Obj->GetClass();
		for (UClass* StopAtClass : StopAtClasses)
		{
			if (ObjClass->IsChildOf(StopAtClass))
			{
				return true;
			}
		}
		return false;
	}

	virtual FArchive& operator<<(UObject*& ObjRef) override
	{
		if (ObjRef != nullptr && ObjRef->IsIn(PackageScope) && !ShouldStopAt(ObjRef))
		{
			if (!VisitedObjects.Contains(ObjRef))
			{
				ReferencedObjects.Add(ObjRef);
				ObjectsToVisit.Add(ObjRef);
			}
		}
		return *this;
	}

private:

	UObject* RootObject;
	UPackage* PackageScope;
	TSet<UClass*> StopAtClasses;

	TArray<UObject*> ObjectsToVisit;
	TSet<UObject*> VisitedObjects;

	TArray<UObject*>& ReferencedObjects;
};

class FObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory interface.
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		return true;
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);
		CreatedObjects.Add(NewObject);
	}

	TArray<UObject*> CreatedObjects;
};

}  // namespace UE::ObjectTreeGraph

UObjectTreeGraphSchema::UObjectTreeGraphSchema(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UObjectTreeGraphSchema::RebuildGraph(UObjectTreeGraph* InGraph) const
{
	RemoveAllNodes(InGraph);
	CreateAllNodes(InGraph);
	InGraph->NotifyGraphChanged();
}

void UObjectTreeGraphSchema::RemoveAllNodes(UObjectTreeGraph* InGraph) const
{
	TArray<UEdGraphNode*> NodesToRemove(InGraph->Nodes);  // Copy all nodes to remove them.
	for (UEdGraphNode* NodeToRemove : NodesToRemove)
	{
		InGraph->RemoveNode(NodeToRemove);
	}
}

void UObjectTreeGraphSchema::CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const
{
	// By default, collect all objects referenced directly or indirectly by the root object, within
	// the same package, unless the root object implements the IObjectTreeGraphRootObject interface,
	// in which case get the list of objects from it.
	// Override this method to collect objects differently.
	const bool bHasRootInterface = CollectAllConnectableObjectsFromRootInterface(InGraph, OutAllObjects, true);
	if (!bHasRootInterface)
	{
		CollectAllReferencedObjects(InGraph, OutAllObjects);
	}
}

void UObjectTreeGraphSchema::CollectAllReferencedObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects)
{
	using namespace UE::ObjectTreeGraph;

	UObject* RootObject = InGraph->GetRootObject();
	if (!RootObject)
	{
		return;
	}

	// Make sure the root object itself is in there.
	OutAllObjects.Add(RootObject);

	// Use a reference collector that doesn't go outside of the root object package.
	TArray<UObject*> ReferencedObjects;
	FPackageReferenceCollector Collector(RootObject, ReferencedObjects);
	Collector.CollectReferences();
	OutAllObjects.Append(ReferencedObjects);
}

bool UObjectTreeGraphSchema::CollectAllConnectableObjectsFromRootInterface(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects, bool bAllowNoRootInterface)
{
	UObject* RootObject = InGraph->GetRootObject();
	if (!RootObject)
	{
		return true;
	}

	// Make sure the root object itself is in there.
	OutAllObjects.Add(RootObject);

	// Get all the objects we need from the dedicated interface for this.
	IObjectTreeGraphRootObject* RootObjectInterface = Cast<IObjectTreeGraphRootObject>(RootObject);
	ensureMsgf(RootObjectInterface || bAllowNoRootInterface,
			TEXT("Root object '%s' was expected to implement IObjectTreeGraphRootObject, but doesn't."),
			*GetNameSafe(RootObject));
	if (RootObjectInterface)
	{
		const FObjectTreeGraphConfig& GraphConfig = InGraph->GetConfig();
		RootObjectInterface->GetConnectableObjects(GraphConfig.GraphName, OutAllObjects);
		return true;
	}
	return false;
}

void UObjectTreeGraphSchema::CreateAllNodes(UObjectTreeGraph* InGraph) const
{
	// Collect the objects.
	TSet<UObject*> AllObjects;
	CollectAllObjects(InGraph, AllObjects);
	
	// Create all the nodes.
	FCreatedNodes CreatedNodes;
	for (UObject* Object : AllObjects)
	{
		if (UEdGraphNode* GraphNode = CreateObjectNode(InGraph, Object))
		{
			CreatedNodes.CreatedNodes.Add(Object, GraphNode);
		}
	}

	// Grab the graph node for the root object.
	InGraph->RootObjectNode = nullptr;
	if (!AllObjects.IsEmpty())
	{
		UObject* RootObject = InGraph->GetRootObject();
		UEdGraphNode* CreatedRootObjectNode = CreatedNodes.CreatedNodes.FindRef(RootObject);
		if (ensureMsgf(CreatedRootObjectNode, 
					TEXT("Can't find root object '%s' in the list of created graph nodes!"),
					*GetNameSafe(RootObject)))
		{
			InGraph->RootObjectNode = CastChecked<UObjectTreeGraphNode>(CreatedRootObjectNode);
		}
	}

	// Create all the connections.
	for (TPair<UObject*, UEdGraphNode*> Pair : CreatedNodes.CreatedNodes)
	{
		if (UObjectTreeGraphNode* Node = Cast<UObjectTreeGraphNode>(Pair.Value))
		{
			CreateConnections(Node, CreatedNodes);
		}
	}

	OnCreateAllNodes(InGraph, CreatedNodes);
}

void UObjectTreeGraphSchema::CreateConnections(UObjectTreeGraphNode* InGraphNode, const FCreatedNodes& InCreatedNodes) const
{
	UObject* Object = InGraphNode->GetObject();
	UClass* ObjectClass = Object->GetClass();

	TArray<FProperty*> ConnectableProperties;
	InGraphNode->GetAllConnectableProperties(ConnectableProperties);

	for (FProperty* ConnectableProperty : ConnectableProperties)
	{
		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ConnectableProperty))
		{
			// Object reference property... if the property value is not null, find the node that corresponds
			// to the referenced object. If we find it, create a graph connection between the two.
			UEdGraphPin* Pin = InGraphNode->GetPinForProperty(ObjectProperty);
			if (!ensure(Pin))
			{
				continue;
			}

			TObjectPtr<UObject> OutConnectedObject;
			ObjectProperty->GetValue_InContainer(Object, &OutConnectedObject);
			if (!OutConnectedObject)
			{
				continue;
			}

			UObjectTreeGraphNode* ConnectedNode = Cast<UObjectTreeGraphNode>(InCreatedNodes.CreatedNodes.FindRef(OutConnectedObject));
			if (ensure(ConnectedNode))
			{
				if (Pin->Direction == EGPD_Input)
				{
					ConnectedNode->OverrideSelfPinDirection(EGPD_Output);
				}
				UEdGraphPin* ConnectedPin = ConnectedNode->GetSelfPin();
				Pin->MakeLinkTo(ConnectedPin);
			}
		}
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ConnectableProperty))
		{
			// Array of object references... add all the pins needed for the array's size, and connect each
			// of those pins to a matching object node, similarly to above with object references.
			FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));

			const int32 ArrayNum = ArrayHelper.Num();
			for (int32 Index = 0; Index < ArrayNum; ++Index)
			{
				UEdGraphPin* Pin = InGraphNode->GetPinForProperty(ArrayProperty, Index);
				if (!ensure(Pin))
				{
					continue;
				}

				UObject* ConnectedObject = InnerProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
				if (!ConnectedObject)
				{
					continue;
				}

				UObjectTreeGraphNode* ConnectedNode = Cast<UObjectTreeGraphNode>(InCreatedNodes.CreatedNodes.FindRef(ConnectedObject));
				if (ensure(ConnectedNode))
				{
					if (Pin->Direction == EGPD_Input)
					{
						ConnectedNode->OverrideSelfPinDirection(EGPD_Output);
					}
					UEdGraphPin* ConnectedPin = ConnectedNode->GetSelfPin();
					Pin->MakeLinkTo(ConnectedPin);
				}
			}
		}
	}
}

void UObjectTreeGraphSchema::OnCreateAllNodes(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const
{
}

UEdGraphNode* UObjectTreeGraphSchema::CreateObjectNode(UObjectTreeGraph* InGraph, UObject* InObject) const
{
	if (!InObject)
	{
		return nullptr;
	}
	
	if (UObjectTreeGraphComment* Comment = Cast<UObjectTreeGraphComment>(InObject))
	{
		return CreateCommentNode(InGraph, Comment);
	}
	else if (InGraph->GetConfig().IsConnectable(InObject->GetClass()))
	{
		return OnCreateObjectNode(InGraph, InObject);
	}

	return nullptr;
}

UEdGraphNode* UObjectTreeGraphSchema::CreateCommentNode(UObjectTreeGraph* InGraph, UObjectTreeGraphComment* InComment) const
{
	InGraph->Modify();

	FGraphNodeCreator<UObjectTreeGraphCommentNode> GraphNodeCreator(*InGraph);
	UObjectTreeGraphCommentNode* NewNode = GraphNodeCreator.CreateNode(false);
	NewNode->Initialize(InComment);
	GraphNodeCreator.Finalize();
	return NewNode;
}

UEdGraphNode* UObjectTreeGraphSchema::OnCreateObjectNode(UObjectTreeGraph* InGraph, UObject* InObject) const
{
	const FObjectTreeGraphConfig& Config = InGraph->GetConfig();
	const FObjectTreeGraphClassConfigs ClassConfigs = Config.GetObjectClassConfigs(InObject->GetClass());

	TSubclassOf<UObjectTreeGraphNode> GraphNodeClass = ClassConfigs.GraphNodeClass();
	if (!GraphNodeClass.Get())
	{
		GraphNodeClass = Config.DefaultGraphNodeClass;
	}

	InGraph->Modify();

	FGraphNodeCreator<UObjectTreeGraphNode> GraphNodeCreator(*InGraph);
	UObjectTreeGraphNode* NewNode = GraphNodeCreator.CreateNode(false, GraphNodeClass);
	NewNode->Initialize(InObject);
	GraphNodeCreator.Finalize();
	return NewNode;
}

void UObjectTreeGraphSchema::AddConnectableObject(UObjectTreeGraph* InGraph, UObject* InNewObject) const
{
	if (!ensure(InNewObject))
	{
		return;
	}
	if (!ensure(!InNewObject->IsA<UEdGraphNode>()))
	{
		return;
	}

	UObjectTreeGraphNode* RootObjectNode = InGraph->GetRootObjectNode();
	IObjectTreeGraphRootObject* RootObjectInterface = Cast<IObjectTreeGraphRootObject>(RootObjectNode->GetObject());
	if (RootObjectInterface)
	{
		const FObjectTreeGraphConfig& GraphConfig = InGraph->GetConfig();
		const FName GraphName = GraphConfig.GraphName;
		RootObjectInterface->AddConnectableObject(GraphName, InNewObject);
	}

	OnAddConnectableObject(InGraph, InNewObject);
}

void UObjectTreeGraphSchema::OnAddConnectableObject(UObjectTreeGraph* InGraph, UObject* InNewObject) const
{
}

void UObjectTreeGraphSchema::RemoveConnectableObject(UObjectTreeGraph* InGraph, UObject* InRemovedObject) const
{
	if (!ensure(InRemovedObject))
	{
		return;
	}
	if (!ensure(!InRemovedObject->IsA<UEdGraphNode>()))
	{
		return;
	}

	IObjectTreeGraphRootObject* RootObjectInterface = Cast<IObjectTreeGraphRootObject>(InGraph->GetRootObject());
	if (RootObjectInterface)
	{
		const FObjectTreeGraphConfig& GraphConfig = InGraph->GetConfig();
		const FName GraphName = GraphConfig.GraphName;
		RootObjectInterface->RemoveConnectableObject(GraphName, InRemovedObject);
	}

	OnRemoveConnectableObject(InGraph, InRemovedObject);
}

void UObjectTreeGraphSchema::OnRemoveConnectableObject(UObjectTreeGraph* InGraph, UObject* InRemovedObject) const
{
}

void UObjectTreeGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const UObjectTreeGraph* Graph = CastChecked<UObjectTreeGraph>(ContextMenuBuilder.CurrentGraph);
	const FObjectTreeGraphConfig& GraphConfig = Graph->GetConfig();

	// Find the common class restriction for all the dragged pins. We will only show actions that
	// are compatible with them.
	UClass* DraggedPinClass = nullptr;
	bool bShouldShowNewObjectActions = true;
	if (const UEdGraphPin* DraggedPin = ContextMenuBuilder.FromPin)
	{
		UObjectTreeGraphNode* OwningNode = Cast<UObjectTreeGraphNode>(DraggedPin->GetOwningNode());
		if (OwningNode)
		{
			if (DraggedPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property)
			{
				DraggedPinClass = OwningNode->GetConnectedObjectClassForPin(DraggedPin);
			}
			else if (DraggedPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Self)
			{
				// Dragged the self pin, so in theory we should offer to create nodes that have
				// output pins (properties) that are compatible with our own class. That would
				// require inspecting all properties on connectable classes, and hopefully
				// caching it all somewhere... instead for now we just encourage the user to
				// connect things from the left to the right and not go up-stream.
				bShouldShowNewObjectActions = false;
			}
			else
			{
				// Dragged an unknown pin...
				bShouldShowNewObjectActions = false;
			}
		}
		else
		{
			// Dragged a pin from an unknown node...
				bShouldShowNewObjectActions = false;
		}
	}
	if (!bShouldShowNewObjectActions)
	{
		// Don't show anything.
		return;
	}

	// Find all the object classes we can create from those pins, for the given graph.
	TArray<UClass*> PossibleObjectClasses;
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		if (ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}
		if (ClassIt->HasAnyClassFlags(CLASS_Hidden | CLASS_NotPlaceable))
		{
			continue;
		}

		if (!GraphConfig.IsConnectable(*ClassIt))
		{
			continue;
		}

		const FObjectTreeGraphClassConfigs ClassConfigs = GraphConfig.GetObjectClassConfigs(*ClassIt);
		if (!ClassConfigs.CanCreateNew())
		{
			continue;
		}

		if (DraggedPinClass && !ClassIt->IsChildOf(DraggedPinClass))
		{
			continue;
		}

		PossibleObjectClasses.Add(*ClassIt);
	}

	FilterGraphContextPlaceableClasses(PossibleObjectClasses);

	const FText MiscellaneousCategoryText = LOCTEXT("MiscellaneousCategory", "Miscellaneous");

	for (UClass* PossibleObjectClass : PossibleObjectClasses)
	{
		if (!PossibleObjectClass)
		{
			continue;
		}

		const FText DisplayName = GraphConfig.GetDisplayNameText(PossibleObjectClass);

		TArray<FString> CategoryNames;
		const FName CreateCategoryMetaData = GraphConfig.GetObjectClassConfigs(PossibleObjectClass).CreateCategoryMetaData();
		for (UClass* CurClass = PossibleObjectClass; CurClass; CurClass = CurClass->GetSuperClass())
		{
			const FString* CategoryNamesMetaData = CurClass->FindMetaData(CreateCategoryMetaData);
			if (CategoryNamesMetaData)
			{
				CategoryNamesMetaData->ParseIntoArray(CategoryNames, TEXT(","), true);
				break;
			}
		}
		if (CategoryNames.IsEmpty())
		{
			CategoryNames.Add(FString());
		}

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Name"), DisplayName);
		const FText ToolTipText = FText::Format(LOCTEXT("NewNodeToolTip", "Adds a {Name} node here"), Arguments);

		checkSlow(PossibleObjectClass);
		for (const FString& CategoryName : CategoryNames)
		{
			FText CategoryText = MiscellaneousCategoryText;
			int32 Grouping = -1;
			if (!CategoryName.IsEmpty())
			{
				CategoryText = FText::FromString(CategoryName);
				Grouping = CategoryName == TEXT("Common") ? 1 : 0;
			}

			FText KeywordsText(FText::FromString(PossibleObjectClass->GetMetaData(TEXT("Keywords"))));

			TSharedRef<FObjectTreeGraphSchemaAction_NewNode> Action = MakeShared<FObjectTreeGraphSchemaAction_NewNode>(
					CategoryText, DisplayName, ToolTipText, Grouping, KeywordsText);
			Action->ObjectClass = PossibleObjectClass;
			ContextMenuBuilder.AddAction(StaticCastSharedPtr<FEdGraphSchemaAction>(Action.ToSharedPtr()));
		}
	}

	GetCommentAction(ContextMenuBuilder);

	// Don't call the base class, we want to control exactly what can be created.
}

void UObjectTreeGraphSchema::FilterGraphContextPlaceableClasses(TArray<UClass*>& InOutClasses) const
{
}

void UObjectTreeGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	using namespace UE::Cameras;

	Super::GetContextMenuActions(Menu, Context);

	const UEdGraph* CurrentGraph = Context->Graph;
	const UEdGraphNode* CurrentNode = Context->Node;
	const UEdGraphPin* CurrentPin = Context->Pin;

	const FObjectTreeGraphEditorCommands& Commands = FObjectTreeGraphEditorCommands::Get();

	if (CurrentPin)
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("ObjectTreeGraphSchemaPinActions");
		Section.InitSection(
				"ObjectTreeGraphSchemaPinActions", 
				LOCTEXT("ObjectPinActionsMenuHeader", "Object Pin Actions"), 
				FToolMenuInsert());

		if (CurrentPin->PinType.PinCategory == PC_Property &&
				CurrentPin->PinType.PinSubCategory == PSC_ArrayPropertyItem)
		{
			Section.AddMenuEntry(Commands.InsertArrayItemPinBefore);
			Section.AddMenuEntry(Commands.InsertArrayItemPinAfter);
			Section.AddMenuEntry(Commands.RemoveArrayItemPin);
		}
	}
}

TSharedPtr<FEdGraphSchemaAction> UObjectTreeGraphSchema::GetCreateCommentAction() const
{
	return MakeShared<FObjectTreeGraphSchemaAction_NewComment>();
}

void UObjectTreeGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		const FText MenuDesc= LOCTEXT("CommentActionDescription", "Add Comment...");
		const FText MenuToolTip = LOCTEXT("CommentToolTip", "Creates a comment.");
		TSharedPtr<FObjectTreeGraphSchemaAction_NewComment> NewAction(MakeShared<FObjectTreeGraphSchemaAction_NewComment>(FText::GetEmpty(), MenuDesc, MenuToolTip));
		ActionMenuBuilder.AddAction(NewAction);
	}
}

FName UObjectTreeGraphSchema::GetParentContextMenuName() const
{
	// Return NAME_None if we don't want the default menu entries.
	return Super::GetParentContextMenuName();
}

FLinearColor UObjectTreeGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

FConnectionDrawingPolicy* UObjectTreeGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, UEdGraph* InGraph) const
{
	return new FObjectTreeConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

bool UObjectTreeGraphSchema::ShouldAlwaysPurgeOnModification() const
{
	return false;
}

FPinConnectionResponse UObjectTreeGraphSchema::CanCreateNewNodes(UEdGraphPin* InSourcePin) const
{
	return Super::CanCreateNewNodes(InSourcePin);
}

const FPinConnectionResponse UObjectTreeGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	UObjectTreeGraphNode* NodeA = Cast<UObjectTreeGraphNode>(A->GetOwningNode());
	UObjectTreeGraphNode* NodeB = Cast<UObjectTreeGraphNode>(B->GetOwningNode());
	if (!NodeA || !NodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Unsupported node types"));
	}
	
	if (A->bOrphanedPin || B->bOrphanedPin)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Can't connect an orphaned pin"));
	}

	if (A->Direction == B->Direction)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Incompatible pins"));
	}
	
	// Try to always reason back to A being the property pin, and B being the self pin of the
	// object we want to set on the property.
	if (A->PinType.PinCategory == PC_Self)
	{
		Swap(A, B);
		Swap(NodeA, NodeB);
	}

	const bool bIsPropertyToSelf = (A->PinType.PinCategory == PC_Property && B->PinType.PinCategory == PC_Self);
	if (!bIsPropertyToSelf)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Connection must be between a property pin and a self pin"));
	}

	UObject* ObjectA = NodeA->GetObject();
	UObject* ObjectB = NodeB->GetObject();
	UClass* ObjectClassB = ObjectB->GetClass();

	FProperty* PropertyA = NodeA->GetPropertyForPin(A);
	if (!PropertyA)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Unsupported source pin"));
	}

	if (!ObjectA->CanEditChange(PropertyA))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Property cannot be changed"));
	}

	if (FObjectProperty* ObjectPropertyA = CastField<FObjectProperty>(PropertyA))
	{
		if (ObjectClassB->IsChildOf(ObjectPropertyA->PropertyClass))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, TEXT("Compatible pin types"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Incompatible pin types"));
		}
	}
	else if (FArrayProperty* ArrayPropertyA = CastField<FArrayProperty>(PropertyA))
	{
		FObjectProperty* InnerPropertyA = CastFieldChecked<FObjectProperty>(ArrayPropertyA->Inner);
		if (ObjectClassB->IsChildOf(InnerPropertyA->PropertyClass))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, TEXT("Compatible array pin types"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Incompatible array pin types"));
		}
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Unsupported source pin type"));
	}
}

bool UObjectTreeGraphSchema::TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	FScopedTransaction Transaction(LOCTEXT("CreateConnection", "Create Connection"));

	const bool bModified = Super::TryCreateConnection(A, B);

	if (!bModified)
	{
		Transaction.Cancel();
		return false;
	}

	if (OnTryCreateCustomConnection(A, B))
	{
		return true;
	}

	UObjectTreeGraphNode* NodeA = Cast<UObjectTreeGraphNode>(A->GetOwningNode());
	UObjectTreeGraphNode* NodeB = Cast<UObjectTreeGraphNode>(B->GetOwningNode());
	if (NodeA && NodeA->GetObject() && NodeB && NodeB->GetObject())
	{
		ApplyConnection(A, B);
	}

	return true;
}

bool UObjectTreeGraphSchema::OnTryCreateCustomConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	return false;
}

void UObjectTreeGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakNodeLinks", "Break Node Links"));

	TArray<UEdGraphPin*> CachedPins = TargetNode.Pins;

#if WITH_EDITOR
	TSet<UEdGraphNode*> NodeList;
	NodeList.Add(&TargetNode);
#endif
	
	for (UEdGraphPin* TargetPin : CachedPins)
	{
		if (TargetPin && TargetPin->SubPins.Num() == 0)
		{
#if WITH_EDITOR
			for (UEdGraphPin* OtherPin : TargetPin->LinkedTo)
			{
				UEdGraphNode* OtherNode = OtherPin ? OtherPin->GetOwningNode() : nullptr;
				if (OtherNode)
				{
					OtherNode->PinConnectionListChanged(OtherPin);
					NodeList.Add(OtherNode);
				}
			}
#endif

			BreakPinLinks(*TargetPin, false);
		}
	}
	
#if WITH_EDITOR
	for (UEdGraphNode* Node : NodeList)
	{
		Node->NodeConnectionListChanged();
	}
#endif
}

void UObjectTreeGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	FScopedTransaction Transaction(LOCTEXT("BreakPinLinks", "Break Pin Links"));

	if (!OnBreakCustomPinLinks(TargetPin))
	{
		UObjectTreeGraphNode* TargetNode = Cast<UObjectTreeGraphNode>(TargetPin.GetOwningNode());
		if (TargetNode && TargetNode->GetObject())
		{
			ApplyDisconnection(&TargetPin);
		}
	}

	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);
}

bool UObjectTreeGraphSchema::OnBreakCustomPinLinks(UEdGraphPin& TargetPin) const
{
	return false;
}

void UObjectTreeGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	FScopedTransaction Transaction(LOCTEXT("BreakSinglePinLink", "Break Pin Link"));

	if (!OnBreakSingleCustomPinLink(SourcePin, TargetPin))
	{
		UObjectTreeGraphNode* SourceNode = Cast<UObjectTreeGraphNode>(SourcePin->GetOwningNode());
		UObjectTreeGraphNode* TargetNode = Cast<UObjectTreeGraphNode>(TargetPin->GetOwningNode());
		if (SourceNode && SourceNode->GetObject() && TargetNode && TargetNode->GetObject())
		{
			ApplyDisconnection(SourcePin, TargetPin);
		}
	}

	Super::BreakSinglePinLink(SourcePin, TargetPin);
}

bool UObjectTreeGraphSchema::OnBreakSingleCustomPinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	return false;
}

void UObjectTreeGraphSchema::ApplyConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	// Input must have been validated prior to calling this method:
	//
	// - no null objects
	// - pins belong to ObjectTreeGraph nodes
	// - these nodes have valid objects
	// - we should have a transaction active
	//
#if WITH_EDITOR
	ensureMsgf(GUndo || !GEditor, TEXT("Setting property values on objects should be called inside a transaction"));
#endif
	
	check(A && B);

	// See if a sub-class is handling this situation.
	if (OnApplyConnection(A, B))
	{
		return;
	}

	// We handle situations where a property pin or array property pin is connected to the "self" pin
	// of an object node. Lets see which pin is which.
	UEdGraphPin* PropertyPin = nullptr;
	UEdGraphPin* ValuePin = nullptr;

	if (A->PinType.PinCategory == PC_Self && B->PinType.PinCategory == PC_Property)
	{
		PropertyPin = B;
		ValuePin = A;
	}
	else if (A->PinType.PinCategory == PC_Property && B->PinType.PinCategory == PC_Self)
	{
		PropertyPin = A;
		ValuePin = B;
	}

	if (!ensureMsgf(PropertyPin && ValuePin, TEXT("Invalid pins passed for setting property values.")))
	{
		return;
	}

	UObjectTreeGraphNode* PropertyNode = CastChecked<UObjectTreeGraphNode>(PropertyPin->GetOwningNode());
	UObjectTreeGraphNode* ValueNode = CastChecked<UObjectTreeGraphNode>(ValuePin->GetOwningNode());

	UObject* PropertyObject = PropertyNode->GetObject();
	UObject* ValueObject = ValueNode->GetObject();
	check(PropertyObject && ValueObject);

	// If it is a property pin, set the value of the underlying property.
	// If it is an array property pin, set the array item value at the pin's index.
	FProperty* Property = PropertyNode->GetPropertyForPin(PropertyPin);

	if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		PropertyObject->PreEditChange(Property);

		PropertyObject->Modify();

		ObjectProperty->SetValue_InContainer(PropertyObject, TObjectPtr<UObject>(ValueObject));

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		PropertyObject->PostEditChangeProperty(PropertyChangedEvent);
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		PropertyObject->PreEditChange(Property);

		PropertyObject->Modify();

		const int32 Index = PropertyNode->GetIndexOfArrayPin(PropertyPin);
		ensure(Index != INDEX_NONE);

		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(PropertyObject));
		ensure(Index < ArrayHelper.Num());

		FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
		InnerProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), ValueObject);

		FPropertyChangedEvent PropertyChangedEvent(Property);
		PropertyChangedEvent.ChangeType = EPropertyChangeType::ValueSet;
		PropertyObject->PostEditChangeProperty(PropertyChangedEvent);
	}
}

bool UObjectTreeGraphSchema::OnApplyConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	return false;
}

void UObjectTreeGraphSchema::ApplyDisconnection(UEdGraphPin* TargetPin) const
{
	// Input must have been validated prior to calling this method:
	//
	// - no null objects
	// - the pin is the property pin to reset, or the self pin connected to a property pin
	// - the pin belongs to an ObjectTreeGraph node
	// - this node has a valid object
	// - we should have a transaction active
	//
#if WITH_EDITOR
	ensureMsgf(GUndo || !GEditor, TEXT("Resetting property values on objects should be called inside a transaction"));
#endif

	check(TargetPin);

	// See if we actually have anything to disconnect.
	if (TargetPin->LinkedTo.IsEmpty())
	{
		return;
	}

	// See if a sub-class is handling this situation.
	if (OnApplyDisconnection(TargetPin))
	{
		return;
	}

	// We may either disconnect a self pin, or a property or array property pin. Let's see
	// what sort of pin we were given: we want the property side of things.
	if (TargetPin->PinType.PinCategory == PC_Self)
	{
		TargetPin = TargetPin->LinkedTo[0];
	}
	if (!ensure(TargetPin->PinType.PinCategory == PC_Property))
	{
		return;
	}

	UObjectTreeGraphNode* PropertyNode = Cast<UObjectTreeGraphNode>(TargetPin->GetOwningNode());
	check(PropertyNode);

	UObject* PropertyObject = PropertyNode->GetObject();
	check(PropertyObject);

	// If it is a property pin, clear the value of the underlying property.
	// If it is an array property pin, clear the value at the given index in the underlying array.
	FProperty* Property = PropertyNode->GetPropertyForPin(TargetPin);

	if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		PropertyObject->PreEditChange(Property);

		PropertyObject->Modify();

		ObjectProperty->ClearValue_InContainer(PropertyObject);

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		PropertyObject->PostEditChangeProperty(PropertyChangedEvent);
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		PropertyObject->PreEditChange(Property);

		PropertyObject->Modify();

		const int32 Index = PropertyNode->GetIndexOfArrayPin(TargetPin);
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(PropertyObject));
		ensure(Index >= 0 && Index < ArrayHelper.Num());

		FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
		InnerProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), nullptr);

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		PropertyObject->PostEditChangeProperty(PropertyChangedEvent);
	}
}

bool UObjectTreeGraphSchema::OnApplyDisconnection(UEdGraphPin* TargetPin) const
{
	return false;
}

void UObjectTreeGraphSchema::ApplyDisconnection(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	// Input must have been validated prior to calling this method:
	//
	// - no null objects
	// - the pins are the property and self pin of a specific link
	//
	check(SourcePin && TargetPin);

	// See if a sub-class is handling this situation.
	if (OnApplyDisconnection(SourcePin, TargetPin))
	{
		return;
	}

	if (SourcePin->PinType.PinCategory == PC_Self && TargetPin->PinType.PinCategory == PC_Property)
	{
		return ApplyDisconnection(TargetPin);
	}
	else if (SourcePin->PinType.PinCategory == PC_Property && TargetPin->PinType.PinCategory == PC_Self)
	{
		return ApplyDisconnection(SourcePin);
	}
	else
	{
		checkf(false, TEXT("Invalid pins passed for setting property values."));
	}
}

bool UObjectTreeGraphSchema::OnApplyDisconnection(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	return false;
}

void UObjectTreeGraphSchema::InsertArrayItemPin(UEdGraphPin* ArrayPin, int32 Index) const
{
	if (!ensure(ArrayPin))
	{
		return;
	}

	UObjectTreeGraphNode* ObjectNode = Cast<UObjectTreeGraphNode>(ArrayPin->GetOwningNode());
	if (!ensure(ObjectNode && ObjectNode->GetObject()))
	{
		return;
	}

	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ObjectNode->GetPropertyForPin(ArrayPin));
	if (!ensure(ArrayProperty))
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
				LOCTEXT("InsertArrayItem", "Add {0} Pin"), FText::FromName(ArrayProperty->GetFName())));

	UObject* Object = ObjectNode->GetObject();

	Object->PreEditChange(ArrayProperty);

	Object->Modify();

	FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));
	FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);

	const bool bIsActualInsert = (Index >= 0 && Index <= ArrayHelper.Num());
	if (bIsActualInsert)
	{
		ArrayHelper.InsertValues(Index, 1);
		InnerProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), nullptr);
		ObjectNode->InsertNewItemPin(ArrayPin, Index);
	}
	else
	{
		const int32 NewIndex = ArrayHelper.AddValues(1);
		InnerProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(NewIndex), nullptr);
		ObjectNode->CreateNewItemPins(ArrayPin, 1);
	}

	FPropertyChangedEvent PropertyChangedEvent(ArrayProperty, EPropertyChangeType::ArrayAdd);
	Object->PostEditChangeProperty(PropertyChangedEvent);

	UEdGraph* Graph = ObjectNode->GetGraph();
	Graph->NotifyNodeChanged(ObjectNode);
}

void UObjectTreeGraphSchema::InsertArrayItemPinBefore(UEdGraphPin* ArrayItemPin) const
{
	if (!ensure(ArrayItemPin))
	{
		return;
	}

	UEdGraphPin* ArrayPin = ArrayItemPin->ParentPin;
	if (!ensure(ArrayPin))
	{
		return;
	}

	const int32 Index = ArrayPin->SubPins.Find(ArrayItemPin);
	if (ensure(Index >= 0))
	{
		InsertArrayItemPin(ArrayPin, Index);
	}
}

void UObjectTreeGraphSchema::InsertArrayItemPinAfter(UEdGraphPin* ArrayItemPin) const
{
	if (!ensure(ArrayItemPin))
	{
		return;
	}

	UEdGraphPin* ArrayPin = ArrayItemPin->ParentPin;
	if (!ensure(ArrayPin))
	{
		return;
	}

	const int32 Index = ArrayPin->SubPins.Find(ArrayItemPin);
	if (ensure(Index >= 0))
	{
		InsertArrayItemPin(ArrayPin, Index + 1);
	}
}

void UObjectTreeGraphSchema::RemoveArrayItemPin(UEdGraphPin* ArrayItemPin) const
{
	if (!ensure(ArrayItemPin))
	{
		return;
	}

	if (!ensure(ArrayItemPin->PinType.PinCategory == PC_Property && 
				ArrayItemPin->PinType.PinSubCategory == PSC_ArrayPropertyItem))
	{
		return;
	}

	UEdGraphPin* ArrayPin = ArrayItemPin->ParentPin;
	if (!ensure(ArrayPin))
	{
		return;
	}

	if (!ensure(ArrayPin->PinType.PinCategory == PC_Property && 
				ArrayPin->PinType.PinSubCategory == PSC_ArrayProperty))
	{
		return;
	}

	const int32 Index = ArrayItemPin->ParentPin->SubPins.Find(ArrayItemPin);
	if (!ensure(Index >= 0))
	{
		return;
	}

	UObjectTreeGraphNode* ObjectNode = Cast<UObjectTreeGraphNode>(ArrayPin->GetOwningNode());
	if (!ensure(ObjectNode && ObjectNode->GetObject()))
	{
		return;
	}

	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ObjectNode->GetPropertyForPin(ArrayPin));
	if (!ensure(ArrayProperty))
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
				LOCTEXT("RemoveArrayItem", "Remove {0} Pin"), FText::FromName(ArrayProperty->GetFName())));

	UObject* Object = ObjectNode->GetObject();

	Object->PreEditChange(ArrayProperty);

	Object->Modify();

	FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));
	ArrayHelper.RemoveValues(Index, 1);
	ObjectNode->RemoveItemPin(ArrayItemPin);

	FPropertyChangedEvent PropertyChangedEvent(ArrayProperty, EPropertyChangeType::ArrayRemove);
	Object->PostEditChangeProperty(PropertyChangedEvent);

	UEdGraph* Graph = ObjectNode->GetGraph();
	Graph->NotifyNodeChanged(ObjectNode);
}

bool UObjectTreeGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	return Super::SupportsDropPinOnNode(InTargetNode, InSourcePinType, InSourcePinDirection, OutErrorMessage);
}

bool UObjectTreeGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const
{
	if (!Graph || !Node)
	{
		return false;
	}
	
	BreakNodeLinks(*Node);

	UObjectTreeGraph* ObjectTreeGraph = CastChecked<UObjectTreeGraph>(Graph);
	OnDeleteNodeFromGraph(ObjectTreeGraph, Node);

	return true;
}

void UObjectTreeGraphSchema::OnDeleteNodeFromGraph(UObjectTreeGraph* Graph, UEdGraphNode* Node) const
{
	if (UObjectTreeGraphNode* ObjectNode = Cast<UObjectTreeGraphNode>(Node))
	{
		RemoveConnectableObject(Graph, ObjectNode->GetObject());
	}
	else if (UObjectTreeGraphCommentNode* CommentNode = Cast<UObjectTreeGraphCommentNode>(Node))
	{
		RemoveConnectableObject(Graph, CommentNode->GetObject());
	}
}

void UObjectTreeGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, FGraphDisplayInfo& OutDisplayInfo) const
{
	const UObjectTreeGraph* ObjectTreeGraph = CastChecked<const UObjectTreeGraph>(&Graph);
	const FObjectTreeGraphConfig& GraphConfig = ObjectTreeGraph->GetConfig();

	OutDisplayInfo = GraphConfig.GraphDisplayInfo;

	if (OutDisplayInfo.PlainName.IsEmpty())
	{
		OutDisplayInfo.PlainName = FText::FromString(Graph.GetName());
	}
	if (OutDisplayInfo.DisplayName.IsEmpty())
	{
		OutDisplayInfo.DisplayName = OutDisplayInfo.PlainName;
	}

	if (GraphConfig.OnGetGraphDisplayInfo.IsBound())
	{
		GraphConfig.OnGetGraphDisplayInfo.Execute(ObjectTreeGraph, OutDisplayInfo);
	}
}

FString UObjectTreeGraphSchema::ExportNodesToText(const FGraphPanelSelectionSet& Nodes, bool bOnlyCanDuplicateNodes, bool bOnlyCanDeleteNodes) const
{
	// Gather up the nodes we need to copy from.
	TSet<UObject*> ObjectsToExport;
	TSet<UObject*> OtherNodesToExport;

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(Nodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt);
		if (Node && 
				(!bOnlyCanDuplicateNodes || Node->CanDuplicateNode()) &&
				(!bOnlyCanDeleteNodes || Node->CanUserDeleteNode()))
		{
			Node->PrepareForCopying();

			if (UObjectTreeGraphNode* ObjectTreeNode = Cast<UObjectTreeGraphNode>(Node))
			{
				ObjectsToExport.Add(ObjectTreeNode->GetObject());
			}
			else if (UObjectTreeGraphCommentNode* CommentNode = Cast<UObjectTreeGraphCommentNode>(Node))
			{
				ObjectsToExport.Add(CommentNode->GetObject());
			}
			else
			{
				OtherNodesToExport.Add(Node);
			}
		}
	}

	if (ObjectsToExport.IsEmpty() && OtherNodesToExport.IsEmpty())
	{
		return FString();
	}

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	UObject* LastOuter = nullptr;
	for (UObject* ObjectToExport : ObjectsToExport)
	{
		// The nodes should all be from the same scope.
		UObject* ThisOuter = ObjectToExport->GetOuter();
		if (LastOuter != nullptr && ThisOuter != LastOuter)
		{
			UE_LOGF(LogCameraSystemEditor, Warning,
					"Cannot copy objects from different outers. Only copying from %ls", *LastOuter->GetName());
			continue;
		}
		LastOuter = ThisOuter;

		UExporter::ExportToOutputDevice(
				&Context,
				ObjectToExport, 
				nullptr, // no exporter
				Archive, 
				TEXT("copy"), // file type
				0, // indent
				PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, // port flags
				false, // selected only
				ThisOuter // export root scope
				);
	}

	if (!OtherNodesToExport.IsEmpty())
	{
		CopyNonObjectNodes(OtherNodesToExport.Array(), Archive);
	}

	return Archive;
}

void UObjectTreeGraphSchema::CopyNonObjectNodes(TArrayView<UObject*> InObjects, FStringOutputDevice& OutDevice) const
{
}

void UObjectTreeGraphSchema::ImportNodesFromText(UObjectTreeGraph* InGraph, const FString& TextToImport, TArray<UEdGraphNode*>& OutPastedNodes) const
{
	using namespace UE::ObjectTreeGraph;

	TArray<UObject*> ImportedObjects;

	InGraph->Modify();

	// Import the given text as new objects.
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/GameplayCamerasEditor/Transient"), RF_Transient);
	TempPackage->AddToRoot();
	{
		FObjectTextFactory Factory;
		Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
		ImportedObjects = Factory.CreatedObjects;
	}
	TempPackage->RemoveFromRoot();

	// Sever references to objects outside of the set being copy/pasted.
	TSet<UObject*> ImportedObjectSet(ImportedObjects);
	const FObjectTreeGraphConfig& GraphConfig = InGraph->GetConfig();
	for (UObject* Object : ImportedObjects)
	{
		UClass* ObjectClass = Object->GetClass();
		for (TFieldIterator<FProperty> PropertyIt(ObjectClass); PropertyIt; ++PropertyIt)
		{
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(*PropertyIt))
			{
				if (!GraphConfig.IsConnectable(ObjectProperty))
				{
					continue;
				}

				TObjectPtr<UObject> OutConnectedObject;
				ObjectProperty->GetValue_InContainer(Object, &OutConnectedObject);
				if (OutConnectedObject && !ImportedObjectSet.Contains(OutConnectedObject))
				{
					ObjectProperty->SetValue_InContainer(Object, nullptr);
				}
			}
			else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*PropertyIt))
			{
				if (!GraphConfig.IsConnectable(ArrayProperty))
				{
					continue;
				}

				FObjectProperty* InnerProperty = CastFieldChecked<FObjectProperty>(ArrayProperty->Inner);
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));

				const int32 ArrayNum = ArrayHelper.Num();
				for (int32 Index = ArrayNum - 1; Index >= 0; --Index)
				{
					UObject* OutConnectedObject = InnerProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
					if (OutConnectedObject && !ImportedObjectSet.Contains(OutConnectedObject))
					{
						ArrayHelper.RemoveValues(Index);
					}
				}
			}
		}
	}

	// Finish setting up the new objects: clear the transient flag from the transient package we used above,
	// and move the objects under the our graph root.
	UObject* GraphRootObject = InGraph->GetRootObject();
	if (ensure(GraphRootObject))
	{
		for (UObject* Object : ImportedObjects)
		{
			Object->ClearFlags(RF_Transient);
			Object->Rename(nullptr, GraphRootObject);
		}
	}

	// Create nodes for all the imported objects, and add them to the root object if it supports the root interface.
	FCreatedNodes CreatedNodes;
	for (UObject* Object : ImportedObjects)
	{
		if (UEdGraphNode* GraphNode = CreateObjectNode(InGraph, Object))
		{
			CreatedNodes.CreatedNodes.Add(Object, GraphNode);

			AddConnectableObject(InGraph, Object);
		}
	}

	// Create all the connections.
	for (TPair<UObject*, UEdGraphNode*> Pair : CreatedNodes.CreatedNodes)
	{
		if (UObjectTreeGraphNode* Node = Cast<UObjectTreeGraphNode>(Pair.Value))
		{
			CreateConnections(Node, CreatedNodes);
		}
	}

	OnCreateAllNodes(InGraph, CreatedNodes);

	for (const TTuple<UObject*, UEdGraphNode*>& Pair : CreatedNodes.CreatedNodes)
	{
		OutPastedNodes.Add(Pair.Value);
	}

	InGraph->NotifyGraphChanged();
}

bool UObjectTreeGraphSchema::CanImportNodesFromText(UObjectTreeGraph* InGraph, const FString& TextToImport) const
{
	using namespace UE::ObjectTreeGraph;

	FObjectTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

const FObjectTreeGraphClassConfigs UObjectTreeGraphSchema::GetObjectClassConfigs(const UObjectTreeGraphNode* InNode) const
{
	const UObjectTreeGraph* Graph = CastChecked<UObjectTreeGraph>(InNode->GetGraph());
	return GetObjectClassConfigs(Graph, InNode->GetObject()->GetClass());
}

const FObjectTreeGraphClassConfigs UObjectTreeGraphSchema::GetObjectClassConfigs(const UObjectTreeGraph* InGraph, UClass* InObjectClass) const
{
	return InGraph->GetConfig().GetObjectClassConfigs(InObjectClass);
}

FObjectTreeGraphSchemaAction_NewNode::FObjectTreeGraphSchemaAction_NewNode()
{
}

FObjectTreeGraphSchemaAction_NewNode::FObjectTreeGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping, InKeywords)
{
}

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
UEdGraphNode* FObjectTreeGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
#else
UEdGraphNode* FObjectTreeGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, bool bSelectNewNode)
#endif
{
	UObjectTreeGraph* ObjectTreeGraph = Cast<UObjectTreeGraph>(ParentGraph);
	if (!ensure(ObjectTreeGraph))
	{
		return nullptr;
	}

	if (!ensure(ObjectClass))
	{
		return nullptr;
	}

	if (!ObjectOuter)
	{
		if (ensure(ObjectTreeGraph))
		{
			ObjectOuter = ObjectTreeGraph->GetRootObject();
		}
	}

	if (!ensure(ObjectOuter))
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("CreateNewNodeAction", "Create {0} Node"), ObjectClass->GetDisplayNameText()));

	const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(ParentGraph->GetSchema());

	UObject* NewObject = CreateObject();

	if (NewObject)
	{
		const FObjectTreeGraphConfig& GraphConfig = ObjectTreeGraph->GetConfig();
		const FObjectTreeGraphClassConfigs ObjectClassConfigs = GraphConfig.GetObjectClassConfigs(ObjectClass);
		ObjectClassConfigs.OnSetupNewObject().ExecuteIfBound(NewObject);

		ObjectTreeGraph->Modify();

		UEdGraphNode* NewGraphNode = Schema->CreateObjectNode(ObjectTreeGraph, NewObject);

		Schema->AddConnectableObject(ObjectTreeGraph, NewObject);

		NewGraphNode->NodePosX = Location.X;
		NewGraphNode->NodePosY = Location.Y;
		if (UObjectTreeGraphNode* NewObjectGraphNode = Cast<UObjectTreeGraphNode>(NewGraphNode))
		{
			NewObjectGraphNode->OnGraphNodeMoved(false);
		}

		AutoSetupNewNode(NewGraphNode, FromPin);

		return NewGraphNode;
	}
	
	return nullptr;
}

UObject* FObjectTreeGraphSchemaAction_NewNode::CreateObject()
{
	return NewObject<UObject>(ObjectOuter, ObjectClass, NAME_None, RF_Transactional);
}

void FObjectTreeGraphSchemaAction_NewNode::AutoSetupNewNode(UEdGraphNode* NewNode, UEdGraphPin* FromPin)
{
	NewNode->AutowireNewNode(FromPin);
}

FObjectTreeGraphSchemaAction_NewComment::FObjectTreeGraphSchemaAction_NewComment()
{
}

FObjectTreeGraphSchemaAction_NewComment::FObjectTreeGraphSchemaAction_NewComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping, InKeywords)
{
}

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
UEdGraphNode* FObjectTreeGraphSchemaAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
#else
UEdGraphNode* FObjectTreeGraphSchemaAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, bool bSelectNewNode)
#endif
{
	UObjectTreeGraph* ObjectTreeGraph = Cast<UObjectTreeGraph>(ParentGraph);
	if (!ensure(ObjectTreeGraph))
	{
		return nullptr;
	}

	UObject* ObjectOuter = ObjectTreeGraph->GetRootObject();
	if (!ensure(ObjectOuter))
	{
		return nullptr;
	}

	FSlateRect Bounds;
	bool bUseBounds = false;
	if (TSharedPtr<SObjectTreeGraphEditor> GraphEditor = SObjectTreeGraphEditor::FindGraphEditor(ObjectTreeGraph))
	{
		bUseBounds = GraphEditor->GetGraphEditor()->GetBoundsForSelectedNodes(Bounds, 50.f);
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateNewCommentAction", "Create Comment"));

	const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(ParentGraph->GetSchema());

	UObjectTreeGraphComment* NewComment = NewObject<UObjectTreeGraphComment>(ObjectOuter, NAME_None, RF_Transactional);

	UEdGraphNode* NewGraphNode = Schema->CreateObjectNode(ObjectTreeGraph, NewComment);

	Schema->AddConnectableObject(ObjectTreeGraph, NewComment);

	if (bUseBounds)
	{
		NewGraphNode->NodePosX = Bounds.Left;
		NewGraphNode->NodePosY = Bounds.Top;

		FVector2D BoundsSize = Bounds.GetSize();
		NewGraphNode->NodeWidth = BoundsSize.X;
		NewGraphNode->NodeHeight = BoundsSize.Y;
	}
	else
	{
		NewGraphNode->NodePosX = Location.X;
		NewGraphNode->NodePosY = Location.Y;

		NewGraphNode->NodeWidth = 400;
		NewGraphNode->NodeHeight = 400;
	}
	if (UObjectTreeGraphCommentNode* NewCommentNode = Cast<UObjectTreeGraphCommentNode>(NewGraphNode))
	{
		NewCommentNode->OnGraphNodeMoved(false);
	}

	NewGraphNode->AutowireNewNode(FromPin);

	return NewGraphNode;
}

#undef LOCTEXT_NAMESPACE

