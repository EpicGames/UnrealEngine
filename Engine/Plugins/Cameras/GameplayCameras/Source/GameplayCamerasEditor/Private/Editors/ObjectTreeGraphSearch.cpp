// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/ObjectTreeGraphSearch.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"

#define LOCTEXT_NAMESPACE "ObjectTreeGraphSearch"

FObjectTreeGraphSearch::FObjectTreeGraphSearch()
{
}

void FObjectTreeGraphSearch::AddGraph(UEdGraph* InGraph)
{
	SearchGraphs.Add(InGraph);
}

void FObjectTreeGraphSearch::SetGraphs(TConstArrayView<UEdGraph*> InGraphs)
{
	SearchGraphs = InGraphs;
}

void FObjectTreeGraphSearch::Search(TArrayView<FString> InTokens, TArray<FObjectTreeGraphSearchResult>& OutResults) const
{
	for (UEdGraph* Graph : SearchGraphs)
	{
		FSearchState State;
		State.Graph = Graph;
		State.Tokens = InTokens;

		SearchGraph(Graph, State);

		OutResults.Append(State.Results);
	}
}

void FObjectTreeGraphSearch::SearchGraph(UEdGraph* Graph, FSearchState& State) const
{
	if (!Graph)
	{
		return;
	}

	for (UEdGraphNode* Node : Graph->Nodes)
	{
		SearchNode(Node, State);
	}
}

void FObjectTreeGraphSearch::SearchNode(UEdGraphNode* Node, FSearchState& State) const
{
	if (!Node)
	{
		return;
	}

	const FText NodeTitle = Node->GetNodeTitle(ENodeTitleType::FullTitle);
	if (MatchString(NodeTitle.ToString(), State))
	{
		State.Results.Add(FSearchResult(State.Graph, Node));
	}

	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (!Pin)
		{
			continue;
		}

		const FText PinName = Pin->GetDisplayName();
		if (MatchString(PinName.ToString(), State))
		{
			State.Results.Add(FSearchResult(State.Graph, Node, Pin->GetFName()));
		}
	}
}

bool FObjectTreeGraphSearch::MatchString(const FString& InString, const FSearchState& InState) const
{
	if (InString.IsEmpty())
	{
		return false;
	}

	for (const FString& Token : InState.Tokens)
	{
		if (!InString.Contains(Token))
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

