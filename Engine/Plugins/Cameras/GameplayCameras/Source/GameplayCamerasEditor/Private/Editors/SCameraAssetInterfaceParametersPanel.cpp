// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraAssetInterfaceParametersPanel.h"

#include "AssetRegistry/ARFilter.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "Helpers/CameraContextDataPinTypeHelper.h"
#include "Helpers/CameraVariablePinTypeHelper.h"
#include "IContentBrowserSingleton.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Toolkits/CameraAssetEditorToolkit.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCameraAssetInterfaceParametersPanel"

namespace UE::Cameras
{

class SCameraAssetSourceCameraRigPicker : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraAssetSourceCameraRigPicker)
	{}
		SLATE_ARGUMENT(UCameraAssetInterfaceParameter*, InterfaceParameter)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args)
	{
		InterfaceParameter = Args._InterfaceParameter;

		CameraAsset = InterfaceParameter->GetTypedOuter<UCameraAsset>();
		if (ensure(CameraAsset))
		{
			if (UCameraDirector* CameraDirector = CameraAsset->GetCameraDirector())
			{
				FCameraDirectorRigUsageInfo UsageInfo;
				CameraDirector->GatherRigUsageInfo(UsageInfo);
				for (const UCameraRigAsset* CameraRig : UsageInfo.CameraRigs)
				{
					ReferencedCameraRigPaths.Add(CameraRig->GetPathName());
				}
			}
		}
		
		ChildSlot
		[
			SAssignNew(ComboButton, SComboButton)
			.OnGetMenuContent(this, &SCameraAssetSourceCameraRigPicker::OnGenerateAssetPicker)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SCameraAssetSourceCameraRigPicker::GetSourceCameraRigName)
			]
		];
	}

private:

	TSharedRef<SWidget> OnGenerateAssetPicker()
	{
		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.Filter.ClassPaths.Add(UCameraRigAsset::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SCameraAssetSourceCameraRigPicker::OnShouldFilterAsset);
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCameraAssetSourceCameraRigPicker::OnAssetSelected);
		AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetPickerEvent);

		IContentBrowserSingleton& ContentBrowser = IContentBrowserSingleton::Get();
		TSharedRef<SWidget> AssetPicker = ContentBrowser.CreateAssetPicker(AssetPickerConfig);

		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.MinHeight(400.f)
			[
				AssetPicker
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.f)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SCameraAssetSourceCameraRigPicker::IsShowingOnlyReferencedCameraRigs)
				.OnCheckStateChanged(this, &SCameraAssetSourceCameraRigPicker::OnToggleOnlyShowReferencedCameraRigs)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowOnlyReferencedCameraRigs", "Show only referenced camera rigs"))
				]
			];
	}

	FText GetSourceCameraRigName() const
	{
		if (UCameraRigAsset* CameraRig = InterfaceParameter->SourceCameraRig.Get())
		{
			return FText::FromName(CameraRig->GetFName());
		}
		return LOCTEXT("NoSourceCameraRig", "None");
	}

	bool OnShouldFilterAsset(const FAssetData& AssetData)
	{
		if (bShowOnlyReferencedCameraRigs)
		{
			return !ReferencedCameraRigPaths.Contains(AssetData.GetObjectPathString());
		}
		return false;
	}

	void OnAssetSelected(const FAssetData& SelectedAsset)
	{
		UCameraRigAsset* SelectedCameraRig = Cast<UCameraRigAsset>(SelectedAsset.GetAsset());
		if (SelectedCameraRig != InterfaceParameter->SourceCameraRig)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetSourceCameraRig", "Set Source Camera Rig"));

			InterfaceParameter->Modify();
			InterfaceParameter->SourceCameraRig = SelectedCameraRig;
		}

		ComboButton->SetIsOpen(false);
	}

	ECheckBoxState IsShowingOnlyReferencedCameraRigs() const
	{
		return (bShowOnlyReferencedCameraRigs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	}

	void OnToggleOnlyShowReferencedCameraRigs(ECheckBoxState CheckBoxState)
	{
		bShowOnlyReferencedCameraRigs = (CheckBoxState == ECheckBoxState::Checked);

		RefreshAssetPickerEvent.ExecuteIfBound(false);
	}

private:

	UCameraAssetInterfaceParameter* InterfaceParameter = nullptr;

	UCameraAsset* CameraAsset = nullptr;
	TSet<FString> ReferencedCameraRigPaths;

	TSharedPtr<SComboButton> ComboButton;
	FRefreshAssetViewDelegate RefreshAssetPickerEvent;
	bool bShowOnlyReferencedCameraRigs = true;
};

class SCameraAssetSourceParameterNamePicker : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCameraAssetSourceParameterNamePicker)
	{}
		SLATE_ARGUMENT(UCameraAssetInterfaceParameter*, InterfaceParameter)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args)
	{
		InterfaceParameter = Args._InterfaceParameter;
		CameraAsset = InterfaceParameter->GetTypedOuter<UCameraAsset>();

		EnsureSourceParameterNamesCached();

		ChildSlot
		[
			SNew(SComboBox<TSharedPtr<FName>>)
			.OptionsSource(&LastKnownSourceParameterNames)
			.OnGenerateWidget(this, &SCameraAssetSourceParameterNamePicker::OnGenerateParameterNameWidget)
			.OnSelectionChanged(this, &SCameraAssetSourceParameterNamePicker::OnParameterNameSelectionChanged)
			[
				SNew(STextBlock)
				.ColorAndOpacity(this, &SCameraAssetSourceParameterNamePicker::GetSourceParameterNameColorAndOpacity)
				.Text(this, &SCameraAssetSourceParameterNamePicker::GetSourceParameterName)
			]
		];
	}

protected:

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// Keep checking in case the source camera rig changes.
		EnsureSourceParameterNamesCached();
	}

private:

	TSharedRef<SWidget> OnGenerateParameterNameWidget(TSharedPtr<FName> Item)
	{
		const FName ItemName = Item ? *Item : NAME_None;
		return SNew(STextBlock)
			.Text(FText::FromName(ItemName));
	}

	void OnParameterNameSelectionChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo)
	{
		FName NewName = Item ? *Item : NAME_None;
		if (InterfaceParameter->SourceParameterName != NewName)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetSourceParameterName", "Set Source Parameter Name"));

			InterfaceParameter->Modify();
			InterfaceParameter->SourceParameterName = NewName;
		}
	}

	FSlateColor GetSourceParameterNameColorAndOpacity() const
	{
		bool bIsValidParameter = false;
		bool bIsParameterNone = InterfaceParameter->SourceParameterName.IsNone();
		if (LastKnownSourceCameraRig && !bIsParameterNone)
		{
			FCameraObjectInterfaceParameterDefinition ParameterDefinition;
			bIsValidParameter = LastKnownSourceCameraRig->FindParameterDefinitionByName(
					InterfaceParameter->SourceParameterName, ParameterDefinition);
		}
		if (bIsValidParameter)
		{
			return FSlateColor::UseForeground();
		}
		else if (bIsParameterNone)
		{
			return FSlateColor::UseSubduedForeground();
		}
		else
		{
			return FStyleColors::Error;
		}
	}

	FText GetSourceParameterName() const
	{
		return FText::FromName(InterfaceParameter->SourceParameterName);
	}

	void EnsureSourceParameterNamesCached()
	{
		UCameraRigAsset* CameraRig = InterfaceParameter->SourceCameraRig.Get();
		if (CameraRig == LastKnownSourceCameraRig)
		{
			return;
		}

		LastKnownSourceCameraRig = CameraRig;
		LastKnownSourceParameterNames.Reset();
		if (CameraRig)
		{
			for (const FCameraObjectInterfaceParameterDefinition& ParameterDefinition : CameraRig->GetParameterDefinitions())
			{
				if (ParameterDefinition.bIsVisible)
				{
					LastKnownSourceParameterNames.Add(MakeShared<FName>(ParameterDefinition.ParameterName));
				}
			}
		}
		if (LastKnownSourceParameterNames.IsEmpty())
		{
			LastKnownSourceParameterNames.Add(MakeShared<FName>(NAME_None));
		}
	}

private:

	UCameraAsset* CameraAsset = nullptr;
	UCameraAssetInterfaceParameter* InterfaceParameter = nullptr;

	TArray<TSharedPtr<FName>> LastKnownSourceParameterNames;
	UCameraRigAsset* LastKnownSourceCameraRig = nullptr;
};

class SCameraAssetInterfaceParameterTableRow : public SMultiColumnTableRow<TObjectPtr<UCameraAssetInterfaceParameter>>
{
public:

	SLATE_BEGIN_ARGS(SCameraAssetInterfaceParameterTableRow)
	{}
		SLATE_ARGUMENT(TObjectPtr<UCameraAssetInterfaceParameter>, Item)
	SLATE_END_ARGS()

	typedef SMultiColumnTableRow<TObjectPtr<UCameraAssetInterfaceParameter>> FSuperRowType;
	typedef typename STableRow<TObjectPtr<UCameraAssetInterfaceParameter>>::FArguments FTableRowArgs;
		
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Item = Args._Item;

		EnsurePinTypeCached();

		FSuperRowType::Construct(
				FTableRowArgs(),
				OwnerTable);
	}

	void EnterNameEditingMode()
	{
		NameTextBlock->EnterEditingMode();
	}
	
protected:

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		FSuperRowType::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// Keep checking in case the source camera rig or parameter name change.
		EnsurePinTypeCached();
	}

private:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == SCameraAssetInterfaceParametersPanel::ParameterTypeColumn)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.f)
				[
					SNew(SImage)
						.Image(this, &SCameraAssetInterfaceParameterTableRow::GetTypeIcon)
						.ColorAndOpacity(this, &SCameraAssetInterfaceParameterTableRow::GetTypeIconColor)
				];
		}
		else if (InColumnName == SCameraAssetInterfaceParametersPanel::ParameterNameColumn)
		{
			return SAssignNew(NameTextBlock, SInlineEditableTextBlock)
				.IsSelected(this, &SCameraAssetInterfaceParameterTableRow::IsSelected)
				.Text_Lambda([this]() { return FText::FromString(Item->InterfaceParameterName); })
				.OnTextCommitted(this, &SCameraAssetInterfaceParameterTableRow::OnParameterNameTextCommitted)
				.ToolTipText(this, &SCameraAssetInterfaceParameterTableRow::GetParameterToolTip);
		}
		else if (InColumnName == SCameraAssetInterfaceParametersPanel::SourceCameraRigColumn)
		{
			return SNew(SCameraAssetSourceCameraRigPicker).InterfaceParameter(Item);
		}
		else if (InColumnName == SCameraAssetInterfaceParametersPanel::SourceParameterNameColumn)
		{
			return SNew(SCameraAssetSourceParameterNamePicker).InterfaceParameter(Item);
		}

		return SNullWidget::NullWidget;
	}

	void OnParameterNameTextCommitted(const FText& Text, ETextCommit::Type CommitType)
	{
		const FString NewParameterName = Text.ToString();

		if (Item->InterfaceParameterName != NewParameterName)
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameInterfaceParameterTransaction", "Rename Interface Parameter"));

			Item->Modify();
			Item->InterfaceParameterName = NewParameterName;
		}
	}

	FText GetParameterToolTip() const
	{
		const FGuid Guid = Item->GetGuid();
		return FText::FromString(Guid.ToString(EGuidFormats::DigitsWithHyphensInParentheses));
	}

	const FSlateBrush* GetTypeIcon() const
	{
		return PinTypeIcon;
	}

	FSlateColor GetTypeIconColor() const
	{
		return PinTypeIconColor;
	}

	void EnsurePinTypeCached()
	{
		UCameraRigAsset* CameraRig = Item->SourceCameraRig.Get();
		if (CameraRig == LastKnownSourceCameraRig && Item->SourceParameterName == LastKnownSourceParameterName)
		{
			return;
		}

		const FSlateBrush* InvalidIcon = FAppStyle::GetBrush("Icons.Warning.Solid");
		const FLinearColor DefaultIconColor = FLinearColor::White;

		LastKnownSourceCameraRig = CameraRig;
		LastKnownSourceParameterName = Item->SourceParameterName;

		if (!CameraRig || Item->SourceParameterName.IsNone())
		{
			PinTypeIcon = InvalidIcon;
			PinTypeIconColor = DefaultIconColor;
			return;
		}

		FCameraObjectInterfaceParameterDefinition ParameterDefinition;
		const bool bFoundSourceParameter = CameraRig->FindParameterDefinitionByName(Item->SourceParameterName, ParameterDefinition);
		if (!bFoundSourceParameter)
		{
			PinTypeIcon = InvalidIcon;
			PinTypeIconColor = DefaultIconColor;
			return;
		}

		FEdGraphPinType PinType;
		if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Blendable)
		{
			PinType = FCameraVariablePinTypeHelper::GetPinType(ParameterDefinition.VariableType, ParameterDefinition.BlendableStructType);
		}
		else if (ParameterDefinition.ParameterType == ECameraObjectInterfaceParameterType::Data)
		{
			PinType = FCameraContextDataPinTypeHelper::GetPinType(ParameterDefinition.DataType, ParameterDefinition.DataContainerType, ParameterDefinition.DataTypeObject);
		}

		PinTypeIcon = FBlueprintEditorUtils::GetIconFromPin(PinType);
		PinTypeIconColor = GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
	}

protected:

	TObjectPtr<UCameraAssetInterfaceParameter> Item;
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;

	UCameraRigAsset* LastKnownSourceCameraRig = nullptr;
	FName LastKnownSourceParameterName;

	const FSlateBrush* PinTypeIcon = nullptr;
	FLinearColor PinTypeIconColor = FLinearColor::White;
};

const FName SCameraAssetInterfaceParametersPanel::ParameterTypeColumn(TEXT("ParameterType"));
const FName SCameraAssetInterfaceParametersPanel::ParameterNameColumn(TEXT("ParameterName"));
const FName SCameraAssetInterfaceParametersPanel::SourceCameraRigColumn(TEXT("SourceCameraRig"));
const FName SCameraAssetInterfaceParametersPanel::SourceParameterNameColumn(TEXT("SourceParameterName"));

void SCameraAssetInterfaceParametersPanel::Construct(const FArguments& Args, FCameraAssetEditorToolkit* OwnerToolkit)
{
	CameraAsset = OwnerToolkit->GetCameraAsset();
	CameraAsset->EventHandlers.Register(EventHandler, this);

	Toolkit = OwnerToolkit;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(3, 5))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				.Padding(5.f)
				[
					SNew(SRichTextBlock)
					.Text(LOCTEXT("ParametersPanelTitle", "Parameters"))
					.TransformPolicy(ETextTransformPolicy::ToUpper)
					.DecoratorStyleSet(&FAppStyle::Get())
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(1, 0))
					.ToolTipText(LOCTEXT("AddParameterToolTip", "Add a parameter"))
					.OnClicked(this, &SCameraAssetInterfaceParametersPanel::OnAddInterfaceParameter)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(1, 0))
					.ToolTipText(LOCTEXT("RemoveParameterToolTip", "Removes the selected parameter(s)"))
					.OnClicked(this, &SCameraAssetInterfaceParametersPanel::OnDeleteSelectedInterfaceParameter)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.MinusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(ParametersListView, SListView<TObjectPtr<UCameraAssetInterfaceParameter>>)
			.ListItemsSource(&CameraAsset->Interface.Parameters)
			.OnGenerateRow(this, &SCameraAssetInterfaceParametersPanel::OnGenerateInterfaceParameterRow)
			.OnSelectionChanged(this, &SCameraAssetInterfaceParametersPanel::OnInterfaceParameterSelectionChanged)
			.OnContextMenuOpening(this, &SCameraAssetInterfaceParametersPanel::OnInterfaceParameterContextMenuOpening)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+SHeaderRow::Column(ParameterTypeColumn)
				.FixedWidth(64)
				.DefaultLabel(LOCTEXT("ParameterTypeColumnLabel", "Type"))

				+SHeaderRow::Column(ParameterNameColumn)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("ParameterNameColumnLabel", "Name"))

				+SHeaderRow::Column(SourceCameraRigColumn)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("SourceCameraRigColumnLabel", "Source Camera Rig"))

				+SHeaderRow::Column(SourceParameterNameColumn)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("SourceParameterNameColumnLabel", "Source Parameter"))
			)
		]
	];
}

void SCameraAssetInterfaceParametersPanel::RequestListRefresh()
{
	bListRefreshRequested = true;
}

void SCameraAssetInterfaceParametersPanel::RenameSelectedParameter()
{
	if (ParametersListView->HasKeyboardFocus())
	{
		TArray<TObjectPtr<UCameraAssetInterfaceParameter>> SelectedItems = ParametersListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnRenameInterfaceParameter(SelectedItems[0]);
		}
	}
}

void SCameraAssetInterfaceParametersPanel::DeleteSelectedParameter()
{
	if (ParametersListView->HasKeyboardFocus())
	{
		OnDeleteSelectedInterfaceParameter();
	}
}

void SCameraAssetInterfaceParametersPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bListRefreshRequested)
	{
		bListRefreshRequested = false;

		ParametersListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SCameraAssetInterfaceParametersPanel::OnGenerateInterfaceParameterRow(TObjectPtr<UCameraAssetInterfaceParameter> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCameraAssetInterfaceParameterTableRow, OwnerTable)
		.Item(Item);
}

void SCameraAssetInterfaceParametersPanel::OnInterfaceParameterSelectionChanged(TObjectPtr<UCameraAssetInterfaceParameter> Item, ESelectInfo::Type Type)
{
	OnInterfaceParameterSelectedDelegate.Broadcast(Item);
}

TSharedPtr<SWidget> SCameraAssetInterfaceParametersPanel::OnInterfaceParameterContextMenuOpening()
{
	TArray<TObjectPtr<UCameraAssetInterfaceParameter>> SelectedItems = ParametersListView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, nullptr);
	TObjectPtr<UCameraAssetInterfaceParameter> Item = SelectedItems[0];

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("InterfaceParameterEditSection", "Edit Interface Parameter"));
	{
		MenuBuilder.AddMenuEntry(
				LOCTEXT("RenameInterfaceParameter", "Rename"),
				LOCTEXT("RenameInterfaceParameterToolTip", "Renames this interface parameter"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
				FExecuteAction::CreateSP(this, &SCameraAssetInterfaceParametersPanel::OnRenameInterfaceParameter, Item));
		MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteInterfaceParameter", "Delete"),
				LOCTEXT("DeleteInterfaceParameterToolTip", "Deletes this interface parameter"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
				FExecuteAction::CreateSP(this, &SCameraAssetInterfaceParametersPanel::OnDeleteInterfaceParameter, Item));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SCameraAssetInterfaceParametersPanel::OnAddInterfaceParameter()
{
	const FScopedTransaction Transaction(LOCTEXT("AddInterfaceParameter", "Add Interface Parameter"));

	CameraAsset->Modify();

	UCameraAssetInterfaceParameter* NewParameter = NewObject<UCameraAssetInterfaceParameter>(CameraAsset, NAME_None, RF_Transactional);
	NewParameter->InterfaceParameterName = GetNewParameterName();

	CameraAsset->Interface.Parameters.Add(NewParameter);
	CameraAsset->EventHandlers.Notify(&ICameraAssetEventHandler::OnCameraAssetInterfaceChanged);

	ParametersListView->RequestListRefresh();

	return FReply::Handled();
}

FReply SCameraAssetInterfaceParametersPanel::OnDeleteSelectedInterfaceParameter()
{
	TArray<TObjectPtr<UCameraAssetInterfaceParameter>> SelectedItems = ParametersListView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return FReply::Handled();
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveInterfaceParameter", "Remove Interface Parameter"));

	CameraAsset->Modify();

	for (TObjectPtr<UCameraAssetInterfaceParameter> Item : SelectedItems)
	{
		const int32 NumRemoved = CameraAsset->Interface.Parameters.Remove(Item);
		ensure(NumRemoved == 1);
	}

	CameraAsset->EventHandlers.Notify(&ICameraAssetEventHandler::OnCameraAssetInterfaceChanged);

	ParametersListView->RequestListRefresh();

	return FReply::Handled();
}

void SCameraAssetInterfaceParametersPanel::OnRenameInterfaceParameter(TObjectPtr<UCameraAssetInterfaceParameter> Item)
{
	TSharedPtr<ITableRow> RowWidget = ParametersListView->WidgetFromItem(Item);
	if (!ensure(RowWidget))
	{
		return;
	}

	TSharedPtr<SCameraAssetInterfaceParameterTableRow> TypedRowWidget = 
		StaticCastSharedPtr<SCameraAssetInterfaceParameterTableRow>(RowWidget);
	TypedRowWidget->EnterNameEditingMode();
}

void SCameraAssetInterfaceParametersPanel::OnDeleteInterfaceParameter(TObjectPtr<UCameraAssetInterfaceParameter> Item)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveInterfaceParameter", "Remove Interface Parameter"));

	CameraAsset->Modify();

	const int32 NumRemoved = CameraAsset->Interface.Parameters.Remove(Item);
	ensure(NumRemoved == 1);

	CameraAsset->EventHandlers.Notify(&ICameraAssetEventHandler::OnCameraAssetInterfaceChanged);

	ParametersListView->RequestListRefresh();
}

FString SCameraAssetInterfaceParametersPanel::GetNewParameterName()
{
	static const FString BaseNewParameterName(TEXT("NewParameter"));

	int32 NameIndex = 0;
	bool bFoundUniqueName = false;
	FString NewParameterName(BaseNewParameterName);

	while (!bFoundUniqueName)
	{
		const bool bFoundConflict = CameraAsset->Interface.Parameters.ContainsByPredicate(
				[&NewParameterName](const UCameraAssetInterfaceParameter* Item)
				{
					return Item->InterfaceParameterName == NewParameterName;
				});
		bFoundUniqueName = !bFoundConflict;
		if (bFoundConflict)
		{
			++NameIndex;
			NewParameterName = FString::Printf(TEXT("%s_%d"), *BaseNewParameterName, NameIndex);
		}
	}

	return NewParameterName;
}

void SCameraAssetInterfaceParametersPanel::OnCameraAssetInterfaceChanged()
{
	bListRefreshRequested = true;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

