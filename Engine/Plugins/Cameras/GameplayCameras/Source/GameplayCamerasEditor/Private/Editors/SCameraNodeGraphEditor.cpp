// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraNodeGraphEditor.h"

#include "Compat/EditorCompat.h"
#include "Editors/CameraNodeGraphDragDropOp.h"
#include "SGraphPanel.h"

FReply SCameraNodeGraphEditor::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FCameraNodeGraphInterfaceParameterDragDropOp> InterfaceParameterOp = 
		DragDropEvent.GetOperationAs<FCameraNodeGraphInterfaceParameterDragDropOp>();
	if (InterfaceParameterOp)
	{
		return InterfaceParameterOp->ExecuteDragOver(GraphEditor);
	}

	return SObjectTreeGraphEditor::OnDragOver(MyGeometry, DragDropEvent);
}

FReply SCameraNodeGraphEditor::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FCameraNodeGraphInterfaceParameterDragDropOp> InterfaceParameterOp = 
		DragDropEvent.GetOperationAs<FCameraNodeGraphInterfaceParameterDragDropOp>();
	if (InterfaceParameterOp)
	{
		SGraphPanel* GraphPanel = GraphEditor->GetGraphPanel();
		FSlateCompatVector2f NewLocation = GraphPanel->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()));

		return InterfaceParameterOp->ExecuteDrop(GraphEditor, NewLocation);
	}

	return SObjectTreeGraphEditor::OnDrop(MyGeometry, DragDropEvent);
}

