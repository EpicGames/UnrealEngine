// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraNodeGraphNode.h"

#include "Editors/CameraObjectGraphSchemaBase.h"
#include "Styles/GameplayCamerasEditorStyle.h"

void SCameraNodeGraphNode::Construct(const FArguments& InArgs)
{
	SObjectTreeGraphNode::Construct(
			SObjectTreeGraphNode::FArguments()
			.GraphNode(InArgs._GraphNode));
}

TSharedPtr<SGraphPin> SCameraNodeGraphNode::CreatePinWidget(UEdGraphPin* InPin) const
{
	using namespace UE::Cameras;

	TSharedPtr<SGraphPin> PinWidget = SObjectTreeGraphNode::CreatePinWidget(InPin);

	if (const UCameraObjectGraphSchemaBase* GraphSchema = Cast<const UCameraObjectGraphSchemaBase>(InPin->GetSchema()))
	{
		TSharedRef<FGameplayCamerasEditorStyle> GraphStyle = FGameplayCamerasEditorStyle::Get();

		if (InPin->PinType.PinCategory == UCameraObjectGraphSchemaBase::PC_CameraParameter ||
				InPin->PinType.PinCategory == UCameraObjectGraphSchemaBase::PC_CameraVariableReference)
		{
			const FSlateBrush* ConnectedBrush = GraphStyle->GetBrush("Graph.CameraRigParameterPin.Connected");
			const FSlateBrush* DisconnectedBrush = GraphStyle->GetBrush("Graph.CameraRigParameterPin.Disconnected");
			PinWidget->SetCustomPinIcon(ConnectedBrush, DisconnectedBrush);
		}
	}

	return PinWidget;
}

