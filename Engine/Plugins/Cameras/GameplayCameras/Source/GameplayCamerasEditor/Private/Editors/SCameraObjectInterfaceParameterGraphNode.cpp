// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraObjectInterfaceParameterGraphNode.h"

#include "Core/CameraRigAsset.h"
#include "Editors/CameraObjectInterfaceParameterGraphNode.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCameraObjectInterfaceParameterGraphNode"

void SCameraObjectInterfaceParameterGraphNode::Construct(const FArguments& InArgs)
{
	SObjectTreeGraphNode::FArguments SuperArgs;
	SuperArgs
		.GraphNode(InArgs._GraphNode);
	SObjectTreeGraphNode::Construct(SuperArgs);
}

void SCameraObjectInterfaceParameterGraphNode::UpdateGraphNode()
{
	using namespace UE::Cameras;

	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

	InputPins.Empty();
	OutputPins.Empty();

	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();

	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(CamerasEditorStyle->GetBrush("Graph.CameraRigParameterNode.Body"))
			]
			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SImage)
				.Image(CamerasEditorStyle->GetBrush("Graph.CameraRigParameterNode.ColorSpill"))
				.ColorAndOpacity(this, &SCameraObjectInterfaceParameterGraphNode::GetNodeTitleColor)
			]
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(CamerasEditorStyle->GetBrush("Graph.CameraRigParameterNode.Gloss"))
			]
			+SOverlay::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(12, 8, 38, 8))
			[
				// NODE TITLE
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.Node.NodeTitle")
				.Text(this, &SCameraObjectInterfaceParameterGraphNode::GetInterfaceParameterName)
			]
			+SOverlay::Slot()
			.Padding(FMargin(0, 4))
			[
				// NODE CONTENT AREA
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				.Padding(FMargin(2, 0))
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding(FMargin(2, 0))
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight() 
		.Padding( FMargin(5.0f, 1.0f) )
		[
			ErrorReporting->AsWidget()
		]
	];

	// Create widgets for each of the real pins
	CreatePinWidgets();
}

const FSlateBrush* SCameraObjectInterfaceParameterGraphNode::GetShadowBrush(bool bSelected) const
{
	using namespace UE::Cameras;
	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();
	return bSelected ? 
		CamerasEditorStyle->GetBrush(TEXT("Graph.CameraRigParameterNode.ShadowSelected")) : 
		CamerasEditorStyle->GetBrush(TEXT("Graph.CameraRigParameterNode.Shadow"));
}

TSharedPtr<SGraphPin> SCameraObjectInterfaceParameterGraphNode::CreatePinWidget(UEdGraphPin* InPin) const
{
	using namespace UE::Cameras;
	TSharedPtr<SGraphPin> PinWidget = SObjectTreeGraphNode::CreatePinWidget(InPin);
	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();
	const FSlateBrush* ConnectedBrush = CamerasEditorStyle->GetBrush("Graph.CameraRigParameterPin.Connected");
	const FSlateBrush* DisconnectedBrush = CamerasEditorStyle->GetBrush("Graph.CameraRigParameterPin.Disconnected");
	PinWidget->SetCustomPinIcon(ConnectedBrush, DisconnectedBrush);
	return PinWidget;
}

FText SCameraObjectInterfaceParameterGraphNode::GetInterfaceParameterName() const
{
	if (UCameraObjectInterfaceParameterGetter* GetterNode = GetObjectGraphNode()->CastObject<UCameraObjectInterfaceParameterGetter>())
	{
		return FText::FromString(GetterNode->GetInterfaceParameterName());
	}
	return LOCTEXT("InvalidParameterName", "Invalid");
}

#undef LOCTEXT_NAMESPACE

