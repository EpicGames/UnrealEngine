// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraObjectInterfaceParametersPanel.h"

#include "Editors/CameraNodeGraphDragDropOp.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Helpers/CameraContextDataPinTypeHelper.h"
#include "Helpers/CameraVariablePinTypeHelper.h"
#include "IGameplayCamerasModule.h"
#include "SPinTypeSelector.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Toolkits/CameraObjectInterfaceParametersToolkit.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCameraObjectInterfaceParametersPanel)

#define LOCTEXT_NAMESPACE "SCameraObjectInterfaceParametersPanel"

namespace UE::Cameras
{

/**
 * List entry for any interface parameter panel.
 */
template<typename ParameterType>
class SCameraObjectInterfaceParameterTableRowBase : public SMultiColumnTableRow<TObjectPtr<ParameterType>>
{
public:

	SLATE_BEGIN_ARGS(SCameraObjectInterfaceParameterTableRowBase<ParameterType>)
	{}
		SLATE_ARGUMENT(TObjectPtr<ParameterType>, Item)
	SLATE_END_ARGS()

	typedef SMultiColumnTableRow<TObjectPtr<ParameterType>> FSuperRowType;
	typedef typename STableRow<TObjectPtr<ParameterType>>::FArguments FTableRowArgs;
		
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Item = Args._Item;

		FSuperRowType::Construct(
				FTableRowArgs(),
				OwnerTable);
	}

	void EnterNameEditingMode()
	{
		NameTextBlock->EnterEditingMode();
	}

protected:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == SCameraObjectInterfaceParametersPanel::ParameterNameColumn)
		{
			return SAssignNew(NameTextBlock, SInlineEditableTextBlock)
				.IsSelected(this, &SCameraObjectInterfaceParameterTableRowBase<ParameterType>::IsSelected)
				.Text_Lambda([this]() { return FText::FromString(Item->InterfaceParameterName); })
				.OnTextCommitted(this, &SCameraObjectInterfaceParameterTableRowBase<ParameterType>::OnParameterNameTextCommitted)
				.ToolTipText(this, &SCameraObjectInterfaceParameterTableRowBase<ParameterType>::GetParameterToolTip);
		}

		return SNullWidget::NullWidget;
	}

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			return FReply::Handled().BeginDragDrop(FCameraNodeGraphInterfaceParameterDragDropOp::New(Item));
		}
		return SMultiColumnTableRow<TObjectPtr<ParameterType>>::OnDragDetected(MyGeometry, MouseEvent);
	}

	void OnParameterNameTextCommitted(const FText& Text, ETextCommit::Type CommitType)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameInterfaceParameterTransaction", "Rename Interface Parameter"));

		Item->Modify();
		Item->InterfaceParameterName = Text.ToString();
	}

	FText GetParameterToolTip() const
	{
		const FGuid Guid = Item->GetGuid();
		return FText::FromString(Guid.ToString(EGuidFormats::DigitsWithHyphensInParentheses));
	}

protected:

	TObjectPtr<ParameterType> Item;
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;
};

/**
 * List entry for the blendable parameters panel.
 */
class SCameraObjectInterfaceBlendableParameterTableRow : public SCameraObjectInterfaceParameterTableRowBase<UCameraObjectInterfaceBlendableParameter>
{
public:

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTable)
	{
		SCameraObjectInterfaceParameterTableRowBase<UCameraObjectInterfaceBlendableParameter>::Construct(Args, OwnerTable);
	}

protected:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == SCameraObjectInterfaceParametersPanel::ParameterTypeColumn)
		{
			TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_CameraNodeK2>();

			return SNew(SBox)
				.MinDesiredWidth(125.f)
				.Padding(this, &SCameraObjectInterfaceBlendableParameterTableRow::GetPinTypeSelectorPadding)
				[
					SNew(SPinTypeSelector, FGetPinTypeTree::CreateSP(this, &SCameraObjectInterfaceBlendableParameterTableRow::GetPinTypeTreeInfos))
					.OnPinTypeChanged(this, &SCameraObjectInterfaceBlendableParameterTableRow::OnBlendableParameterPinTypeChanged)
					.TargetPinType(this, &SCameraObjectInterfaceBlendableParameterTableRow::GetBlendableParameterPinType)
					.ReadOnly(this, &SCameraObjectInterfaceBlendableParameterTableRow::IsPinTypeSelectorReadOnly)
					.Schema(K2Schema)
					.bAllowArrays(false)
				];
		}

		return SCameraObjectInterfaceParameterTableRowBase<UCameraObjectInterfaceBlendableParameter>::GenerateWidgetForColumn(InColumnName);
	}

	FMargin GetPinTypeSelectorPadding() const
	{
		// Add some horizontal margin when the pin type selector is read-only, so that the transition on hover is seamless.
		return IsHovered() ? FMargin(0) : FMargin(7, 0);
	}

	void GetPinTypeTreeInfos(TArray<FPinTypeTreeItem>& TypeTree, ETypeTreeFilter TypeTreeFilter) const
	{
		using FPinTypeTreeInfo = UEdGraphSchema_K2::FPinTypeTreeInfo;

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_CameraNodeK2>();

		TypeTree.Reset();

		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::GetCategoryText(UEdGraphSchema_K2::PC_Boolean, true), UEdGraphSchema_K2::PC_Boolean, K2Schema, LOCTEXT("BooleanType", "True or false value")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::GetCategoryText(UEdGraphSchema_K2::PC_Int, true), UEdGraphSchema_K2::PC_Int, K2Schema, LOCTEXT("IntegerType", "Integer number")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::GetCategoryText(UEdGraphSchema_K2::PC_Float, true), UEdGraphSchema_K2::PC_Float, K2Schema, LOCTEXT("FloatType", "Floating point number")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::GetCategoryText(UEdGraphSchema_K2::PC_Double, true), UEdGraphSchema_K2::PC_Double, K2Schema, LOCTEXT("DoubleType", "Double precision floating point number")));

		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TVariantStructure<FVector2f>::Get(), LOCTEXT("Vector2fType", "A 2D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FVector2D>::Get(), LOCTEXT("Vector2dType", "A double precision 2D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TVariantStructure<FVector3f>::Get(), LOCTEXT("Vector3fType", "A 3D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FVector>::Get(), LOCTEXT("Vector3dType", "A double precision 3D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TVariantStructure<FVector4f>::Get(), LOCTEXT("Vector4fType", "A 4D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FVector4>::Get(), LOCTEXT("Vector4dType", "A double precision 4D vector")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TVariantStructure<FRotator3f>::Get(), LOCTEXT("Rotator3fType", "A 3D rotation")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FRotator>::Get(), LOCTEXT("Rotator3dType", "A double precision 3D rotation")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TVariantStructure<FTransform3f>::Get(), LOCTEXT("Transform3fType", "A 3D transformation")));
		TypeTree.Add(MakeShared<FPinTypeTreeInfo>(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FTransform>::Get(), LOCTEXT("Transform3dType", "A double precision 3D transformation")));

		TSharedPtr<FPinTypeTreeInfo> Structs = MakeShared<FPinTypeTreeInfo>(
				LOCTEXT("BlendableStructPinTypeLabel", "Blendable Structures"),
				UEdGraphSchema_K2::PC_Struct,
				K2Schema,
				LOCTEXT("BlendableStructPinTypeToolTip", "Blendable structure types"),
				true);
		IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
		for (const FBlendableStructInfo& BlendableStruct : GameplayCamerasModule.GetBlendableStructs())
		{
			if (const UScriptStruct* StructType = BlendableStruct.StructType)
			{
				Structs->Children.Add(
						MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(
							UEdGraphSchema_K2::PC_Struct,
							const_cast<UScriptStruct*>(StructType),
							StructType->GetToolTipText(),
							false,
							(uint8)EObjectReferenceType::NotAnObject));
			}
		}
		TypeTree.Add(Structs);
	}

	FEdGraphPinType GetBlendableParameterPinType() const
	{
		return FCameraVariablePinTypeHelper::GetPinType(
				Item->VariableType, Item->BlendableStructType.Get());
	}

	void OnBlendableParameterPinTypeChanged(const FEdGraphPinType& PinType)
	{
		bool bIsValidType = true;
		ECameraVariableType NewParameterType = ECameraVariableType::Boolean;

		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
		{
			NewParameterType = ECameraVariableType::Boolean;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Int)
		{
			NewParameterType = ECameraVariableType::Integer32;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Float)
		{
			NewParameterType = ECameraVariableType::Float;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Double)
		{
			NewParameterType = ECameraVariableType::Double;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			const UObject* TypeObject = PinType.PinSubCategoryObject.Get();
			if (TypeObject == TVariantStructure<FVector2f>::Get())
			{
				NewParameterType = ECameraVariableType::Vector2f;
			}
			else if (TypeObject == TBaseStructure<FVector2D>::Get())
			{
				NewParameterType = ECameraVariableType::Vector2d;
			}
			else if (TypeObject == TVariantStructure<FVector3f>::Get())
			{
				NewParameterType = ECameraVariableType::Vector3f;
			}
			else if (TypeObject == TBaseStructure<FVector>::Get())
			{
				NewParameterType = ECameraVariableType::Vector3d;
			}
			else if (TypeObject == TVariantStructure<FVector4f>::Get())
			{
				NewParameterType = ECameraVariableType::Vector4f;
			}
			else if (TypeObject == TBaseStructure<FVector4>::Get())
			{
				NewParameterType = ECameraVariableType::Vector4d;
			}
			else if (TypeObject == TVariantStructure<FRotator3f>::Get())
			{
				NewParameterType = ECameraVariableType::Rotator3f;
			}
			else if (TypeObject == TBaseStructure<FRotator>::Get())
			{
				NewParameterType = ECameraVariableType::Rotator3d;
			}
			else if (TypeObject == TVariantStructure<FTransform3f>::Get())
			{
				NewParameterType = ECameraVariableType::Transform3f;
			}
			else if (TypeObject == TBaseStructure<FTransform>::Get())
			{
				NewParameterType = ECameraVariableType::Transform3d;
			}
			else
			{
				bIsValidType = false;
			}
		}
		else
		{
			bIsValidType = false;
		}

		if (ensure(bIsValidType) && Item->VariableType != NewParameterType)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeBlendableParameterType", "Change Blendable Parameter Type"));

			Item->Modify();
			Item->VariableType = NewParameterType;
		}
	}

	bool IsPinTypeSelectorReadOnly() const
	{
		return !IsHovered();
	}
};

class FDataParameterPinTypeSelectorFilter : public IPinTypeSelectorFilter
{
	virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const override
	{
		// Remove types that are for blendable parameters.
		FEdGraphPinType PinType = InItem->GetPinType(false);
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean ||
				PinType.PinCategory == UEdGraphSchema_K2::PC_Int ||
				PinType.PinCategory == UEdGraphSchema_K2::PC_Int64 ||
				PinType.PinCategory == UEdGraphSchema_K2::PC_Float ||
				PinType.PinCategory == UEdGraphSchema_K2::PC_Double ||
				PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
		{
			return false;
		}

		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Byte && PinType.PinSubCategoryObject == nullptr)
		{
			return false;
		}

		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			const UObject* TypeObject = PinType.PinSubCategoryObject.Get();
			if (TypeObject == TVariantStructure<FVector2f>::Get() ||
					TypeObject == TBaseStructure<FVector2D>::Get() ||
					TypeObject == TVariantStructure<FVector3f>::Get() ||
					TypeObject == TBaseStructure<FVector>::Get() ||
					TypeObject == TVariantStructure<FVector4f>::Get() ||
					TypeObject == TBaseStructure<FVector4>::Get() ||
					TypeObject == TVariantStructure<FRotator3f>::Get() ||
					TypeObject == TBaseStructure<FRotator>::Get() ||
					TypeObject == TVariantStructure<FTransform3f>::Get() ||
					TypeObject == TBaseStructure<FTransform>::Get())
			{
				return false;
			}
		}
		
		return true;
	};
};

/**
 * List entry for the data parameters panel.
 */
class SCameraObjectInterfaceDataParameterTableRow : public SCameraObjectInterfaceParameterTableRowBase<UCameraObjectInterfaceDataParameter>
{
protected:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == SCameraObjectInterfaceParametersPanel::ParameterTypeColumn)
		{
			TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_CameraNodeK2>();
			TArray<TSharedPtr<IPinTypeSelectorFilter>> PinTypeSelectorFilters{ MakeShared<FDataParameterPinTypeSelectorFilter>() };

			return SNew(SBox)
				.MinDesiredWidth(125.f)
				.Padding(this, &SCameraObjectInterfaceDataParameterTableRow::GetPinTypeSelectorPadding)
				[
					SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(K2Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
					.OnPinTypeChanged(this, &SCameraObjectInterfaceDataParameterTableRow::OnDataParameterPinTypeChanged)
					.TargetPinType(this, &SCameraObjectInterfaceDataParameterTableRow::GetDataParameterPinType)
					.ReadOnly(this, &SCameraObjectInterfaceDataParameterTableRow::IsPinTypeSelectorReadOnly)
					.Schema(K2Schema)
					.CustomFilters(PinTypeSelectorFilters)
					.bAllowArrays(true)
				];
		}

		return SCameraObjectInterfaceParameterTableRowBase<UCameraObjectInterfaceDataParameter>::GenerateWidgetForColumn(InColumnName);
	}

	FMargin GetPinTypeSelectorPadding() const
	{
		// Add some horizontal margin when the pin type selector is read-only, so that the transition on hover is seamless.
		return IsHovered() ? FMargin(0) : FMargin(7, 0);
	}

	FEdGraphPinType GetDataParameterPinType() const
	{
		return FCameraContextDataPinTypeHelper::GetPinType(
				Item->DataType, Item->DataContainerType, Item->DataTypeObject.Get());
	}

	void OnDataParameterPinTypeChanged(const FEdGraphPinType& PinType)
	{
		bool bIsValidType = true;
		ECameraContextDataType NewDataType = ECameraContextDataType::Name;
		ECameraContextDataContainerType NewDataContainerType = ECameraContextDataContainerType::None;
		const UObject* NewDataTypeObject = PinType.PinSubCategoryObject.Get();

		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
		{
			NewDataType = ECameraContextDataType::Name;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_String)
		{
			NewDataType = ECameraContextDataType::String;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
		{
			NewDataType = ECameraContextDataType::Enum;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Byte && 
				PinType.PinSubCategoryObject != nullptr && PinType.PinSubCategoryObject->IsA<UEnum>())
		{
			NewDataType = ECameraContextDataType::Enum;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			NewDataType = ECameraContextDataType::Struct;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
		{
			NewDataType = ECameraContextDataType::Object;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
		{
			NewDataType = ECameraContextDataType::Class;
		}
		else
		{
			bIsValidType = false;
		}

		switch (PinType.ContainerType)
		{
			case EPinContainerType::None:
				NewDataContainerType = ECameraContextDataContainerType::None;
				break;
			case EPinContainerType::Array:
				NewDataContainerType = ECameraContextDataContainerType::Array;
				break;
			default:
				bIsValidType = false;
				break;
		}

		if (ensure(bIsValidType) && (NewDataType != Item->DataType || NewDataContainerType != Item->DataContainerType || NewDataTypeObject != Item->DataTypeObject))
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeDataParameterType", "Change Data Parameter Type"));

			Item->Modify();

			Item->DataType = NewDataType;
			Item->DataContainerType = NewDataContainerType;
			Item->DataTypeObject = NewDataTypeObject;
		}
	}

	bool IsPinTypeSelectorReadOnly() const
	{
		return !IsHovered();
	}
};

const FName SCameraObjectInterfaceParametersPanel::ParameterTypeColumn(TEXT("ParameterType"));
const FName SCameraObjectInterfaceParametersPanel::ParameterNameColumn(TEXT("ParameterName"));

void SCameraObjectInterfaceParametersPanel::Construct(const FArguments& Args, FCameraObjectInterfaceParametersToolkit* OwnerToolkit)
{
	CameraObject = OwnerToolkit->GetCameraObject();
	CameraObject->EventHandlers.Register(EventHandler, this);

	Toolkit = OwnerToolkit;

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)

		// The blendable parameters panel.
		+SSplitter::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(3, 5))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.Padding(5.f)
					[
						SNew(SRichTextBlock)
						.Text(LOCTEXT("BlendableParametersPanelTitle", "Blendable Parameters"))
						.TransformPolicy(ETextTransformPolicy::ToUpper)
						.DecoratorStyleSet(&FAppStyle::Get())
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("AddBlendableToolTip", "Add a blendable parameter"))
						.OnClicked(this, &SCameraObjectInterfaceParametersPanel::OnAddBlendableParameter)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("RemoveBlendableToolTip", "Removes the selected blendable parameter(s)"))
						.OnClicked(this, &SCameraObjectInterfaceParametersPanel::OnDeleteSelectedBlendableParameter)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.MinusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(BlendableParametersListView, SListView<TObjectPtr<UCameraObjectInterfaceBlendableParameter>>)
				.ListItemsSource(&CameraObject->Interface.BlendableParameters)
				.OnGenerateRow(this, &SCameraObjectInterfaceParametersPanel::OnGenerateBlendableParameterRow)
				.OnSelectionChanged(this, &SCameraObjectInterfaceParametersPanel::OnBlendableSelectionChanged)
				.OnContextMenuOpening(this, &SCameraObjectInterfaceParametersPanel::OnBlendableParameterContextMenuOpening)
				.HeaderRow
				(
					SNew(SHeaderRow)

					+SHeaderRow::Column(ParameterTypeColumn)
					.FillWidth(0.3f)
					.DefaultLabel(LOCTEXT("ParameterTypeColumnLabel", "Type"))

					+SHeaderRow::Column(ParameterNameColumn)
					.FillWidth(0.5f)
					.DefaultLabel(LOCTEXT("ParameterNameColumnLabel", "Name"))
				)
			]
		]

		// The data parameters panel.
		+SSplitter::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(3, 5))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.Padding(5.f)
					[
						SNew(SRichTextBlock)
						.Text(LOCTEXT("DataParametersPanelTitle", "Data Parameters"))
						.TransformPolicy(ETextTransformPolicy::ToUpper)
						.DecoratorStyleSet(&FAppStyle::Get())
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("AddDataParameterToolTip", "Add a data parameter"))
						.OnClicked(this, &SCameraObjectInterfaceParametersPanel::OnAddDataParameter)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("RemoveDataParameterToolTip", "Removes the selected data parameter(s)"))
						.OnClicked(this, &SCameraObjectInterfaceParametersPanel::OnDeleteSelectedDataParameter)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.MinusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(DataParametersListView, SListView<TObjectPtr<UCameraObjectInterfaceDataParameter>>)
				.ListItemsSource(&CameraObject->Interface.DataParameters)
				.OnGenerateRow(this, &SCameraObjectInterfaceParametersPanel::OnGenerateDataParameterRow)
				.OnSelectionChanged(this, &SCameraObjectInterfaceParametersPanel::OnDataParameterSelectionChanged)
				.OnContextMenuOpening(this, &SCameraObjectInterfaceParametersPanel::OnDataParameterContextMenuOpening)
				.HeaderRow
				(
					SNew(SHeaderRow)

					+SHeaderRow::Column(ParameterTypeColumn)
					.FillWidth(0.3f)
					.DefaultLabel(LOCTEXT("ParameterTypeColumnLabel", "Type"))

					+SHeaderRow::Column(ParameterNameColumn)
					.FillWidth(0.5f)
					.DefaultLabel(LOCTEXT("ParameterNameColumnLabel", "Name"))
				)
			]
		]
	];
}

void SCameraObjectInterfaceParametersPanel::RequestListRefresh()
{
	bListRefreshRequested = true;
}

void SCameraObjectInterfaceParametersPanel::RenameSelectedParameter()
{
	if (BlendableParametersListView->HasKeyboardFocus())
	{
		OnRenameSelectedBlendableParameter();
	}
	else if (DataParametersListView->HasKeyboardFocus())
	{
		OnRenameSelectedDataParameter();
	}
}

void SCameraObjectInterfaceParametersPanel::DeleteSelectedParameter()
{
	if (BlendableParametersListView->HasKeyboardFocus())
	{
		OnDeleteSelectedBlendableParameter();
	}
	else if (DataParametersListView->HasKeyboardFocus())
	{
		OnDeleteSelectedDataParameter();
	}
}

void SCameraObjectInterfaceParametersPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bListRefreshRequested)
	{
		bListRefreshRequested = false;

		BlendableParametersListView->RequestListRefresh();
		DataParametersListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SCameraObjectInterfaceParametersPanel::OnGenerateBlendableParameterRow(TObjectPtr<UCameraObjectInterfaceBlendableParameter> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCameraObjectInterfaceBlendableParameterTableRow, OwnerTable)
		.Item(Item);
}

TSharedRef<ITableRow> SCameraObjectInterfaceParametersPanel::OnGenerateDataParameterRow(TObjectPtr<UCameraObjectInterfaceDataParameter> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCameraObjectInterfaceDataParameterTableRow, OwnerTable)
		.Item(Item);
}

void SCameraObjectInterfaceParametersPanel::OnBlendableSelectionChanged(TObjectPtr<UCameraObjectInterfaceBlendableParameter> Item, ESelectInfo::Type Type)
{
	Toolkit->OnInterfaceParameterSelected().Broadcast(Item);
}

void SCameraObjectInterfaceParametersPanel::OnDataParameterSelectionChanged(TObjectPtr<UCameraObjectInterfaceDataParameter> Item, ESelectInfo::Type Type)
{
	Toolkit->OnInterfaceParameterSelected().Broadcast(Item);
}

TSharedPtr<SWidget> SCameraObjectInterfaceParametersPanel::OnBlendableParameterContextMenuOpening()
{
	TArray<TObjectPtr<UCameraObjectInterfaceBlendableParameter>> SelectedItems = BlendableParametersListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return OnInterfaceParameterContextMenuOpening(BlendableParametersListView, SelectedItems[0]);
	}
	return nullptr;
}

TSharedPtr<SWidget> SCameraObjectInterfaceParametersPanel::OnDataParameterContextMenuOpening()
{
	TArray<TObjectPtr<UCameraObjectInterfaceDataParameter>> SelectedItems = DataParametersListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return OnInterfaceParameterContextMenuOpening(DataParametersListView, SelectedItems[0]);
	}
	return nullptr;
}

void SCameraObjectInterfaceParametersPanel::OnSearchInterfaceParameter(TObjectPtr<UCameraObjectInterfaceParameterBase> Item)
{
	Toolkit->OnSearchInterfaceParameterNodes().Broadcast(Item);
}

template<typename ItemType>
TSharedPtr<SWidget> SCameraObjectInterfaceParametersPanel::OnInterfaceParameterContextMenuOpening(
		TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("InterfaceParameterActionsSection", "Actions"));
	{
		TObjectPtr<UCameraObjectInterfaceParameterBase> ItemBase(Item);
		MenuBuilder.AddMenuEntry(
				LOCTEXT("SearchInterfaceParameterNodes", "Find References"),
				LOCTEXT("SearchInterfaceParameterNodesToolTip", "Find references to this interface parameter"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
				FExecuteAction::CreateSP(this, &SCameraObjectInterfaceParametersPanel::OnSearchInterfaceParameter, ItemBase));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("InterfaceParameterEditSection", "Edit Interface Parameter"));
	{
		MenuBuilder.AddMenuEntry(
				LOCTEXT("RenameInterfaceParameter", "Rename"),
				LOCTEXT("RenameInterfaceParameterToolTip", "Renames this interface parameter"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
				FExecuteAction::CreateSP(this, &SCameraObjectInterfaceParametersPanel::OnRenameInterfaceParameter, ListView, Item));
		MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteInterfaceParameter", "Delete"),
				LOCTEXT("DeleteInterfaceParameterToolTip", "Deletes this interface parameter"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
				FExecuteAction::CreateSP(this, &SCameraObjectInterfaceParametersPanel::OnDeleteInterfaceParameter, ListView, Item));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

template<typename ItemType>
void SCameraObjectInterfaceParametersPanel::OnRenameInterfaceParameter(
		TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item)
{
	TSharedPtr<ITableRow> RowWidget = ListView->WidgetFromItem(Item);
	if (!ensure(RowWidget))
	{
		return;
	}

	TSharedPtr<SCameraObjectInterfaceParameterTableRowBase<ItemType>> TypedRowWidget = 
		StaticCastSharedPtr<SCameraObjectInterfaceParameterTableRowBase<ItemType>>(RowWidget);
	TypedRowWidget->EnterNameEditingMode();
}

template<typename ItemType>
void SCameraObjectInterfaceParametersPanel::OnDeleteInterfaceParameter(
		TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveInterfaceParameter", "Remove Interface Parameter"));

	CameraObject->Modify();

	if constexpr (std::is_same_v<ItemType, UCameraObjectInterfaceBlendableParameter>)
	{
		const int32 NumRemoved = CameraObject->Interface.BlendableParameters.Remove(Item);
		ensure(NumRemoved == 1);
	}
	else if constexpr (std::is_same_v<ItemType, UCameraObjectInterfaceDataParameter>)
	{
		const int32 NumRemoved = CameraObject->Interface.DataParameters.Remove(Item);
		ensure(NumRemoved == 1);
	}

	CameraObject->EventHandlers.Notify(&ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	ListView->RequestListRefresh();
}

template<typename ItemType>
void SCameraObjectInterfaceParametersPanel::OnRenameSelectedInterfaceParameter(TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView)
{
	TArray<TObjectPtr<ItemType>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		OnRenameInterfaceParameter(ListView, SelectedItems[0]);
	}
}

template<typename ItemType>
void SCameraObjectInterfaceParametersPanel::OnDeleteSelectedInterfaceParameter(TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView)
{
	TArray<TObjectPtr<ItemType>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveInterfaceParameter", "Remove Interface Parameter"));

	CameraObject->Modify();

	for (TObjectPtr<ItemType> Item : SelectedItems)
	{
		if constexpr (std::is_same_v<ItemType, UCameraObjectInterfaceBlendableParameter>)
		{
			const int32 NumRemoved = CameraObject->Interface.BlendableParameters.Remove(Item);
			ensure(NumRemoved == 1);
		}
		else if constexpr (std::is_same_v<ItemType, UCameraObjectInterfaceDataParameter>)
		{
			const int32 NumRemoved = CameraObject->Interface.DataParameters.Remove(Item);
			ensure(NumRemoved == 1);
		}
	}

	CameraObject->EventHandlers.Notify(&ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	ListView->RequestListRefresh();
}

FReply SCameraObjectInterfaceParametersPanel::OnAddBlendableParameter()
{
	const FScopedTransaction Transaction(LOCTEXT("AddBlendableParameter", "Add Blendable Parameter"));

	UCameraObjectInterfaceBlendableParameter* NewBlendableParameter = NewObject<UCameraObjectInterfaceBlendableParameter>(CameraObject, NAME_None, RF_Transactional);
	NewBlendableParameter->InterfaceParameterName = GetNewParameterName();

	CameraObject->Modify();
	CameraObject->Interface.BlendableParameters.Add(NewBlendableParameter);
	CameraObject->EventHandlers.Notify(&ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	return FReply::Handled();
}

FReply SCameraObjectInterfaceParametersPanel::OnAddDataParameter()
{
	const FScopedTransaction Transaction(LOCTEXT("AddDataParameter", "Add Data Parameter"));

	UCameraObjectInterfaceDataParameter* NewDataParameter = NewObject<UCameraObjectInterfaceDataParameter>(CameraObject, NAME_None, RF_Transactional);
	NewDataParameter->InterfaceParameterName = GetNewParameterName();

	CameraObject->Modify();
	CameraObject->Interface.DataParameters.Add(NewDataParameter);
	CameraObject->EventHandlers.Notify(&ICameraObjectEventHandler::OnCameraObjectInterfaceChanged);

	return FReply::Handled();
}

void SCameraObjectInterfaceParametersPanel::OnRenameSelectedBlendableParameter()
{
	OnRenameSelectedInterfaceParameter(BlendableParametersListView);
}

void SCameraObjectInterfaceParametersPanel::OnRenameSelectedDataParameter()
{
	OnRenameSelectedInterfaceParameter(DataParametersListView);
}

FReply SCameraObjectInterfaceParametersPanel::OnDeleteSelectedBlendableParameter()
{
	OnDeleteSelectedInterfaceParameter(BlendableParametersListView);
	return FReply::Handled();
}

FReply SCameraObjectInterfaceParametersPanel::OnDeleteSelectedDataParameter()
{
	OnDeleteSelectedInterfaceParameter(DataParametersListView);
	return FReply::Handled();
}

FString SCameraObjectInterfaceParametersPanel::GetNewParameterName() const
{
	static const FString BaseNewParameterName(TEXT("NewParameter"));

	int32 NameIndex = 0;
	bool bFoundUniqueName = false;
	FString NewParameterName(BaseNewParameterName);

	TArray<UCameraObjectInterfaceParameterBase*> AllParameters;
	AllParameters.Append(CameraObject->Interface.BlendableParameters);
	AllParameters.Append(CameraObject->Interface.DataParameters);

	while (!bFoundUniqueName)
	{
		const bool bFoundConflict = AllParameters.ContainsByPredicate(
				[&NewParameterName](const UCameraObjectInterfaceParameterBase* Item)
				{
					return Item->InterfaceParameterName == NewParameterName;
				});
		bFoundUniqueName = !bFoundConflict;
		if (bFoundConflict)
		{
			++NameIndex;
			NewParameterName = FString::Printf(TEXT("%s_%d"), *BaseNewParameterName, NameIndex);
		}
	}

	return NewParameterName;
}

void SCameraObjectInterfaceParametersPanel::OnCameraObjectInterfaceChanged()
{
	bListRefreshRequested = true;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

