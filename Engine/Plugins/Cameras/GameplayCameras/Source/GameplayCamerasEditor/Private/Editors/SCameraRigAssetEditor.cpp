// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraRigAssetEditor.h"

#include "Core/CameraRigAsset.h"
#include "Core/CameraRigTransition.h"
#include "Editors/CameraObjectInterfaceParameterGraphNode.h"
#include "Editors/CameraRigCameraNodeGraphSchema.h"
#include "Editors/CameraRigTransitionGraphSchema.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/SCameraNodeGraphEditor.h"
#include "Editors/SObjectTreeGraphEditor.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SCameraRigAssetEditor"

namespace UE::Cameras
{

void SCameraRigAssetEditor::Construct(const FArguments& InArgs)
{
	CameraRigAsset = InArgs._CameraRigAsset;
	DetailsView = InArgs._DetailsView;
	AssetEditorToolkit = InArgs._AssetEditorToolkit;

	CurrentMode = ECameraRigAssetEditorMode::NodeGraph;

	CreateGraphEditors();

	ChildSlot
	[
		SAssignNew(BoxPanel, SBox)
		[
			NodeGraphEditor.ToSharedRef()
		]
	];
}

SCameraRigAssetEditor::~SCameraRigAssetEditor()
{
	if (!GExitPurge)
	{
		DiscardGraphEditors();
	}
}

void SCameraRigAssetEditor::SetCameraRigAsset(UCameraRigAsset* InCameraRig)
{
	if (CameraRigAsset != InCameraRig)
	{
		DiscardGraphEditors();

		CameraRigAsset = InCameraRig;

		CreateGraphEditors();

		SetEditorModeImpl(CurrentMode, true);
	}
}

void SCameraRigAssetEditor::CreateGraphEditors()
{
	CreateNodeGraphEditor();
	CreateTransitionGraphEditor();
}

void SCameraRigAssetEditor::CreateNodeGraphEditor()
{
	UClass* SchemaClass = UCameraRigCameraNodeGraphSchema::StaticClass();
	UCameraRigCameraNodeGraphSchema* DefaultSchemaObject = Cast<UCameraRigCameraNodeGraphSchema>(SchemaClass->GetDefaultObject());
	FObjectTreeGraphConfig GraphConfig = DefaultSchemaObject->BuildGraphConfig();

	NodeGraph = NewObject<UObjectTreeGraph>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Standalone);
	NodeGraph->Schema = SchemaClass;
	NodeGraph->Reset(CameraRigAsset, GraphConfig);

	NodeGraphChangedHandle = NodeGraph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &SCameraRigAssetEditor::OnGraphChanged));

	FGraphAppearanceInfo Appearance;
	Appearance.CornerText = LOCTEXT("CameraRigGraphText", "CAMERA NODES");

	NodeGraphEditor = SNew(SCameraNodeGraphEditor)
		.Appearance(Appearance)
		.DetailsView(DetailsView)
		.GraphTitle(this, &SCameraRigAssetEditor::GetCameraRigAssetName, NodeGraph.Get())
		.IsEnabled(this, &SCameraRigAssetEditor::IsGraphEditorEnabled)
		.GraphToEdit(NodeGraph)
		.AssetEditorToolkit(AssetEditorToolkit);
	NodeGraphEditor->RegisterEditor();
}

void SCameraRigAssetEditor::CreateTransitionGraphEditor()
{
	UClass* SchemaClass = UCameraRigTransitionGraphSchema::StaticClass();
	UCameraRigTransitionGraphSchema* DefaultSchemaObject = Cast<UCameraRigTransitionGraphSchema>(SchemaClass->GetDefaultObject());
	FObjectTreeGraphConfig GraphConfig = DefaultSchemaObject->BuildGraphConfig();

	TransitionGraph = NewObject<UObjectTreeGraph>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Standalone);
	TransitionGraph->Schema = SchemaClass;
	TransitionGraph->Reset(CameraRigAsset, GraphConfig);

	TransitionGraphChangedHandle = TransitionGraph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &SCameraRigAssetEditor::OnGraphChanged));

	FGraphAppearanceInfo Appearance;
	Appearance.CornerText = LOCTEXT("TransitionGraphText", "TRANSITIONS");

	TransitionGraphEditor = SNew(SObjectTreeGraphEditor)
		.Appearance(Appearance)
		.DetailsView(DetailsView)
		.GraphTitle(this, &SCameraRigAssetEditor::GetCameraRigAssetName, TransitionGraph.Get())
		.IsEnabled(this, &SCameraRigAssetEditor::IsGraphEditorEnabled)
		.GraphToEdit(TransitionGraph)
		.AssetEditorToolkit(AssetEditorToolkit);
	TransitionGraphEditor->RegisterEditor();
}

void SCameraRigAssetEditor::DiscardGraphEditors()
{
	TArray<TTuple<UObjectTreeGraph*, FDelegateHandle>> Graphs 
	{ 
		{ NodeGraph, NodeGraphChangedHandle },
		{ TransitionGraph, TransitionGraphChangedHandle } 
	};
	for (auto& Pair : Graphs)
	{
		UObjectTreeGraph* Graph(Pair.Key);
		FDelegateHandle GraphChangedHandle(Pair.Value);
		if (Graph)
		{
			Graph->RemoveFromRoot();

			if (GraphChangedHandle.IsValid())
			{
				Graph->RemoveOnGraphChangedHandler(GraphChangedHandle);
			}
		}
	}

	NodeGraphEditor->UnregisterEditor();
	TransitionGraphEditor->UnregisterEditor();

	NodeGraphChangedHandle.Reset();
	TransitionGraphChangedHandle.Reset();

	// WARNING: the graph editors (and their graphs) are still in use as widgets 
	//			in the layout until they are replaced!
}

ECameraRigAssetEditorMode SCameraRigAssetEditor::GetEditorMode() const
{
	return CurrentMode;
}

bool SCameraRigAssetEditor::IsEditorMode(ECameraRigAssetEditorMode InMode) const
{
	return CurrentMode == InMode;
}

void SCameraRigAssetEditor::SetEditorMode(ECameraRigAssetEditorMode InMode)
{
	SetEditorModeImpl(InMode, false);
}

void SCameraRigAssetEditor::SetEditorModeImpl(ECameraRigAssetEditorMode InMode, bool bForceSet)
{
	if (bForceSet || InMode != CurrentMode)
	{
		TSharedPtr<SObjectTreeGraphEditor> CurrentGraphEditor;
		switch(InMode)
		{
			case ECameraRigAssetEditorMode::NodeGraph:
			default:
				CurrentGraphEditor = NodeGraphEditor;
				break;
			case ECameraRigAssetEditorMode::TransitionGraph:
				CurrentGraphEditor = TransitionGraphEditor;
				break;
		}

		BoxPanel->SetContent(CurrentGraphEditor.ToSharedRef());
		CurrentGraphEditor->ResyncDetailsView();
		CurrentMode = InMode;
	}
}

void SCameraRigAssetEditor::GetGraphs(TArray<UEdGraph*>& OutGraphs) const
{
	OutGraphs.Add(NodeGraph);
	OutGraphs.Add(TransitionGraph);
}

UEdGraph* SCameraRigAssetEditor::GetFocusedGraph() const
{
	switch (CurrentMode)
	{
		case ECameraRigAssetEditorMode::NodeGraph:
			return NodeGraph;
		case ECameraRigAssetEditorMode::TransitionGraph:
			return TransitionGraph;
		default:
			ensure(false);
			return nullptr;
	}
}

const FObjectTreeGraphConfig& SCameraRigAssetEditor::GetFocusedGraphConfig() const
{
	static const FObjectTreeGraphConfig DefaultConfig;

	switch (CurrentMode)
	{
		case ECameraRigAssetEditorMode::NodeGraph:
			return NodeGraph->GetConfig();
		case ECameraRigAssetEditorMode::TransitionGraph:
			return TransitionGraph->GetConfig();
		default:
			ensure(false);
			return DefaultConfig;
	}
}

void SCameraRigAssetEditor::FocusHome()
{
	FindAndJumpToObjectNode(CameraRigAsset, CurrentMode);
}

bool SCameraRigAssetEditor::JumpToNode(UEdGraphNode* InNode, FName InPinName)
{
	UEdGraph* OwningGraph = InNode->GetGraph();
	if (OwningGraph == NodeGraph)
	{
		SetEditorMode(ECameraRigAssetEditorMode::NodeGraph);
		NodeGraphEditor->JumpToNode(InNode, InPinName);
		return true;
	}
	else if (OwningGraph == TransitionGraph)
	{
		SetEditorMode(ECameraRigAssetEditorMode::TransitionGraph);
		TransitionGraphEditor->JumpToNode(InNode, InPinName);
		return true;
	}
	return false;
}

bool SCameraRigAssetEditor::FindAndJumpToObjectNode(UObject* InObject)
{
	if (FindAndJumpToObjectNode(InObject, ECameraRigAssetEditorMode::NodeGraph))
	{
		SetEditorMode(ECameraRigAssetEditorMode::NodeGraph);
		return true;
	}
	if (FindAndJumpToObjectNode(InObject, ECameraRigAssetEditorMode::TransitionGraph))
	{
		SetEditorMode(ECameraRigAssetEditorMode::TransitionGraph);
		return true;
	}
	return false;
}

bool SCameraRigAssetEditor::FindAndJumpToObjectNode(UObject* InObject, ECameraRigAssetEditorMode InEditorMode)
{
	UObjectTreeGraph* FocusGraph = nullptr;
	TSharedPtr<SObjectTreeGraphEditor> FocusGraphEditor = nullptr;

	switch (InEditorMode)
	{
		case ECameraRigAssetEditorMode::NodeGraph:
			FocusGraph = NodeGraph;
			FocusGraphEditor = NodeGraphEditor;
			break;
		case ECameraRigAssetEditorMode::TransitionGraph:
			FocusGraph = TransitionGraph;
			FocusGraphEditor = TransitionGraphEditor;
	}

	if (FocusGraph && FocusGraphEditor)
	{
		if (UObjectTreeGraphNode* NodeGraphObjectNode = FocusGraph->FindObjectNode(InObject))
		{
			FocusGraphEditor->JumpToNode(NodeGraphObjectNode);
			return true;
		}
	}
	return false;
}

FText SCameraRigAssetEditor::GetCameraRigAssetName(UObjectTreeGraph* ForGraph) const
{
	if (CameraRigAsset && ForGraph)
	{
		const FObjectTreeGraphConfig& GraphConfig = ForGraph->GetConfig();
		return GraphConfig.GetDisplayNameText(CameraRigAsset);
	}
	return LOCTEXT("NoCameraRig", "No Camera Rig");
}

bool SCameraRigAssetEditor::IsGraphEditorEnabled() const
{
	return CameraRigAsset != nullptr;
}

void SCameraRigAssetEditor::OnGraphChanged(const FEdGraphEditAction& InEditAction)
{
	OnAnyGraphChanged.Broadcast(InEditAction);
}

FDelegateHandle SCameraRigAssetEditor::AddOnAnyGraphChanged(FOnGraphChanged::FDelegate InAddDelegate)
{
	return OnAnyGraphChanged.Add(InAddDelegate);
}

void SCameraRigAssetEditor::RemoveOnAnyGraphChanged(FDelegateHandle InDelegateHandle)
{
	if (InDelegateHandle.IsValid())
	{
		OnAnyGraphChanged.Remove(InDelegateHandle);
	}
}

void SCameraRigAssetEditor::RemoveOnAnyGraphChanged(FDelegateUserObjectConst InUserObject)
{
	OnAnyGraphChanged.RemoveAll(InUserObject);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

