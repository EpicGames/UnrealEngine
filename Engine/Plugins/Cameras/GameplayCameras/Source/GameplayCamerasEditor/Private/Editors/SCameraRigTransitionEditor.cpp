// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraRigTransitionEditor.h"

#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigTransition.h"
#include "EdGraph/EdGraphPin.h"
#include "Editors/CameraRigTransitionGraphSchema.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/SObjectTreeGraphEditor.h"
#include "GameplayCamerasEditorSettings.h"
#include "ObjectEditorUtils.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SCameraRigTransitionEditor"

namespace UE::Cameras
{

void SCameraRigTransitionEditor::Construct(const FArguments& InArgs)
{
	TransitionOwner = InArgs._TransitionOwner;
	TransitionGraphSchemaClass = InArgs._TransitionGraphSchemaClass;
	DetailsView = InArgs._DetailsView;
	AssetEditorToolkit = InArgs._AssetEditorToolkit;
	TransitionGraphEditorAppearance = InArgs._TransitionGraphEditorAppearance;

	CreateTransitionGraphEditor();

	ChildSlot
	[
		SAssignNew(BoxPanel, SBox)
		[
			TransitionGraphEditor.ToSharedRef()
		]
	];
}

SCameraRigTransitionEditor::~SCameraRigTransitionEditor()
{
	if (!GExitPurge)
	{
		DiscardTransitionGraphEditor();
	}
}

void SCameraRigTransitionEditor::SetTransitionOwner(UObject* InTransitionOwner)
{
	if (TransitionOwner != InTransitionOwner)
	{
		DiscardTransitionGraphEditor();

		TransitionOwner = InTransitionOwner;

		CreateTransitionGraphEditor();

		BoxPanel->SetContent(TransitionGraphEditor.ToSharedRef());
		TransitionGraphEditor->ResyncDetailsView();
	}
}

void SCameraRigTransitionEditor::CreateTransitionGraphEditor()
{
	UCameraRigTransitionGraphSchemaBase* DefaultSchemaObject = Cast<UCameraRigTransitionGraphSchemaBase>(TransitionGraphSchemaClass->GetDefaultObject());
	FObjectTreeGraphConfig GraphConfig = DefaultSchemaObject->BuildGraphConfig();

	TransitionGraph = NewObject<UObjectTreeGraph>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Standalone);
	TransitionGraph->Schema = TransitionGraphSchemaClass;
	TransitionGraph->Reset(TransitionOwner, GraphConfig);

	TransitionGraphChangedHandle = TransitionGraph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &SCameraRigTransitionEditor::OnGraphChanged));

	FGraphAppearanceInfo Appearance;
	Appearance.CornerText = LOCTEXT("TransitionGraphText", "TRANSITIONS");
	if (TransitionGraphEditorAppearance.IsSet())
	{
		Appearance = TransitionGraphEditorAppearance.Get();
	}

	TransitionGraphEditor = SNew(SObjectTreeGraphEditor)
		.Appearance(Appearance)
		.DetailsView(DetailsView)
		.GraphTitle(this, &SCameraRigTransitionEditor::GetTransitionOwnerName)
		.GraphToEdit(TransitionGraph)
		.AssetEditorToolkit(AssetEditorToolkit);
	TransitionGraphEditor->RegisterEditor();
}

void SCameraRigTransitionEditor::DiscardTransitionGraphEditor()
{
	if (TransitionGraph)
	{
		TransitionGraph->RemoveFromRoot();

		if (TransitionGraphChangedHandle.IsValid())
		{
			TransitionGraph->RemoveOnGraphChangedHandler(TransitionGraphChangedHandle);
		}
	}

	TransitionGraphEditor->UnregisterEditor();
	TransitionGraph = nullptr;
	TransitionGraphChangedHandle.Reset();

	// WARNING: the graph editor (and its graph) is still in use as a widget in the layout 
	//			until it is replaced!
}

void SCameraRigTransitionEditor::OnGraphChanged(const FEdGraphEditAction& InEditAction)
{
	OnTransitionGraphChanged.Broadcast(InEditAction);
}

FDelegateHandle SCameraRigTransitionEditor::AddOnGraphChanged(FOnGraphChanged::FDelegate InAddDelegate)
{
	return OnTransitionGraphChanged.Add(InAddDelegate);
}

void SCameraRigTransitionEditor::RemoveOnGraphChanged(FDelegateHandle InDelegateHandle)
{
	OnTransitionGraphChanged.Remove(InDelegateHandle);
}

void SCameraRigTransitionEditor::RemoveOnGraphChanged(FDelegateUserObjectConst InUserObject)
{
	OnTransitionGraphChanged.RemoveAll(InUserObject);
}

UEdGraph* SCameraRigTransitionEditor::GetTransitionGraph() const
{
	return TransitionGraph;
}

const FObjectTreeGraphConfig& SCameraRigTransitionEditor::GetTransitionGraphConfig() const
{
	return TransitionGraph->GetConfig();
}

void SCameraRigTransitionEditor::FocusHome()
{
	if (UObjectTreeGraphNode* RootObjectNode = TransitionGraph->GetRootObjectNode())
	{
		JumpToNode(RootObjectNode);
	}
}

void SCameraRigTransitionEditor::JumpToNode(UEdGraphNode* InGraphNode, FName InPinName)
{
	if (InGraphNode)
	{
		TransitionGraphEditor->JumpToNode(InGraphNode, InPinName);
	}
}

bool SCameraRigTransitionEditor::FindAndJumpToObjectNode(UObject* InObject)
{
	if (UObjectTreeGraphNode* NodeGraphObjectNode = TransitionGraph->FindObjectNode(InObject))
	{
		TransitionGraphEditor->JumpToNode(NodeGraphObjectNode);
		return true;
	}
	return false;
}

FText SCameraRigTransitionEditor::GetTransitionOwnerName() const
{
	if (TransitionOwner && TransitionGraph)
	{
		const FObjectTreeGraphConfig& GraphConfig = TransitionGraph->GetConfig();
		return GraphConfig.GetDisplayNameText(TransitionOwner);
	}
	return LOCTEXT("NoTransitionOwner", "No Transition Owner");
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

