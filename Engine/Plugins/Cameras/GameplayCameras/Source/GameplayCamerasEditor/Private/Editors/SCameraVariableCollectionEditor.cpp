// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraVariableCollectionEditor.h"

#include "Commands/CameraVariableCollectionEditorCommands.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableCollection.h"
#include "IDetailsView.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenus.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCameraVariableCollectionEditor"

namespace UE::Cameras
{

class SCameraVariableCollectionListRow : public SMultiColumnTableRow<UCameraVariableAsset*>
{
public:

	using FSuperRowType = SMultiColumnTableRow<UCameraVariableAsset*>;

	SLATE_BEGIN_ARGS(SCameraVariableCollectionListRow)
		: _CameraVariable(nullptr)
	{}
		/** The camera variable corresponding to this entry. */
		SLATE_ARGUMENT(UCameraVariableAsset*, CameraVariable)
		/** Text to highlight if a search is ongoing. */
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	void EnterNameEditingMode(FSimpleDelegate InOnTextCommitted);

protected:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	FText GetVariableName() const;
	FText GetVariableType() const;
	FText GetDefaultValue() const;

	bool OnVerifyVariableNameChanged(const FText& Text, FText& OutErrorMessage);
	void OnVariableNameCommitted(const FText& Text, ETextCommit::Type CommitType);

private:

	UCameraVariableAsset* CameraVariable = nullptr;

	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;
	FSimpleDelegate OnTextComitted;

	TAttribute<FText> HighlightText;
};

void SCameraVariableCollectionListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	CameraVariable = InArgs._CameraVariable;
	HighlightText = InArgs._HighlightText;

	FSuperRowType::Construct(
		FSuperRowType::FArguments().Padding(1.0f),
		OwnerTableView);
}

void SCameraVariableCollectionListRow::EnterNameEditingMode(FSimpleDelegate InOnTextCommitted)
{
	OnTextComitted = InOnTextCommitted;
	EditableTextBlock->EnterEditingMode();
}

TSharedRef<SWidget> SCameraVariableCollectionListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();

	if (ColumnName == TEXT("VariableName"))
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
				.Style(CamerasEditorStyle, "CameraVariableCollectionEditor.Entry.Name")
				.Text(this, &SCameraVariableCollectionListRow::GetVariableName)
				.OnTextCommitted(this, &SCameraVariableCollectionListRow::OnVariableNameCommitted)
				.OnVerifyTextChanged(this, &SCameraVariableCollectionListRow::OnVerifyVariableNameChanged)
				.HighlightText(HighlightText)
				.IsSelected(this, &FSuperRowType::IsSelectedExclusively)
			];
	}
	else if (ColumnName == TEXT("VariableType"))
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(CamerasEditorStyle, "CameraVariableCollectionEditor.Entry.Type")
				.Text(this, &SCameraVariableCollectionListRow::GetVariableType)
				.HighlightText(HighlightText)
			];
	}
	else if (ColumnName == TEXT("DefaultValue"))
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(CamerasEditorStyle, "CameraVariableCollectionEditor.Entry.Value")
				.Text(this, &SCameraVariableCollectionListRow::GetDefaultValue)
			];
	}
	return SNullWidget::NullWidget;
}

FText SCameraVariableCollectionListRow::GetVariableName() const
{
	return CameraVariable->DisplayName.IsEmpty() ?
		FText::FromName(CameraVariable->GetFName()) :
		FText::FromString(CameraVariable->DisplayName);
}

FText SCameraVariableCollectionListRow::GetVariableType() const
{
	FString ClassName = CameraVariable->GetClass()->GetName();
	ClassName.RemoveFromEnd(TEXT("CameraVariable"));
	return FText::FromString(ClassName);
}

FText SCameraVariableCollectionListRow::GetDefaultValue() const
{
	return FText::FromString(CameraVariable->FormatDefaultValue());
}

bool SCameraVariableCollectionListRow::OnVerifyVariableNameChanged(const FText& Text, FText& OutErrorMessage)
{
	return true;
}

void SCameraVariableCollectionListRow::OnVariableNameCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CameraVariable)
	{
		FScopedTransaction Transaction(LOCTEXT("RenameCameraVariable", "Rename camera variable"));

		const FString NewDisplayName = Text.ToString();
		CameraVariable->Modify();
		CameraVariable->DisplayName = NewDisplayName;
	}

	if (OnTextComitted.IsBound())
	{
		OnTextComitted.Execute();
		OnTextComitted.Unbind();
	}
}

void SCameraVariableCollectionEditor::Construct(const FArguments& InArgs)
{
	VariableCollection = InArgs._VariableCollection;

	WeakDetailsView = InArgs._DetailsView;

	CommandList = MakeShared<FUICommandList>();
	if (InArgs._AdditionalCommands)
	{
		CommandList->Append(InArgs._AdditionalCommands.ToSharedRef());
	}

	SearchTextFilter = MakeShareable(new FEntryTextFilter(
				FEntryTextFilter::FItemToStringArray::CreateSP(this, &SCameraVariableCollectionEditor::GetEntryStrings)));

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Search"))
				.OnTextChanged(this, &SCameraVariableCollectionEditor::OnSearchTextChanged)
				.OnTextCommitted(this, &SCameraVariableCollectionEditor::OnSearchTextCommitted)
			]
		]

		+SVerticalBox::Slot()
		.Padding(0, 3.f)
		[
			SAssignNew(ListView, SListView<UCameraVariableAsset*>)
			.ListItemsSource(&FilteredItemSource)
			.OnGenerateRow(this, &SCameraVariableCollectionEditor::OnListGenerateRow)
			.OnSelectionChanged(this, &SCameraVariableCollectionEditor::OnListSectionChanged)
			.OnItemScrolledIntoView(this, &SCameraVariableCollectionEditor::OnListItemScrolledIntoView)
			.OnContextMenuOpening(this, &SCameraVariableCollectionEditor::OnListContextMenuOpening)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+SHeaderRow::Column("VariableName")
				.DefaultLabel(LOCTEXT("VariableNameColumn", "Variable Name"))
				.FillWidth(0.4f)

				+SHeaderRow::Column("VariableType")
				.DefaultLabel(LOCTEXT("VariableTypeColumn", "Variable Type"))
				.FillWidth(0.3f)

				+SHeaderRow::Column("DefaultValue")
				.DefaultLabel(LOCTEXT("DefaultValueColumn", "Default Value"))
				.FillWidth(0.3f)
			)
		]
	];

	bUpdateFilteredItemSource = true;

	SetDetailsViewObject(nullptr);
}

SCameraVariableCollectionEditor::~SCameraVariableCollectionEditor()
{
}

void SCameraVariableCollectionEditor::GetSelectedVariables(TArray<UCameraVariableAsset*>& OutSelection) const
{
	ListView->GetSelectedItems(OutSelection);
}

void SCameraVariableCollectionEditor::SelectVariable(UCameraVariableAsset* InItem)
{
	ListView->SetSelection(InItem, ESelectInfo::Direct);
}

void SCameraVariableCollectionEditor::RequestRenameVariable(UCameraVariableAsset* InItem, FSimpleDelegate InOnRenamedItem)
{
	bDeferredRequestRenameItem = true;
	OnDeferredRenamedItem = InOnRenamedItem;
	ListView->RequestScrollIntoView(InItem);
}

void SCameraVariableCollectionEditor::RequestRenameSelectedVariable()
{
	TArray<UCameraVariableAsset*> SelectedVariables = ListView->GetSelectedItems();
	if (SelectedVariables.IsEmpty())
	{
		return;
	}

	bDeferredRequestRenameItem = true;
	ListView->RequestScrollIntoView(SelectedVariables[0]);
}

void SCameraVariableCollectionEditor::RequestListRefresh()
{
	bUpdateFilteredItemSource = true;
}

void SCameraVariableCollectionEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bUpdateFilteredItemSource)
	{
		UpdateFilteredItemSource();
		ListView->RequestListRefresh();
	}
	bUpdateFilteredItemSource = false;

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SCameraVariableCollectionEditor::UpdateFilteredItemSource()
{
	FilteredItemSource.Reset();

	if (!SearchTextFilter->GetRawFilterText().IsEmpty())
	{
		for (UCameraVariableAsset* Item : VariableCollection->Variables)
		{
			if (SearchTextFilter->PassesFilter(Item))
			{
				FilteredItemSource.Add(Item);
			}
		}
	}
	else
	{
		FilteredItemSource = VariableCollection->Variables;
	}
}

void SCameraVariableCollectionEditor::SetDetailsViewObject(UObject* InObject) const
{
	if (TSharedPtr<IDetailsView> DetailsView = WeakDetailsView.Pin())
	{
		DetailsView->SetObject(InObject);
	}
}

TSharedRef<ITableRow> SCameraVariableCollectionEditor::OnListGenerateRow(UCameraVariableAsset* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCameraVariableCollectionListRow, OwnerTable)
		.CameraVariable(Item)
		.HighlightText(this, &SCameraVariableCollectionEditor::GetHighlightText);
}

void SCameraVariableCollectionEditor::OnListSectionChanged(UCameraVariableAsset* Item, ESelectInfo::Type SelectInfo) const
{
	SetDetailsViewObject(Item);
}

void SCameraVariableCollectionEditor::OnListItemScrolledIntoView(UCameraVariableAsset* Item, const TSharedPtr<ITableRow>& ItemWidget)
{
	if (bDeferredRequestRenameItem)
	{
		bDeferredRequestRenameItem = false;

		TSharedPtr<ITableRow> RowWidget = ListView->WidgetFromItem(Item);
		if (!ensure(RowWidget))
		{
			return;
		}

		TSharedPtr<SCameraVariableCollectionListRow> TypedRowWidget = StaticCastSharedPtr<SCameraVariableCollectionListRow>(RowWidget);
		if (!ensure(TypedRowWidget))
		{
			return;
		}

		TypedRowWidget->EnterNameEditingMode(OnDeferredRenamedItem);

		// The delegate was copied into the row widget so we can unbind it here.
		OnDeferredRenamedItem.Unbind();
	}
}

TSharedPtr<SWidget> SCameraVariableCollectionEditor::OnListContextMenuOpening()
{
	static const FName ContextMenuName("CameraVariableList.ContextMenu");

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		const FCameraVariableCollectionEditorCommands& Commands = FCameraVariableCollectionEditorCommands::Get();

		UToolMenu* ContextMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = ContextMenu->AddSection("Actions");
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.RenameVariable));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.DeleteVariable));
	}

	FToolMenuContext MenuContext;
	MenuContext.AppendCommandList(CommandList);
	return ToolMenus->GenerateWidget(ContextMenuName, MenuContext);
}

void SCameraVariableCollectionEditor::GetEntryStrings(const UCameraVariableAsset* InItem, TArray<FString>& OutStrings)
{
	OutStrings.Add(InItem->GetName());
	OutStrings.Add(InItem->GetClass()->GetName());
}

void SCameraVariableCollectionEditor::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(SearchTextFilter->GetFilterErrorText());

	bUpdateFilteredItemSource = true;
}

void SCameraVariableCollectionEditor::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	OnSearchTextChanged(InFilterText);
}

FText SCameraVariableCollectionEditor::GetHighlightText() const
{
	return SearchTextFilter->GetRawFilterText();
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE
