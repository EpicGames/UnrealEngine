// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraVariableNameGraphPin.h"

#include "Core/CameraVariableAssets.h"
#include "Editors/CameraVariablePickerConfig.h"
#include "IContentBrowserSingleton.h"
#include "IGameplayCamerasEditorModule.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SCameraVariableNameGraphPin"

namespace UE::Cameras
{

void SCameraVariableNameGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SCameraVariableNameGraphPin::GetDefaultValueWidget()
{
	if (!GraphPinObj)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		.MaxWidth(200.f)
		[
			SAssignNew(CameraVariablePickerButton, SComboButton)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ContentPadding(FMargin(2.f, 2.f, 2.f, 1.f))
			.ForegroundColor(this, &SCameraVariableNameGraphPin::OnGetComboForeground)
			.ButtonColorAndOpacity(this, &SCameraVariableNameGraphPin::OnGetWidgetBackground)
			.MenuPlacement(MenuPlacement_BelowAnchor)
			.IsEnabled(this, &SCameraVariableNameGraphPin::IsEditingEnabled)
			.ButtonContent()
			[
				SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.ColorAndOpacity(this, &SCameraVariableNameGraphPin::OnGetComboForeground)
					.Text(this, &SCameraVariableNameGraphPin::OnGetSelectedCameraVariableName)
					.ToolTipText(this, &SCameraVariableNameGraphPin::OnGetCameraVariablePickerToolTipText)
			]
			.OnGetMenuContent(this, &SCameraVariableNameGraphPin::OnBuildCameraVariablePicker)
		]
		// Reset button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ButtonColorAndOpacity(this, &SCameraVariableNameGraphPin::OnGetWidgetBackground)
			.OnClicked(this, &SCameraVariableNameGraphPin::OnResetButtonClicked)
			.ContentPadding(1.f)
			.ToolTipText(LOCTEXT("ResetButtonToolTip", "Reset the camera rig reference."))
			.IsEnabled(this, &SGraphPin::IsEditingEnabled)
			[
				SNew(SImage)
				.ColorAndOpacity(this, &SCameraVariableNameGraphPin::OnGetWidgetForeground)
				.Image(FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")))
			]
		];
}

bool SCameraVariableNameGraphPin::DoesWidgetHandleSettingEditingEnabled() const
{
	return true;
}

FSlateColor SCameraVariableNameGraphPin::OnGetComboForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? ActiveComboAlpha : InactiveComboAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SCameraVariableNameGraphPin::OnGetWidgetForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? ActivePinForegroundAlpha : InactivePinForegroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SCameraVariableNameGraphPin::OnGetWidgetBackground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? ActivePinBackgroundAlpha : InactivePinBackgroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FText SCameraVariableNameGraphPin::OnGetSelectedCameraVariableName() const
{
	FText Value;
	if (GraphPinObj != nullptr)
	{
		if (const UCameraVariableAsset* CameraVariable = Cast<const UCameraVariableAsset>(GraphPinObj->DefaultObject))
		{
			Value = FText::FromString(CameraVariable->GetDisplayName());
		}
	}
	if (Value.IsEmpty())
	{
		return LOCTEXT("DefaultComboText", "Select Camera Variable");
	}
	return Value;
}

FText SCameraVariableNameGraphPin::OnGetCameraVariablePickerToolTipText() const
{
	return LOCTEXT("ComboToolTipText", "The name of the camera variable.");
}

TSharedRef<SWidget> SCameraVariableNameGraphPin::OnBuildCameraVariablePicker() const
{
	FCameraVariablePickerConfig CameraVariablePickerConfig;
	CameraVariablePickerConfig.CameraAssetViewType = EAssetViewType::List;
	CameraVariablePickerConfig.CameraVariableCollectionSaveSettingsName = TEXT("CameraVariableNameGraphPinAssetPicker");
	CameraVariablePickerConfig.OnCameraVariableSelected = FOnCameraVariableSelected::CreateSP(
			this, &SCameraVariableNameGraphPin::OnPickerAssetSelected);

	// Find the already specified camera variable, if any.
	if (GraphPinObj)
	{
		UClass* VariableClass = Cast<UClass>(GraphPinObj->PinType.PinSubCategoryObject);
		CameraVariablePickerConfig.CameraVariableClass = VariableClass;

		UCameraVariableAsset* DefaultCameraVariable = Cast<UCameraVariableAsset>(GraphPinObj->DefaultObject);
		if (DefaultCameraVariable)
		{
			CameraVariablePickerConfig.InitialCameraVariableSelection = DefaultCameraVariable;
		}
	}

	IGameplayCamerasEditorModule& CamerasEditorModule = IGameplayCamerasEditorModule::Get();
	return CamerasEditorModule.CreateCameraVariablePicker(CameraVariablePickerConfig);
}

void SCameraVariableNameGraphPin::OnPickerAssetSelected(UCameraVariableAsset* SelectedItem) const
{
	if (SelectedItem)
	{
		CameraVariablePickerButton->SetIsOpen(false);
		SetCameraVariable(SelectedItem);
	}
}

FReply SCameraVariableNameGraphPin::OnResetButtonClicked()
{
	CameraVariablePickerButton->SetIsOpen(false);
	SetCameraVariable(nullptr);
	return FReply::Handled();
}

void SCameraVariableNameGraphPin::SetCameraVariable(UCameraVariableAsset* SelectedCameraVariable) const
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeObjectPinValue", "Change Object Pin Value"));

	GraphPinObj->Modify();
	GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, SelectedCameraVariable);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

