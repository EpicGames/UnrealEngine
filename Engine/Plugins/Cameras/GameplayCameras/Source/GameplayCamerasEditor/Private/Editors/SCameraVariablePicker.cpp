// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SCameraVariablePicker.h"

#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableCollection.h"
#include "Styles/GameplayCamerasEditorStyle.h"

#define LOCTEXT_NAMESPACE "SCameraVariablePicker"

namespace UE::Cameras
{

void SCameraVariablePicker::Construct(const FArguments& InArgs)
{
	const FCameraVariablePickerConfig& PickerConfig = InArgs._CameraVariablePickerConfig;

	VariableClass = PickerConfig.CameraVariableClass;
	OnCameraVariableSelected = PickerConfig.OnCameraVariableSelected;

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(400)
		.WidthOverride(350)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SNew(SVerticalBox)
				// Camera variable collection asset picker.
				+SVerticalBox::Slot()
				.FillHeight(0.55f)
				[
					BuildVariableCollectionAssetPicker(PickerConfig)
				]
				// Camera variable list.
				+SVerticalBox::Slot()
				.FillHeight(0.45f)
				.Padding(0.f, 3.f)
				[
					SAssignNew(CameraVariableListView, SListView<UCameraVariableAsset*>)
					.ListItemsSource(&CameraVariableItemsSource)
					.OnGenerateRow(this, &SCameraVariablePicker::OnVariableListGenerateRow)
					.OnSelectionChanged(this, &SCameraVariablePicker::OnVariableListSelectionChanged)
				]
				// Number of items in the camera variable list.
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.Padding(8, 5)
					[
						SNew(STextBlock)
						.Text(this, &SCameraVariablePicker::GetCameraVariableCountText)
					]
				]
			]
		]
	];

	if (PickerConfig.InitialCameraVariableSelection)
	{
		SetupInitialSelections(PickerConfig.InitialCameraVariableSelection);
	}
}

void SCameraVariablePicker::SetupInitialSelections(UCameraVariableAsset* InSelectedCameraVariable)
{
	UCameraVariableCollection* InitialVariableCollection = nullptr;
	if (InSelectedCameraVariable)
	{
		InitialVariableCollection = InSelectedCameraVariable->GetTypedOuter<UCameraVariableCollection>();
	}

	UpdateVariableListItemsSource(InitialVariableCollection);

	CameraVariableListView->SetSelection(InSelectedCameraVariable);
	CameraVariableListView->RequestScrollIntoView(InSelectedCameraVariable);
}

TSharedRef<SWidget> SCameraVariablePicker::BuildVariableCollectionAssetPicker(const FCameraVariablePickerConfig& InPickerConfig)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;

	FARFilter ARFilter;
	ARFilter.ClassPaths.Add(FTopLevelAssetPath(UCameraVariableCollection::StaticClass()->GetPathName()));

	FAssetData InitialVariableCollection = InPickerConfig.InitialCameraVariableCollectionSelection;
	if (InPickerConfig.InitialCameraVariableSelection)
	{
		InitialVariableCollection = InPickerConfig.InitialCameraVariableSelection->GetTypedOuter<UCameraVariableCollection>();
	}

	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = true;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.Filter = ARFilter;
	AssetPickerConfig.SaveSettingsName = InPickerConfig.CameraVariableCollectionSaveSettingsName;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.InitialAssetSelection = InitialVariableCollection;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCameraVariablePicker::OnAssetSelected);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentAssetPickerSelection);

	return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
}

void SCameraVariablePicker::OnAssetSelected(const FAssetData& SelectedAsset)
{
	UpdateVariableListItemsSource();
}

void SCameraVariablePicker::UpdateVariableListItemsSource(UCameraVariableCollection* InCameraVariableCollection)
{
	UCameraVariableCollection* CameraVariableCollection = InCameraVariableCollection;
	if (!CameraVariableCollection)
	{
		TArray<FAssetData> SelectedAssets;
		if (GetCurrentAssetPickerSelection.IsBound())
		{
			SelectedAssets = GetCurrentAssetPickerSelection.Execute();
		}

		if (!SelectedAssets.IsEmpty())
		{
			CameraVariableCollection = Cast<UCameraVariableCollection>(SelectedAssets[0].GetAsset());
		}
	}

	CameraVariableItemsSource.Reset();
	if (CameraVariableCollection)
	{
		if (VariableClass)
		{
			CameraVariableItemsSource = CameraVariableCollection->Variables.FilterByPredicate(
					[this](UCameraVariableAsset* Item) { return Item->GetClass() == VariableClass; });
		}
		else
		{
			CameraVariableItemsSource = CameraVariableCollection->Variables;
		}
	}

	CameraVariableListView->RequestListRefresh();
}

TSharedRef<ITableRow> SCameraVariablePicker::OnVariableListGenerateRow(UCameraVariableAsset* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasStyle = FGameplayCamerasEditorStyle::Get();

	const FText DisplayName = Item->DisplayName.IsEmpty() ?
		FText::FromName(Item->GetFName()) : FText::FromString(Item->DisplayName);

	return SNew(STableRow<UCameraVariableAsset*>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(GameplayCamerasStyle->GetBrush("CameraParameter.VariableBrowser"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(4.f, 2.f)
			[
				SNew(STextBlock)
				.Text(DisplayName)
			]
		];
}

void SCameraVariablePicker::OnVariableListSelectionChanged(UCameraVariableAsset* Item, ESelectInfo::Type SelectInfo) const
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		OnCameraVariableSelected.ExecuteIfBound(Item);
	}
}

FText SCameraVariablePicker::GetCameraVariableCountText() const
{
	const int32 NumCameraVariables = CameraVariableItemsSource.Num();

	FText CountText = FText::GetEmpty();
	if (NumCameraVariables == 1)
	{
		CountText = LOCTEXT("CameraVariableCountTextSingular", "1 item");
	}
	else
	{
		CountText = FText::Format(LOCTEXT("CameraVariableCountTextPlural", "{0} items"), FText::AsNumber(NumCameraVariables));
	}
	return CountText;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

