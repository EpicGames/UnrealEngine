// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserModule.h"
#include "Editors/CameraVariablePickerConfig.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SSearchBox;
class UCameraVariableAsset;
class UCameraVariableCollection;

namespace UE::Cameras
{

/**
 * A picker widget for selecting a camera variable.
 */
class SCameraVariablePicker : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraVariablePicker)
	{}
		SLATE_ARGUMENT(FCameraVariablePickerConfig, CameraVariablePickerConfig)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	void SetupInitialSelections(UCameraVariableAsset* InSelectedCameraVariable);

	TSharedRef<SWidget> BuildVariableCollectionAssetPicker(const FCameraVariablePickerConfig& InPickerConfig);

	void OnAssetSelected(const FAssetData& SelectedAsset);

	void UpdateVariableListItemsSource(UCameraVariableCollection* InCameraVariableCollection = nullptr);
	TSharedRef<ITableRow> OnVariableListGenerateRow(UCameraVariableAsset* Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnVariableListSelectionChanged(UCameraVariableAsset* Item, ESelectInfo::Type SelectInfo) const;

	FText GetCameraVariableCountText() const;

private:

	TSharedPtr<SListView<UCameraVariableAsset*>> CameraVariableListView;
	TArray<UCameraVariableAsset*> CameraVariableItemsSource;

	UClass* VariableClass = nullptr;

	FGetCurrentSelectionDelegate GetCurrentAssetPickerSelection;

	FOnCameraVariableSelected OnCameraVariableSelected;
};

}  // namespace UE::Cameras

