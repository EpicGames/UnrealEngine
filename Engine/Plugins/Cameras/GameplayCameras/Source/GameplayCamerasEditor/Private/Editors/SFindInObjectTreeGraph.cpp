// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SFindInObjectTreeGraph.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/ObjectTreeGraphSearch.h"
#include "Framework/Application/SlateApplication.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SFindInObjectTreeGraph"

FFindInObjectTreeGraphResult::FFindInObjectTreeGraphResult(const FText& InCustomText)
	: CustomText(InCustomText)
{
}

FFindInObjectTreeGraphResult::FFindInObjectTreeGraphResult(
		TSharedPtr<FFindInObjectTreeGraphResult>& InParent, 
		UEdGraphNode* InNode, 
		FName InPinName)
	: Parent(InParent)
	, WeakNode(InNode)
	, PinName(InPinName)
{
}

TSharedRef<SWidget>	FFindInObjectTreeGraphResult::GetIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = NULL;

	UEdGraphNode* Node = WeakNode.Get();

	if (Node && !PinName.IsNone())
	{
		if (UEdGraphPin* Pin = Node->FindPin(PinName))
		{
			const UEdGraphSchema* Schema = Node->GetSchema();
			IconColor = Schema->GetPinTypeColor(Pin->PinType);
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		}
	}
	else if (Node)
	{
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.NodeGlyph"));
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(GetCategory());
}

FText FFindInObjectTreeGraphResult::GetCategory() const
{
	if (WeakNode.IsValid())
	{
		if (PinName.IsNone())
		{
			return LOCTEXT("NodeCategory", "Node");
		}
		else
		{
			return LOCTEXT("PinCategory", "Pin");
		}
	}
	return FText::GetEmpty();
}

FText FFindInObjectTreeGraphResult::GetText() const
{
	UEdGraphNode* Node = WeakNode.Get();
	if (Node)
	{
		if (PinName.IsNone())
		{
			return Node->GetNodeTitle(ENodeTitleType::FullTitle);
		}
		else if (UEdGraphPin* Pin = Node->FindPin(PinName))
		{
			return Pin->GetDisplayName();
		}
		else
		{
			return FText::FromName(PinName);
		}
	}
	return CustomText;
}

FText FFindInObjectTreeGraphResult::GetCommentText() const
{
	if (UEdGraphNode* Node = WeakNode.Get())
	{
		return FText::FromString(Node->NodeComment);
	}
	return FText::GetEmpty();
}

FReply FFindInObjectTreeGraphResult::OnClick(TSharedRef<SFindInObjectTreeGraph> FindInObjectTreeGraph)
{
	if (UEdGraphNode* Node = WeakNode.Get())
	{
		FindInObjectTreeGraph->OnJumpToNodeRequested.ExecuteIfBound(Node, PinName);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SFindInObjectTreeGraph::Construct(const FArguments& InArgs)
{
	OnGetGraphsToSearch = InArgs._OnGetGraphsToSearch;
	OnJumpToNodeRequested = InArgs._OnJumpToNodeRequested;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Search"))
				.OnTextChanged(this, &SFindInObjectTreeGraph::OnSearchTextChanged)
				.OnTextCommitted(this, &SFindInObjectTreeGraph::OnSearchTextCommitted)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0, 4, 0, 0)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(ResultTreeView, SResultTreeView)
				.TreeItemsSource(&Results)
				.SelectionMode(ESelectionMode::Multi)
				.OnGenerateRow(this, &SFindInObjectTreeGraph::OnResultTreeViewGenerateRow)
				.OnGetChildren(this, &SFindInObjectTreeGraph::OnResultTreeViewGetChildren)
				.OnSelectionChanged(this, &SFindInObjectTreeGraph::OnResultTreeViewSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SFindInObjectTreeGraph::OnResultTreeViewMouseButtonDoubleClick)
			]
		]
	];
}

void SFindInObjectTreeGraph::FocusSearchEditBox()
{
	FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
}

void SFindInObjectTreeGraph::Search(const FString& InSearchQuery)
{
	SearchBox->SetText(FText::FromString(InSearchQuery));
	SearchQuery = InSearchQuery;
	StartSearch();
}

void SFindInObjectTreeGraph::OnSearchTextChanged(const FText& Text)
{
	SearchQuery = Text.ToString();
}

void SFindInObjectTreeGraph::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		StartSearch();
	}
}

TSharedRef<ITableRow> SFindInObjectTreeGraph::OnResultTreeViewGenerateRow(FResultPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const FText CommentText = InItem->GetCommentText();

	return SNew(STableRow<FResultPtr>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				InItem->GetIcon()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2, 0)
			[
				SNew(STextBlock)
				.Text(InItem->GetText())
				.HighlightText(HighlightText)
				.ToolTipText(FText::Format(LOCTEXT("ResultToolTipFmt", "{0} : {1}"), InItem->GetCategory(), InItem->GetText()))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2, 0)
			[
				SNew(STextBlock)
				.Text(CommentText.IsEmpty() 
						? FText::GetEmpty() 
						: FText::Format(LOCTEXT("NodeCommentFmt", "Node Comment: {0}"), CommentText))
				.HighlightText(HighlightText)
			]
		];
}

void SFindInObjectTreeGraph::OnResultTreeViewGetChildren(FResultPtr InItem, TArray<FResultPtr>& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInObjectTreeGraph::OnResultTreeViewSelectionChanged(FResultPtr Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		Item->OnClick(SharedThis(this));
	}
}

void SFindInObjectTreeGraph::OnResultTreeViewMouseButtonDoubleClick(FResultPtr Item)
{
	if (Item.IsValid())
	{
		Item->OnClick(SharedThis(this));
	}
}

void SFindInObjectTreeGraph::StartSearch()
{
	TArray<FString> Tokens;
	if (SearchQuery.Contains("\"") && SearchQuery.ParseIntoArray(Tokens, TEXT("\""), true) > 0)
	{
		for (FString& Token : Tokens)
		{
			Token = Token.TrimQuotes();
		}
	}
	else
	{
		SearchQuery.ParseIntoArray(Tokens, TEXT(" "), true);
	}
	Tokens.RemoveAll([](const FString& Item) { return Item.IsEmpty(); });

	Results.Empty();
	HighlightText = FText::GetEmpty();
	TArray<FObjectTreeGraphSearchResult> SearchResults;
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchQuery);

		TArray<FFindInObjectTreeGraphSource> Sources;
		OnGetGraphsToSearch.ExecuteIfBound(Sources);

		FObjectTreeGraphSearch Searcher;
		for (const FFindInObjectTreeGraphSource& Source : Sources)
		{
			Searcher.AddGraph(Source.Graph);
		}

		Searcher.Search(Tokens, SearchResults);
	}
	
	// Convert simple flat search results to hierarchical results, where property results
	// are always under an object result, object results are always under a graph result, 
	// and so on. We do this by creating blank results if we need to, although it makes
	// the assumption that the flat results come ordered (e.g. an object result won't be
	// found after a property result for that object).
	{
		TMap<UEdGraph*, FResultPtr> RootGraphToWidgetResult;
		TMap<UEdGraphNode*, FResultPtr> NodeToWidgetResult;
		for (const FObjectTreeGraphSearchResult& SearchResult : SearchResults)
		{
			FResultPtr GraphResult;
			if (SearchResult.Graph)
			{
				GraphResult = RootGraphToWidgetResult.FindRef(SearchResult.Graph);
				if (!GraphResult)
				{
					FGraphDisplayInfo GraphDisplayInfo;
					const UEdGraphSchema* Schema = SearchResult.Graph->GetSchema();
					Schema->GetGraphDisplayInformation(*SearchResult.Graph, GraphDisplayInfo);

					FText GraphResultText;
					if (UObjectTreeGraph* ObjectTreeGraph = Cast<UObjectTreeGraph>(SearchResult.Graph))
					{
						const UObject* RootObject = ObjectTreeGraph->GetRootObject();
						const FText RootObjectText = ObjectTreeGraph->GetConfig().GetDisplayNameText(RootObject);
						GraphResultText = FText::Format(
								LOCTEXT("GraphResultFmt", "{0}: {1}"), { RootObjectText, GraphDisplayInfo.DisplayName });
					}
					else
					{
						GraphResultText = GraphDisplayInfo.DisplayName;
					}

					GraphResult = MakeShared<FFindInObjectTreeGraphResult>(GraphResultText);
					RootGraphToWidgetResult.Add(SearchResult.Graph, GraphResult);
					Results.Add(GraphResult);
				}
			}

			FResultPtr NodeResult;
			if (SearchResult.Node)
			{
				NodeResult = NodeToWidgetResult.FindRef(SearchResult.Node);
				if (!NodeResult)
				{
					ensure(GraphResult);
					NodeResult = MakeShared<FFindInObjectTreeGraphResult>(GraphResult, SearchResult.Node);
					NodeToWidgetResult.Add(SearchResult.Node, NodeResult);
					GraphResult->Children.Add(NodeResult);
				}
			}

			if (!SearchResult.PinName.IsNone())
			{
				ensure(NodeResult);
				FResultPtr PropertyResult = MakeShared<FFindInObjectTreeGraphResult>(
						NodeResult, SearchResult.Node, SearchResult.PinName);
				NodeResult->Children.Add(PropertyResult);
			}
		}
	}

	if (Results.IsEmpty())
	{
		Results.Add(MakeShared<FFindInObjectTreeGraphResult>(LOCTEXT("NoResults", "No results found")));
	}

	ResultTreeView->RequestTreeRefresh();
	for (FResultPtr Result : Results)
	{
		ResultTreeView->SetItemExpansion(Result, true);
	}
}

#undef LOCTEXT_NAMESPACE

