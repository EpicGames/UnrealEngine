// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SObjectTreeGraphCommentNode.h"

#include "Editors/ObjectTreeGraphCommentNode.h"
#include "Editors/ObjectTreeGraphNode.h"

void SObjectTreeGraphCommentNode::Construct(const FArguments& InArgs)
{
	SGraphNodeComment::Construct(SGraphNodeComment::FArguments(), InArgs._GraphNode);

	ObjectGraphNode = InArgs._GraphNode;
}

void SObjectTreeGraphCommentNode::MoveTo(const FSlateCompatVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNodeComment::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	if (ObjectGraphNode)
	{
		ObjectGraphNode->OnGraphNodeMoved(bMarkDirty);

		if (ObjectGraphNode && ObjectGraphNode->MoveMode == ECommentBoxMode::GroupMovement)
		{
			// Also notify any nodes that were moved along.
			for (FCommentNodeSet::TConstIterator NodeIt(ObjectGraphNode->GetNodesUnderComment()); NodeIt; ++NodeIt)
			{
				if (UObjectTreeGraphNode* Node = Cast<UObjectTreeGraphNode>(*NodeIt))
				{
					Node->OnGraphNodeMoved(bMarkDirty);
				}
			}
		}
	}
}

