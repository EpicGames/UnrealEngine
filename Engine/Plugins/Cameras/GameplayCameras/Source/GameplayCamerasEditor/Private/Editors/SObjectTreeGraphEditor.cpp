// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SObjectTreeGraphEditor.h"

#include "Algo/AnyOf.h"
#include "Commands/ObjectTreeGraphEditorCommands.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor.h"
#include "Editors/ObjectTreeDragDropOp.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/ObjectTreeGraphSchema.h"
#include "Editors/SObjectTreeGraphTitleBar.h"
#include "Editors/SObjectTreeGraphToolbox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "SGraphPanel.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SObjectTreeGraphEditor"

TMap<UObjectTreeGraph*, TSharedPtr<SObjectTreeGraphEditor>> SObjectTreeGraphEditor::ActiveGraphEditors;

TSharedPtr<SObjectTreeGraphEditor> SObjectTreeGraphEditor::FindGraphEditor(UObjectTreeGraph* InGraph)
{
	return ActiveGraphEditors.FindRef(InGraph);
}

void SObjectTreeGraphEditor::OnBeginEditingGraph(UObjectTreeGraph* InGraph, TSharedRef<SObjectTreeGraphEditor> InGraphEditor)
{
	ActiveGraphEditors.Add(InGraph, InGraphEditor);
}

void SObjectTreeGraphEditor::OnEndEditingGraph(UObjectTreeGraph* InGraph, TSharedRef<SObjectTreeGraphEditor> InGraphEditor)
{
	TSharedPtr<SObjectTreeGraphEditor> RemovedGraphEditor;
	ActiveGraphEditors.RemoveAndCopyValue(InGraph, RemovedGraphEditor);
	ensure(RemovedGraphEditor == InGraphEditor);
}

void SObjectTreeGraphEditor::Construct(const FArguments& InArgs)
{
	DetailsView = InArgs._DetailsView;

	TSharedPtr<SWidget> GraphTitleBar = InArgs._GraphTitleBar;
	if (!GraphTitleBar.IsValid())
	{
		TSharedRef<SWidget> DefaultTitleBar = SNew(SObjectTreeGraphTitleBar)
			.Graph(InArgs._GraphToEdit)
			.TitleText(InArgs._GraphTitle);
		GraphTitleBar = DefaultTitleBar.ToSharedPtr();
	}

	SGraphEditor::FGraphEditorEvents GraphEditorEvents;
	GraphEditorEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SObjectTreeGraphEditor::OnGraphSelectionChanged);
	GraphEditorEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &SObjectTreeGraphEditor::OnNodeTextCommitted);
	GraphEditorEvents.OnDoubleClicked = SGraphEditor::FOnDoubleClicked::CreateSP(this, &SObjectTreeGraphEditor::OnDoubleClicked);
	GraphEditorEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SObjectTreeGraphEditor::OnNodeDoubleClicked);

	InitializeBuiltInCommands();

	TSharedPtr<FUICommandList> AdditionalCommands = BuiltInCommands;
	if (InArgs._AdditionalCommands)
	{
		AdditionalCommands = MakeShared<FUICommandList>();
		AdditionalCommands->Append(BuiltInCommands.ToSharedRef());
		AdditionalCommands->Append(InArgs._AdditionalCommands.ToSharedRef());
	}

	GraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(AdditionalCommands)
		.Appearance(InArgs._Appearance)
		.TitleBar(GraphTitleBar)
		.GraphToEdit(InArgs._GraphToEdit)
		.GraphEvents(GraphEditorEvents)
		.AssetEditorToolkit(InArgs._AssetEditorToolkit);

	ChildSlot
	[
		GraphEditor.ToSharedRef()
	];

	GEditor->RegisterForUndo(this);
}

SObjectTreeGraphEditor::~SObjectTreeGraphEditor()
{
	GEditor->UnregisterForUndo(this);
}

void SObjectTreeGraphEditor::RegisterEditor()
{
	if (UObjectTreeGraph* CurrentGraph = Cast<UObjectTreeGraph>(GraphEditor->GetCurrentGraph()))
	{
		OnBeginEditingGraph(CurrentGraph, SharedThis(this));
	}
}

void SObjectTreeGraphEditor::UnregisterEditor()
{
	if (UObjectTreeGraph* CurrentGraph = Cast<UObjectTreeGraph>(GraphEditor->GetCurrentGraph()))
	{
		OnEndEditingGraph(CurrentGraph, SharedThis(this));
	}
}

void SObjectTreeGraphEditor::InitializeBuiltInCommands()
{
	using namespace UE::Cameras;

	if (BuiltInCommands.IsValid())
	{
		return;
	}

	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	const FGraphEditorCommandsImpl& GraphEditorCommands = FGraphEditorCommands::Get();
	const FObjectTreeGraphEditorCommands& ObjectTreeGraphEditorCommands = FObjectTreeGraphEditorCommands::Get();

	BuiltInCommands = MakeShared<FUICommandList>();

	// Generic commands.
	BuiltInCommands->MapAction(GenericCommands.SelectAll,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::SelectAllNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanSelectAllNodes)
		);

	BuiltInCommands->MapAction(GenericCommands.Delete,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanDeleteSelectedNodes)
		);

	BuiltInCommands->MapAction(GenericCommands.Copy,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanCopySelectedNodes)
		);

	BuiltInCommands->MapAction(GenericCommands.Cut,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanCutSelectedNodes)
		);

	BuiltInCommands->MapAction(GenericCommands.Paste,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::PasteNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanPasteNodes)
		);

	BuiltInCommands->MapAction(GenericCommands.Duplicate,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::DuplicateNodes),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanDuplicateNodes)
		);
	BuiltInCommands->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnRenameNode),
		FCanExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::CanRenameNode)
	);

	// Alignment commands.
	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesTop,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignTop)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesMiddle,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignMiddle)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesBottom,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignBottom)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesLeft,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignLeft)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesCenter,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignCenter)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.AlignNodesRight,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnAlignRight)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.StraightenConnections,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnStraightenConnections)
			);

	// Distribution commands.
	BuiltInCommands->MapAction(GraphEditorCommands.DistributeNodesHorizontally,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnDistributeNodesHorizontally)
			);

	BuiltInCommands->MapAction(GraphEditorCommands.DistributeNodesVertically,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnDistributeNodesVertically)
			);

	// Custom commands.
	BuiltInCommands->MapAction(ObjectTreeGraphEditorCommands.InsertArrayItemPinBefore,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnInsertArrayItemPinBefore)
			);
	BuiltInCommands->MapAction(ObjectTreeGraphEditorCommands.InsertArrayItemPinAfter,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnInsertArrayItemPinAfter)
			);
	BuiltInCommands->MapAction(ObjectTreeGraphEditorCommands.RemoveArrayItemPin,
			FExecuteAction::CreateSP(this, &SObjectTreeGraphEditor::OnRemoveArrayItemPin)
			);
}

void SObjectTreeGraphEditor::JumpToNode(UEdGraphNode* InNode, FName InPinName)
{
	if (InPinName != NAME_None)
	{
		if (UEdGraphPin* Pin = InNode->FindPin(InPinName))
		{
			GraphEditor->JumpToPin(Pin);
			return;
		}
	}
	GraphEditor->JumpToNode(InNode);
}

bool SObjectTreeGraphEditor::FindAndJumpToObjectNode(UObject* InObject, FName InPropertyName)
{
	if (UObjectTreeGraph* CurrentGraph = Cast<UObjectTreeGraph>(GraphEditor->GetCurrentGraph()))
	{
		if (UObjectTreeGraphNode* Node = CurrentGraph->FindObjectNode(InObject))
		{
			JumpToNode(Node, InPropertyName);
			return true;
		}
	}
	return false;
}

void SObjectTreeGraphEditor::ResyncDetailsView()
{
	OnGraphSelectionChanged(GraphEditor->GetSelectedNodes());
}

FReply SObjectTreeGraphEditor::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FObjectTreeClassDragDropOp> ObjectClassOp = DragDropEvent.GetOperationAs<FObjectTreeClassDragDropOp>();
	if (ObjectClassOp)
	{
		return ObjectClassOp->ExecuteDragOver(GraphEditor);
	}

	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}

FReply SObjectTreeGraphEditor::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FObjectTreeClassDragDropOp> ObjectClassOp = DragDropEvent.GetOperationAs<FObjectTreeClassDragDropOp>();
	if (ObjectClassOp)
	{
		SGraphPanel* GraphPanel = GraphEditor->GetGraphPanel();
		FSlateCompatVector2f NewLocation = GraphPanel->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()));

		return ObjectClassOp->ExecuteDrop(GraphEditor, NewLocation);
	}

	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

void SObjectTreeGraphEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		GraphEditor->ClearSelectionSet();

		GraphEditor->NotifyGraphChanged();

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SObjectTreeGraphEditor::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SObjectTreeGraphEditor::OnGraphSelectionChanged(const FGraphPanelSelectionSet& SelectionSet)
{
	if (DetailsView)
	{
		TArray<UObject*> SelectedObjects;
		for (UObject* Selection : SelectionSet)
		{
			if (UObjectTreeGraphNode* GraphNode = Cast<UObjectTreeGraphNode>(Selection))
			{
				SelectedObjects.Add(GraphNode->GetObject());
			}
		}

		DetailsView->SetObjects(SelectedObjects);
	}
}

void SObjectTreeGraphEditor::OnNodeTextCommitted(const FText& InText, ETextCommit::Type InCommitType, UEdGraphNode* InEditedNode)
{
	if (InEditedNode)
	{
		FString NewName = InText.ToString().TrimStartAndEnd();
		if (NewName.IsEmpty())
		{
			return;
		}

		if (NewName.Len() >= NAME_SIZE)
		{
			NewName = NewName.Left(NAME_SIZE - 1);
		}

		const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));

		InEditedNode->Modify();
		InEditedNode->OnRenameNode(NewName);

		GraphEditor->GetCurrentGraph()->NotifyNodeChanged(InEditedNode);
	}
}

void SObjectTreeGraphEditor::OnNodeDoubleClicked(UEdGraphNode* InClickedNode)
{
	UObjectTreeGraphNode* SelectedNode = Cast<UObjectTreeGraphNode>(InClickedNode);
	if (SelectedNode)
	{
		SelectedNode->OnDoubleClicked();
	}
}

void SObjectTreeGraphEditor::OnDoubleClicked()
{
}

FString SObjectTreeGraphEditor::ExportNodesToText(const FGraphPanelSelectionSet& Nodes, bool bOnlyCanDuplicateNodes, bool bOnlyCanDeleteNodes)
{
	UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
	const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(CurrentGraph->GetSchema());

	return Schema->ExportNodesToText(Nodes, bOnlyCanDuplicateNodes, bOnlyCanDeleteNodes);
}

void SObjectTreeGraphEditor::ImportNodesFromText(const FSlateCompatVector2f& Location, const FString& TextToImport)
{
	// Start a transaction and flag things as modified.
	const FScopedTransaction Transaction(LOCTEXT("PasteNodes", "Paste Nodes"));

	UObjectTreeGraph* Graph = CastChecked<UObjectTreeGraph>(GraphEditor->GetCurrentGraph());
	Graph->Modify();

	UPackage* ObjectPackage = Graph->GetRootObject()->GetOutermost();
	ObjectPackage->Modify();

	// Import the nodes.
	TArray<UEdGraphNode*> PastedNodes;
	const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(Graph->GetSchema());
	Schema->ImportNodesFromText(Graph, TextToImport, PastedNodes);

	// Compute the center of the pasted nodes.
	FVector2D PastedNodesClusterCenter(FVector2D::ZeroVector);
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		PastedNodesClusterCenter.X += PastedNode->NodePosX;
		PastedNodesClusterCenter.Y += PastedNode->NodePosY;
	}
	if (PastedNodes.Num() > 0)
	{
		float InvNumNodes = 1.0f / float(PastedNodes.Num());
		PastedNodesClusterCenter.X *= InvNumNodes;
		PastedNodesClusterCenter.Y *= InvNumNodes;
	}

	// Move all pasted nodes to the new location, and select them.
	GraphEditor->ClearSelectionSet();

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		PastedNode->NodePosX = (PastedNode->NodePosX - PastedNodesClusterCenter.X) + Location.X ;
		PastedNode->NodePosY = (PastedNode->NodePosY - PastedNodesClusterCenter.Y) + Location.Y ;

		PastedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

		// Notify object nodes of having been moved so that we save the new position
		// in the underlying data.
		if (UObjectTreeGraphNode* PastedObjectNode = Cast<UObjectTreeGraphNode>(PastedNode))
		{
			PastedObjectNode->OnGraphNodeMoved(false);
		}

		GraphEditor->SetNodeSelection(PastedNode, true);
	}

	// Update the UI.
	GraphEditor->NotifyGraphChanged();
}

bool SObjectTreeGraphEditor::CanImportNodesFromText(const FString& TextToImport)
{
	UObjectTreeGraph* CurrentGraph = CastChecked<UObjectTreeGraph>(GraphEditor->GetCurrentGraph());
	const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(CurrentGraph->GetSchema());

	return Schema->CanImportNodesFromText(CurrentGraph, TextToImport);
}

void SObjectTreeGraphEditor::DeleteNodes(TArrayView<UEdGraphNode*> NodesToDelete)
{
	UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
	const UEdGraphSchema* Schema = CurrentGraph->GetSchema();

	const FScopedTransaction Transaction(LOCTEXT("DeleteNode", "Delete Node(s)"));

	for (UEdGraphNode* Node : NodesToDelete)
	{
		if (Node)
		{
			Schema->SafeDeleteNodeFromGraph(CurrentGraph, Node);

			Node->DestroyNode();
		}
	}
}

void SObjectTreeGraphEditor::SelectAllNodes()
{
	GraphEditor->SelectAllNodes();
}

bool SObjectTreeGraphEditor::CanSelectAllNodes()
{
	return true;
}

void SObjectTreeGraphEditor::DeleteSelectedNodes()
{
	TArray<UEdGraphNode*> NodesToDelete;
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* GraphNode = Cast<UEdGraphNode>(*NodeIt);
		if (GraphNode && GraphNode->CanUserDeleteNode())
		{
			NodesToDelete.Add(GraphNode);
		}
	}
	
	DeleteNodes(NodesToDelete);

	// Remove deleted nodes from the details view.
	GraphEditor->ClearSelectionSet();
}

bool SObjectTreeGraphEditor::CanDeleteSelectedNodes()
{
	bool bDeletableNodeExists = false;
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* GraphNode = Cast<UEdGraphNode>(*NodeIt);
		if (GraphNode && GraphNode->CanUserDeleteNode())
		{
			bDeletableNodeExists = true;
		}
	}

	return SelectedNodes.Num() > 0 && bDeletableNodeExists;
}

void SObjectTreeGraphEditor::CopySelectedNodes()
{
	const FString Buffer = ExportNodesToText(GraphEditor->GetSelectedNodes(), true, false);
	FPlatformApplicationMisc::ClipboardCopy(*Buffer);
}

bool SObjectTreeGraphEditor::CanCopySelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt);
		if (Node != nullptr && Node->CanDuplicateNode())
		{
			return true;
		}
	}

	return false;
}

void SObjectTreeGraphEditor::CutSelectedNodes()
{
	const FString Buffer = ExportNodesToText(GraphEditor->GetSelectedNodes(), true, true);
	FPlatformApplicationMisc::ClipboardCopy(*Buffer);

	DeleteSelectedNodes();
}

bool SObjectTreeGraphEditor::CanCutSelectedNodes()
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void SObjectTreeGraphEditor::PasteNodes()
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	FVector2f PasteLocation = GraphEditor->GetPasteLocation2f();
#else
	FVector2D PasteLocation = GraphEditor->GetPasteLocation();
#endif
	ImportNodesFromText(PasteLocation, TextToImport);
}

bool SObjectTreeGraphEditor::CanPasteNodes()
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return CanImportNodesFromText(ClipboardContent);
}

void SObjectTreeGraphEditor::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool SObjectTreeGraphEditor::CanDuplicateNodes()
{
	return CanCopySelectedNodes();
}

void SObjectTreeGraphEditor::OnRenameNode()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
		if (SelectedNode != nullptr && SelectedNode->GetCanRenameNode())
		{
			const bool bRequestRename = true;
			GraphEditor->IsNodeTitleVisible(SelectedNode, bRequestRename);
			break;
		}
	}
}

bool SObjectTreeGraphEditor::CanRenameNode()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt);
		if (Node != nullptr && Node->GetCanRenameNode())
		{
			return true;
		}
	}

	return false;
}

void SObjectTreeGraphEditor::OnAlignTop()
{
	GraphEditor->OnAlignTop();
}

void SObjectTreeGraphEditor::OnAlignMiddle()
{
	GraphEditor->OnAlignMiddle();
}

void SObjectTreeGraphEditor::OnAlignBottom()
{
	GraphEditor->OnAlignBottom();
}

void SObjectTreeGraphEditor::OnAlignLeft()
{
	GraphEditor->OnAlignLeft();
}

void SObjectTreeGraphEditor::OnAlignCenter()
{
	GraphEditor->OnAlignCenter();
}

void SObjectTreeGraphEditor::OnAlignRight()
{
	GraphEditor->OnAlignRight();
}

void SObjectTreeGraphEditor::OnStraightenConnections()
{
	GraphEditor->OnStraightenConnections();
}

void SObjectTreeGraphEditor::OnDistributeNodesHorizontally()
{
	GraphEditor->OnDistributeNodesH();
}

void SObjectTreeGraphEditor::OnDistributeNodesVertically()
{
	GraphEditor->OnDistributeNodesV();
}

void SObjectTreeGraphEditor::OnInsertArrayItemPinBefore()
{
	if (UEdGraphPin* SelectedPin = GraphEditor->GetGraphPinForMenu())
	{
		UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
		const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(CurrentGraph->GetSchema());

		Schema->InsertArrayItemPinBefore(SelectedPin);

		GraphEditor->RefreshNode(*SelectedPin->GetOwningNode());
	}
}

void SObjectTreeGraphEditor::OnInsertArrayItemPinAfter()
{
	if (UEdGraphPin* SelectedPin = GraphEditor->GetGraphPinForMenu())
	{
		UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
		const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(CurrentGraph->GetSchema());

		Schema->InsertArrayItemPinAfter(SelectedPin);

		GraphEditor->RefreshNode(*SelectedPin->GetOwningNode());
	}
}

void SObjectTreeGraphEditor::OnRemoveArrayItemPin()
{
	if (UEdGraphPin* SelectedPin = GraphEditor->GetGraphPinForMenu())
	{
		UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
		const UObjectTreeGraphSchema* Schema = CastChecked<UObjectTreeGraphSchema>(CurrentGraph->GetSchema());

		// Get owning node before we remove the pin.
		UEdGraphNode* OwningNode = SelectedPin->GetOwningNode();
		
		Schema->RemoveArrayItemPin(SelectedPin);

		if (ensure(OwningNode))
		{
			GraphEditor->RefreshNode(*OwningNode);
		}
	}
}

#undef LOCTEXT_NAMESPACE

