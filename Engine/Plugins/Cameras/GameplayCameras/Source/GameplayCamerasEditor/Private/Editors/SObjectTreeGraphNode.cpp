// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SObjectTreeGraphNode.h"

#include "EdGraph/EdGraph.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/ObjectTreeGraphSchema.h"
#include "GraphEditorSettings.h"
#include "Styles/ObjectTreeGraphEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SObjectTreeGraphNode"

void SObjectTreeGraphNode::Construct(const FArguments& InArgs)
{
	GraphNode = InArgs._GraphNode;
	ObjectGraphNode = InArgs._GraphNode;

	TitleBorderMargin = FMargin(12, 6, 6, 6);

	SetCursor(EMouseCursor::CardinalCross);

	UpdateGraphNode();
}

void SObjectTreeGraphNode::MoveTo(const FSlateCompatVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	if (ObjectGraphNode)
	{
		ObjectGraphNode->OnGraphNodeMoved(bMarkDirty);
	}
}

const FSlateBrush* SObjectTreeGraphNode::GetShadowBrush(bool bSelected) const
{
	TSharedRef<FObjectTreeGraphEditorStyle> GraphStyle = FObjectTreeGraphEditorStyle::Get();
	return bSelected ? 
		GraphStyle->GetBrush(TEXT("ObjectTreeGraphNode.ShadowSelected")) :
		GraphStyle->GetBrush(TEXT("ObjectTreeGraphNode.Shadow"));
}

void SObjectTreeGraphNode::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	MakeAllAddArrayPropertyPinButtons(InputBox, EEdGraphPinDirection::EGPD_Input);
}

void SObjectTreeGraphNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	MakeAllAddArrayPropertyPinButtons(OutputBox, EEdGraphPinDirection::EGPD_Output);
}

TSharedPtr<SGraphPin> SObjectTreeGraphNode::CreatePinWidget(UEdGraphPin* InPin) const
{
	TSharedPtr<SGraphPin> PinWidget = SGraphNode::CreatePinWidget(InPin);

	if (const UObjectTreeGraphSchema* GraphSchema = Cast<const UObjectTreeGraphSchema>(InPin->GetSchema()))
	{
		TSharedRef<FObjectTreeGraphEditorStyle> GraphStyle = FObjectTreeGraphEditorStyle::Get();

		if (InPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Self)
		{
			const FSlateBrush* ConnectedBrush = GraphStyle->GetBrush("ObjectTreeGraphNode.SelfPin.Connected");
			const FSlateBrush* DisconnectedBrush = GraphStyle->GetBrush("ObjectTreeGraphNode.SelfPin.Disconnected");
			PinWidget->SetCustomPinIcon(ConnectedBrush, DisconnectedBrush);
		}
		else if (InPin->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property ||
				InPin->PinType.PinCategory == UObjectTreeGraphSchema::PSC_ArrayProperty)
		{
			const FSlateBrush* ConnectedBrush = GraphStyle->GetBrush("ObjectTreeGraphNode.ObjectPin.Connected");
			const FSlateBrush* DisconnectedBrush = GraphStyle->GetBrush("ObjectTreeGraphNode.ObjectPin.Disconnected");
			PinWidget->SetCustomPinIcon(ConnectedBrush, DisconnectedBrush);
		}
	}

	return PinWidget;
}

void SObjectTreeGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	SGraphNode::AddPin(PinToAdd);

	if (const UEdGraphPin* PinObj = PinToAdd->GetPinObj())
	{
		// If this is the last of a list of array pins, add a spacer below.
		const UEdGraphPin* ArrayPinObj = PinObj->ParentPin;
		if (PinObj->PinType.PinCategory == UObjectTreeGraphSchema::PC_Property &&
				PinObj->PinType.PinSubCategory == UObjectTreeGraphSchema::PSC_ArrayPropertyItem &&
				ArrayPinObj && ArrayPinObj->SubPins.Num() > 0 && ArrayPinObj->SubPins.Last() == PinObj)
		{
			if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
			{
				LeftNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					SNew(SSpacer)
					.Size(FVector2D(12, 12))
				];
			}
			else // Direction == EEdGraphPinDirection::EGPD_Output
			{
				RightNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetOutputPinPadding())
				[
					SNew(SSpacer)
					.Size(FVector2D(12, 12))
				];
			}
		}
	}
}

void SObjectTreeGraphNode::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	SGraphNode::SetDefaultTitleAreaWidget(DefaultTitleAreaWidget);

	// Somewhat of a hack, but clear up the title area and rebuild it the way we want.
	DefaultTitleAreaWidget->ClearChildren();

	// Node title.
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	// Node icon.
	IconColor = FLinearColor::White;
	const FSlateBrush* IconBrush = nullptr;
	if (GraphNode != NULL && GraphNode->ShowPaletteIconOnNode())
	{
		IconBrush = GraphNode->GetIconAndTint(IconColor).GetOptionalIcon();
	}

	TSharedRef<FObjectTreeGraphEditorStyle> GraphStyle = FObjectTreeGraphEditorStyle::Get();

	DefaultTitleAreaWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(GraphStyle->GetBrush("ObjectTreeGraphNode.TitleBackground"))
			.BorderBackgroundColor(this, &SGraphNode::GetNodeTitleColor)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Padding(TitleBorderMargin)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Top)
						.AutoWidth()
						.Padding(FMargin(0, 0, 4, 0))
						[
							SNew(SImage)
							.Image(IconBrush)
							.ColorAndOpacity(this, &SGraphNode::GetNodeTitleIconColor)
						]
						+SHorizontalBox::Slot()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							[
								CreateTitleWidget(NodeTitle)
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								NodeTitle.ToSharedRef()
							]
						]
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(6, 6, 12, 6)
				.AutoWidth()
				[
					CreateTitleRightWidget()
				]
			]
		];
}

const FSlateBrush* SObjectTreeGraphNode::GetNodeBodyBrush() const
{
	TSharedRef<FObjectTreeGraphEditorStyle> GraphStyle = FObjectTreeGraphEditorStyle::Get();
	return GraphStyle->GetBrush("ObjectTreeGraphNode.Body");
}

void SObjectTreeGraphNode::MakeAllAddArrayPropertyPinButtons(TSharedPtr<SVerticalBox> Box, EEdGraphPinDirection Direction)
{
	TArray<FArrayProperty*> ArrayProperties;
	ObjectGraphNode->GetArrayProperties(ArrayProperties, Direction);
	for (FArrayProperty* ArrayProperty : ArrayProperties)
	{
		TSharedRef<SWidget> AddPinButton = MakeAddArrayPropertyPinButton(ArrayProperty);

		FMargin AddPinPadding = Settings->GetOutputPinPadding();
		AddPinPadding.Top += 6.0f;

		Box->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(AddPinPadding)
		[
			AddPinButton
		];
	}
}

TSharedRef<SWidget> SObjectTreeGraphNode::MakeAddArrayPropertyPinButton(FArrayProperty* ArrayProperty)
{
	TSharedRef<SWidget> ButtonContent = SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Text(FText::Format(
					LOCTEXT("AddPropertyPinButtonLabelFmt", "Add {0} pin"),
					FText::FromName(ArrayProperty->GetFName())))
		.ColorAndOpacity(FLinearColor::White)
	]
	+SHorizontalBox::Slot()
	.AutoWidth()
	. VAlign(VAlign_Center)
	. Padding(7, 0, 0, 0)
	[
		SNew(SImage)
		.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
	];

	TSharedRef<SButton> AddPinButton = SNew(SButton)
	.ContentPadding(0.0f)
	.ButtonStyle( FAppStyle::Get(), "NoBorder" )
	.OnClicked(this, &SObjectTreeGraphNode::OnAddArrayPropertyPin, ArrayProperty)
	.IsEnabled(this, &SGraphNode::IsNodeEditable)
	.ToolTipText(FText::Format(
				LOCTEXT("AddPropertyPinButtonTooltipFmt", "Adds a new pin for the '{0}' property on this node"),
				FText::FromName(ArrayProperty->GetFName())))
	[
		ButtonContent
	];

	AddPinButton->SetCursor( EMouseCursor::Hand );

	return AddPinButton;
}

FReply SObjectTreeGraphNode::OnAddArrayPropertyPin(FArrayProperty* ArrayProperty)
{
	if (ObjectGraphNode)
	{
		UEdGraph* Graph = ObjectGraphNode->GetGraph();
		const UObjectTreeGraphSchema* Schema = Cast<const UObjectTreeGraphSchema>(Graph->GetSchema());

		UEdGraphPin* ArrayPin = ObjectGraphNode->GetPinForProperty(ArrayProperty);
		Schema->InsertArrayItemPin(ArrayPin, INDEX_NONE);

		UpdateGraphNode();
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

