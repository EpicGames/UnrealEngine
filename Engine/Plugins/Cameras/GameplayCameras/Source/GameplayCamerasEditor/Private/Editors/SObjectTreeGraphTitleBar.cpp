// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SObjectTreeGraphTitleBar.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SObjectTreeGraphTitleBar"

void SObjectTreeGraphTitleBar::Construct(const FArguments& InArgs)
{
	Graph = InArgs._Graph;
	OnBreadcrumbClicked = InArgs._OnBreadcrumbClicked;

	const ISlateStyle& AppStyle = FAppStyle::Get();
	const FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
	const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		];

	if (InArgs._HistoryNavigationWidget)
	{
		// Navigation widget.
		HorizontalBox->AddSlot()
		.AutoWidth()
		[
			InArgs._HistoryNavigationWidget.ToSharedRef()
		];
		HorizontalBox->AddSlot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		];
	}

	{
		// Title icon and breadcrumb trail.
		HorizontalBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 10.0f,5.0f )
				.VAlign(VAlign_Center)
				[
					// Icon.
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_24x")))
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					// Breadcrumb trail.
					SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
					.Orientation(Orient_Horizontal)
					.ScrollBarVisibility(EVisibility::Collapsed)

					+SScrollBox::Slot()
					.Padding(0.f)
					.VAlign(VAlign_Center)
					[
						// Root breadcrumb, defined by title text.
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(BreadcrumbTrailPadding)
						[
							SNew(STextBlock)
							.Text(InArgs._TitleText)
							.TextStyle(AppStyle, TEXT("GraphBreadcrumbButtonText"))
							.Visibility(EVisibility::Visible)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(BreadcrumbButtonImage)
							.Visibility(EVisibility::Visible)
						]

						// Graph name, defined by current graph.
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<UEdGraph*>)
							.ButtonStyle(AppStyle, "GraphBreadcrumbButton")
							.TextStyle(AppStyle, "GraphBreadcrumbButtonText")
							.ButtonContentPadding(BreadcrumbTrailPadding)
							.DelimiterImage(BreadcrumbButtonImage)
							.PersistentBreadcrumbs(true)
							.OnCrumbClicked(this, &SObjectTreeGraphTitleBar::OnBreadcrumbClickedImpl)
						]
					]
				]
			]
		];
	}
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.HAlign(HAlign_Fill)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EventGraphTitleBar")))
			[
				HorizontalBox
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SAssignNew(GraphListView, SListView<TSharedPtr<FObjectTreeGraphInfo>>)
				.ListItemsSource(InArgs._GraphList)
				.OnGenerateRow(this, &SObjectTreeGraphTitleBar::GenerateGraphInfoRow)
				.SelectionMode( ESelectionMode::None )
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	RebuildBreadcrumbTrail();
	BreadcrumbTrailScrollBox->ScrollToEnd();
}

TSharedRef<ITableRow> SObjectTreeGraphTitleBar::GenerateGraphInfoRow(TSharedPtr<FObjectTreeGraphInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	const int32 FontSize = 9;

	const FObjectTreeGraphInfo* GraphInfo = Item.Get();

	if (GraphInfo->GraphName.IsEmpty())
	{
		return SNew(STableRow<TSharedPtr<FObjectTreeGraphInfo>>, OwnerTable)
			[
				SNew(SSpacer)
			];
	}
	else 
	{
		return SNew(STableRow<TSharedPtr<FObjectTreeGraphInfo>>, OwnerTable)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", FontSize))
				.Text(FText::FromString(GraphInfo->GraphName))
			];
	}
}

void SObjectTreeGraphTitleBar::OnBreadcrumbClickedImpl(UEdGraph* const& Item)
{
	OnBreadcrumbClicked.ExecuteIfBound(Item);
}

void SObjectTreeGraphTitleBar::RebuildBreadcrumbTrail()
{
	TArray<UEdGraph*> GraphStack;
	for (UEdGraph* OuterChain = Graph; OuterChain != nullptr; OuterChain = UEdGraph::GetOuterGraph(OuterChain))
	{
		GraphStack.Push(OuterChain);
	}

	BreadcrumbTrail->ClearCrumbs(false);

	const UEdGraph* LastGraph = (GraphStack.Num() > 0 ? GraphStack.Last() : nullptr);

	while (GraphStack.Num() > 0)
	{
		UEdGraph* CurGraph = GraphStack.Pop();
		
		TAttribute<FText> TitleText = TAttribute<FText>::Create(
				TAttribute<FText>::FGetter::CreateStatic(&SObjectTreeGraphTitleBar::GetTitleForOneCrumb, LastGraph, (const UEdGraph*)CurGraph));
		BreadcrumbTrail->PushCrumb(TitleText, CurGraph);
	}
}

FText SObjectTreeGraphTitleBar::GetTitleForOneCrumb(const UEdGraph* BaseGraph, const UEdGraph* CurGraph)
{
	const UEdGraphSchema* Schema = CurGraph->GetSchema();

	FGraphDisplayInfo DisplayInfo;
	Schema->GetGraphDisplayInformation(*CurGraph, DisplayInfo);

	FFormatNamedArguments Args;
	Args.Add(TEXT("BreadcrumbDisplayName"), DisplayInfo.DisplayName);
	Args.Add(TEXT("BreadcrumbNotes"), FText::FromString(DisplayInfo.GetNotesAsString()));
	return FText::Format(LOCTEXT("BreadcrumbTitle", "{BreadcrumbDisplayName} {BreadcrumbNotes}"), Args);
}

#undef LOCTEXT_NAMESPACE

