// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editors/SObjectTreeGraphToolbox.h"

#include "Algo/AnyOf.h"
#include "EditorClassUtils.h"
#include "Editors/ObjectTreeDragDropOp.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Styles/ObjectTreeGraphEditorStyle.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SObjectTreeGraphToolbox"

void SObjectTreeGraphToolboxEntry::Construct(const FArguments& InArgs)
{
	ObjectClass = InArgs._ObjectClass;

	if (ObjectClass)
	{
		if (InArgs._GraphConfig)
		{
			DisplayNameText = InArgs._GraphConfig->GetDisplayNameText(ObjectClass);
		}
		else
		{
			DisplayNameText = ObjectClass->GetDisplayNameText();
		}
	}

	TSharedRef<FObjectTreeGraphEditorStyle> ObjectTreeStyle = FObjectTreeGraphEditorStyle::Get();

	TSharedPtr<IToolTip> EntryToolTip = FEditorClassUtils::GetTooltip(ObjectClass);
	TSharedRef<SWidget> DocWidget = FEditorClassUtils::GetDocumentationLinkWidget(ObjectClass);

	const FButtonStyle& ButtonStyle = ObjectTreeStyle->GetWidgetStyle<FButtonStyle>("ObjectTreeGraphToolbox.Entry");
	NormalImage = &ButtonStyle.Normal;
	HoverImage = &ButtonStyle.Hovered;
	PressedImage = &ButtonStyle.Pressed; 

	ChildSlot
	.Padding(FMargin(8.f, 2.f, 12.f, 2.f))
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(ObjectTreeStyle->GetBrush("ObjectTreeGraphToolbox.Entry.Background"))
			.Cursor(EMouseCursor::GrabHand)
			.ToolTip(EntryToolTip)
			.Padding(0)
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[
					SNew(SBorder)
					.BorderImage(ObjectTreeStyle->GetBrush("ObjectTreeGraphToolbox.Entry.LabelBack"))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.Padding(8.f, 4.f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(ObjectTreeStyle, "ObjectTreeGraphToolbox.Entry.Name")
							.Text(DisplayNameText)
							.HighlightText(InArgs._HighlightText)
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							DocWidget
						]
					]
				]
			]
		]
		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(this, &SObjectTreeGraphToolboxEntry::GetBorder)
			.Cursor(EMouseCursor::GrabHand)
			.ToolTip(EntryToolTip)
		]
	];
}

FReply SObjectTreeGraphToolboxEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsPressed = true;

		return FReply::Handled().DetectDrag(SharedThis(this), MouseEvent.GetEffectingButton());
	}

	return FReply::Unhandled();
}

FReply SObjectTreeGraphToolboxEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsPressed = false;
	}

	return FReply::Unhandled();
}

FReply SObjectTreeGraphToolboxEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;

	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FObjectTreeClassDragDropOp::New(ObjectClass));
	}
	else
	{
		return FReply::Handled();
	}
}

const FSlateBrush* SObjectTreeGraphToolboxEntry::GetBorder() const
{
	if (bIsPressed)
	{
		return PressedImage;
	}
	else if (IsHovered())
	{
		return HoverImage;
	}
	else
	{
		return NormalImage;
	}
}

void SObjectTreeGraphToolbox::Construct(const FArguments& InArgs)
{
	GraphConfig = InArgs._GraphConfig;

	SearchTextFilter = MakeShareable(new FEntryTextFilter(
		FEntryTextFilter::FItemToStringArray::CreateSP(this, &SObjectTreeGraphToolbox::GetEntryStrings)));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Search"))
				.OnTextChanged(this, &SObjectTreeGraphToolbox::OnSearchTextChanged)
				.OnTextCommitted(this, &SObjectTreeGraphToolbox::OnSearchTextCommitted)
			]
		]
		+SVerticalBox::Slot()
		.Padding(0.f, 3.f)
		[
			SAssignNew(ListView, SListView<UClass*>)
				.ListItemsSource(&FilteredItemSource)
				.OnGenerateRow(this, &SObjectTreeGraphToolbox::OnGenerateItemRow)
		]
	];

	bUpdateItemSource = true;
	bUpdateFilteredItemSource = true;
}

void SObjectTreeGraphToolbox::GetEntryStrings(const UClass* InItem, TArray<FString>& OutStrings)
{
	FText DisplayNameText = GraphConfig.GetDisplayNameText(InItem);
	OutStrings.Add(DisplayNameText.ToString());
}

void SObjectTreeGraphToolbox::SetGraphConfig(const FObjectTreeGraphConfig& InGraphConfig)
{
	GraphConfig = InGraphConfig;
	bUpdateItemSource = true;
}

void SObjectTreeGraphToolbox::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bUpdateItemSource)
	{
		UpdateItemSource();
	}

	if (bUpdateItemSource || bUpdateFilteredItemSource)
	{
		UpdateFilteredItemSource();
	}

	const bool bRequestListRefresh = bUpdateItemSource || bUpdateFilteredItemSource;
	bUpdateItemSource = false;
	bUpdateFilteredItemSource = false;

	if (bRequestListRefresh)
	{
		ListView->RequestListRefresh();
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SObjectTreeGraphToolbox::UpdateItemSource()
{
	ItemSource.Reset();
	GraphConfig.GetConnectableClasses(ItemSource, true);
	ItemSource.Sort([](UClass& A, UClass& B) { return A.GetFName().Compare(B.GetFName()) < 0; });
}

void SObjectTreeGraphToolbox::UpdateFilteredItemSource()
{
	FilteredItemSource.Reset();

	if (!SearchTextFilter->GetRawFilterText().IsEmpty())
	{
		for (UClass* Item : ItemSource)
		{
			if (SearchTextFilter->PassesFilter(Item))
			{
				FilteredItemSource.Add(Item);
			}
		}
	}
	else
	{
		FilteredItemSource = ItemSource;
	}
}

TSharedRef<ITableRow> SObjectTreeGraphToolbox::OnGenerateItemRow(UClass* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<UClass*>, OwnerTable)
		[
			SNew(SObjectTreeGraphToolboxEntry)
				.ObjectClass(Item)
				.GraphConfig(&GraphConfig)
				.HighlightText(this, &SObjectTreeGraphToolbox::GetHighlightText)
		];
}

void SObjectTreeGraphToolbox::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(SearchTextFilter->GetFilterErrorText());

	bUpdateFilteredItemSource = true;
}

void SObjectTreeGraphToolbox::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	OnSearchTextChanged(InFilterText);
}

FText SObjectTreeGraphToolbox::GetHighlightText() const
{
	return SearchTextFilter->GetRawFilterText();
}

#undef LOCTEXT_NAMESPACE

