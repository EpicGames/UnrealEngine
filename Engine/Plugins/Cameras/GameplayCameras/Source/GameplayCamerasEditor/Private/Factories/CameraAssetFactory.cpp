// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraAssetFactory.h"

#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Helpers/CameraDirectorClassPicker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAssetFactory)

#define LOCTEXT_NAMESPACE "CameraAssetFactory"

UCameraAssetFactory::UCameraAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraAsset::StaticClass();
}

UObject* UCameraAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCameraAsset* NewCameraAsset = NewObject<UCameraAsset>(Parent, Class, Name, Flags | RF_Transactional);

	if (CameraDirectorClass)
	{
		UCameraDirector* NewCameraDirector = NewObject<UCameraDirector>(NewCameraAsset, CameraDirectorClass, NAME_None, RF_Transactional);
		NewCameraAsset->SetCameraDirector(NewCameraDirector);

		// Let the camera director do some scaffolding.
		FCameraDirectorFactoryCreateParams CreateParams;
		NewCameraDirector->FactoryCreateAsset(CreateParams);
	}

	return NewCameraAsset;
}

bool UCameraAssetFactory::ConfigureProperties()
{
	using namespace UE::Cameras;

	FCameraDirectorClassPicker Picker;

	CameraDirectorClass = nullptr;
	TSubclassOf<UCameraDirector> ChosenClass;
	const bool bPressedOk = Picker.PickCameraDirectorClass(ChosenClass);
	if (bPressedOk)
	{
		CameraDirectorClass = ChosenClass;
	}
	return bPressedOk;
}

bool UCameraAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE


