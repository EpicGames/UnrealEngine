// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraAssetFactory.generated.h"

class UCameraAsset;
class UCameraDirector;

/**
 * Implements a factory for UCameraAsset objects.
 */
UCLASS(hidecategories=Object)
class UCameraAssetFactory : public UFactory
{
	GENERATED_BODY()

public:

	UCameraAssetFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;

private:

	/** The type of camera director to create for this asset. */
	UPROPERTY(EditAnywhere, Category=Camera)
	TSubclassOf<UCameraDirector> CameraDirectorClass;
};

