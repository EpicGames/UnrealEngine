// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraRigAssetFactory.h"

#include "Core/CameraRigAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigAssetFactory)

#define LOCTEXT_NAMESPACE "CameraRigAssetFactory"

UCameraRigAssetFactory::UCameraRigAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraRigAsset::StaticClass();
}

UObject* UCameraRigAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCameraRigAsset* NewCameraRig = NewObject<UCameraRigAsset>(Parent, Class, Name, Flags | RF_Transactional);
	return NewCameraRig;
}

bool UCameraRigAssetFactory::ConfigureProperties()
{
	return true;
}

bool UCameraRigAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE


