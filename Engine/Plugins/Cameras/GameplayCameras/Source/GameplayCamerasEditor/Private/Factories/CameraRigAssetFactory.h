// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraRigAssetFactory.generated.h"

class UCameraRigAsset;

/**
 * Implements a factory for UCameraRigAsset objects.
 */
UCLASS(hidecategories=Object)
class UCameraRigAssetFactory : public UFactory
{
	GENERATED_BODY()

	UCameraRigAssetFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
};

