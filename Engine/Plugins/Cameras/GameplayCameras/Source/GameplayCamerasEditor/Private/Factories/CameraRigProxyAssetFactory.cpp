// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraRigProxyAssetFactory.h"

#include "Core/CameraRigProxyAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigProxyAssetFactory)

#define LOCTEXT_NAMESPACE "CameraRigProxyAssetFactory"

UCameraRigProxyAssetFactory::UCameraRigProxyAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraRigProxyAsset::StaticClass();
}

UObject* UCameraRigProxyAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCameraRigProxyAsset* NewCameraRig = NewObject<UCameraRigProxyAsset>(Parent, Class, Name, Flags | RF_Transactional);
	return NewCameraRig;
}

bool UCameraRigProxyAssetFactory::ConfigureProperties()
{
	return true;
}

bool UCameraRigProxyAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE

