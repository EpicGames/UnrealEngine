// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraRigProxyAssetFactory.generated.h"

class UCameraRigProxyAsset;

/**
 * Implements a factory for UCameraRigProxyAsset objects.
 */
UCLASS(hidecategories=Object)
class UCameraRigProxyAssetFactory : public UFactory
{
	GENERATED_BODY()

	UCameraRigProxyAssetFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
};

