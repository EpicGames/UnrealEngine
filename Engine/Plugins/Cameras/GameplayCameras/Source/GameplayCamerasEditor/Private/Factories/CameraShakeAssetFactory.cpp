// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraShakeAssetFactory.h"

#include "Core/CameraShakeAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeAssetFactory)

#define LOCTEXT_NAMESPACE "CameraShakeAssetFactory"

UCameraShakeAssetFactory::UCameraShakeAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraShakeAsset::StaticClass();
}

UObject* UCameraShakeAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCameraShakeAsset* NewCameraShake = NewObject<UCameraShakeAsset>(Parent, Class, Name, Flags | RF_Transactional);
	return NewCameraShake;
}

bool UCameraShakeAssetFactory::ConfigureProperties()
{
	return true;
}

bool UCameraShakeAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE


