// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraShakeAssetFactory.generated.h"

class UCameraShakeAsset;

/**
 * Implements a factory for UCameraShakeAsset objects.
 */
UCLASS(hidecategories=Object)
class UCameraShakeAssetFactory : public UFactory
{
	GENERATED_BODY()

	UCameraShakeAssetFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
};

