// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraVariableCollectionFactory.h"

#include "Core/CameraVariableCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableCollectionFactory)

#define LOCTEXT_NAMESPACE "CameraVariableCollectionFactory"

UCameraVariableCollectionFactory::UCameraVariableCollectionFactory(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraVariableCollection::StaticClass();
}

FText UCameraVariableCollectionFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Camera Variable Collection");
}

UObject* UCameraVariableCollectionFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCameraVariableCollection* NewVariableCollection = NewObject<UCameraVariableCollection>(Parent, Class, Name, Flags | RF_Transactional);
	return NewVariableCollection;
}

#undef LOCTEXT_NAMESPACE

