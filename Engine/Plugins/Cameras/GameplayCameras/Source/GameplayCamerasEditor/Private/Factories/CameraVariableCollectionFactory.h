// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraVariableCollectionFactory.generated.h"

class UCameraVariableCollection;

/**
 * Implements a factory for camera variable collections.
 */
UCLASS(hidecategories=Object)
class UCameraVariableCollectionFactory : public UFactory
{
	GENERATED_BODY()

public:

	UCameraVariableCollectionFactory(const FObjectInitializer& ObjectInit);

	// UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

