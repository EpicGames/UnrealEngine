// Copyright Epic Games, Inc. All Rights Reserved.

#include "Family/CameraAssetFamily.h"

#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Family/GameplayCamerasFamilyHelper.h"

#define LOCTEXT_NAMESPACE "CameraAssetFamily"

namespace UE::Cameras
{

FCameraAssetFamily::FCameraAssetFamily(UCameraAsset* InRootAsset)
	: RootAsset(InRootAsset)
{
	ensure(InRootAsset);
}

UObject* FCameraAssetFamily::GetRootAsset() const
{
	return RootAsset;
}

void FCameraAssetFamily::GetAssetTypes(TArray<UClass*>& OutAssetTypes) const
{
	OutAssetTypes.Add(UCameraAsset::StaticClass());
	OutAssetTypes.Add(UCameraDirector::StaticClass());
	OutAssetTypes.Add(UCameraRigAsset::StaticClass());
	OutAssetTypes.Add(UCameraRigProxyAsset::StaticClass());
}

void FCameraAssetFamily::FindAssetsOfType(UClass* InAssetType, TArray<FAssetData>& OutAssets) const
{
	if (!RootAsset)
	{
		return;
	}

	if (InAssetType == UCameraAsset::StaticClass())
	{
		OutAssets.Add(RootAsset);
		return;
	}

	UCameraDirector* CameraDirector = RootAsset->GetCameraDirector();
	if (!CameraDirector)
	{
		return;
	}

	if (InAssetType == UCameraDirector::StaticClass())
	{
		TArray<FAssetData> ThisAsset{ FAssetData(RootAsset) };
		TArray<FAssetData> ExternalCameraDirectors;
		FGameplayCamerasFamilyHelper::GetExternalCameraDirectorAssets(ThisAsset, OutAssets);
		return;
	}

	TSet<UCameraRigAsset*> AllCameraRigs;
	TSet<UCameraRigProxyAsset*> AllCameraRigProxies;

	// Get camera rigs and proxies from the director's logic.
	{
		FCameraDirectorRigUsageInfo UsageInfo;
		CameraDirector->GatherRigUsageInfo(UsageInfo);

		AllCameraRigs.Append(UsageInfo.CameraRigs);
		AllCameraRigProxies.Append(UsageInfo.CameraRigProxies);
	}
	// Add camera rigs and proxies from the proxy table.
	for (const FCameraRigProxyRedirectTableEntry& Entry : CameraDirector->CameraRigProxyRedirectTable.Entries)
	{
		if (Entry.CameraRigProxy)
		{
			AllCameraRigProxies.Add(Entry.CameraRigProxy);
		}
		if (Entry.CameraRig)
		{
			AllCameraRigs.Add(Entry.CameraRig);
		}
	}

	if (InAssetType == UCameraRigAsset::StaticClass())
	{
		for (UCameraRigAsset* CameraRig : AllCameraRigs)
		{
			OutAssets.Add(FAssetData(CameraRig));
		}
	}
	else if (InAssetType == UCameraRigProxyAsset::StaticClass())
	{
		for (UCameraRigProxyAsset* CameraRigProxy : AllCameraRigProxies)
		{
			OutAssets.Add(FAssetData(CameraRigProxy));
		}
	}
}

FText FCameraAssetFamily::GetAssetTypeTooltip(UClass* InAssetType) const
{
	if (InAssetType == UCameraRigAsset::StaticClass())
	{
		return LOCTEXT("CameraRigAssetTypeTooltip", "Open camera rigs referenced by this asset.");
	}
	else if (InAssetType == UCameraRigProxyAsset::StaticClass())
	{
		return LOCTEXT("CameraRigProxyAssetTypeTooltip", "Open camera rig proxies referenced by this asset.");
	}
	return FText();
}

const FSlateBrush* FCameraAssetFamily::GetAssetIcon(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetIcon(InAssetType);
}

FSlateColor FCameraAssetFamily::GetAssetTint(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetTint(InAssetType);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

