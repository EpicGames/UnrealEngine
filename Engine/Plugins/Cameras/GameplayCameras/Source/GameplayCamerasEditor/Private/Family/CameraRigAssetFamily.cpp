// Copyright Epic Games, Inc. All Rights Reserved.

#include "Family/CameraRigAssetFamily.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Family/GameplayCamerasFamilyHelper.h"
#include "UObject/ReferencerFinder.h"

#define LOCTEXT_NAMESPACE "CameraRigAssetFamily"

namespace UE::Cameras
{

FCameraRigAssetFamily::FCameraRigAssetFamily(UCameraRigAsset* InRootAsset)
	: RootAsset(InRootAsset)
{
	ensure(InRootAsset);
}

UObject* FCameraRigAssetFamily::GetRootAsset() const
{
	return RootAsset;
}

void FCameraRigAssetFamily::GetAssetTypes(TArray<UClass*>& OutAssetTypes) const
{
	OutAssetTypes.Add(UCameraAsset::StaticClass());
	OutAssetTypes.Add(UCameraDirector::StaticClass());
	OutAssetTypes.Add(UCameraRigAsset::StaticClass());
}

void FCameraRigAssetFamily::FindAssetsOfType(UClass* InAssetType, TArray<FAssetData>& OutAssets) const
{
	if (!RootAsset)
	{
		return;
	}

	if (InAssetType == UCameraRigAsset::StaticClass())
	{
		OutAssets.Add(RootAsset);
	}
	else if (InAssetType == UCameraAsset::StaticClass())
	{
		FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(RootAsset, OutAssets);
	}
	else if (InAssetType == UCameraDirector::StaticClass())
	{
		TArray<FAssetData> CameraAssets;
		FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(RootAsset, CameraAssets);
		FGameplayCamerasFamilyHelper::GetExternalCameraDirectorAssets(CameraAssets, OutAssets);
	}
}

FText FCameraRigAssetFamily::GetAssetTypeTooltip(UClass* InAssetType) const
{
	if (InAssetType == UCameraAsset::StaticClass())
	{
		return LOCTEXT("CameraAssetTypeTooltip", "Open camera assets referencing this asset.");
	}
	return FText();
}

const FSlateBrush* FCameraRigAssetFamily::GetAssetIcon(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetIcon(InAssetType);
}

FSlateColor FCameraRigAssetFamily::GetAssetTint(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetTint(InAssetType);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

