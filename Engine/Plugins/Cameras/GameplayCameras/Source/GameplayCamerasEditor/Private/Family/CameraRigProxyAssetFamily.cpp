// Copyright Epic Games, Inc. All Rights Reserved.

#include "Family/CameraRigProxyAssetFamily.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigProxyAsset.h"
#include "Family/GameplayCamerasFamilyHelper.h"

#define LOCTEXT_NAMESPACE "CameraRigProxyAssetFamily"

namespace UE::Cameras
{

FCameraRigProxyAssetFamily::FCameraRigProxyAssetFamily(UCameraRigProxyAsset* InRootAsset)
	: RootAsset(InRootAsset)
{
	ensure(InRootAsset);
}

UObject* FCameraRigProxyAssetFamily::GetRootAsset() const
{
	return RootAsset;
}

void FCameraRigProxyAssetFamily::GetAssetTypes(TArray<UClass*>& OutAssetTypes) const
{
	OutAssetTypes.Add(UCameraAsset::StaticClass());
	OutAssetTypes.Add(UCameraDirector::StaticClass());
	OutAssetTypes.Add(UCameraRigProxyAsset::StaticClass());
}

void FCameraRigProxyAssetFamily::FindAssetsOfType(UClass* InAssetType, TArray<FAssetData>& OutAssets) const
{
	if (!RootAsset)
	{
		return;
	}

	if (InAssetType == UCameraRigProxyAsset::StaticClass())
	{
		OutAssets.Add(RootAsset);
	}
	else if (InAssetType == UCameraAsset::StaticClass())
	{
		FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(RootAsset, OutAssets);
	}
	else if (InAssetType == UCameraDirector::StaticClass())
	{
		TArray<FAssetData> CameraAssets;
		FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(RootAsset, CameraAssets);
		FGameplayCamerasFamilyHelper::GetExternalCameraDirectorAssets(CameraAssets, OutAssets);
	}
}

FText FCameraRigProxyAssetFamily::GetAssetTypeTooltip(UClass* InAssetType) const
{
	if (InAssetType == UCameraAsset::StaticClass())
	{
		return LOCTEXT("CameraRigAssetTypeTooltip", "Open camera assets referencing this asset.");
	}
	return FText();
}

const FSlateBrush* FCameraRigProxyAssetFamily::GetAssetIcon(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetIcon(InAssetType);
}

FSlateColor FCameraRigProxyAssetFamily::GetAssetTint(UClass* InAssetType) const
{
	return FGameplayCamerasFamilyHelper::GetAssetTint(InAssetType);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

