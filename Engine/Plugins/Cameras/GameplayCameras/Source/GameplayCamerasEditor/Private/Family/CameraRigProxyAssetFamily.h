// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGameplayCamerasFamily.h"

class UCameraRigProxyAsset;

namespace UE::Cameras
{

class FCameraRigProxyAssetFamily : public IGameplayCamerasFamily
{
public:

	FCameraRigProxyAssetFamily(UCameraRigProxyAsset* InRootAsset);

	// IGameplayCamerasFamily interface.
	virtual UObject* GetRootAsset() const override;
	virtual void GetAssetTypes(TArray<UClass*>& OutAssetTypes) const override;
	virtual void FindAssetsOfType(UClass* InAssetType, TArray<FAssetData>& OutAssets) const override;
	virtual FText GetAssetTypeTooltip(UClass* InAssetType) const override;
	virtual const FSlateBrush* GetAssetIcon(UClass* InAssetType) const override;
	virtual FSlateColor GetAssetTint(UClass* InAssetType) const override;

private:

	UCameraRigProxyAsset* RootAsset;
};

}  // namespace UE::Cameras

