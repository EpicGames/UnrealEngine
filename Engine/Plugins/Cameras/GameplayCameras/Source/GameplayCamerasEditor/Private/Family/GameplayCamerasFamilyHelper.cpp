// Copyright Epic Games, Inc. All Rights Reserved.

#include "Family/GameplayCamerasFamilyHelper.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "UObject/Package.h"

namespace UE::Cameras
{

const FSlateBrush* FGameplayCamerasFamilyHelper::GetAssetIcon(UClass* InAssetType)
{
	if (InAssetType == UCameraAsset::StaticClass())
	{
		return FGameplayCamerasEditorStyle::Get()->GetBrush("Family.CameraAsset");
	}
	else if (InAssetType == UCameraDirector::StaticClass())
	{
		return FGameplayCamerasEditorStyle::Get()->GetBrush("Family.CameraDirector");
	}
	else if (InAssetType == UCameraRigAsset::StaticClass())
	{
		return FGameplayCamerasEditorStyle::Get()->GetBrush("Family.CameraRigAsset");
	}
	else if (InAssetType == UCameraRigProxyAsset::StaticClass())
	{
		return FGameplayCamerasEditorStyle::Get()->GetBrush("Family.CameraRigProxyAsset");
	}
	return nullptr;
}

FSlateColor FGameplayCamerasFamilyHelper::GetAssetTint(UClass* InAssetType)
{
	static const FColor Teal(23, 126, 137);
	static const FColor MidnightGreen(8, 76, 97);
	static const FColor Poppy(219, 58, 52);
	static const FColor Sunglow(255, 200, 87);

	if (InAssetType == UCameraAsset::StaticClass())
	{
		return FSlateColor(Sunglow);
	}
	else if (InAssetType == UCameraDirector::StaticClass())
	{
		return FSlateColor(Poppy);
	}
	else if (InAssetType == UCameraRigAsset::StaticClass())
	{
		return FSlateColor(Teal);
	}
	else if (InAssetType == UCameraRigProxyAsset::StaticClass())
	{
		return FSlateColor(MidnightGreen);
	}
	return FSlateColor();
}

namespace Internal
{

static void FindRelatedCameraAssets(UObject* Object, FName TagName, TFunctionRef<bool(const FCameraDirectorRigUsageInfo&)> ContainsPredicate, TArray<FAssetData>& OutCameraAssets)
{
	const FName RootPackageName = Object->GetPackage()->GetFName();

	TArray<FAssetData> AllCameraAssets;
	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
	AssetRegistry.GetAssetsByClass(UCameraAsset::StaticClass()->GetClassPathName(), AllCameraAssets);
	for (const FAssetData& CameraAsset : AllCameraAssets)
	{
		// By default, use the asset tags to know which camera asset uses which camera rigs.
		// If the asset is loaded, it might have been modified and not saved yet, so use the in-memory
		// object instead.
		// If the asset doesn't have tags, it hasn't been saved since tags were added, so load it
		// and also use it directly in memory.
		bool bUseObjectDirectly = (CameraAsset.IsAssetLoaded());
		if (!bUseObjectDirectly)
		{
			FString UsedReferencesTag = CameraAsset.GetTagValueRef<FString>(TagName);
			if (!UsedReferencesTag.IsEmpty())
			{
				TArray<FString> UsedReferences;
				UsedReferencesTag.ParseIntoArrayLines(UsedReferences);
				if (UsedReferences.Contains(RootPackageName))
				{
					OutCameraAssets.Add(CameraAsset);
				}
			}
			else if (!CameraAsset.FindTag(TagName))
			{
				bUseObjectDirectly = true;
			}
		}
		if (bUseObjectDirectly)
		{
			if (UCameraAsset* LoadedCameraAsset = Cast<UCameraAsset>(CameraAsset.GetAsset()))
			{
				if (UCameraDirector* LoadedCameraDirector = LoadedCameraAsset->GetCameraDirector())
				{
					FCameraDirectorRigUsageInfo UsageInfo;
					LoadedCameraDirector->GatherRigUsageInfo(UsageInfo);
					if (ContainsPredicate(UsageInfo))
					{
						OutCameraAssets.Add(CameraAsset);
					}
				}
			}
		}
	}
}

}  // namespace Internal

void FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(UCameraRigAsset* CameraRig, TArray<FAssetData>& OutCameraAssets)
{
	Internal::FindRelatedCameraAssets(
			CameraRig, 
			TEXT("UsedCameraRigs"),
			[CameraRig](const FCameraDirectorRigUsageInfo& UsageInfo) { return UsageInfo.CameraRigs.Contains(CameraRig); },
			OutCameraAssets);
}

void FGameplayCamerasFamilyHelper::FindRelatedCameraAssets(UCameraRigProxyAsset* CameraRigProxy, TArray<FAssetData>& OutCameraAssets)
{
	Internal::FindRelatedCameraAssets(
			CameraRigProxy, 
			TEXT("UsedCameraRigProxies"),
			[CameraRigProxy](const FCameraDirectorRigUsageInfo& UsageInfo) { return UsageInfo.CameraRigProxies.Contains(CameraRigProxy); },
			OutCameraAssets);
}

void FGameplayCamerasFamilyHelper::GetExternalCameraDirectorAssets(TArrayView<FAssetData> CameraAssets, TArray<FAssetData>& OutExternalCameraDirectors)
{
	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();

	for (const FAssetData& CameraAsset : CameraAssets)
	{
		FString ExternalDirectorName = CameraAsset.GetTagValueRef<FString>(TEXT("ExternalDirector"));
		if (CameraAsset.IsAssetLoaded())
		{
			// If the camera asset is in memory, let's ask for up-to-date information.
			if (UCameraAsset* LoadedCameraAsset = Cast<UCameraAsset>(CameraAsset.GetAsset()))
			{
				if (UCameraDirector* CameraDirector = LoadedCameraAsset->GetCameraDirector())
				{
					FAssetRegistryTagsContextData ContextData(LoadedCameraAsset);
					FAssetRegistryTagsContext Context(ContextData);
					CameraDirector->ExtendAssetRegistryTags(Context);

					if (UObject::FAssetRegistryTag* NewTag = Context.FindTag(TEXT("ExternalDirector")))
					{
						ExternalDirectorName = NewTag->Value;
					}
				}
			}
		}
		if (!ExternalDirectorName.IsEmpty())
		{
			FAssetData ExternalDirectorAsset = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(ExternalDirectorName));
			OutExternalCameraDirectors.Add(ExternalDirectorAsset);
		}
	}
}

}  // namespace UE::Cameras

