// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"

class UClass;
class UObject;
class UCameraAsset;
class UCameraRigAsset;
class UCameraRigProxyAsset;
struct FAssetData;
struct FSlateBrush;

namespace UE::Cameras
{

class FGameplayCamerasFamilyHelper
{
public:

	static const FSlateBrush* GetAssetIcon(UClass* InAssetType);

	static FSlateColor GetAssetTint(UClass* InAssetType);

	static void FindRelatedCameraAssets(UCameraRigAsset* CameraRig, TArray<FAssetData>& OutCameraAssets);
	static void FindRelatedCameraAssets(UCameraRigProxyAsset* CameraRigProxy, TArray<FAssetData>& OutCameraAssets);
	static void GetExternalCameraDirectorAssets(TArrayView<FAssetData> CameraAssets, TArray<FAssetData>& OutExternalCameraDirectors);
};

}  // namespace UE::Cameras

