// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGameplayCamerasEditorModule.h"

#include "ActorFactories/GameplayCameraActorFactory.h"
#include "ActorFactories/GameplayCameraRigActorFactory.h"
#include "AssetTools/CameraAssetEditor.h"
#include "AssetTools/CameraRigAssetEditor.h"
#include "AssetTools/CameraRigProxyAssetEditor.h"
#include "AssetTools/CameraShakeAssetEditor.h"
#include "AssetTools/CameraVariableCollectionEditor.h"
#include "Commands/CameraAssetEditorCommands.h"
#include "Commands/CameraRigAssetEditorCommands.h"
#include "Commands/CameraRigTransitionEditorCommands.h"
#include "Commands/CameraShakeAssetEditorCommands.h"
#include "Commands/CameraVariableCollectionEditorCommands.h"
#include "Commands/GameplayCamerasDebuggerCommands.h"
#include "Commands/ObjectTreeGraphEditorCommands.h"
#include "ComponentVisualizers/GameplayCameraComponentVisualizer.h"
#include "Core/CameraVariableCollection.h"
#include "Customizations/CameraAssetReferenceDetailsCustomization.h"
#include "Customizations/CameraObjectInterfaceParameterDetailsCustomization.h"
#include "Customizations/CameraParameterDetailsCustomizations.h"
#include "Customizations/CameraRigAssetReferenceDetailsCustomization.h"
#include "Customizations/CameraShakeAssetReferenceDetailsCustomization.h"
#include "Customizations/CameraVariableReferenceDetailsCustomizations.h"
#include "Customizations/FilmbackCameraNodeDetailsCustomization.h"
#include "Customizations/RichCurveDetailsCustomizations.h"
#include "Debug/CameraDebugCategories.h"
#include "Debugger/SBlendStacksDebugPanel.h"
#include "Debugger/SCameraNodeTreeDebugPanel.h"
#include "Debugger/SCameraPoseStatsDebugPanel.h"
#include "Debugger/SEvaluationServicesDebugPanel.h"
#include "Debugger/SGameplayCamerasDebugger.h"
#include "Directors/BlueprintCameraDirector.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Editors/GameplayCamerasGraphPanelPinFactory.h"
#include "Editors/SCameraVariablePicker.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/GameplayCameraComponentBase.h"
#include "GameplayCameras.h"
#include "GameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasEditorModule.h"
#include "IGameplayCamerasModule.h"
#include "IRewindDebuggerExtension.h"
#include "ISequencerModule.h"
#include "ISettingsModule.h"
#include "K2Node_Event.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "Nodes/Utility/BlueprintCameraNode.h"
#include "ObjectTools.h"
#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#include "Sequencer/CameraFramingZoneTrackEditor.h"
#include "Sequencer/GameplayCameraComponentTrackEditor.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Subsystems/PlacementSubsystem.h"
#include "ToolMenus.h"
#include "Toolkits/BlueprintCameraDirectorAssetEditorMode.h"
#include "Toolkits/CameraAssetEditorToolkit.h"
#include "Toolkits/CameraRigAssetEditorToolkit.h"
#include "Toolkits/CameraShakeAssetEditorToolkit.h"
#include "Toolkits/SingleCameraDirectorAssetEditorMode.h"
#include "Trace/CameraSystemRewindDebuggerExtension.h"
#include "Trace/CameraSystemRewindDebuggerTrack.h"
#include "Trace/CameraSystemTraceModule.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "GameplayCamerasEditor"

DEFINE_LOG_CATEGORY(LogCameraSystemEditor);

const FName IGameplayCamerasEditorModule::GameplayCamerasEditorAppIdentifier("GameplayCamerasEditorApp");
const FName IGameplayCamerasEditorModule::CameraRigAssetEditorToolBarName("CameraRigAssetEditor.ToolBar");

IGameplayCamerasEditorModule& IGameplayCamerasEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<IGameplayCamerasEditorModule>("GameplayCamerasEditor");
}

/**
 * Implements the FGameplayCamerasEditor module.
 */
class FGameplayCamerasEditorModule : public IGameplayCamerasEditorModule
{
public:
	FGameplayCamerasEditorModule()
	{
	}

	virtual void StartupModule() override
	{
		if (GEditor)
		{
			OnPostEngineInit();
		}
		else
		{
			FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGameplayCamerasEditorModule::OnPostEngineInit);
		}

		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FGameplayCamerasEditorModule::OnPreExit);

		FEditorDelegates::OnPreForceDeleteObjects.AddRaw(this, &FGameplayCamerasEditorModule::OnPreForceDeleteObjects);

		RegisterCameraDirectorEditors();
		RegisterCoreDebugCategories();
		RegisterRewindDebuggerFeatures();
		RegisterDetailsCustomizations();
		RegisterEdGraphUtilities();
		RegisterComponentVisualizers();
		RegisterSequencerTracks();

		InitializeLiveEditManager();

		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(
					this, &FGameplayCamerasEditorModule::RegisterMenus));
	}

	virtual void ShutdownModule() override
	{
		using namespace UE::Cameras;

		UToolMenus::UnRegisterStartupCallback(this);

		FCameraAssetEditorCommands::Unregister();
		FCameraRigAssetEditorCommands::Unregister();
		FCameraRigTransitionEditorCommands::Unregister();
		FCameraShakeAssetEditorCommands::Unregister();
		FCameraVariableCollectionEditorCommands::Unregister();
		FGameplayCamerasDebuggerCommands::Unregister();
		FObjectTreeGraphEditorCommands::Unregister();

		UnregisterCameraDirectorEditors();
		UnregisterCoreDebugCategories();
		UnregisterRewindDebuggerFeatures();
		UnregisterDetailsCustomizations();
		UnregisterEdGraphUtilities();
		UnregisterComponentVisualizers();
		UnregisterSequencerTracks();

		TeardownLiveEditManager();

		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
		FCoreDelegates::OnEnginePreExit.RemoveAll(this);

		FEditorDelegates::OnPreForceDeleteObjects.RemoveAll(this);
	}

	virtual UCameraAssetEditor* CreateCameraAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraAsset* CameraAsset) override
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UCameraAssetEditor* AssetEditor = NewObject<UCameraAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(CameraAsset);
		return AssetEditor;
	}

	virtual UCameraRigAssetEditor* CreateCameraRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraRigAsset* CameraRig) override
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UCameraRigAssetEditor* AssetEditor = NewObject<UCameraRigAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(CameraRig);
		return AssetEditor;
	}

	virtual UCameraRigProxyAssetEditor* CreateCameraRigProxyEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraRigProxyAsset* CameraRigProxy) override
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UCameraRigProxyAssetEditor* AssetEditor = NewObject<UCameraRigProxyAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(CameraRigProxy);
		return AssetEditor;
	}

	virtual UCameraShakeAssetEditor* CreateCameraShakeEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraShakeAsset* CameraShake) override
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UCameraShakeAssetEditor* AssetEditor = NewObject<UCameraShakeAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(CameraShake);
		return AssetEditor;
	}

	virtual UCameraVariableCollectionEditor* CreateCameraVariableCollectionEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraVariableCollection* VariableCollection) override
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UCameraVariableCollectionEditor* AssetEditor = NewObject<UCameraVariableCollectionEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(VariableCollection);
		return AssetEditor;
	}

	virtual TSharedRef<SWidget> CreateCameraVariablePicker(const FCameraVariablePickerConfig& InPickerConfig) override
	{
		using namespace UE::Cameras;

		return SNew(SCameraVariablePicker)
			.CameraVariablePickerConfig(InPickerConfig);
	}

	virtual FDelegateHandle RegisterCameraDirectorEditor(FOnCreateCameraDirectorAssetEditorMode InOnCreateEditor) override
	{
		CameraDirectorEditorCreators.Add(InOnCreateEditor);
		return CameraDirectorEditorCreators.Last().GetHandle();
	}

	virtual TArrayView<const FOnCreateCameraDirectorAssetEditorMode> GetCameraDirectorEditorCreators() const override
	{
		return CameraDirectorEditorCreators;
	}

	virtual void UnregisterCameraDirectorEditor(FDelegateHandle InHandle) override
	{
		CameraDirectorEditorCreators.RemoveAll(
				[=](const FOnCreateCameraDirectorAssetEditorMode& Delegate) 
				{
					return Delegate.GetHandle() == InHandle; 
				});
	}

	virtual void RegisterDebugCategory(const UE::Cameras::FCameraDebugCategoryInfo& InCategoryInfo) override
	{
		if (!ensureMsgf(!InCategoryInfo.Name.IsEmpty(), TEXT("A debug category must at least specify a name!")))
		{
			return;
		}

		DebugCategoryInfos.Add(InCategoryInfo.Name, InCategoryInfo);
	}

	virtual void GetRegisteredDebugCategories(TArray<UE::Cameras::FCameraDebugCategoryInfo>& OutCategoryInfos) override
	{
		DebugCategoryInfos.GenerateValueArray(OutCategoryInfos);
	}

	virtual void UnregisterDebugCategory(const FString& InCategoryName)
	{
		DebugCategoryInfos.Remove(InCategoryName);

	}

	virtual void RegisterDebugCategoryPanel(const FString& InDebugCategory, FOnCreateDebugCategoryPanel OnCreatePanel) override
	{
		if (!DebugCategoryPanelCreators.Contains(InDebugCategory))
		{
			DebugCategoryPanelCreators.Add(InDebugCategory, OnCreatePanel);
		}
		else
		{
			// Override existing creator... for games and projects that want to extend a panel with extra controls.
			DebugCategoryPanelCreators[InDebugCategory] = OnCreatePanel;
		}
	}

	virtual TSharedPtr<SWidget> CreateDebugCategoryPanel(const FString& InDebugCategory) override
	{
		if (FOnCreateDebugCategoryPanel* PanelCreator = DebugCategoryPanelCreators.Find(InDebugCategory))
		{
			return PanelCreator->Execute(InDebugCategory).ToSharedPtr();
		}
		return nullptr;
	}

	virtual void UnregisterDebugCategoryPanel(const FString& InDebugCategory) override
	{
		DebugCategoryPanelCreators.Remove(InDebugCategory);
	}

private:

	void OnPostEngineInit()
	{
		using namespace UE::Cameras;

		SGameplayCamerasDebugger::RegisterTabSpawners();

		GameplayCameraActorFactory.Reset(NewObject<UGameplayCameraActorFactory>());
		GameplayCameraRigActorFactory.Reset(NewObject<UGameplayCameraRigActorFactory>());
		GEditor->ActorFactories.Add(GameplayCameraActorFactory.Get());
		GEditor->ActorFactories.Add(GameplayCameraRigActorFactory.Get());
		if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
		{
			PlacementSubsystem->RegisterAssetFactory(GameplayCameraActorFactory.Get());
			PlacementSubsystem->RegisterAssetFactory(GameplayCameraRigActorFactory.Get());
		}
	}

	void OnPreExit()
	{
		using namespace UE::Cameras;

		SGameplayCamerasDebugger::UnregisterTabSpawners();

		GEditor->ActorFactories.RemoveAll([this](const UActorFactory* ActorFactory)
				{
					return ActorFactory == GameplayCameraActorFactory.Get() || ActorFactory == GameplayCameraRigActorFactory.Get();
				});
		if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
		{
			PlacementSubsystem->UnregisterAssetFactory(GameplayCameraActorFactory.Get());
		}
	}
	
	void OnPreForceDeleteObjects(const TArray<UObject*>& ObjectsToDelete)
	{
		TArray<UCameraVariableCollection*> VariableCollectionsToDelete;
		for (UObject* Object : ObjectsToDelete)
		{
			if (UCameraVariableCollection* VariableCollection = Cast<UCameraVariableCollection>(Object))
			{
				VariableCollectionsToDelete.Add(VariableCollection);
			}
		}

		if (VariableCollectionsToDelete.IsEmpty())
		{
			return;
		}

		// If any variable collection is being force-deleted, let's clear up references
		// to variables from inside it.

		TArray<UObject*> SubObjectsToDelete;
		for (UCameraVariableCollection* VariableCollection : VariableCollectionsToDelete)
		{
			for (UCameraVariableAsset* Variable : VariableCollection->Variables)
			{
				SubObjectsToDelete.Add(Variable);
			}
		}
		ObjectTools::ForceReplaceReferences(nullptr, SubObjectsToDelete);
	}

	void RegisterCameraDirectorEditors()
	{
		using namespace UE::Cameras;

		BuiltInDirectorCreatorHandles.Add(
				RegisterCameraDirectorEditor(FOnCreateCameraDirectorAssetEditorMode::CreateStatic(
						&FSingleCameraDirectorAssetEditorMode::CreateInstance)));
		BuiltInDirectorCreatorHandles.Add(
				RegisterCameraDirectorEditor(FOnCreateCameraDirectorAssetEditorMode::CreateStatic(
						&FBlueprintCameraDirectorAssetEditorMode::CreateInstance)));
	}

	void UnregisterCameraDirectorEditors()
	{
		for (FDelegateHandle Handle : BuiltInDirectorCreatorHandles)
		{
			UnregisterCameraDirectorEditor(Handle);
		}
		BuiltInDirectorCreatorHandles.Reset();
	}

	void RegisterCoreDebugCategories()
	{
		using namespace UE::Cameras;

		TSharedRef<FGameplayCamerasEditorStyle> GameplayCamerasEditorStyle = FGameplayCamerasEditorStyle::Get();
		const FName& GameplayCamerasEditorStyleName = GameplayCamerasEditorStyle->GetStyleSetName();

		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::NodeTree,
				LOCTEXT("NodeTreeDebugCategory", "Node Tree"),
				LOCTEXT("NodeTreeDebugCategoryToolTip", "Shows the entire camrera node evaluator tree"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.NodeTree.Icon")
			});
		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::DirectorTree,
				LOCTEXT("DirectorTreeDebugCategory", "Director Tree"),
				LOCTEXT("DirectorTreeDebugCategoryToolTip", "Shows the active/inactive directors, and their evaluation context"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.DirectorTree.Icon")
			});
		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::BlendStacks,
				LOCTEXT("BlendStacksDebugCategory", "Blend Stacks"),
				LOCTEXT("BlendStacksDebugCategoryToolTip", "Shows a summary of the blend stacks"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.BlendStacks.Icon")
			});
		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::Services,
				LOCTEXT("ServicesDebugCategory", "Services"),
				LOCTEXT("ServicesDebugCategoryToolTip", "Shows the debug information from evaluation services"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.Services.Icon")
			});
		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::PoseStats,
				LOCTEXT("PoseStatsDebugCategory", "Pose Stats"),
				LOCTEXT("PoseStatsDebugCategoryToolTip", "Shows the evaluated camera pose"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.PoseStats.Icon")
			});
		RegisterDebugCategory(FCameraDebugCategoryInfo{
				FCameraDebugCategories::Viewfinder,
				LOCTEXT("ViewfinderDebugCategory", "Viewfinder"),
				LOCTEXT("ViewfinderDebugCategoryToolTip", "Shows an old-school viewfinder on screen"),
				FSlateIcon(GameplayCamerasEditorStyleName, "DebugCategory.Viewfinder.Icon")
			});

		RegisterDebugCategoryPanel(FCameraDebugCategories::NodeTree, FOnCreateDebugCategoryPanel::CreateLambda([](const FString&)
					{
						return SNew(SCameraNodeTreeDebugPanel);
					}));
		RegisterDebugCategoryPanel(FCameraDebugCategories::BlendStacks, FOnCreateDebugCategoryPanel::CreateLambda([](const FString&)
					{
						return SNew(SBlendStacksDebugPanel);
					}));
		RegisterDebugCategoryPanel(FCameraDebugCategories::Services, FOnCreateDebugCategoryPanel::CreateLambda([](const FString&)
					{
						return SNew(SEvaluationServicesDebugPanel);
					}));
		RegisterDebugCategoryPanel(FCameraDebugCategories::PoseStats, FOnCreateDebugCategoryPanel::CreateLambda([](const FString&)
					{
						return SNew(SCameraPoseStatsDebugPanel);
					}));
	}

	void UnregisterCoreDebugCategories()
	{
		using namespace UE::Cameras;

		UnregisterDebugCategoryPanel(FCameraDebugCategories::BlendStacks);
		UnregisterDebugCategoryPanel(FCameraDebugCategories::NodeTree);
	}

	void RegisterMenus()
	{
		using namespace UE::Cameras;

		FCameraAssetEditorCommands::Register();
		FCameraRigAssetEditorCommands::Register();	
		FCameraRigTransitionEditorCommands::Register();
		FCameraShakeAssetEditorCommands::Register();	
		FCameraVariableCollectionEditorCommands::Register();
		FGameplayCamerasDebuggerCommands::Register();
		FObjectTreeGraphEditorCommands::Register();
	}

	void RegisterRewindDebuggerFeatures()
	{
#if UE_GAMEPLAY_CAMERAS_TRACE
		using namespace UE::Cameras;

		TraceModule = MakeShared<UE::Cameras::FCameraSystemTraceModule>();
		RewindDebuggerExtension = MakeShared<FCameraSystemRewindDebuggerExtension>();
		RewindDebuggerTrackCreator = MakeShared<FCameraSystemRewindDebuggerTrackCreator>();

		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerExtension.Get());
		ModularFeatures.RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, RewindDebuggerTrackCreator.Get());
		ModularFeatures.RegisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
	}

	void UnregisterRewindDebuggerFeatures()
	{
#if UE_GAMEPLAY_CAMERAS_TRACE
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerExtension.Get());
		ModularFeatures.UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, RewindDebuggerTrackCreator.Get());
		ModularFeatures.UnregisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
	}

	void RegisterDetailsCustomizations()
	{
		using namespace UE::Cameras;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FCameraParameterDetailsCustomization::Register(PropertyEditorModule);
		FCameraVariableReferenceDetailsCustomization::Register(PropertyEditorModule);
		FRichCurveDetailsCustomization::Register(PropertyEditorModule);

		PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"InstancedOverridablePropertyBag",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(
					&FPropertyBagDetails::MakeInstance));

		PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"CameraAssetReference",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(
					&FCameraAssetReferenceDetailsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"CameraRigAssetReference", 
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(
					&FCameraRigAssetReferenceDetailsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"CameraShakeAssetReference", 
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(
					&FCameraShakeAssetReferenceDetailsCustomization::MakeInstance));

		PropertyEditorModule.RegisterCustomClassLayout(
				"FilmbackCameraNode", 
				FOnGetDetailCustomizationInstance::CreateStatic(
					&FFilmbackCameraNodeDetailsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(
				"CameraObjectInterfaceParameterBase",
				FOnGetDetailCustomizationInstance::CreateStatic(
					&FCameraObjectInterfaceParameterDetailsCustomization::MakeInstance));
	}

	void UnregisterDetailsCustomizations()
	{
		using namespace UE::Cameras;

		FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");

		if (PropertyEditorModule)
		{
			FCameraParameterDetailsCustomization::Unregister(*PropertyEditorModule);
			FCameraVariableReferenceDetailsCustomization::Unregister(*PropertyEditorModule);
			FRichCurveDetailsCustomization::Unregister(*PropertyEditorModule);

			PropertyEditorModule->UnregisterCustomPropertyTypeLayout("InstancedOverridablePropertyBag");

			PropertyEditorModule->UnregisterCustomPropertyTypeLayout("CameraAssetReference");
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout("CameraRigAssetReference");
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout("CameraShakeAssetReference");

			PropertyEditorModule->UnregisterCustomClassLayout("FilmbackCameraNode");
			PropertyEditorModule->UnregisterCustomClassLayout("CameraObjectInterfaceParameterBase");
		}
	}

	void RegisterEdGraphUtilities()
	{
		using namespace UE::Cameras;

		GraphPanelPinFactory = MakeShared<FGameplayCamerasGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

		FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(
				this,
				UBlueprintCameraDirectorEvaluator::StaticClass(),
				GET_FUNCTION_NAME_CHECKED(UBlueprintCameraDirectorEvaluator, RunCameraDirector));
		FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(
				this, 
				UBlueprintCameraDirectorEvaluator::StaticClass(),
				FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(
					this, &FGameplayCamerasEditorModule::OnNewBlueprintCameraDirectorEvaluatorCreated));

		FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(
				this,
				UBlueprintCameraNodeEvaluator::StaticClass(),
				GET_FUNCTION_NAME_CHECKED(UBlueprintCameraNodeEvaluator, InitializeCameraNode));
		FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(
				this,
				UBlueprintCameraNodeEvaluator::StaticClass(),
				GET_FUNCTION_NAME_CHECKED(UBlueprintCameraNodeEvaluator, TickCameraNode));
		FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(
				this, 
				UBlueprintCameraNodeEvaluator::StaticClass(),
				FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(
					this, &FGameplayCamerasEditorModule::OnNewBlueprintCameraNodeEvaluatorCreated));
	}

	static UK2Node_Event* FindEventNodeInEventGraph(UBlueprint* InBlueprint, const FName InEventName) 
	{
		if (InBlueprint->BlueprintType != BPTYPE_Normal)
		{
			return nullptr;
		}
		TObjectPtr<UEdGraph>* FoundItem = InBlueprint->UbergraphPages.FindByPredicate(
					[](UEdGraph* Item) { return Item->GetFName() == TEXT("EventGraph"); });
		if (!FoundItem)
		{
			return nullptr;
		}

		TObjectPtr<UEdGraph> EventGraph(*FoundItem);

		TArray<UK2Node_Event*> EventNodes;
		EventGraph->GetNodesOfClass(EventNodes);
		if (EventNodes.IsEmpty())
		{
			return nullptr;
		}

		UK2Node_Event** FoundEventNode = EventNodes.FindByPredicate(
				[&InEventName](UK2Node_Event* Item) 
				{
					return Item->EventReference.GetMemberName() == InEventName;
				});
		if (FoundEventNode)
		{
			return *FoundEventNode;
		}
		return nullptr;
	}

	void OnNewBlueprintCameraDirectorEvaluatorCreated(UBlueprint* InBlueprint)
	{
		const FName EventName = GET_FUNCTION_NAME_CHECKED(UBlueprintCameraDirectorEvaluator, RunCameraDirector);
		if (UK2Node_Event* RunEventNode = FindEventNodeInEventGraph(InBlueprint, EventName))
		{
			const FText RunEventNodeCommentText = LOCTEXT(
					"BlueprintCameraDirector_RunEventComment",
					"Implement your camera director logic starting from here.\n"
					"This node is currently disabled, but start dragging off pins to enable it.\n"
					"Call ActivateCameraRig at least once to declare which camera rig(s) should be active this frame.");
			RunEventNode->NodeComment = RunEventNodeCommentText.ToString();
			RunEventNode->bCommentBubbleVisible = true;
		}
	}

	void OnNewBlueprintCameraNodeEvaluatorCreated(UBlueprint* InBlueprint)
	{
		const FName EventName = GET_FUNCTION_NAME_CHECKED(UBlueprintCameraNodeEvaluator, TickCameraNode);
		if (UK2Node_Event* RunEventNode = FindEventNodeInEventGraph(InBlueprint, EventName))
		{
			const FText RunEventNodeCommentText = LOCTEXT(
					"BlueprintCameraNode_TickEventComment",
					"Implement your camera node logic starting from here.\n"
					"This node is currently disabled, but start dragging off pins to enable it.\n"
					"Use the CameraPose and CameraData variables to affect the camera and its parameters. Other useful variables "
					"and functions are available in the 'Evaluation' category.");
			RunEventNode->NodeComment = RunEventNodeCommentText.ToString();
			RunEventNode->bCommentBubbleVisible = true;
		}
	}

	void UnregisterEdGraphUtilities()
	{
		if (GraphPanelPinFactory)
		{
			FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);
		}

		FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);
	}

	void RegisterComponentVisualizers()
	{
		using namespace UE::Cameras;

		if (GUnrealEd)
		{
			GUnrealEd->RegisterComponentVisualizer(UGameplayCameraComponentBase::StaticClass()->GetFName(), MakeShared<FGameplayCameraComponentVisualizer>());
		}
	}

	void UnregisterComponentVisualizers()
	{
		if (GUnrealEd)
		{
			GUnrealEd->UnregisterComponentVisualizer(UGameplayCameraComponentBase::StaticClass()->GetFName());
		}
	}
	
	void RegisterSequencerTracks()
	{
		using namespace UE::Cameras;

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		GameplayCameraComponentTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FGameplayCameraComponentTrackEditor::CreateTrackEditor));
		CameraFramingZoneTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FCameraFramingZoneTrackEditor>();
#endif
	}

	void UnregisterSequencerTracks()
	{
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.UnRegisterTrackEditor(GameplayCameraComponentTrackCreateEditorHandle);
		SequencerModule.UnRegisterTrackEditor(CameraFramingZoneTrackCreateEditorHandle);
#endif
	}

	void InitializeLiveEditManager()
	{
		using namespace UE::Cameras;

		LiveEditManager = MakeShared<FGameplayCamerasLiveEditManager>();

		IGameplayCamerasModule& CamerasModule = FModuleManager::LoadModuleChecked<IGameplayCamerasModule>("GameplayCameras");
		CamerasModule.SetLiveEditManager(LiveEditManager);
	}

	void TeardownLiveEditManager()
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

		IGameplayCamerasModule& CamerasModule = FModuleManager::LoadModuleChecked<IGameplayCamerasModule>("GameplayCameras");
		CamerasModule.SetLiveEditManager(nullptr);

		LiveEditManager.Reset();
	}
	
private:

	TSharedPtr<UE::Cameras::FGameplayCamerasLiveEditManager> LiveEditManager;

	TArray<FOnCreateCameraDirectorAssetEditorMode> CameraDirectorEditorCreators;
	TArray<FDelegateHandle> BuiltInDirectorCreatorHandles;

	TSharedPtr<UE::Cameras::FGameplayCamerasGraphPanelPinFactory> GraphPanelPinFactory;

	TMap<FString, UE::Cameras::FCameraDebugCategoryInfo> DebugCategoryInfos;
	TMap<FString, FOnCreateDebugCategoryPanel> DebugCategoryPanelCreators;
	
	FDelegateHandle GameplayCameraComponentTrackCreateEditorHandle;
	FDelegateHandle CameraFramingZoneTrackCreateEditorHandle;

	TStrongObjectPtr<UGameplayCameraActorFactory> GameplayCameraActorFactory;
	TStrongObjectPtr<UGameplayCameraRigActorFactory> GameplayCameraRigActorFactory;

#if UE_GAMEPLAY_CAMERAS_TRACE
	TSharedPtr<UE::Cameras::FCameraSystemTraceModule> TraceModule;
	TSharedPtr<UE::Cameras::FCameraSystemRewindDebuggerExtension> RewindDebuggerExtension;
	TSharedPtr<UE::Cameras::FCameraSystemRewindDebuggerTrackCreator> RewindDebuggerTrackCreator;
#endif  // UE_GAMEPLAY_CAMERAS_TRACE
};

IMPLEMENT_MODULE(FGameplayCamerasEditorModule, GameplayCamerasEditor);

#undef LOCTEXT_NAMESPACE

