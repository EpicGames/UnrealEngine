// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCamerasEditorSettings)

UGameplayCamerasEditorSettings::UGameplayCamerasEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CategoryName = TEXT("Plugins");

	CameraNodeTitleColor = FLinearColor(0.190525f, 0.583898f, 1.0f, 1.0f);           // Light blue
	CameraAssetTitleColor = FLinearColor(1.0f, 0.170000f, 0.0f, 1.0f);				 // Orange
	CameraRigAssetTitleColor = FLinearColor(1.0f, 0.170000f, 0.0f, 1.0f);            // Orange
	CameraShakeAssetTitleColor = FLinearColor(1.0f, 0.170000f, 0.0f, 1.0f);          // Orange
	CameraRigTransitionTitleColor = FLinearColor(1.0f, 0.65f, 0.4f, 1.0f);           // Beige
	CameraRigTransitionConditionTitleColor = FLinearColor(0.8f, 0.4f, 0.4f, 1.0f);   // Salmon
	CameraBlendNodeTitleColor = FLinearColor(0.6f, 0.0f, 1.0f, 1.0f);                // Purple
}

