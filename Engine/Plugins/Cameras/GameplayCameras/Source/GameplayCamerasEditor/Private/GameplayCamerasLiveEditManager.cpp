// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCamerasLiveEditManager.h"

#include "Build/CameraAssetBuilder.h"
#include "Build/CameraBuildContext.h"
#include "Build/CameraRigAssetBuilder.h"
#include "Build/CameraShakeAssetBuilder.h"
#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraShakeAsset.h"
#include "Editor.h"
#include "GameplayCamerasSettings.h"
#include "IGameplayCamerasEditorModule.h"
#include "IGameplayCamerasLiveEditListener.h"
#include "GameplayCamerasEditorSettings.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "GameplayCamerasLiveEditManager"

namespace UE::Cameras
{

namespace Internal
{

using FListenerArray = TArray<IGameplayCamerasLiveEditListener*, TInlineAllocator<4>>;

template<typename ObjectType>
void AddListenerImpl(
		TMap<TWeakObjectPtr<const ObjectType>, FListenerArray>& ListenerMap,
		const ObjectType* Object,
		IGameplayCamerasLiveEditListener* Listener)
{
	if (ensure(Object && Listener))
	{
		FListenerArray& Listeners = ListenerMap.FindOrAdd(Object);
		Listeners.Add(Listener);
	}
}

template<typename ObjectType>
void RemoveListenerImpl(
		TMap<TWeakObjectPtr<const ObjectType>, FListenerArray>& ListenerMap,
		const ObjectType* Object,
		IGameplayCamerasLiveEditListener* Listener)
{
	if (ensure(Object && Listener))
	{
		FListenerArray* Listeners = ListenerMap.Find(Object);
		if (ensure(Listeners))
		{
			const int32 NumRemoved = Listeners->RemoveSwap(Listener);
			ensure(NumRemoved == 1);
			if (Listeners->IsEmpty())
			{
				ListenerMap.Remove(Object);
			}
		}
	}
}

}  // namespace Internal

FGameplayCamerasLiveEditManager::FGameplayCamerasLiveEditManager()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FGameplayCamerasLiveEditManager::OnPostGarbageCollection);
	FEditorDelegates::BeginPIE.AddRaw(this, &FGameplayCamerasLiveEditManager::OnBeginPIE);
}

FGameplayCamerasLiveEditManager::~FGameplayCamerasLiveEditManager()
{
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
}

bool FGameplayCamerasLiveEditManager::CanRunInEditor() const
{
	const UGameplayCamerasEditorSettings* Settings = GetDefault<UGameplayCamerasEditorSettings>();
	return Settings->bEnableRunInEditor;
}

void FGameplayCamerasLiveEditManager::NotifyPostBuildAsset(const UPackage* InAssetPackage) const
{
	if (const FListenerArray* Listeners = PackageListenerMap.Find(InAssetPackage))
	{
		FGameplayCameraAssetBuildEvent BuildEvent;
		BuildEvent.AssetPackage = InAssetPackage;

		for (IGameplayCamerasLiveEditListener* Listener : *Listeners)
		{
			Listener->PostBuildAsset(BuildEvent);
		}
	}
}

void FGameplayCamerasLiveEditManager::AddListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener)
{
	Internal::AddListenerImpl(PackageListenerMap, InAssetPackage, Listener);
}

void FGameplayCamerasLiveEditManager::RemoveListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener)
{
	Internal::RemoveListenerImpl(PackageListenerMap, InAssetPackage, Listener);
}

void FGameplayCamerasLiveEditManager::NotifyPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (const FListenerArray* Listeners = NodeListenerMap.Find(InCameraNode))
	{
		for (IGameplayCamerasLiveEditListener* Listener : *Listeners)
		{
			Listener->PostEditChangeProperty(InCameraNode, PropertyChangedEvent);
		}
	}
}

void FGameplayCamerasLiveEditManager::AddListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener)
{
	Internal::AddListenerImpl(NodeListenerMap, InCameraNode, Listener);
}

void FGameplayCamerasLiveEditManager::RemoveListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener)
{
	Internal::RemoveListenerImpl(NodeListenerMap, InCameraNode, Listener);
}

void FGameplayCamerasLiveEditManager::RemoveListener(IGameplayCamerasLiveEditListener* Listener)
{
	if (ensure(Listener))
	{
		for (auto It = PackageListenerMap.CreateIterator(); It; ++It)
		{
			It.Value().Remove(Listener);
			if (It.Value().IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
		for (auto It = NodeListenerMap.CreateIterator(); It; ++It)
		{
			It.Value().Remove(Listener);
			if (It.Value().IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FGameplayCamerasLiveEditManager::OnPostGarbageCollection()
{
	RemoveGarbage();
}

void FGameplayCamerasLiveEditManager::RemoveGarbage()
{
	for (auto It = PackageListenerMap.CreateIterator(); It; ++It)
	{
		if (!It.Key().IsValid())
		{
			It.RemoveCurrent();
		}
	}
	for (auto It = NodeListenerMap.CreateIterator(); It; ++It)
	{
		if (!It.Key().IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

namespace Internal
{

template<typename CameraObjectType>
struct TCameraObjectBuilderTraits;

template<>
struct TCameraObjectBuilderTraits<UCameraAsset>
{
	static void Build(UCameraAsset* InObject, FCameraBuildContext& InBuildContext)
	{
		const bool bBuildReferencedAssets = false; // We are going to build rigs first.
		FCameraAssetBuilder Builder(InBuildContext);
		Builder.BuildCamera(InObject, bBuildReferencedAssets); 
	}
};

template<>
struct TCameraObjectBuilderTraits<UCameraRigAsset>
{
	static void Build(UCameraRigAsset* InObject, FCameraBuildContext& InBuildContext)
	{
		FCameraRigAssetBuilder Builder(InBuildContext);
		Builder.BuildCameraRig(InObject);
	}
};

template<>
struct TCameraObjectBuilderTraits<UCameraShakeAsset>
{
	static void Build(UCameraShakeAsset* InObject, FCameraBuildContext& InBuildContext)
	{
		FCameraShakeAssetBuilder Builder(InBuildContext);
		Builder.BuildCameraShake(InObject); 
	}
};

template<typename CameraObjectType>
struct TCameraObjectBuilderUtil
{
	static int32 Gather(TArray<CameraObjectType*>& OutCameraObjectsToBuild)
	{
		int32 TotalCameraObjects = 0;
		for (TObjectIterator<CameraObjectType> It; It; ++It)
		{
			CameraObjectType* Obj(*It);
			++TotalCameraObjects;

			const ECameraBuildStatus BuildStatus = Obj->GetBuildStatus();
			if (BuildStatus == ECameraBuildStatus::Clean || BuildStatus == ECameraBuildStatus::CleanWithWarnings)
			{
				continue;
			}

			OutCameraObjectsToBuild.Add(Obj);
		}
		return TotalCameraObjects;
	}

	static void Build(TArrayView<CameraObjectType*> InCameraObjectsToBuild)
	{
		FCameraBuildLog BuildLog;
		BuildLog.SetForwardMessagesToLogging(true);
		FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::StartingPIE);

		for (CameraObjectType* CameraObject : InCameraObjectsToBuild)
		{
			BuildLog.ResetMessages();
			BuildLog.SetLoggingPrefix(CameraObject->GetName());

			TCameraObjectBuilderTraits<CameraObjectType>::Build(CameraObject, BuildContext);
		}
	}
};

}  // namespace Internal

void FGameplayCamerasLiveEditManager::OnBeginPIE(const bool bSimulate)
{
	using namespace Internal;

	const UGameplayCamerasSettings* Settings = GetDefault<UGameplayCamerasSettings>();
	if (!Settings->bAutoBuildInPIE)
	{
		return;
	}

	TArray<UCameraRigAsset*> CameraRigsToBuild;
	const int32 NumCameraRigs = TCameraObjectBuilderUtil<UCameraRigAsset>::Gather(CameraRigsToBuild);

	TArray<UCameraShakeAsset*> CameraShakesToBuild;
	const int32 NumCameraShakes = TCameraObjectBuilderUtil<UCameraShakeAsset>::Gather(CameraShakesToBuild);

	TArray<UCameraAsset*> CamerasToBuild;
	const int32 NumCameras = TCameraObjectBuilderUtil<UCameraAsset>::Gather(CamerasToBuild);

	const int32 NumCameraObjects = (NumCameras + NumCameraRigs + NumCameraShakes);
	const int32 NumCameraObjectsToBuild = (CamerasToBuild.Num() + CameraRigsToBuild.Num() + CameraShakesToBuild.Num());
	if (NumCameraObjectsToBuild > 0)
	{
		const double BuildStartTime = FPlatformTime::Seconds();

		TCameraObjectBuilderUtil<UCameraRigAsset>::Build(CameraRigsToBuild);
		TCameraObjectBuilderUtil<UCameraShakeAsset>::Build(CameraShakesToBuild);
		TCameraObjectBuilderUtil<UCameraAsset>::Build(CamerasToBuild);

		const double BuildEndTime = FPlatformTime::Seconds();
		UE_LOGF(LogCameraSystemEditor, Log, 
				"Built %d/%d camera objects in %d ms",
				NumCameraObjectsToBuild, NumCameraObjects,
				(int32)((BuildEndTime - BuildStartTime) * 1000));
	}
	else
	{
		UE_LOGF(LogCameraSystemEditor, Log, 
				"No camera objects needed building (inspected %d objects)",
				NumCameraObjects);
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

