// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGameplayCamerasLiveEditManager.h"

#include "Core/CameraNode.h"
#include "CoreTypes.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::Cameras
{

class FGameplayCamerasLiveEditManager : public IGameplayCamerasLiveEditManager
{
public:

	FGameplayCamerasLiveEditManager();
	~FGameplayCamerasLiveEditManager();

public:

	// IGameplayCamerasLiveEditManager interface
	virtual bool CanRunInEditor() const override;
	virtual void NotifyPostBuildAsset(const UPackage* InAssetPackage) const override;
	virtual void AddListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener) override;
	virtual void RemoveListener(const UPackage* InAssetPackage, IGameplayCamerasLiveEditListener* Listener) override;
	virtual void NotifyPostEditChangeProperty(const UCameraNode* InCameraNode, const FPropertyChangedEvent& PropertyChangedEvent) const override;
	virtual void AddListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener) override;
	virtual void RemoveListener(const UCameraNode* InCameraNode, IGameplayCamerasLiveEditListener* Listener) override;
	virtual void RemoveListener(IGameplayCamerasLiveEditListener* Listener) override;

private:

	void OnPostGarbageCollection();
	void OnBeginPIE(const bool bSimulate);

	void RemoveGarbage();

private:

	using FListenerArray = TArray<IGameplayCamerasLiveEditListener*, TInlineAllocator<4>>;

	using FPackageListenerMap = TMap<TWeakObjectPtr<const UPackage>, FListenerArray>;
	FPackageListenerMap PackageListenerMap;

	using FNodeListenerMap = TMap<TWeakObjectPtr<const UCameraNode>, FListenerArray>;
	FNodeListenerMap NodeListenerMap;
};

}  // namespace UE::Cameras

