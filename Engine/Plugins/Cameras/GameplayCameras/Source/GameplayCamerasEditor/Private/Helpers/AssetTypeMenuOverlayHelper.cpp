// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/AssetTypeMenuOverlayHelper.h"

#include "EditorClassUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AssetTypeMenuOverlayHelper"

namespace UE::Cameras
{

TSharedRef<SWidget> FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UClass* InAssetType)
{
	TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.ShadowOffset(FVector2D::UnitVector)
			.Text(LOCTEXT("CameraAssetType", "Asset Type: "))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		[
			FEditorClassUtils::GetSourceLink(InAssetType)
		];

	return MenuOverlayBox;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

