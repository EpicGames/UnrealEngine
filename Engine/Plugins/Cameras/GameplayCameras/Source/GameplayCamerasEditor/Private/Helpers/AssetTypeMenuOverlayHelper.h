// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

class SWidget;
class UClass;

namespace UE::Cameras
{

class FAssetTypeMenuOverlayHelper
{
public:

	static TSharedRef<SWidget> CreateMenuOverlay(UClass* InAssetType);
};

}  // namespace UE::Cameras

