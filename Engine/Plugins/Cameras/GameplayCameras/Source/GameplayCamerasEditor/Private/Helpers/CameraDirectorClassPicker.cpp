// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CameraDirectorClassPicker.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Directors/BlueprintCameraDirector.h"
#include "Directors/SingleCameraDirector.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "CameraDirectorClassPicker"

namespace UE::Cameras
{

class FCameraDirectorClassFilter : public IClassViewerFilter
{
public:
	TSet<const UClass*> AllowedClasses;
	EClassFlags DisallowedClassFlags;

	FCameraDirectorClassFilter()
	{
		AllowedClasses.Add(UCameraDirector::StaticClass());
		DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated;
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return 
			!InClass->HasAnyClassFlags(DisallowedClassFlags) &&
			InFilterFuncs->IfInChildOfClassesSet(AllowedClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return 
			!InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags) && 
			InFilterFuncs->IfInChildOfClassesSet(AllowedClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

FCameraDirectorClassPicker::FCameraDirectorClassPicker()
{
	CommonCameraDirectorClasses.Add(UBlueprintCameraDirector::StaticClass());
	CommonCameraDirectorClasses.Add(USingleCameraDirector::StaticClass());
}

void FCameraDirectorClassPicker::AddCommonCameraDirector(TSubclassOf<UCameraDirector> InClass)
{
	CommonCameraDirectorClasses.Add(InClass);
}

void FCameraDirectorClassPicker::ResetCommonCameraDirectors()
{
	CommonCameraDirectorClasses.Reset();
}

bool FCameraDirectorClassPicker::PickCameraDirectorClass(TSubclassOf<UCameraDirector>& OutChosenClass)
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.bShowNoneOption = true;
	Options.ExtraPickerCommonClasses.Append(CommonCameraDirectorClasses);

	TSharedPtr<FCameraDirectorClassFilter> Filter = MakeShareable(new FCameraDirectorClassFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	UClass* ChosenClass = nullptr;
	const FText TitleText = LOCTEXT("CameraDirectorPicker", "Pick Camera Director Type");
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UCameraDirector::StaticClass());
	if (bPressedOk)
	{
		OutChosenClass = ChosenClass;
	}
	return bPressedOk;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

