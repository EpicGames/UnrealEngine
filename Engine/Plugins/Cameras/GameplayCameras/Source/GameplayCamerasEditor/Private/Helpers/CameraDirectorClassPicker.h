// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SubclassOf.h"

class UCameraDirector;

namespace UE::Cameras
{

class FCameraDirectorClassPicker
{
public:

	FCameraDirectorClassPicker();

	void AddCommonCameraDirector(TSubclassOf<UCameraDirector> InClass);
	void ResetCommonCameraDirectors();

	bool PickCameraDirectorClass(TSubclassOf<UCameraDirector>& OutChosenClass);

private:

	TArray<TSubclassOf<UCameraDirector>> CommonCameraDirectorClasses;
};

}  // namespace UE::Cameras

