// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGameplayCamerasFamily.h"

#include "Core/CameraAsset.h"
#include "Core/CameraRigAsset.h"
#include "Core/CameraRigProxyAsset.h"
#include "Family/CameraAssetFamily.h"
#include "Family/CameraRigAssetFamily.h"
#include "Family/CameraRigProxyAssetFamily.h"

namespace UE::Cameras
{

TSharedPtr<IGameplayCamerasFamily> IGameplayCamerasFamily::CreateFamily(UObject* InAsset)
{
	if (!ensure(InAsset))
	{
		return nullptr;
	}

	UClass* AssetType = InAsset->GetClass();
	if (AssetType == UCameraAsset::StaticClass())
	{
		return MakeShared<FCameraAssetFamily>(CastChecked<UCameraAsset>(InAsset));
	}
	else if (AssetType == UCameraRigAsset::StaticClass())
	{
		return MakeShared<FCameraRigAssetFamily>(CastChecked<UCameraRigAsset>(InAsset));
	}
	else if (AssetType == UCameraRigProxyAsset::StaticClass())
	{
		return MakeShared<FCameraRigProxyAssetFamily>(CastChecked<UCameraRigProxyAsset>(InAsset));
	}

	return nullptr;
}

}  // namespace UE::Cameras

