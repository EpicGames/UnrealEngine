// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/CameraFramingZoneTrackEditor.h"

#include "Channels/MovieSceneDoubleChannel.h"
#include "KeyPropertyParams.h"
#include "PropertyPath.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

namespace UE::Cameras
{

FName FCameraFramingZoneTrackEditor::LeftName("Left");
FName FCameraFramingZoneTrackEditor::TopName("Top");
FName FCameraFramingZoneTrackEditor::RightName("Right");
FName FCameraFramingZoneTrackEditor::BottomName("Bottom");

TSharedRef<ISequencerTrackEditor> FCameraFramingZoneTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCameraFramingZoneTrackEditor(InSequencer));
}

void FCameraFramingZoneTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	using namespace UE::MovieScene;

	FPropertyPath StructPath = PropertyChangedParams.StructPathToKey;
	FName ChannelName = StructPath.GetNumProperties() != 0 ? StructPath.GetLeafMostProperty().Property->GetFName() : NAME_None;

	FCameraFramingZone CameraFramingZone = PropertyChangedParams.GetPropertyValue<FCameraFramingZone>();

	const bool bKeyLeft   = ChannelName == NAME_None || ChannelName == LeftName;
	const bool bKeyTop    = ChannelName == NAME_None || ChannelName == TopName;
	const bool bKeyRight  = ChannelName == NAME_None || ChannelName == RightName;
	const bool bKeyBottom = ChannelName == NAME_None || ChannelName == BottomName;

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(0, CameraFramingZone.Left, bKeyLeft));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(1, CameraFramingZone.Top, bKeyTop));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(2, CameraFramingZone.Right, bKeyRight));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(3, CameraFramingZone.Bottom, bKeyBottom));
}

}  // namespace UE::Cameras

