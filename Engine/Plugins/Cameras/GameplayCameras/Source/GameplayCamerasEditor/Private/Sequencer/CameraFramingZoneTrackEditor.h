// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrackEditor.h"

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "KeyframeTrackEditor.h"
#include "MovieScene/MovieSceneCameraFramingZoneTrack.h"
#include "Nodes/Framing/CameraFramingZone.h"
#include "Templates/SharedPointer.h"

class FName;
class FPropertyChangedParams;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

namespace UE::Cameras
{

class FCameraFramingZoneTrackEditor : public FPropertyTrackEditor<UMovieSceneCameraFramingZoneTrack>
{
public:

	FCameraFramingZoneTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{
	}

	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromStructType(FCameraFramingZone::StaticStruct()) });
	}

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:

	virtual void GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;

private:

	static FName LeftName;
	static FName TopName;
	static FName RightName;
	static FName BottomName;
};

}  // namespace UE::Cameras

