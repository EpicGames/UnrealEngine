// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/GameplayCameraComponentTrackEditor.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

#include "Core/CameraAsset.h"
#include "Core/CameraParameters.h"  // IWYU pragma: keep
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/GameplayCameraComponent.h"
#include "GameFramework/GameplayCameraRigComponent.h"
#include "ISequencer.h"
#include "KeyPropertyParams.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "StructUtils/PropertyBag.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Tracks/MovieScenePropertyTrack.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
#include "Misc/SequencerObjectBindingHelper.h"
#endif  // >=5.7.0

#define LOCTEXT_NAMESPACE "GameplayCameraComponentTrackEditor"

namespace UE::Cameras::Internal
{

#if UE_VERSION_OLDER_THAN(5,7,0)

// TODO: duplicated from GetKeyablePropertyPaths in ObjectBindingModel.cpp
void GetKeyablePropertyPathsImpl(TSharedPtr<ISequencer> Sequencer, const UClass* BaseObjectClass, const UStruct* Struct, const void* StructValuePtr, FPropertyPath PropertyPath, TArray<FPropertyPath>& KeyablePropertyPaths)
{
	for (TFieldIterator<FProperty> PropertyIterator(Struct); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (!Property || Property->HasAnyPropertyFlags(CPF_Deprecated))
		{
			continue;
		}

		PropertyPath.AddProperty(FPropertyInfo(Property));

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(StructValuePtr));
			for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
			{
				PropertyPath.AddProperty(FPropertyInfo(ArrayProperty->Inner, Index));

				if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(BaseObjectClass, PropertyPath)))
				{
					KeyablePropertyPaths.Add(PropertyPath);
				}
				else if (FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
				{
					GetKeyablePropertyPathsImpl(Sequencer, BaseObjectClass, StructProperty->Struct, ArrayHelper.GetRawPtr(Index), PropertyPath, KeyablePropertyPaths);
				}

				PropertyPath = *PropertyPath.TrimPath(1);
			}
		}
		else if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(BaseObjectClass, PropertyPath)))
		{
			KeyablePropertyPaths.Add(PropertyPath);
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			GetKeyablePropertyPathsImpl(Sequencer, BaseObjectClass, StructProperty->Struct, StructProperty->ContainerPtrToValuePtr<void>(StructValuePtr), PropertyPath, KeyablePropertyPaths);
		}

		PropertyPath = *PropertyPath.TrimPath(1);
	}
}

#endif  // pre-5.7.0

void GetKeyablePropertyPaths(TSharedPtr<ISequencer> Sequencer, const UGameplayCameraComponentBase* CameraComponentBase, TArray<FPropertyPath>& KeyablePropertyPaths)
{
	// Start us off with the property path of the parameters struct, and then get all keyable property paths from there.
	FPropertyPath PropertyPath;
	const UPropertyBag* CameraParametersStruct = nullptr;
	const uint8* CameraParametersMemory = nullptr;

	if (const UGameplayCameraComponent* CameraComponent = Cast<UGameplayCameraComponent>(CameraComponentBase))
	{
		const UCameraAsset* CameraAsset = CameraComponent->CameraReference.GetCameraAsset();
		if (CameraAsset)
		{
			const FInstancedPropertyBag& CameraParameters = CameraComponent->CameraReference.GetParameters();
			CameraParametersStruct = CameraParameters.GetPropertyBagStruct();
			CameraParametersMemory = CameraParameters.GetValue().GetMemory();

			PropertyPath.AddProperty(FPropertyInfo(UGameplayCameraComponent::StaticClass()->FindPropertyByName(TEXT("CameraReference"))));
			PropertyPath.AddProperty(FPropertyInfo(FCameraAssetReference::StaticStruct()->FindPropertyByName(TEXT("Parameters"))));
		}
	}
	else if (const UGameplayCameraRigComponent* CameraRigComponent = Cast<UGameplayCameraRigComponent>(CameraComponentBase))
	{
		const UCameraRigAsset* CameraRigAsset = CameraRigComponent->CameraRigReference.GetCameraRig();
		if (CameraRigAsset)
		{
			const FInstancedPropertyBag& CameraParameters = CameraRigComponent->CameraRigReference.GetParameters();
			CameraParametersStruct = CameraParameters.GetPropertyBagStruct();
			CameraParametersMemory = CameraParameters.GetValue().GetMemory();

			PropertyPath.AddProperty(FPropertyInfo(UGameplayCameraRigComponent::StaticClass()->FindPropertyByName(TEXT("CameraRigReference"))));
			PropertyPath.AddProperty(FPropertyInfo(FCameraRigAssetReference::StaticStruct()->FindPropertyByName(TEXT("Parameters"))));
		}
	}

	if (!CameraParametersStruct || !CameraParametersMemory)
	{
		return;
	}

#if UE_VERSION_OLDER_THAN(5,7,0)
	// Before 5.7.0, Sequencer can animate instanced structs directly but not property bags, so we need to add the "Value" field to
	// dive inside the property bag's instanced struct.
	const UStruct* PropertyBagStruct = FInstancedPropertyBag::StaticStruct();
	PropertyPath.AddProperty(FPropertyInfo(PropertyBagStruct->FindPropertyByName(TEXT("Value"))));
#endif  // pre-5.7.0

	const UClass* ComponentClass = CameraComponentBase->GetClass();
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
	FSequencerObjectBindingHelper::GetKeyablePropertyPaths(ComponentClass, CameraParametersMemory, CameraParametersStruct, PropertyPath, Sequencer.ToSharedRef(), KeyablePropertyPaths);
#else
	GetKeyablePropertyPathsImpl(Sequencer, ComponentClass, CameraParametersStruct, CameraParametersMemory, PropertyPath, KeyablePropertyPaths);
#endif  // 5.7 and above
}

class FCameraParameterTrackSetupHandler : public UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISequenceDataEventHandler>
{
public:

	FCameraParameterTrackSetupHandler(const FGuid& ObjectBindingID)
		: MonitoredObjectBindingID(ObjectBindingID)
	{
	}

	void SetDesiredTrackName(const FText& InDesiredTrackName)
	{
		DesiredTrackName = InDesiredTrackName;
	}

	virtual void OnTrackAddedToBinding(UMovieSceneTrack* Track, const FGuid& ObjectBindingID) override
	{
		if (ObjectBindingID != MonitoredObjectBindingID)
		{
			return;
		}

		UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track);
		if (!PropertyTrack)
		{
			return;
		}

		if (!DesiredTrackName.IsEmpty())
		{
			PropertyTrack->SetDisplayName(DesiredTrackName);
		}
	}
	
private:

	FGuid MonitoredObjectBindingID;
	FText DesiredTrackName;
};

}  // namespace UE::Cameras::Internal

TSharedRef<ISequencerTrackEditor> FGameplayCameraComponentTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShareable(new FGameplayCameraComponentTrackEditor(OwningSequencer));
}

FGameplayCameraComponentTrackEditor::FGameplayCameraComponentTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer) 
{
}

void FGameplayCameraComponentTrackEditor::BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings)
{
}

void FGameplayCameraComponentTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
}

void FGameplayCameraComponentTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
}

void FGameplayCameraComponentTrackEditor::ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass && ObjectClass->IsChildOf<UGameplayCameraComponentBase>())
	{
		Extender->AddMenuExtension(
				TEXT("Tracks"), EExtensionHook::After, nullptr, 
				FMenuExtensionDelegate::CreateSP(this, &FGameplayCameraComponentTrackEditor::OnExtendObjectBindingTrackMenu, ObjectBindings));
	}
}

void FGameplayCameraComponentTrackEditor::OnExtendObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	if (ObjectBindings.Num() != 1)
	{
		return;
	}

	const UGameplayCameraComponentBase* CameraComponent = GetCameraComponentForBinding(ObjectBindings[0]);
	if (CameraComponent)
	{
		using namespace UE::Cameras::Internal;

		TArray<FPropertyPath> KeyablePropertyPaths;
		GetKeyablePropertyPaths(GetSequencer(), CameraComponent, KeyablePropertyPaths);

		MenuBuilder.BeginSection(TEXT("CameraParameters"), LOCTEXT("AddCameraParametersMenuSection", "Camera Parameters"));

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
		// Start at level 2 to skip adding menus for CameraReference > Parameters
		BuildAddParameterTrackMenuItems(ObjectBindings[0], MenuBuilder, KeyablePropertyPaths, 2);
#else
		// Start at level 3 to skip adding menus for CameraReference > Parameters > Value
		BuildAddParameterTrackMenuItems(ObjectBindings[0], MenuBuilder, KeyablePropertyPaths, 3);
#endif  // >= 5.7.0

		MenuBuilder.EndSection();
	}
}

TSharedPtr<SWidget> FGameplayCameraComponentTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return FMovieSceneTrackEditor::BuildOutlinerEditWidget(ObjectBinding, Track, Params);
}

TSharedPtr<SWidget> FGameplayCameraComponentTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

TSharedRef<ISequencerSection> FGameplayCameraComponentTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return FMovieSceneTrackEditor::MakeSectionInterface(SectionObject, Track, ObjectBinding);
}

void FGameplayCameraComponentTrackEditor::OnRelease()
{
}

bool FGameplayCameraComponentTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	// This track editor doesn't support any track, it just extends object bindings.
	return false;
}

void FGameplayCameraComponentTrackEditor::Tick(float DeltaTime)
{
}

const FSlateBrush* FGameplayCameraComponentTrackEditor::GetIconBrush() const
{
	using namespace UE::Cameras;
	TSharedRef<FGameplayCamerasEditorStyle> CamerasEditorStyle = FGameplayCamerasEditorStyle::Get();
	return CamerasEditorStyle->GetBrush("Sequencer.CameraRigTrack");
}

bool FGameplayCameraComponentTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	return false;
}

FReply FGameplayCameraComponentTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	return FReply::Unhandled();
}

UGameplayCameraComponentBase* FGameplayCameraComponentTrackEditor::GetCameraComponentForBinding(const FGuid& ObjectBinding) const
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		UObject* BoundObject = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);
		return Cast<UGameplayCameraComponentBase>(BoundObject);
	}
	return nullptr;
}

void FGameplayCameraComponentTrackEditor::AddCameraParameterTrack(FPropertyMenuData PropertyMenuData, FGuid ObjectBinding)
{
	using namespace UE::Cameras::Internal;
	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UObject* BoundObject = Sequencer->FindSpawnedObjectOrTemplate(ObjectBinding);

	if (BoundObject != nullptr)
	{
		UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
		FCameraParameterTrackSetupHandler TrackSetupHandler(ObjectBinding);
		FocusedSequence->GetMovieScene()->EventHandlers.Link(&TrackSetupHandler);

		if (PropertyMenuData.PropertyIndexForMenuName != INDEX_NONE)
		{
			const FPropertyInfo& PropertyInfoForTrackName = PropertyMenuData.PropertyPath.GetPropertyInfo(PropertyMenuData.PropertyIndexForMenuName);
			const FProperty* PropertyForTrackName = PropertyInfoForTrackName.Property.Get();
			TrackSetupHandler.SetDesiredTrackName(PropertyForTrackName->GetDisplayNameText());
		}

		TArray<UObject*> KeyableBoundObjects;
		KeyableBoundObjects.Add(BoundObject);

		ESequencerKeyMode KeyMode = Sequencer->GetAutoSetTrackDefaults() == false ? ESequencerKeyMode::ManualKeyForced : ESequencerKeyMode::ManualKey;

		FKeyPropertyParams KeyPropertyParams(KeyableBoundObjects, PropertyMenuData.PropertyPath, KeyMode);

		Sequencer->KeyProperty(KeyPropertyParams);
	}
}

bool FGameplayCameraComponentTrackEditor::CanAddCameraParameterTrack(FPropertyMenuData PropertyMenuData, FGuid ObjectBinding) const
{
	return true;
}

// TODO: most of the below stuff is duplicated from ObjectBindingModel.cpp but without classifying in categories, 
//		 and with special handling of camera parameters.

void FGameplayCameraComponentTrackEditor::BuildAddParameterTrackMenuItem(FMenuBuilder& MenuBuilder, const FPropertyMenuData& KeyablePropertyMenuData, const FGuid& ObjectBinding)
{
	FUIAction AddTrackMenuAction(
			FExecuteAction::CreateSP(this, &FGameplayCameraComponentTrackEditor::AddCameraParameterTrack, KeyablePropertyMenuData, ObjectBinding),
			FCanExecuteAction::CreateSP(this, &FGameplayCameraComponentTrackEditor::CanAddCameraParameterTrack, KeyablePropertyMenuData, ObjectBinding));
	MenuBuilder.AddMenuEntry(FText::FromString(KeyablePropertyMenuData.MenuName), FText(), FSlateIcon(), AddTrackMenuAction);
}

void FGameplayCameraComponentTrackEditor::BuildAddParameterTrackMenuItems(const FGuid& ObjectBinding, FMenuBuilder& MenuBuilder, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart)
{
	if (KeyablePropertyPaths.IsEmpty())
	{
		return;
	}

	// Create property menu data based on keyable property paths
	TArray<FPropertyMenuData> KeyablePropertyMenuDatas;
	for (const FPropertyPath& KeyablePropertyPath : KeyablePropertyPaths)
	{
		if (!ensure(KeyablePropertyPath.GetNumProperties() > PropertyNameIndexStart))
		{
			continue;
		}

		const FPropertyInfo& PropertyInfo = KeyablePropertyPath.GetPropertyInfo(PropertyNameIndexStart);
		if (const FProperty* Property = PropertyInfo.Property.Get())
		{
			FPropertyMenuData KeyableMenuData;
			KeyableMenuData.PropertyPath = KeyablePropertyPath;
			if (PropertyInfo.ArrayIndex != INDEX_NONE)
			{
				KeyableMenuData.MenuName = FText::Format(LOCTEXT("PropertyMenuTextFormat", "{0} [{1}]"), Property->GetDisplayNameText(), FText::AsNumber(PropertyInfo.ArrayIndex)).ToString();
			}
			else
			{
				KeyableMenuData.MenuName = Property->GetDisplayNameText().ToString();
			}

			if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
			{
				bool bIsCameraParameter = false;
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
				if (StructProperty->Struct == F##ValueName##CameraParameter::StaticStruct())\
				{\
					bIsCameraParameter = true;\
				}\
				else
				UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE
				{
				}

				if (bIsCameraParameter)
				{
					KeyableMenuData.PropertyIndexForMenuName = PropertyNameIndexStart;
				}
			}

			KeyablePropertyMenuDatas.Add(KeyableMenuData);
		}
	}

	KeyablePropertyMenuDatas.Sort();

	for (int32 MenuDataIndex = 0; MenuDataIndex < KeyablePropertyMenuDatas.Num(); )
	{
		// If this menu data only has one property name left in it, add the menu item
		if (KeyablePropertyMenuDatas[MenuDataIndex].PropertyPath.GetNumProperties() == PropertyNameIndexStart + 1 || 
				KeyablePropertyMenuDatas[MenuDataIndex].PropertyIndexForMenuName == PropertyNameIndexStart)
		{
			BuildAddParameterTrackMenuItem(MenuBuilder, KeyablePropertyMenuDatas[MenuDataIndex], ObjectBinding);
			++MenuDataIndex;
		}
		// Otherwise, look to the next menu data to gather up new data
		else
		{
			TArray<FPropertyPath> KeyableSubMenuPropertyPaths;
			KeyableSubMenuPropertyPaths.Add(KeyablePropertyMenuDatas[MenuDataIndex].PropertyPath);

			for (; MenuDataIndex < KeyablePropertyMenuDatas.Num() - 1; )
			{
				if (KeyablePropertyMenuDatas[MenuDataIndex].MenuName == KeyablePropertyMenuDatas[MenuDataIndex + 1].MenuName)
				{	
					++MenuDataIndex;
					KeyableSubMenuPropertyPaths.Add(KeyablePropertyMenuDatas[MenuDataIndex].PropertyPath);
				}
				else
				{
					break;
				}
			}

			MenuBuilder.AddSubMenu(
				FText::FromString(KeyablePropertyMenuDatas[MenuDataIndex].MenuName),
				FText::GetEmpty(), 
				FNewMenuDelegate::CreateSP(this, &FGameplayCameraComponentTrackEditor::BuildAddParameterTrackSubMenuItems, ObjectBinding, KeyableSubMenuPropertyPaths, PropertyNameIndexStart + 1));

			++MenuDataIndex;
		}
	}
}

void FGameplayCameraComponentTrackEditor::BuildAddParameterTrackSubMenuItems(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart)
{
	BuildAddParameterTrackMenuItems(ObjectBinding, MenuBuilder, KeyablePropertyPaths, PropertyNameIndexStart);
}

#undef LOCTEXT_NAMESPACE

#endif  // >=5.6.0

