// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EngineVersionComparison.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)

#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointerFwd.h"

class ISequencer;
class UGameplayCameraComponentBase;
class UMovieScenePropertyTrack;
struct FCameraObjectInterfaceParameterDefinition;

namespace UE::Sequencer
{
	class FObjectBindingModel;
	class FObjectBindingPropertyMenuData;
}

class FGameplayCameraComponentTrackEditor : public FMovieSceneTrackEditor
{
public:

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	FGameplayCameraComponentTrackEditor(TSharedRef<ISequencer> InSequencer);

protected:

	// ISequencerTrackEditor interface.
	virtual void BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual void ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void OnRelease() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual void Tick(float DeltaTime) override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;

private:

	UGameplayCameraComponentBase* GetCameraComponentForBinding(const FGuid& ObjectBinding) const;

	void OnExtendObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);

private:

	struct FPropertyMenuData
	{
		FString MenuName;
		FPropertyPath PropertyPath;
		int32 PropertyIndexForMenuName = INDEX_NONE;

		bool operator< (const FPropertyMenuData& Other) const
		{
			int32 CompareResult = MenuName.Compare(Other.MenuName);
			return CompareResult < 0;
		}
	};

	void BuildAddParameterTrackMenuItems(const FGuid& ObjectBinding, FMenuBuilder& MenuBuilder, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart);
	void BuildAddParameterTrackSubMenuItems(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart);
	void BuildAddParameterTrackMenuItem(FMenuBuilder& MenuBuilder, const FPropertyMenuData& KeyablePropertyMenuData, const FGuid& ObjectBinding);

	void AddCameraParameterTrack(FPropertyMenuData PropertyMenuData, FGuid ObjectBinding);
	bool CanAddCameraParameterTrack(FPropertyMenuData PropertyMenuData, FGuid ObjectBinding) const;
};

#endif  // >= 5.6.0

