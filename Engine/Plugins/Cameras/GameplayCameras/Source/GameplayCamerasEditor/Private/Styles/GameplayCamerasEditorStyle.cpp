// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/GameplayCamerasEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

namespace UE::Cameras
{

TSharedPtr<FGameplayCamerasEditorStyle> FGameplayCamerasEditorStyle::Singleton;

FGameplayCamerasEditorStyle::FGameplayCamerasEditorStyle()
	: FSlateStyleSet("GameplayCamerasEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("GameplayCameras"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const ISlateStyle& CoreStyle = FAppStyle::Get();
	const FTextBlockStyle& NormalText = CoreStyle.GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FInlineEditableTextBlockStyle& NormalInlineEditableText = CoreStyle.GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle");
	const FTableRowStyle AlternatingTableRowStyle = CoreStyle.GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");

	// Camera assets.
	Set("ClassIcon.CameraAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraKit", Icon16x16));
	Set("ClassThumbnail.CameraAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraKit", Icon64x64));
	Set("ClassIcon.CameraRigAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRig", Icon16x16));
	Set("ClassThumbnail.CameraRigAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRig", Icon64x64));
	Set("ClassIcon.CameraRigProxyAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRigProxy", Icon16x16));
	Set("ClassThumbnail.CameraRigProxyAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRigProxy", Icon64x64));
	Set("ClassIcon.CameraVariableCollection", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraVariableCollection", Icon16x16));
	Set("ClassThumbnail.CameraVariableCollection", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraVariableCollection", Icon64x64));
	Set("ClassIcon.CameraShakeAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraShake", Icon16x16));
	Set("ClassThumbnail.CameraShakeAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraShake", Icon64x64));

	// Camera actors and components.
	Set("ClassIcon.GameplayCameraRigComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCameraRig_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraRigComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCameraRig_64", Icon64x64));
	Set("ClassIcon.GameplayCameraRigActor", new IMAGE_BRUSH_SVG("Icons/GameplayCameraRig_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraRigActor", new IMAGE_BRUSH_SVG("Icons/GameplayCameraRig_64", Icon64x64));
	Set("ClassIcon.GameplayCameraComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCamera_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCamera_64", Icon64x64));
	Set("ClassIcon.GameplayCameraActor", new IMAGE_BRUSH_SVG("Icons/GameplayCamera_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraActor", new IMAGE_BRUSH_SVG("Icons/GameplayCamera_64", Icon64x64));
	Set("ClassIcon.GameplayCameraSystemComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCameraSystem_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraSystemComponent", new IMAGE_BRUSH_SVG("Icons/GameplayCameraSystem_64", Icon64x64));
	Set("ClassIcon.GameplayCameraSystemActor", new IMAGE_BRUSH_SVG("Icons/GameplayCameraSystem_16", Icon16x16));
	Set("ClassThumbnail.GameplayCameraSystemActor", new IMAGE_BRUSH_SVG("Icons/GameplayCameraSystem_64", Icon64x64));

	// Camera asset editor icons.
	Set("CameraAssetEditor.ShowCameraDirector", new IMAGE_BRUSH_SVG("Icons/CameraEditor-CameraDirector", Icon20x20));
	Set("CameraAssetEditor.ShowCameraRigs", new IMAGE_BRUSH_SVG("Icons/CameraEditor-CameraRigs", Icon20x20));
	Set("CameraAssetEditor.ShowSharedTransitions", new IMAGE_BRUSH_SVG("Icons/CameraEditor-SharedTransitions", Icon20x20));
	
	Set("CameraAssetEditor.Tabs.Search", new CORE_IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16));
	Set("CameraAssetEditor.Tabs.Messages", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16));
	Set("CameraAssetEditor.Tabs.CameraRigs", new IMAGE_BRUSH_SVG("Icons/CameraEditor-CameraRigs", Icon16x16));

	Set("CameraAssetEditor.FindInCamera", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Search", Icon20x20));

	Set("CameraAssetEditor.AddCameraRig", new IMAGE_BRUSH_SVG("Icons/CameraEditor-AddCameraRig", Icon16x16));
	Set("CameraAssetEditor.DeleteCameraRig", new IMAGE_BRUSH_SVG("Icons/CameraEditor-DeleteCameraRig", Icon16x16));
	Set("CameraAssetEditor.RenameCameraRig", new IMAGE_BRUSH_SVG("Icons/CameraEditor-RenameCameraRig", Icon16x16));

	Set("CameraAssetEditor.CameraRigsList.RowStyle", AlternatingTableRowStyle);

	// Camera rig editor icons.
	Set("CameraRigAssetEditor.Tabs.Toolbox", new IMAGE_BRUSH_SVG("Icons/CameraRig-Toolbox", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.Search", new CORE_IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.Messages", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.NodeHierarchy", new IMAGE_BRUSH_SVG("Icons/CameraRig-NodeHierarchy", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.Transitions", new IMAGE_BRUSH_SVG("Icons/CameraRig-Transitions", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.Curves", new IMAGE_BRUSH_SVG("Icons/CurveEditor", Icon16x16));
	Set("CameraRigAssetEditor.Tabs.InterfaceParameters", new IMAGE_BRUSH_SVG("Icons/CameraRig-InterfaceParameters", Icon16x16));

	Set("CameraRigAssetEditor.ShowNodeHierarchy", new IMAGE_BRUSH_SVG("Icons/CameraRig-NodeHierarchy", Icon20x20));
	Set("CameraRigAssetEditor.ShowTransitions", new IMAGE_BRUSH_SVG("Icons/CameraRig-Transitions", Icon20x20));
	Set("CameraRigAssetEditor.FocusHome", new IMAGE_BRUSH_SVG("Icons/GraphEditor-Home", Icon20x20));
	Set("CameraRigAssetEditor.FindInCameraRig", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Search", Icon20x20));

	// Camera shake editor icons.
	Set("CameraShakeAssetEditor.Tabs.Toolbox", new IMAGE_BRUSH_SVG("Icons/CameraRig-Toolbox", Icon16x16));
	Set("CameraShakeAssetEditor.Tabs.Search", new CORE_IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16));
	Set("CameraShakeAssetEditor.Tabs.Messages", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16));
	Set("CameraShakeAssetEditor.Tabs.InterfaceParameters", new IMAGE_BRUSH_SVG("Icons/CameraRig-InterfaceParameters", Icon16x16));

	Set("CameraShakeAssetEditor.FocusHome", new IMAGE_BRUSH_SVG("Icons/GraphEditor-Home", Icon20x20));
	Set("CameraShakeAssetEditor.FindInCameraShake", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Search", Icon20x20));

	// Camera object build statuses.
	Set("CameraObjectEditor.BuildStatus.Background", new IMAGE_BRUSH_SVG("Icons/CameraRig-BuildStatus_Background", Icon20x20));
	Set("CameraObjectEditor.BuildStatus.Overlay.Error", new IMAGE_BRUSH_SVG("Icons/CameraRig-BuildStatus_Fail_Badge", Icon20x20, FStyleColors::Error));
	Set("CameraObjectEditor.BuildStatus.Overlay.Good", new IMAGE_BRUSH_SVG("Icons/CameraRig-BuildStatus_Good_Badge", Icon20x20, FStyleColors::AccentGreen));
	Set("CameraObjectEditor.BuildStatus.Overlay.Unknown", new IMAGE_BRUSH_SVG("Icons/CameraRig-BuildStatus_Unknown_Badge", Icon20x20, FStyleColors::AccentYellow));
	Set("CameraObjectEditor.BuildStatus.Overlay.Warning", new IMAGE_BRUSH_SVG("Icons/CameraRig-BuildStatus_Warning_Badge", Icon20x20, FStyleColors::Warning));

	// Camera object editor icons.
	Set("CameraObjectEditor.InterfaceParameter.Message", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Italic", 10)));

	// Camera parameters icons.
	Set("CameraParameter.VariableBrowser", new IMAGE_BRUSH_SVG("Icons/CameraParameter-Variable", Icon16x16));
	Set("CameraParameter.TypeIcon", new IMAGE_BRUSH_SVG("Icons/CameraParameter-Pill", Icon16x16));

	// Camera variable collection icons.
	Set("CameraVariableCollectionEditor.CreateVariable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
	Set("CameraVariableCollectionEditor.RenameVariable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Rename", Icon16x16));
	Set("CameraVariableCollectionEditor.DeleteVariable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/minus", Icon16x16));

	FInlineEditableTextBlockStyle CameraVariableCollectionEntryNameStyle(NormalInlineEditableText);
	CameraVariableCollectionEntryNameStyle.TextStyle.SetFont(DEFAULT_FONT("Bold", 12));
	Set("CameraVariableCollectionEditor.Entry.Name", CameraVariableCollectionEntryNameStyle);
	Set("CameraVariableCollectionEditor.Entry.Type", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Italic", 10)));
	Set("CameraVariableCollectionEditor.Entry.Value", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Regular", 10)));

	// Curve editor icons.
	Set("CurveEditor.ShowInCurvesTab", new IMAGE_BRUSH_SVG("Icons/CurveEditor", Icon16x16));
	//Set("NiagaraEditor.CurveDetails.ShowInOverview.Small", new CORE_IMAGE_BRUSH("Common/GoToSource", Icon12x12, FLinearColor(.9f, .9f, .9f, 1.0f)));

	// Debugger tool icons.
	Set("Debugger.TabIcon", new IMAGE_BRUSH_SVG("Icons/GameplayCameraSystem_16", Icon16x16));

	Set("Debugger.BindToCameraSystem", new IMAGE_BRUSH_SVG("Icons/DebugLink", Icon16x16));

	Set("Debugger.DebugInfoEnabled.Icon", new IMAGE_BRUSH_SVG("Icons/DebugInfo-ToggleCheck", Icon16x16, FStyleColors::AccentGreen));
	Set("Debugger.DebugInfoDisabled.Icon", new IMAGE_BRUSH_SVG("Icons/DebugInfo-ToggleCheck", Icon16x16, FStyleColors::AccentGray));

	Set("DebugCategory.NodeTree.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-NodeTree", Icon16x16));
	Set("DebugCategory.DirectorTree.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-DirectorTree", Icon16x16));
	Set("DebugCategory.BlendStacks.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-BlendStacks", Icon16x16));
	Set("DebugCategory.Services.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-Services", Icon16x16));
	Set("DebugCategory.PoseStats.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-PoseStats", Icon16x16));
	Set("DebugCategory.Viewfinder.Icon", new IMAGE_BRUSH_SVG("Icons/DebugCategory-Viewfinder", Icon16x16));

	// Graph editor brushes.
	Set("Graph.CameraRigParameterNode.Body", new BOX_BRUSH("Graph/CameraRigParameterNode_Body", FMargin(16.f/64.f, 12.f/28.f)));
	Set("Graph.CameraRigParameterNode.ColorSpill", new IMAGE_BRUSH("Graph/CameraRigParameterNode_ColorSpill", FVector2D(132,28)));
	Set("Graph.CameraRigParameterNode.Gloss", new BOX_BRUSH("Graph/CameraRigParameterNode_Gloss", FMargin(16.f/64.f, 16.f/28.f, 16.f/64.f, 4.f/28.f)));
	Set("Graph.CameraRigParameterNode.Shadow", new BOX_BRUSH("Graph/CameraRigParameterNode_Shadow", FMargin(26.0f/64.0f)));
	Set("Graph.CameraRigParameterNode.ShadowSelected", new BOX_BRUSH("Graph/CameraRigParameterNode_ShadowSelected", FMargin(26.0f/64.0f)));
	Set("Graph.CameraRigParameterNode.DiffHighlight", new BOX_BRUSH("Graph/CameraRigParameterNode_DiffHighlight", FMargin(18.0f/64.0f)));
	Set("Graph.CameraRigParameterNode.DiffHighlightShading", new BOX_BRUSH("Graph/CameraRigParameterNode_DiffHighlightShading", FMargin(18.0f/64.0f)));

	Set("Graph.CameraRigParameterPin.Connected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_DiamondPin_Connected", Icon16x16));
	Set("Graph.CameraRigParameterPin.Disconnected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_DiamondPin_Disconnected", Icon16x16));

	// Family icons.
	Set("Family.CameraAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraKit", Icon20x20));
	Set("Family.CameraDirector", new IMAGE_BRUSH_SVG("Icons/CameraEditor-CameraDirector", Icon20x20));
	Set("Family.CameraRigAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRig", Icon20x20));
	Set("Family.CameraRigProxyAsset", new IMAGE_BRUSH_SVG("Icons/ContentBrowser-CameraRigProxy", Icon20x20));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FGameplayCamerasEditorStyle::~FGameplayCamerasEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

TSharedRef<FGameplayCamerasEditorStyle> FGameplayCamerasEditorStyle::Get()
{
	if (!Singleton.IsValid())
	{
		Singleton = MakeShareable(new FGameplayCamerasEditorStyle);
	}
	return Singleton.ToSharedRef();
}

}  // namespace UE::Cameras

