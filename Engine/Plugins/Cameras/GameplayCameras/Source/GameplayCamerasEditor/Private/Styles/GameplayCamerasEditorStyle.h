// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::Cameras
{

/**
 * Implements the visual style of the gameplay cameras editors.
 */
class FGameplayCamerasEditorStyle final : public FSlateStyleSet
{
public:
	
	FGameplayCamerasEditorStyle();
	virtual ~FGameplayCamerasEditorStyle();

	static TSharedRef<FGameplayCamerasEditorStyle> Get();

private:

	static TSharedPtr<FGameplayCamerasEditorStyle> Singleton;
};

}  // namespace UE::Cameras

