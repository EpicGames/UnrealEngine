// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/ObjectTreeGraphEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

TSharedPtr<FObjectTreeGraphEditorStyle> FObjectTreeGraphEditorStyle::Singleton;

FObjectTreeGraphEditorStyle::FObjectTreeGraphEditorStyle()
	: FSlateStyleSet("ObjectTreeGraphEditorStyle")
{
	const FVector2D Icon12x16(12.0f, 16.0f);
	const FVector2D Icon15x11(15.0f, 11.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("GameplayCameras"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FButtonStyle& DefaultButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Object tree graph node styles.
	Set("ObjectTreeGraphNode.Body", new BOX_BRUSH("Graph/ObjectTreeGraphNode_Body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f)));
	Set("ObjectTreeGraphNode.Shadow", new BOX_BRUSH("Graph/ObjectTreeGraphNode_Shadow", FMargin(18.0f/64.0f)));
	Set("ObjectTreeGraphNode.ShadowSelected", new BOX_BRUSH("Graph/ObjectTreeGraphNode_Shadow_Selected", FMargin(18.0f/64.0f)));
	Set("ObjectTreeGraphNode.TitleHighlight", new BOX_BRUSH("Graph/ObjectTreeGraphNode_TitleHighlight", FMargin(16.f/64.f, 1.f, 16.f/64.f, 0.f)));
	Set("ObjectTreeGraphNode.TitleBackground", new BOX_BRUSH("Graph/ObjectTreeGraphNode_TitleBackground", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f)));
	Set("ObjectTreeGraphNode.DiffHighlight", new BOX_BRUSH("Graph/ObjectTreeGraphNode_DiffHighlight", FMargin(18.0f/64.0f)));
	Set("ObjectTreeGraphNode.DiffHighlightShading", new BOX_BRUSH("Graph/ObjectTreeGraphNode_DiffHighlightShading", FMargin(18.0f/64.0f)));

	// Object tree graph pin styles.
	Set("ObjectTreeGraphNode.SelfPin.Connected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_SelfPin_Connected", Icon12x16));
	Set("ObjectTreeGraphNode.SelfPin.Disconnected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_SelfPin_Disconnected", Icon12x16));

	Set("ObjectTreeGraphNode.ObjectPin.Connected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_ArrowPin_Connected", Icon15x11));
	Set("ObjectTreeGraphNode.ObjectPin.Disconnected", new IMAGE_BRUSH("Graph/ObjectTreeGraphNode_ArrowPin_Disconnected", Icon15x11));

	// Object tree graph toolbox styles.
	Set("ObjectTreeGraphToolbox.Entry", FButtonStyle(DefaultButton)
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Dropdown, 1.0f))
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Hover, 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Primary, 1.0f))
		.SetNormalPadding(0)
		.SetPressedPadding(0));

	Set("ObjectTreeGraphToolbox.Entry.Name", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 10))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	Set("ObjectTreeGraphToolbox.Entry.Type", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	Set("ObjectTreeGraphToolbox.Entry.Background", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 6.f));
	Set("ObjectTreeGraphToolbox.Entry.LabelBack", new BOX_BRUSH("Icons/Toolbox-LabelBack", 6.f/18.f, FStyleColors::Dropdown));

	Set("ObjectTreeGraphToolbox.EntryToolTip.Name", FTextBlockStyle(NormalText) .SetFont(DEFAULT_FONT("Bold", 9)));
	Set("ObjectTreeGraphToolbox.EntryToolTip.ClassName", FTextBlockStyle(NormalText) .SetFont(DEFAULT_FONT("Regular", 9)));
	Set("ObjectTreeGraphToolbox.EntryToolTip.Path", FTextBlockStyle(NormalText) .SetFont(DEFAULT_FONT("Regular", 8)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FObjectTreeGraphEditorStyle::~FObjectTreeGraphEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

TSharedRef<FObjectTreeGraphEditorStyle> FObjectTreeGraphEditorStyle::Get()
{
	if (!Singleton.IsValid())
	{
		Singleton = MakeShareable(new FObjectTreeGraphEditorStyle);
	}
	return Singleton.ToSharedRef();
}

