// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Implements the visual style of generic object tree graph editors
 */
class FObjectTreeGraphEditorStyle final : public FSlateStyleSet
{
public:
	
	FObjectTreeGraphEditorStyle();
	virtual ~FObjectTreeGraphEditorStyle();

	static TSharedRef<FObjectTreeGraphEditorStyle> Get();

private:

	static TSharedPtr<FObjectTreeGraphEditorStyle> Singleton;
};

