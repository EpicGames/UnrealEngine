// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/AssetEditorMode.h"

#include "Framework/Docking/LayoutExtender.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

namespace UE::Cameras
{

FAssetEditorMode::FAssetEditorMode()
{
}

FAssetEditorMode::FAssetEditorMode(FName InModeName)
	: ModeName(InModeName)
{
}

FAssetEditorMode::~FAssetEditorMode()
{
}

void FAssetEditorMode::ActivateMode(const FAssetEditorModeActivateParams& InParams)
{
	OnActivateMode(InParams);
}

void FAssetEditorMode::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	OnInitToolMenuContext(MenuContext);
}

void FAssetEditorMode::DeactivateMode(const FAssetEditorModeDeactivateParams& InParams)
{
	OnDeactivateMode(InParams);
}

}  // namespace UE::Cameras

