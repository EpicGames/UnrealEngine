// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/AssetEditorModeManagerToolkit.h"

#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorMode.h"

namespace UE::Cameras
{

FAssetEditorModeManagerToolkit::FAssetEditorModeManagerToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
}

FAssetEditorModeManagerToolkit::~FAssetEditorModeManagerToolkit()
{
}

void FAssetEditorModeManagerToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(MenuContext);

	if (CurrentEditorMode)
	{
		CurrentEditorMode->InitToolMenuContext(MenuContext);
	}
}

void FAssetEditorModeManagerToolkit::AddEditorMode(TSharedRef<FAssetEditorMode> InMode)
{
	if (!ensureMsgf(
				!EditorModes.Contains(InMode->GetModeName()),
				TEXT("An editor mode named '%s' has already been added!"), *InMode->GetModeName().ToString()))
	{
		return;
	}

	EditorModes.Add(InMode->GetModeName(), InMode);
}

void FAssetEditorModeManagerToolkit::RemoveEditorMode(TSharedRef<FAssetEditorMode> InMode)
{
	RemoveEditorMode(InMode->GetModeName());
}

void FAssetEditorModeManagerToolkit::RemoveEditorMode(FName InModeName)
{
	TSharedPtr<FAssetEditorMode> ModeToRemove = EditorModes.FindRef(InModeName);
	if (!ensureMsgf(
				ModeToRemove,
				TEXT("No editor mode named '%s' was added!"), *InModeName.ToString()))
	{
		return;
	}

	if (CurrentEditorMode == ModeToRemove)
	{
		SetEditorMode(NAME_None);
	}

	EditorModes.Remove(InModeName);
}

void FAssetEditorModeManagerToolkit::GetEditorModes(TArray<TSharedPtr<FAssetEditorMode>>& OutModes) const
{
	for (const TPair<FName, TSharedPtr<FAssetEditorMode>>& Pair : EditorModes)
	{
		OutModes.Add(Pair.Value);
	}
}

TSharedPtr<FAssetEditorMode> FAssetEditorModeManagerToolkit::GetEditorMode(FName InModeName) const
{
	return EditorModes.FindRef(InModeName);
}

void FAssetEditorModeManagerToolkit::SetEditorMode(FName InModeName)
{
	if (CurrentEditorModeName == InModeName)
	{
		return;
	}

	if (!ensure(TabManager))
	{
		return;
	}

	// Remove current editor mode and setup.
	if (CurrentEditorMode)
	{
		// Deactivating should in theory remove all of the mode's tab-spawners.
		FAssetEditorModeDeactivateParams DeactivateParams;
		DeactivateParams.Toolkit = SharedThis(this);
		DeactivateParams.TabManager = TabManager;
		CurrentEditorMode->DeactivateMode(DeactivateParams);

		if (TSharedPtr<FExtender> OldToolbarExtender = CurrentEditorMode->GetToolbarExtender())
		{
			RemoveToolbarExtender(OldToolbarExtender);
		}

		if (TSharedPtr<FLayoutExtender> OldLayoutExtender = CurrentEditorMode->GetLayoutExtender())
		{
			LayoutExtenders.Remove(OldLayoutExtender);
		}
	}

	RemoveAllToolbarWidgets();

	// Set the new current editor mode.
	TSharedPtr<FAssetEditorMode> NewMode = EditorModes.FindRef(InModeName);
	CurrentEditorMode = NewMode;
	CurrentEditorModeName = InModeName;

	// Setup the new editor mode.
	if (NewMode)
	{
		FName ParentName;
		const FName ToolbarMenuName = GetToolMenuToolbarName(ParentName);

		// Activating should in theory add all of the mode's tab-spawners.
		FAssetEditorModeActivateParams ActivateParams;
		ActivateParams.Toolkit = SharedThis(this);
		ActivateParams.TabManager = TabManager;
		ActivateParams.AssetEditorTabsCategory = AssetEditorTabsCategory;
		ActivateParams.CommandList = ToolkitCommands;
		ActivateParams.ToolbarMenuName = ToolbarMenuName;
		NewMode->ActivateMode(ActivateParams);

		if (TSharedPtr<FLayoutExtender> NewLayoutExtender = NewMode->GetLayoutExtender())
		{
			LayoutExtenders.Add(NewLayoutExtender);
		}

		if (TSharedPtr<FTabManager::FLayout> NewLayout = NewMode->GetDefaultLayout())
		{
			RestoreFromLayout(NewLayout.ToSharedRef());
		}

		if (TSharedPtr<FExtender> NewToolbarExtender = NewMode->GetToolbarExtender())
		{
			AddToolbarExtender(NewToolbarExtender);
		}

		OnEditorToolkitModeActivated();
	}

	RegenerateMenusAndToolbars();
}

bool FAssetEditorModeManagerToolkit::CanSetEditorMode(FName InModeName) const
{
	return EditorModes.Contains(InModeName);
}

bool FAssetEditorModeManagerToolkit::IsEditorMode(FName InModeName) const
{
	return CurrentEditorModeName == InModeName;
}

FName FAssetEditorModeManagerToolkit::GetCurrentEditorModeName() const
{
	return CurrentEditorModeName;
}

}  // namespace UE::Cameras

