// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/BlueprintCameraDirectorAssetEditorMode.h"

#include "Directors/BlueprintCameraDirector.h"

namespace UE::Cameras
{

TSharedPtr<FCameraDirectorAssetEditorMode> FBlueprintCameraDirectorAssetEditorMode::CreateInstance(UCameraAsset* InCameraAsset)
{
	UCameraDirector* CameraDirector = InCameraAsset->GetCameraDirector();
	if (Cast<UBlueprintCameraDirector>(CameraDirector))
	{
		return MakeShared<FBlueprintCameraDirectorAssetEditorMode>(InCameraAsset);
	}
	return nullptr;
}

FBlueprintCameraDirectorAssetEditorMode::FBlueprintCameraDirectorAssetEditorMode(UCameraAsset* InCameraAsset)
	: FCameraDirectorAssetEditorMode(InCameraAsset)
{
}

}  // namespace UE::Cameras

