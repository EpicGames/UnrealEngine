// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/BuildButtonToolkit.h"

#include "Build/CameraBuildStatus.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuEntry.h"

#define LOCTEXT_NAMESPACE "BuildButtonToolkit"

namespace UE::Cameras
{

FBuildButtonToolkit::FBuildButtonToolkit()
{
}

FBuildButtonToolkit::FBuildButtonToolkit(TScriptInterface<IHasCameraBuildStatus> InTarget)
	: Target(InTarget)
{
}

void FBuildButtonToolkit::SetTarget(TScriptInterface<IHasCameraBuildStatus> InTarget)
{
	Target = InTarget;
}

FToolMenuEntry FBuildButtonToolkit::MakeToolbarButton(TSharedPtr<FUICommandInfo> InCommand)
{
	FToolMenuEntry BuildButton = FToolMenuEntry::InitToolBarButton(InCommand);
	BuildButton.Icon = TAttribute<FSlateIcon>(this, &FBuildButtonToolkit::GetBuildButtonIcon);
	BuildButton.ToolTip = TAttribute<FText>(this, &FBuildButtonToolkit::GetBuildButtonTooltip);
	return BuildButton;
}

FSlateIcon FBuildButtonToolkit::GetBuildButtonIcon() const
{
	static const FName BuildStatusBackground("CameraObjectEditor.BuildStatus.Background");
	static const FName BuildStatusError("CameraObjectEditor.BuildStatus.Overlay.Error");
	static const FName BuildStatusGood("CameraObjectEditor.BuildStatus.Overlay.Good");
	static const FName BuildStatusUnknown("CameraObjectEditor.BuildStatus.Overlay.Unknown");
	static const FName BuildStatusWarning("CameraObjectEditor.BuildStatus.Overlay.Warning");

	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	IHasCameraBuildStatus* Interface = Target.GetInterface();
	if (!Interface)
	{
		return FSlateIcon(CamerasStyleSetName, BuildStatusBackground, NAME_None, BuildStatusError);
	}

	switch (Interface->GetBuildStatus())
	{
		default:
		case ECameraBuildStatus::Dirty:
			return FSlateIcon(CamerasStyleSetName, BuildStatusBackground, NAME_None, BuildStatusUnknown);
		case ECameraBuildStatus::WithErrors:
			return FSlateIcon(CamerasStyleSetName, BuildStatusBackground, NAME_None, BuildStatusError);
		case ECameraBuildStatus::Clean:
			return FSlateIcon(CamerasStyleSetName, BuildStatusBackground, NAME_None, BuildStatusGood);
		case ECameraBuildStatus::CleanWithWarnings:
			return FSlateIcon(CamerasStyleSetName, BuildStatusBackground, NAME_None, BuildStatusWarning);
	}
}

FText FBuildButtonToolkit::GetBuildButtonTooltip() const
{
	IHasCameraBuildStatus* Interface = Target.GetInterface();
	if (!Interface)
	{
		return LOCTEXT("BuildButtonStatusNoAsset", "No asset is open");
	}

	switch (Interface->GetBuildStatus())
	{
		default:
		case ECameraBuildStatus::Dirty:
			return LOCTEXT("BuildButtonStatusDirty", "Dirty or unknown, should rebuild");
		case ECameraBuildStatus::WithErrors:
			return LOCTEXT("BuildButtonStatusWithErrors", "There were errors during the build, see the log window for details");
		case ECameraBuildStatus::Clean:
			return LOCTEXT("BuildButtonStatusClean", "Good to go");
		case ECameraBuildStatus::CleanWithWarnings:
			return LOCTEXT("BuildButtonStatusCleanWithWarnings", "There were warnings during the build, see the log window for details");
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

