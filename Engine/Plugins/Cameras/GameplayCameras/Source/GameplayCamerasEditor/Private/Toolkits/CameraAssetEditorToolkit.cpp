// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraAssetEditorToolkit.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetTools/CameraAssetEditor.h"
#include "BlueprintActionDatabase.h"
#include "Build/CameraAssetBuilder.h"
#include "Build/CameraBuildLog.h"
#include "Commands/CameraAssetEditorCommands.h"
#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Core/CameraRigAsset.h"
#include "Editor.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Editors/SCameraAssetInterfaceParametersPanel.h"
#include "Editors/SFindInObjectTreeGraph.h"
#include "FileHelpers.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "GameplayCamerasEditorSettings.h"
#include "Helpers/AssetTypeMenuOverlayHelper.h"
#include "Helpers/CameraDirectorClassPicker.h"
#include "Helpers/ObjectReferenceFinder.h"
#include "IAssetTools.h"
#include "IGameplayCamerasEditorModule.h"
#include "IGameplayCamerasFamily.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasModule.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenus.h"
#include "Toolkits/BuildButtonToolkit.h"
#include "Toolkits/CameraBuildLogToolkit.h"
#include "Toolkits/CameraDirectorAssetEditorMode.h"
#include "Toolkits/CameraSharedTransitionsAssetEditorMode.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "UObject/Package.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCameraFamilyShortcutBar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAssetEditorToolkit)

#define LOCTEXT_NAMESPACE "CameraAssetEditorToolkit"

namespace UE::Cameras
{

const FName FCameraAssetEditorToolkit::SearchTabId(TEXT("CameraAssetEditor_Search"));
const FName FCameraAssetEditorToolkit::MessagesTabId(TEXT("CameraAssetEditor_Messages"));
const FName FCameraAssetEditorToolkit::InterfaceParametersTabId(TEXT("CameraAssetEditor_InterfaceParameters"));

FCameraAssetEditorToolkit::FCameraAssetEditorToolkit(UCameraAssetEditor* InOwningAssetEditor)
	: FAssetEditorModeManagerToolkit(InOwningAssetEditor)
	, CameraAsset(InOwningAssetEditor->GetCameraAsset())
	, StandardLayout(new FStandardToolkitLayout(TEXT("CameraAssetEditor_Layout_v2")))
	, BuildButtonToolkit(MakeShared<FBuildButtonToolkit>(CameraAsset))
	, BuildLogToolkit(MakeShared<FCameraBuildLogToolkit>())
{
	CameraAsset->EventHandlers.Register(CameraAssetEventHandler, this);

	StandardLayout->AddBottomTab(SearchTabId);
	StandardLayout->AddBottomTab(MessagesTabId);
	StandardLayout->AddLeftTab(InterfaceParametersTabId, ETabState::OpenedTab);

	TSharedPtr<FLayoutExtender> NewLayoutExtender = MakeShared<FLayoutExtender>();
	{
		NewLayoutExtender->ExtendStack(
				FStandardToolkitLayout::BottomStackExtensionId,
				ELayoutExtensionPosition::After,
				FTabManager::FTab(SearchTabId, ETabState::ClosedTab));

		NewLayoutExtender->ExtendStack(
				FStandardToolkitLayout::BottomStackExtensionId,
				ELayoutExtensionPosition::After,
				FTabManager::FTab(MessagesTabId, ETabState::ClosedTab));

		NewLayoutExtender->ExtendStack(
				FStandardToolkitLayout::LeftStackExtensionId,
				ELayoutExtensionPosition::Above,
				FTabManager::FTab(InterfaceParametersTabId, ETabState::OpenedTab));
	}
	LayoutExtenders.Add(NewLayoutExtender);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FCameraAssetEditorToolkit::~FCameraAssetEditorToolkit()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void FCameraAssetEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraAsset);
}

FString FCameraAssetEditorToolkit::GetReferencerName() const
{
	return TEXT("FCameraAssetEditorToolkit");
}

void FCameraAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(SearchTabId, FOnSpawnTab::CreateSP(this, &FCameraAssetEditorToolkit::SpawnTab_Search))
		.SetDisplayName(LOCTEXT("Search", "Search"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraAssetEditor.Tabs.Search"));

	InTabManager->RegisterTabSpawner(MessagesTabId, FOnSpawnTab::CreateSP(this, &FCameraAssetEditorToolkit::SpawnTab_Messages))
		.SetDisplayName(LOCTEXT("Messages", "Messages"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraAssetEditor.Tabs.Messages"));

	InTabManager->RegisterTabSpawner(InterfaceParametersTabId, FOnSpawnTab::CreateSP(this, &FCameraAssetEditorToolkit::SpawnTab_InterfaceParameters))
		.SetDisplayName(LOCTEXT("InterfaceParameters", "Interface Parameters"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.InterfaceParameters"));
}

TSharedRef<SDockTab> FCameraAssetEditorToolkit::SpawnTab_Search(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> SearchTab = SNew(SDockTab)
		.Label(LOCTEXT("SearchTabTitle", "Search"))
		[
			SearchWidget.ToSharedRef()
		];

	return SearchTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraAssetEditorToolkit::SpawnTab_Messages(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> MessagesTab = SNew(SDockTab)
		.Label(LOCTEXT("MessagesTabTitle", "Messages"))
		[
			BuildLogToolkit->GetMessagesWidget().ToSharedRef()
		];

	return MessagesTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraAssetEditorToolkit::SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> InterfaceParametersTab = SNew(SDockTab)
		.Label(LOCTEXT("InterfaceParametersTabTitle", "Parameters"))
		[
			InterfaceParametersPanel.ToSharedRef()
		];

	return InterfaceParametersTab.ToSharedRef();
}

void FCameraAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(SearchTabId);
	InTabManager->UnregisterTabSpawner(MessagesTabId);
	InTabManager->UnregisterTabSpawner(InterfaceParametersTabId);
}

void FCameraAssetEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	// ...no up-call...

	RegisterToolbar();
	CreateEditorModeManager();
	LayoutExtender = MakeShared<FLayoutExtender>();
	// We don't want a details view, but we need to because otherwise it crashes.
	DetailsView = CreateDetailsView();

	// Do our custom stuff.

	// Create the search panel.
	SearchWidget = SNew(SFindInObjectTreeGraph)
		.OnGetGraphsToSearch(this, &FCameraAssetEditorToolkit::OnGetGraphsToSearch)
		.OnJumpToNodeRequested(this, &FCameraAssetEditorToolkit::OnJumpToNode);

	// Create the interface parameters panel.
	InterfaceParametersPanel = SNew(SCameraAssetInterfaceParametersPanel, this);
	InterfaceParametersPanel->OnInterfaceParameterSelected().AddSP(this, &FCameraAssetEditorToolkit::OnInterfaceParameterSelected);

	// Create the message log.
	BuildLogToolkit->Initialize("CameraAssetBuildMessages");
}

void FCameraAssetEditorToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(
				MenuName, ParentName, EMultiBoxType::ToolBar);

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		const FCameraAssetEditorCommands& Commands = FCameraAssetEditorCommands::Get();

		ToolbarMenu->AddDynamicSection("Tools", FNewToolMenuDelegate::CreateLambda(
				[&Commands](UToolMenu* InMenu)
				{
					UCameraAssetEditorMenuContext* Context = InMenu->FindContext<UCameraAssetEditorMenuContext>();
					FCameraAssetEditorToolkit* This = Context ? Context->Toolkit.Pin().Get() : nullptr;
					if (!ensure(This))
					{
						return;
					}

					FToolMenuSection& ToolsSection = InMenu->AddSection("Tools");
					ToolsSection.AddEntry(This->BuildButtonToolkit->MakeToolbarButton(Commands.Build));
					ToolsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.FindInCamera));
				}),
				InsertAfterAssetSection);

		FToolMenuSection& ModesSection = ToolbarMenu->AddSection("EditorModes", TAttribute<FText>(), InsertAfterAssetSection);
		ModesSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ShowCameraDirector));
		ModesSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ShowSharedTransitions));
	}

	TSharedRef<IGameplayCamerasFamily> Family = IGameplayCamerasFamily::CreateFamily(CameraAsset).ToSharedRef();
	TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
	AddToolbarExtender(ToolbarExtender);
	ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this, Family](FToolBarBuilder& Builder)
				{
					AddToolbarWidget(SNew(SCameraFamilyShortcutBar, SharedThis(this), Family));
				})
			);
}

void FCameraAssetEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(MenuContext);

	UCameraAssetEditorMenuContext* Context = NewObject<UCameraAssetEditorMenuContext>();
	Context->Toolkit = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FCameraAssetEditorToolkit::PostInitAssetEditor()
{
	TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();
	{
		MenuExtender->AddMenuExtension(
			"AssetEditorActions",
			EExtensionHook::After,
			GetToolkitCommands(),
			FMenuExtensionDelegate::CreateSP(this, &FCameraAssetEditorToolkit::FillCameraMenu));
	}
	AddMenuExtender(MenuExtender);

	Settings = GetMutableDefault<UGameplayCamerasEditorSettings>();

	const FName CameraDirectorModeName = FCameraDirectorAssetEditorMode::ModeName;
	AddEditorMode(CreateCameraDirectorAssetEditorMode().ToSharedRef());

	const FName SharedTransitionsModeName = FCameraSharedTransitionsAssetEditorMode::ModeName;
	AddEditorMode(MakeShared<FCameraSharedTransitionsAssetEditorMode>(CameraAsset));

	const FCameraAssetEditorCommands& Commands = FCameraAssetEditorCommands::Get();
	TMap<FName, TSharedPtr<FUICommandInfo>> ModeCommands;
	ModeCommands.Add(CameraDirectorModeName, Commands.ShowCameraDirector);
	ModeCommands.Add(SharedTransitionsModeName, Commands.ShowSharedTransitions);
	for (auto& Pair : ModeCommands)
	{
		ToolkitCommands->MapAction(
			Pair.Value,
			FExecuteAction::CreateSP(this, &FCameraAssetEditorToolkit::SetEditorMode, Pair.Key),
			FCanExecuteAction::CreateSP(this, &FCameraAssetEditorToolkit::CanSetEditorMode, Pair.Key),
			FIsActionChecked::CreateSP(this, &FCameraAssetEditorToolkit::IsEditorMode, Pair.Key));
	}

	ToolkitCommands->MapAction(
		Commands.ChangeCameraDirector,
		FExecuteAction::CreateSP(this, &FCameraAssetEditorToolkit::OnChangeCameraDirector));

	ToolkitCommands->MapAction(
		Commands.Build,
		FExecuteAction::CreateSP(this, &FCameraAssetEditorToolkit::OnBuild));

	ToolkitCommands->MapAction(
		Commands.FindInCamera,
		FExecuteAction::CreateSP(this, &FCameraAssetEditorToolkit::OnFindInCamera));

	BuildLogToolkit->OnRequestJumpToObject().BindSP(this, &FCameraAssetEditorToolkit::OnJumpToObject);

	ToolkitCommands->MapAction(
			Commands.RenameInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersPanel->RenameSelectedParameter();
				}));

	ToolkitCommands->MapAction(
			Commands.DeleteInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersPanel->DeleteSelectedParameter();
				}));

	IGameplayCamerasModule& GameplayCamerasModule = IGameplayCamerasModule::Get();
	LiveEditManager = GameplayCamerasModule.GetLiveEditManager();

	const FName InitialModeName = !Settings->LastCameraAssetToolkitModeName.IsNone() ? 
		Settings->LastCameraAssetToolkitModeName : CameraDirectorModeName;
	SetEditorMode(InitialModeName);

	UpgradeLegacyCameraAssets();

	// If the asset predates the FCameraAssetInterface, upgrade it. We don't mark it as modified in this case.
	// We either let the user modify the asset and save it, or we'll re-upgrade it next time if needed again.
	// This prevents prompting the user to save changes they don't know about.
	FCameraAssetBuilder::BuildDefaultInterfaceIfNeeded(CameraAsset);
}

TSharedPtr<FAssetEditorMode> FCameraAssetEditorToolkit::CreateCameraDirectorAssetEditorMode()
{
	IGameplayCamerasEditorModule& GameplayCamerasEditorModule = IGameplayCamerasEditorModule::Get();

	TSharedPtr<FCameraDirectorAssetEditorMode> CameraDirectorEditor;
	for (const FOnCreateCameraDirectorAssetEditorMode& EditorCreator : GameplayCamerasEditorModule.GetCameraDirectorEditorCreators())
	{
		CameraDirectorEditor = EditorCreator.Execute(CameraAsset);
		if (CameraDirectorEditor)
		{
			break;
		}
	}
	if (!CameraDirectorEditor)
	{
		CameraDirectorEditor = MakeShared<FCameraDirectorAssetEditorMode>(CameraAsset);
	}
	return CameraDirectorEditor;
}

void FCameraAssetEditorToolkit::FillCameraMenu(FMenuBuilder& MenuBuilder)
{
	const FCameraAssetEditorCommands& Commands = FCameraAssetEditorCommands::Get();

	MenuBuilder.BeginSection(TEXT("Camera"), LOCTEXT("CameraMenuTitle", "Camera"));
	MenuBuilder.AddMenuEntry(Commands.ChangeCameraDirector);
	MenuBuilder.EndSection();
}

void FCameraAssetEditorToolkit::PostRegenerateMenusAndToolbars()
{
	SetMenuOverlay(FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UCameraAsset::StaticClass()));
}

void FCameraAssetEditorToolkit::OnEditorToolkitModeActivated()
{
	Settings->LastCameraAssetToolkitModeName = GetCurrentEditorModeName();
	Settings->SaveConfig();
}

void FCameraAssetEditorToolkit::OnCameraDirectorChanged(UCameraAsset* InCameraAsset, const TCameraPropertyChangedEvent<UCameraDirector*>& Event)
{
	const FName CameraDirectorModeName = FCameraDirectorAssetEditorMode::ModeName;

	RemoveEditorMode(CameraDirectorModeName);

	AddEditorMode(CreateCameraDirectorAssetEditorMode().ToSharedRef());

	if (GetCurrentEditorModeName().IsNone())
	{
		SetEditorMode(CameraDirectorModeName);
	}
}

void FCameraAssetEditorToolkit::OnChangeCameraDirector()
{
	FCameraDirectorClassPicker Picker;
	TSubclassOf<UCameraDirector> ChosenClass;
	const bool bPressedOk = Picker.PickCameraDirectorClass(ChosenClass);
	if (bPressedOk && ChosenClass != CameraAsset->GetCameraDirector()->GetClass())
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeCameraDirector", "Change Camera Director"));

		UCameraDirector* NewCameraDirector = NewObject<UCameraDirector>(CameraAsset, ChosenClass, NAME_None, RF_Transactional);
		CameraAsset->SetCameraDirector(NewCameraDirector);
	}
}

void FCameraAssetEditorToolkit::OnBuild()
{
	using namespace UE::Cameras;

	if (!CameraAsset)
	{
		return;
	}

	FCameraBuildLog BuildLog;
	FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::UserAction);
	FCameraAssetBuilder Builder(BuildContext);
	Builder.BuildCamera(CameraAsset);
	
	BuildLogToolkit->PopulateMessageListing(BuildLog);

	if (CameraAsset->GetBuildStatus() != ECameraBuildStatus::Clean)
	{
		TabManager->TryInvokeTab(MessagesTabId);
	}

	DetailsView->RequestForceRefresh();

	TConstArrayView<UObject*> BuiltObjects = Builder.GetBuiltReferencedAssets();
	for (UObject* BuiltObject : BuiltObjects)
	{
		if (UCameraRigAsset* CameraRigAsset = Cast<UCameraRigAsset>(BuiltObject))
		{
			FCameraRigPackages BuiltPackages;
			CameraRigAsset->GatherPackages(BuiltPackages);

			for (const UPackage* BuiltPackage : BuiltPackages)
			{
				LiveEditManager->NotifyPostBuildAsset(BuiltPackage);
			}
		}
	}

	if (FBlueprintActionDatabase* Database = FBlueprintActionDatabase::TryGet())
	{
		Database->RefreshAssetActions(CameraAsset);
	}
}

void FCameraAssetEditorToolkit::OnFindInCamera()
{
	TabManager->TryInvokeTab(SearchTabId);
	SearchWidget->FocusSearchEditBox();
}

void FCameraAssetEditorToolkit::OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources)
{
	TSharedPtr<FCameraSharedTransitionsAssetEditorMode> SharedTransitionsMode = GetTypedEditorMode<FCameraSharedTransitionsAssetEditorMode>(
			FCameraSharedTransitionsAssetEditorMode::ModeName);
	SharedTransitionsMode->OnGetGraphsToSearch(OutSources);
}

void FCameraAssetEditorToolkit::OnJumpToNode(UEdGraphNode* Node, FName PinName)
{
	// We only have one graph, the shared transition graph, so go there directly.
	TSharedPtr<FCameraSharedTransitionsAssetEditorMode> SharedTransitionsMode = GetTypedEditorMode<FCameraSharedTransitionsAssetEditorMode>(
			FCameraSharedTransitionsAssetEditorMode::ModeName);
	SetEditorMode(FCameraSharedTransitionsAssetEditorMode::ModeName);
	SharedTransitionsMode->JumpToNode(Node, PinName);
}

void FCameraAssetEditorToolkit::OnJumpToObject(UObject* Object)
{
	OnJumpToObject(Object, NAME_None);
}

void FCameraAssetEditorToolkit::OnJumpToObject(UObject* Object, FName PropertyName)
{
	bool bFindInCameraDirector = false;
	bool bFindInCameraRig = false;
	bool bFindInSharedTranstions = false;
	UObject* CurOuter = Object;
	while (CurOuter != nullptr)
	{
		if (CurOuter->IsA<UCameraDirector>())
		{
			bFindInCameraDirector = true;
			break;
		}
		if (CurOuter->IsA<UCameraRigAsset>())
		{
			bFindInCameraRig = true;
			break;
		}
		if (CurOuter == CameraAsset)
		{
			bFindInSharedTranstions = true;
			break;
		}

		CurOuter = CurOuter->GetOuter();
	}

	if (bFindInCameraDirector)
	{
		TSharedPtr<FCameraDirectorAssetEditorMode> DirectorMode = GetTypedEditorMode<FCameraDirectorAssetEditorMode>(
				FCameraDirectorAssetEditorMode::ModeName);
		SetEditorMode(FCameraDirectorAssetEditorMode::ModeName);
		DirectorMode->JumpToObject(Object, PropertyName);
		return;
	}
	
	if (bFindInSharedTranstions)
	{
		TSharedPtr<FCameraSharedTransitionsAssetEditorMode> SharedTransitionsMode = GetTypedEditorMode<FCameraSharedTransitionsAssetEditorMode>(
				FCameraSharedTransitionsAssetEditorMode::ModeName);
		SetEditorMode(FCameraSharedTransitionsAssetEditorMode::ModeName);
		SharedTransitionsMode->JumpToObject(Object, PropertyName);
		return;
	}
}

void FCameraAssetEditorToolkit::OnInterfaceParameterSelected(UCameraAssetInterfaceParameter* InterfaceParameter)
{
	DetailsView->SetObject(InterfaceParameter);
}

void FCameraAssetEditorToolkit::PostUndo(bool bSuccess)
{
	InterfaceParametersPanel->RequestListRefresh();
}

void FCameraAssetEditorToolkit::PostRedo(bool bSuccess)
{
	InterfaceParametersPanel->RequestListRefresh();
}

FText FCameraAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Asset");
}

FName FCameraAssetEditorToolkit::GetToolkitFName() const
{
	static FName SequencerName("CameraAssetEditor");
	return SequencerName;
}

FString FCameraAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Asset ").ToString();
}

FLinearColor FCameraAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

void FCameraAssetEditorToolkit::UpgradeLegacyCameraAssets()
{
	if (!CameraAsset)
	{
		return;
	}

	UPackage* CameraAssetPackage = CameraAsset->GetOutermost();
	if (!CameraAssetPackage || CameraAssetPackage == GetTransientPackage())
	{
		return;
	}

	// Gather all camera rigs found inside the camera package. Camera rigs used to be "owned"
	// by the camera asset, but now we want them all to be shared assets.
	TSet<UCameraRigAsset*> KnownCameraRigs;
	{
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(CameraAssetPackage, ObjectsInPackage);
		for (UObject* Object : ObjectsInPackage)
		{
			if (UCameraRigAsset* CameraRig = Cast<UCameraRigAsset>(Object))
			{
				KnownCameraRigs.Add(CameraRig);
			}

			// Also clean up any redirectors. They used to be created for a short time when
			// renaming owned camera rigs wasn't doing the right thing.
			if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
			{
				if (KnownCameraRigs.Contains(Cast<UCameraRigAsset>(Redirector->DestinationObject)))
				{
					CameraAsset->Modify();

					Redirector->ClearFlags(RF_Public | RF_Standalone);
					Redirector->DestinationObject = nullptr;
				}
			}
		}
	}

	// No owned camera rigs? We are done.
	if (KnownCameraRigs.IsEmpty())
	{
		return;
	}

	// Start working!
	FScopedSlowTask SlowTask(KnownCameraRigs.Num() + 1,
			LOCTEXT("UpgradeLegacyCameraAssets", "Upgrading legacy camera asset"));
	SlowTask.MakeDialog(true);

	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
	IAssetTools& AssetTools = IAssetTools::Get();

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(CameraAssetPackage);

	// Look for packages that reference one of our camera rigs. We don't need to patch up their references
	// since they will still point to the same object, but we will need to re-save those packages because
	// the serialized soft-object-path will have changed to the new standalone camera rig asset package.
	{
		SlowTask.EnterProgressFrame();

		TArray<FName> OnDiskReferencers;
		AssetRegistry.GetReferencers(CameraAssetPackage->GetFName(), OnDiskReferencers);
		TArray<UObject*> KnownCameraRigsArray(KnownCameraRigs.Array());

		for (FName OnDiskReferencer : OnDiskReferencers)
		{
			UPackage* ReferencerPackage = FindPackage(nullptr, *OnDiskReferencer.ToString());
			if (!ReferencerPackage)
			{
				FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(OnDiskReferencer);
				ReferencerPackage = LoadPackage(nullptr, PackagePath, LOAD_None);
			}
			if (!ensure(ReferencerPackage))
			{
				continue;
			}

			UObject* ReferencingAsset = ReferencerPackage->FindAssetInPackage();
			if (!ensure(ReferencingAsset))
			{
				continue;
			}

			FObjectReferenceFinder ReferenceFinder(ReferencingAsset, KnownCameraRigsArray);
			ReferenceFinder.CollectReferences();

			if (ReferenceFinder.HasAnyObjectReference())
			{
				ReferencingAsset->Modify();
				ReferencerPackage->MarkPackageDirty();
				PackagesToSave.Add(ReferencerPackage);
			}
		}
	}

	// Now create individual assets for each camera rig.
	const FString CameraRigsBaseName = CameraAsset->GetName();
	const FString CameraRigsBasePath = FPaths::GetPath(CameraAssetPackage->GetPathName());
	const EObjectFlags CameraRigFlags = RF_Public | RF_Standalone | RF_Transactional;

	for (UCameraRigAsset* CameraRig : KnownCameraRigs)
	{
		SlowTask.EnterProgressFrame();

		// Name the new camera rigs like this: "<CameraAsset>_<CameraRig>"
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FString CameraRigName = CameraRig->Interface.GetDisplayName();
		if (CameraRigName.IsEmpty())
		{
			CameraRigName = CameraRig->GetName();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const FString CameraRigPackageName = FString::Printf(TEXT("%s_%s"), *CameraRigsBaseName, *CameraRigName);

		FString CameraRigPackagePath;
		FString CameraRigAssetName;
		AssetTools.CreateUniqueAssetName(
				CameraRigsBasePath / CameraRigPackageName, FString(), CameraRigPackagePath, CameraRigAssetName);

		// Create the new package, and move the camera rig inside it as its main asset.
		UPackage* CameraRigPackage = CreatePackage(*CameraRigPackagePath);
		CameraRig->Rename(*CameraRigAssetName, CameraRigPackage, REN_DontCreateRedirectors);
		CameraRig->SetFlags(CameraRigFlags);

		// Notify the asset registry that a new asset was created.
		AssetRegistry.AssetCreated(CameraRig);

		CameraRigPackage->MarkPackageDirty();

		PackagesToSave.Add(CameraRigPackage);
	}

	// Prompt the user to save all assets.
	FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
	SaveParams.bCheckDirty = false;
	SaveParams.bPromptToSave = true;
	SaveParams.Title = LOCTEXT("SaveUpgradedAsset_Title", "Save upgraded packages");
	SaveParams.Message = LOCTEXT(
			"SaveUpgradedAsset_Message", 
			"This camera asset had legacy private camera rigs. "
			"They have been re-created as standalone assets, and referencing packages have been fixed up. "
			"Please save all new and modified packages.");
	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, SaveParams);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

