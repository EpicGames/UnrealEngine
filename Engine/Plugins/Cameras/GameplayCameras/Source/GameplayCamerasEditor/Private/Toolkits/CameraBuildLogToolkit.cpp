// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraBuildLogToolkit.h"

#include "Build/CameraBuildLog.h"
#include "IMessageLogListing.h"
#include "Logging/TokenizedMessage.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"

namespace UE::Cameras
{

FCameraBuildLogToolkit::FCameraBuildLogToolkit()
{
}

void FCameraBuildLogToolkit::Initialize(FName InLogName)
{
	if (InLogName.IsNone())
	{
		InLogName = "CameraBuildLogMessages";
	}

	// Create the message log.
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;
	MessageListing = MessageLogModule.CreateLogListing(InLogName, LogOptions);
	MessageListing->OnMessageTokenClicked().AddSP(this, &FCameraBuildLogToolkit::OnMessageTokenClicked);

	// Create the messages widget.
	MessagesWidget = MessageLogModule.CreateLogListingWidget(MessageListing.ToSharedRef());
}

void FCameraBuildLogToolkit::PopulateMessageListing(FCameraBuildLog& InBuildLog)
{
	if (!ensureMsgf(MessageListing, TEXT("This toolkit wasn't initialized")))
	{
		return;
	}

	MessageListing->ClearMessages();

	for (const FCameraBuildLogMessage& Message : InBuildLog.GetMessages())
	{
		TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(Message.Severity);

		if (Message.Object)
		{
			TSharedRef<FUObjectToken> ObjectToken = FUObjectToken::Create(
					Message.Object, FText::FromName(Message.Object->GetFName()));

			// Suppress default activation callback that opens the content browser. We handle that
			// sort of stuff in a more general fashion in OnMessageTokenClicked.
			static auto DummyActivation = [](const TSharedRef<class IMessageToken>&) {};
			ObjectToken->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda(DummyActivation));
			
			TokenizedMessage->AddToken(ObjectToken);
		}

		TokenizedMessage->AddToken(FTextToken::Create(Message.Text));

		MessageListing->AddMessage(TokenizedMessage);
	}
}

void FCameraBuildLogToolkit::OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken)
{
	if (InMessageToken->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> ObjectToken = StaticCastSharedRef<FUObjectToken>(InMessageToken);
		if (UObject* Object = ObjectToken->GetObject().Get())
		{
			RequestJumpToObject.ExecuteIfBound(Object);
		}
	}
}

}  // namespace UE::Cameras

