// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraDirectorAssetEditorMode.h"

#include "Core/CameraAsset.h"
#include "Core/CameraDirector.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "CameraDirectorAssetEditorMode"

namespace UE::Cameras
{

FName FCameraDirectorAssetEditorMode::ModeName(TEXT("CameraDirector"));

FName FCameraDirectorAssetEditorMode::DirectorEditorTabId(TEXT("DirectorEditor"));

FCameraDirectorAssetEditorMode::FCameraDirectorAssetEditorMode(UCameraAsset* InCameraAsset)
	: FAssetEditorMode(ModeName)
	, CameraAsset(InCameraAsset)
{
	StandardLayout = MakeShared<FStandardToolkitLayout>("CameraAssetEditor_Mode_CameraDirector_Layout_v1");
	{
		StandardLayout->AddCenterTab(DirectorEditorTabId);
	}

	DefaultLayout = StandardLayout->GetLayout();
}

void FCameraDirectorAssetEditorMode::OnActivateMode(const FAssetEditorModeActivateParams& InParams)
{
	if (!bInitializedToolkit)
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.NotifyHook = this;
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		bInitializedToolkit = true;
	}

	DetailsView->SetObject(CameraAsset->GetCameraDirector());

	InParams.TabManager->RegisterTabSpawner(DirectorEditorTabId, FOnSpawnTab::CreateSP(this, &FCameraDirectorAssetEditorMode::SpawnTab_DirectorEditor))
		.SetDisplayName(LOCTEXT("CameraDirectorEditor", "Camera Director"))
		.SetGroup(InParams.AssetEditorTabsCategory.ToSharedRef());
}

TSharedRef<SDockTab> FCameraDirectorAssetEditorMode::SpawnTab_DirectorEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DirectorEditorTab = SNew(SDockTab)
		.Label(LOCTEXT("CameraDirectorEditorTabTitle", "Camera Director"))
		[
			DetailsView.ToSharedRef()
		];

	return DirectorEditorTab.ToSharedRef();
}

void FCameraDirectorAssetEditorMode::OnDeactivateMode(const FAssetEditorModeDeactivateParams& InParams)
{
	InParams.TabManager->UnregisterTabSpawner(DirectorEditorTabId);
}

void FCameraDirectorAssetEditorMode::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (CameraAsset)
	{
		CameraAsset->DirtyBuildStatus();
	}
}

bool FCameraDirectorAssetEditorMode::JumpToObject(UObject* InObject, FName InPropertyName)
{
	if (InObject == CameraAsset->GetCameraDirector())
	{
		return true;
	}
	return false;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

