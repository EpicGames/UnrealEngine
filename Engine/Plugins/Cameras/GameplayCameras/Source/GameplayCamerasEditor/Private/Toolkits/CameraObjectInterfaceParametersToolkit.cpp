// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraObjectInterfaceParametersToolkit.h"

#include "Core/BaseCameraObject.h"
#include "EdGraph/EdGraphNode.h"
#include "Editor.h"
#include "Editors/SCameraObjectInterfaceParametersPanel.h"
#include "SPinTypeSelector.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "CameraObjectInterfaceParametersToolkit"

namespace UE::Cameras
{

FCameraObjectInterfaceParametersToolkit::FCameraObjectInterfaceParametersToolkit()
{
	SAssignNew(PanelContainer, SBox);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FCameraObjectInterfaceParametersToolkit::~FCameraObjectInterfaceParametersToolkit()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void FCameraObjectInterfaceParametersToolkit::SetCameraObject(UBaseCameraObject* InCameraObject)
{
	if (CameraObject != InCameraObject)
	{
		PanelContainer->SetContent(SNullWidget::NullWidget);

		CameraObject = InCameraObject;

		if (CameraObject)
		{
			Panel = SNew(SCameraObjectInterfaceParametersPanel, this);
			PanelContainer->SetContent(Panel.ToSharedRef());
		}
	}
}

TSharedPtr<SWidget> FCameraObjectInterfaceParametersToolkit::GetInterfaceParametersPanel() const
{
	return PanelContainer;
}

void FCameraObjectInterfaceParametersToolkit::PostUndo(bool bSuccess)
{
	Panel->RequestListRefresh();
}

void FCameraObjectInterfaceParametersToolkit::PostRedo(bool bSuccess)
{
	Panel->RequestListRefresh();
}

void FCameraObjectInterfaceParametersToolkit::RenameSelectedParameter()
{
	Panel->RenameSelectedParameter();
}

void FCameraObjectInterfaceParametersToolkit::DeleteSelectedParameter()
{
	Panel->DeleteSelectedParameter();
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

