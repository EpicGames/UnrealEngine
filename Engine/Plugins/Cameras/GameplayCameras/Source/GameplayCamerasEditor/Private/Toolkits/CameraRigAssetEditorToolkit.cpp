// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraRigAssetEditorToolkit.h"

#include "BlueprintActionDatabase.h"
#include "Build/CameraBuildLog.h"
#include "Build/CameraRigAssetBuilder.h"
#include "Commands/CameraRigAssetEditorCommands.h"
#include "Core/CameraRigAsset.h"
#include "Customizations/RichCurveDetailsCustomizations.h"
#include "Editors/CameraRigCameraNodeGraphSchema.h"
#include "Editors/CameraRigTransitionGraphSchema.h"
#include "Editors/SCameraRigAssetEditor.h"
#include "Editors/SFindInObjectTreeGraph.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Helpers/AssetTypeMenuOverlayHelper.h"
#include "IGameplayCamerasEditorModule.h"
#include "IGameplayCamerasFamily.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "IGameplayCamerasModule.h"
#include "Modules/ModuleManager.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenus.h"
#include "Toolkits/BuildButtonToolkit.h"
#include "Toolkits/CameraBuildLogToolkit.h"
#include "Toolkits/CameraObjectInterfaceParametersToolkit.h"
#include "Toolkits/CameraRigAssetEditorToolkitBase.h"
#include "Toolkits/CurveEditorToolkit.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCameraFamilyShortcutBar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRigAssetEditorToolkit)

#define LOCTEXT_NAMESPACE "CameraRigAssetEditorToolkit"

namespace UE::Cameras
{

const FName FCameraRigAssetEditorToolkit::SearchTabId(TEXT("CameraRigAssetEditor_Search"));
const FName FCameraRigAssetEditorToolkit::MessagesTabId(TEXT("CameraRigAssetEditor_Messages"));
const FName FCameraRigAssetEditorToolkit::CurvesTabId(TEXT("CameraRigAssetEditor_Curves"));
const FName FCameraRigAssetEditorToolkit::InterfaceParametersTabId(TEXT("CameraRigAssetEditor_InterfaceParameters"));

FCameraRigAssetEditorToolkit::FCameraRigAssetEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	Impl = MakeShared<FCameraRigAssetEditorToolkitBase>(TEXT("CameraRigAssetEditor_Layout_v6"));
	BuildButtonToolkit = MakeShared<FBuildButtonToolkit>();
	BuildLogToolkit = MakeShared<FCameraBuildLogToolkit>();
	CurveEditorToolkit = MakeShared<FCurveEditorToolkit>();
	InterfaceParametersToolkit = MakeShared<FCameraObjectInterfaceParametersToolkit>();

	// Override base class default layout.
	TSharedPtr<FStandardToolkitLayout> StandardLayout = Impl->GetStandardLayout();
	{
		StandardLayout->AddBottomTab(SearchTabId);
		StandardLayout->AddBottomTab(MessagesTabId);
		StandardLayout->AddBottomTab(CurvesTabId);
		StandardLayout->AddLeftTab(InterfaceParametersTabId, ETabState::OpenedTab);
	}
	StandaloneDefaultLayout = StandardLayout->GetLayout();

	UClass* NodeGraphSchemaClass = UCameraRigCameraNodeGraphSchema::StaticClass();
	UCameraRigCameraNodeGraphSchema* DefaultNodeGraphSchema = Cast<UCameraRigCameraNodeGraphSchema>(NodeGraphSchemaClass->GetDefaultObject());
	NodeGraphConfig = DefaultNodeGraphSchema->BuildGraphConfig();

	UClass* TransitionSchemaClass = UCameraRigTransitionGraphSchema::StaticClass();
	UCameraRigTransitionGraphSchema* DefaultTransitionGraphSchema = Cast<UCameraRigTransitionGraphSchema>(TransitionSchemaClass->GetDefaultObject());
	TransitionGraphConfig = DefaultTransitionGraphSchema->BuildGraphConfig();
}

FCameraRigAssetEditorToolkit::~FCameraRigAssetEditorToolkit()
{
	FRichCurveDetailsCustomization::OnInvokeCurveEditor().RemoveAll(this);
}

void FCameraRigAssetEditorToolkit::SetCameraRigAsset(UCameraRigAsset* InCameraRig)
{
	EventHandler.Unlink();

	Impl->SetCameraRigAsset(InCameraRig);
	BuildButtonToolkit->SetTarget(InCameraRig);
	InterfaceParametersToolkit->SetCameraObject(InCameraRig);

	if (InCameraRig && bIsInitialized)
	{
		InCameraRig->EventHandlers.Register(EventHandler, this);
	}
}

void FCameraRigAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	Impl->RegisterTabSpawners(InTabManager, AssetEditorTabsCategory);

	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(SearchTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkit::SpawnTab_Search))
		.SetDisplayName(LOCTEXT("Search", "Search"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.Search"));

	InTabManager->RegisterTabSpawner(MessagesTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkit::SpawnTab_Messages))
		.SetDisplayName(LOCTEXT("Messages", "Messages"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.Messages"));

	InTabManager->RegisterTabSpawner(CurvesTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkit::SpawnTab_Curves))
		.SetDisplayName(LOCTEXT("Curves", "Curves"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.Curves"));

	InTabManager->RegisterTabSpawner(InterfaceParametersTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkit::SpawnTab_InterfaceParameters))
		.SetDisplayName(LOCTEXT("InterfaceParameters", "Interface Parameters"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.InterfaceParameters"));
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkit::SpawnTab_Search(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> SearchTab = SNew(SDockTab)
		.Label(LOCTEXT("SearchTabTitle", "Search"))
		[
			SearchWidget.ToSharedRef()
		];

	return SearchTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkit::SpawnTab_Messages(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> MessagesTab = SNew(SDockTab)
		.Label(LOCTEXT("MessagesTabTitle", "Messages"))
		[
			BuildLogToolkit->GetMessagesWidget().ToSharedRef()
		];

	return MessagesTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkit::SpawnTab_Curves(const FSpawnTabArgs& Args)
{
	if (!CurveEditorToolkit->IsInitialized())
	{
		CurveEditorToolkit->Initialize();
	}

	TSharedPtr<SDockTab> CurvesTab = SNew(SDockTab)
		.Label(LOCTEXT("CurvesTabTitle", "Curves"))
		.OnTabClosed(this, &FCameraRigAssetEditorToolkit::OnCurvesTabClosed)
		[
			CurveEditorToolkit->GetCurveEditorWidget().ToSharedRef()
		];

	return CurvesTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkit::SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> InterfaceParametersTab = SNew(SDockTab)
		.Label(LOCTEXT("InterfaceParametersTabTitle", "Parameters"))
		[
			InterfaceParametersToolkit->GetInterfaceParametersPanel().ToSharedRef()
		];

	return InterfaceParametersTab.ToSharedRef();
}

void FCameraRigAssetEditorToolkit::OnCurvesTabClosed(TSharedRef<SDockTab> InTab)
{
	if (CurveEditorToolkit->IsInitialized())
	{
		InTab->ClearContent();

		// Clear the curve editor when the tab is closed.
		CurveEditorToolkit->Shutdown();
	}
}

void FCameraRigAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	
	Impl->UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(SearchTabId);
	InTabManager->UnregisterTabSpawner(MessagesTabId);
	InTabManager->UnregisterTabSpawner(CurvesTabId);
	InTabManager->UnregisterTabSpawner(InterfaceParametersTabId);
}

void FCameraRigAssetEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab, and our base class
	// has its own details view in order to get a notify hook.
	// ...no up-call...

	RegisterToolbar();
	CreateEditorModeManager();
	LayoutExtender = MakeShared<FLayoutExtender>();

	// Now do our custom stuff.

	Impl->CreateWidgets();

	// We need to set this for our FBaseAssetToolkit parent because otherwise it crashes
	// unhappily in SetObjectsToEdit.
	DetailsView = Impl->GetDetailsView();

	// Create the search panel.
	TSharedPtr<SCameraRigAssetEditor> CameraRigEditorWidget = Impl->GetCameraRigAssetEditor();
	TArray<UEdGraph*> CameraRigGraphs;
	CameraRigEditorWidget->GetGraphs(CameraRigGraphs);
	SearchWidget = SNew(SFindInObjectTreeGraph)
		.OnGetGraphsToSearch(this, &FCameraRigAssetEditorToolkit::OnGetGraphsToSearch)
		.OnJumpToNodeRequested(this, &FCameraRigAssetEditorToolkit::OnJumpToNode);

	// Create the message log.
	BuildLogToolkit->Initialize("CameraRigAssetBuildMessages");

	// Hook-up interface parameters events.
	InterfaceParametersToolkit->OnInterfaceParameterSelected().AddSP(this, &FCameraRigAssetEditorToolkit::OnCameraObjectInterfaceParameterSelected);
	InterfaceParametersToolkit->OnSearchInterfaceParameterNodes().AddSP(this, &FCameraRigAssetEditorToolkit::OnSearchCameraObjectInterfaceParameterNodes);
}

void FCameraRigAssetEditorToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);
		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);

		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(
				MenuName, ParentName, EMultiBoxType::ToolBar);

		ToolbarMenu->AddDynamicSection("Tools", FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					UCameraRigAssetEditorMenuContext* Context = InMenu->FindContext<UCameraRigAssetEditorMenuContext>();
					FCameraRigAssetEditorToolkit* This = Context ? Context->Toolkit.Pin().Get() : nullptr;
					if (!ensure(This))
					{
						return;
					}

					const FCameraRigAssetEditorCommands& Commands = FCameraRigAssetEditorCommands::Get();

					FToolMenuSection& ToolsSection = InMenu->AddSection("Tools");
					ToolsSection.AddEntry(This->BuildButtonToolkit->MakeToolbarButton(Commands.Build));
					ToolsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.FindInCameraRig));
				}),
				InsertAfterAssetSection);

		Impl->BuildToolbarMenu(ToolbarMenu);
	}
	
	TSharedRef<IGameplayCamerasFamily> Family = IGameplayCamerasFamily::CreateFamily(Impl->GetCameraRigAsset()).ToSharedRef();
	TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
	AddToolbarExtender(ToolbarExtender);
	ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this, Family](FToolBarBuilder& Builder)
				{
					AddToolbarWidget(SNew(SCameraFamilyShortcutBar, SharedThis(this), Family));
				})
			);
}

void FCameraRigAssetEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(MenuContext);

	UCameraRigAssetEditorMenuContext* Context = NewObject<UCameraRigAssetEditorMenuContext>();
	Context->Toolkit = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FCameraRigAssetEditorToolkit::PostInitAssetEditor()
{
	Impl->BindCommands(ToolkitCommands);

	const FCameraRigAssetEditorCommands& Commands = FCameraRigAssetEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Build,
		FExecuteAction::CreateSP(this, &FCameraRigAssetEditorToolkit::OnBuild));

	ToolkitCommands->MapAction(
		Commands.FindInCameraRig,
		FExecuteAction::CreateSP(this, &FCameraRigAssetEditorToolkit::OnFindInCameraRig));

	BuildLogToolkit->OnRequestJumpToObject().BindSPLambda(this, [this](UObject* Object)
		{
			TSharedPtr<SCameraRigAssetEditor> CameraRigEditorWidget = Impl->GetCameraRigAssetEditor();
			CameraRigEditorWidget->FindAndJumpToObjectNode(Object);
		});

	ToolkitCommands->MapAction(
			Commands.RenameInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersToolkit->RenameSelectedParameter();
				}));

	ToolkitCommands->MapAction(
			Commands.DeleteInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersToolkit->DeleteSelectedParameter();
				}));

	IGameplayCamerasModule& GameplayCamerasModule = FModuleManager::GetModuleChecked<IGameplayCamerasModule>("GameplayCameras");
	LiveEditManager = GameplayCamerasModule.GetLiveEditManager();

	Impl->SetLiveEditManager(LiveEditManager);

	RegenerateMenusAndToolbars();

	if (UCameraRigAsset* CameraRig = Impl->GetCameraRigAsset())
	{
		TArray<UObject*> CameraRigObjects;
		UPackage* CameraRigPackage = CameraRig->GetPackage();
		GetObjectsWithPackage(CameraRigPackage, CameraRigObjects);
		CurveEditorToolkit->AddCurveOwners(CameraRigObjects);

		CameraRig->EventHandlers.Register(EventHandler, this);
	}

	FRichCurveDetailsCustomization::OnInvokeCurveEditor().AddSP(this, &FCameraRigAssetEditorToolkit::OnInvokeCurveEditor);

	bIsInitialized = true;
}

void FCameraRigAssetEditorToolkit::PostRegenerateMenusAndToolbars()
{
	SetMenuOverlay(FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UCameraRigAsset::StaticClass()));
}

void FCameraRigAssetEditorToolkit::OnCameraObjectInterfaceParameterSelected(UCameraObjectInterfaceParameterBase* Object)
{
	DetailsView->SetObject(Object);
}

void FCameraRigAssetEditorToolkit::OnSearchCameraObjectInterfaceParameterNodes(UCameraObjectInterfaceParameterBase* Object)
{
	TabManager->TryInvokeTab(SearchTabId);

	SearchWidget->Search(Object->InterfaceParameterName);
}

void FCameraRigAssetEditorToolkit::OnBuild()
{
	UCameraRigAsset* CameraRigAsset = Impl->GetCameraRigAsset();
	if (!CameraRigAsset)
	{
		return;
	}

	FCameraBuildLog BuildLog;
	FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::UserAction);
	FCameraRigAssetBuilder Builder(BuildContext);
	Builder.BuildCameraRig(CameraRigAsset);

	BuildLogToolkit->PopulateMessageListing(BuildLog);

	if (CameraRigAsset->BuildStatus != ECameraBuildStatus::Clean)
	{
		TabManager->TryInvokeTab(MessagesTabId);
	}

	DetailsView->RequestForceRefresh();

	FCameraRigPackages BuiltPackages;
	CameraRigAsset->GatherPackages(BuiltPackages);

	for (const UPackage* BuiltPackage : BuiltPackages)
	{
		LiveEditManager->NotifyPostBuildAsset(BuiltPackage);
	}

	if (FBlueprintActionDatabase* Database = FBlueprintActionDatabase::TryGet())
	{
		Database->RefreshAssetActions(CameraRigAsset);
	}
}

void FCameraRigAssetEditorToolkit::OnFindInCameraRig()
{
	TabManager->TryInvokeTab(SearchTabId);
	SearchWidget->FocusSearchEditBox();
}

void FCameraRigAssetEditorToolkit::OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources)
{
	TArray<UEdGraph*> Graphs;
	TSharedPtr<SCameraRigAssetEditor> CameraRigEditor = Impl->GetCameraRigAssetEditor();
	CameraRigEditor->GetGraphs(Graphs);
	for (UEdGraph* Graph : Graphs)
	{
		OutSources.Add(FFindInObjectTreeGraphSource{ Graph });
	}
}

void FCameraRigAssetEditorToolkit::OnJumpToNode(UEdGraphNode* Node, FName PinName)
{
	TSharedPtr<SCameraRigAssetEditor> CameraRigEditor = Impl->GetCameraRigAssetEditor();
	CameraRigEditor->JumpToNode(Node, PinName);
}

void FCameraRigAssetEditorToolkit::OnInvokeCurveEditor(UObject* Object, FName PropertyName)
{
	UPackage* Package = Impl->GetCameraRigAsset()->GetPackage();
	if (Object->IsIn(Package))
	{
		TabManager->TryInvokeTab(CurvesTabId);

		CurveEditorToolkit->SelectCurves(Object, PropertyName);
	}
}

void FCameraRigAssetEditorToolkit::OnObjectAddedToGraph(const FName GraphName, UObject* Object)
{
	ECameraRigAssetEditorMode CurrentMode = Impl->GetCameraRigEditorMode();
	if ((GraphName == UCameraRigAsset::NodeTreeGraphName && CurrentMode == ECameraRigAssetEditorMode::NodeGraph) ||
			(GraphName == UCameraRigAsset::TransitionsGraphName && CurrentMode == ECameraRigAssetEditorMode::TransitionGraph))
	{
		CurveEditorToolkit->AddCurveOwner(Object);
	}
}

void FCameraRigAssetEditorToolkit::OnObjectRemovedFromGraph(const FName GraphName, UObject* Object)
{
	ECameraRigAssetEditorMode CurrentMode = Impl->GetCameraRigEditorMode();
	if ((GraphName == UCameraRigAsset::NodeTreeGraphName && CurrentMode == ECameraRigAssetEditorMode::NodeGraph) ||
			(GraphName == UCameraRigAsset::TransitionsGraphName && CurrentMode == ECameraRigAssetEditorMode::TransitionGraph))
	{
		CurveEditorToolkit->RemoveCurveOwner(Object);
	}
}

FText FCameraRigAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Rig Asset");
}

FName FCameraRigAssetEditorToolkit::GetToolkitFName() const
{
	static FName ToolkitName("CameraRigAssetEditor");
	return ToolkitName;
}

FString FCameraRigAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Rig Asset ").ToString();
}

FLinearColor FCameraRigAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

