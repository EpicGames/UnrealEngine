// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraRigAssetEditorToolkitBase.h"

#include "Commands/CameraRigAssetEditorCommands.h"
#include "Core/CameraNode.h"
#include "Core/CameraRigAsset.h"
#include "Editors/SCameraRigAssetEditor.h"
#include "Editors/SObjectTreeGraphToolbox.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "IGameplayCamerasLiveEditManager.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenu.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "CameraRigAssetEditorToolkitBase"

namespace UE::Cameras
{

const FName FCameraRigAssetEditorToolkitBase::ToolboxTabId(TEXT("CameraRigAssetEditor_Toolbox"));
const FName FCameraRigAssetEditorToolkitBase::CameraRigEditorTabId(TEXT("CameraRigAssetEditor_CameraRigEditor"));
const FName FCameraRigAssetEditorToolkitBase::DetailsViewTabId(TEXT("CameraRigAssetEditor_DetailsView"));

FCameraRigAssetEditorToolkitBase::FCameraRigAssetEditorToolkitBase(FName InLayoutName)
	: StandardLayout(new FStandardToolkitLayout(InLayoutName))
{
	StandardLayout->AddLeftTab(ToolboxTabId);
	StandardLayout->AddCenterTab(CameraRigEditorTabId);
	StandardLayout->AddRightTab(DetailsViewTabId);
}

FCameraRigAssetEditorToolkitBase::~FCameraRigAssetEditorToolkitBase()
{
	if (CameraRigEditorWidget)
	{
		CameraRigEditorWidget->RemoveOnAnyGraphChanged(this);
	}
}

void FCameraRigAssetEditorToolkitBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraRigAsset);
}

FString FCameraRigAssetEditorToolkitBase::GetReferencerName() const
{
	return TEXT("FCameraRigAssetEditorToolkitBase");
}

void FCameraRigAssetEditorToolkitBase::SetCameraRigAsset(UCameraRigAsset* InCameraRig)
{
	CameraRigAsset = InCameraRig;

	if (CameraRigEditorWidget)
	{
		CameraRigEditorWidget->SetCameraRigAsset(InCameraRig);
	}
}

void FCameraRigAssetEditorToolkitBase::RegisterTabSpawners(TSharedRef<FTabManager> InTabManager, TSharedPtr<FWorkspaceItem> InAssetEditorTabsCategory)
{
	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(ToolboxTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkitBase::SpawnTab_Toolbox))
		.SetDisplayName(LOCTEXT("Toolbox", "Toolbox"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.Toolbox"));

	InTabManager->RegisterTabSpawner(CameraRigEditorTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkitBase::SpawnTab_CameraRigEditor))
		.SetDisplayName(LOCTEXT("CameraRigEditor", "Camera Rig"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(DetailsViewTabId, FOnSpawnTab::CreateSP(this, &FCameraRigAssetEditorToolkitBase::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkitBase::SpawnTab_Toolbox(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> ToolboxTab = SNew(SDockTab)
		.Label(LOCTEXT("ToolboxTabTitle", "Toolbox"))
		[
			ToolboxWidget.ToSharedRef()
		];

	return ToolboxTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkitBase::SpawnTab_CameraRigEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> CameraRigEditorTab = SNew(SDockTab)
		.Label(LOCTEXT("CameraRigEditorTabTitle", "Camera Rig Editor"))
		[
			CameraRigEditorWidget.ToSharedRef()
		];

	return CameraRigEditorTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigAssetEditorToolkitBase::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label(LOCTEXT("DetailsTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DetailsTab.ToSharedRef();
}

void FCameraRigAssetEditorToolkitBase::UnregisterTabSpawners(TSharedRef<FTabManager> InTabManager)
{
	InTabManager->UnregisterTabSpawner(ToolboxTabId);
	InTabManager->UnregisterTabSpawner(CameraRigEditorTabId);
	InTabManager->UnregisterTabSpawner(DetailsViewTabId);
}

void FCameraRigAssetEditorToolkitBase::CreateWidgets()
{
	// Create the details view.
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// Create the camera rig editor.
	CameraRigEditorWidget = SNew(SCameraRigAssetEditor)
		.DetailsView(DetailsView)
		.CameraRigAsset(CameraRigAsset);
	CameraRigEditorWidget->AddOnAnyGraphChanged(FOnGraphChanged::FDelegate::CreateSP(
				this, &FCameraRigAssetEditorToolkitBase::OnAnyGraphChanged));

	// Create the toolbox, default to the rig editor items.
	ToolboxWidget = SNew(SObjectTreeGraphToolbox)
		.GraphConfig(CameraRigEditorWidget->GetFocusedGraphConfig());
}

void FCameraRigAssetEditorToolkitBase::BuildToolbarMenu(UToolMenu* ToolbarMenu)
{
	FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
	const FCameraRigAssetEditorCommands& Commands = FCameraRigAssetEditorCommands::Get();

	FToolMenuSection& GraphsSection = ToolbarMenu->AddSection("Graphs", TAttribute<FText>(), InsertAfterAssetSection);

	GraphsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.FocusHome));
	GraphsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ShowNodeHierarchy));
	GraphsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ShowTransitions));
}

void FCameraRigAssetEditorToolkitBase::BindCommands(TSharedRef<FUICommandList> CommandList)
{
	const FCameraRigAssetEditorCommands& Commands = FCameraRigAssetEditorCommands::Get();

	TSharedRef<SCameraRigAssetEditor> CameraRigEditor = CameraRigEditorWidget.ToSharedRef();

	CommandList->MapAction(
		Commands.FocusHome,
		FExecuteAction::CreateSP(CameraRigEditor, &SCameraRigAssetEditor::FocusHome));

	CommandList->MapAction(
		Commands.ShowNodeHierarchy,
		FExecuteAction::CreateSP(this, &FCameraRigAssetEditorToolkitBase::SetCameraRigEditorMode, ECameraRigAssetEditorMode::NodeGraph),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCameraRigAssetEditorToolkitBase::IsCameraRigEditorMode, ECameraRigAssetEditorMode::NodeGraph));

	CommandList->MapAction(
		Commands.ShowTransitions,
		FExecuteAction::CreateSP(this, &FCameraRigAssetEditorToolkitBase::SetCameraRigEditorMode, ECameraRigAssetEditorMode::TransitionGraph),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCameraRigAssetEditorToolkitBase::IsCameraRigEditorMode, ECameraRigAssetEditorMode::TransitionGraph));
}

void FCameraRigAssetEditorToolkitBase::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// Called when something is modified in the details view.
	if (CameraRigAsset)
	{
		CameraRigAsset->BuildStatus = ECameraBuildStatus::Dirty;
	}

	if (LiveEditManager && PropertyChangedEvent.GetNumObjectsBeingEdited() > 0)
	{
		const UCameraNode* EditedCameraNode = Cast<UCameraNode>(PropertyChangedEvent.GetObjectBeingEdited(0));
		if (EditedCameraNode)
		{
			LiveEditManager->NotifyPostEditChangeProperty(EditedCameraNode, PropertyChangedEvent);
		}
	}
}

void FCameraRigAssetEditorToolkitBase::SetCameraRigEditorMode(ECameraRigAssetEditorMode InEditorMode)
{
	CameraRigEditorWidget->SetEditorMode(InEditorMode);
	ToolboxWidget->SetGraphConfig(CameraRigEditorWidget->GetFocusedGraphConfig());
}

bool FCameraRigAssetEditorToolkitBase::IsCameraRigEditorMode(ECameraRigAssetEditorMode InEditorMode) const
{
	return CameraRigEditorWidget->IsEditorMode(InEditorMode);
}

ECameraRigAssetEditorMode FCameraRigAssetEditorToolkitBase::GetCameraRigEditorMode() const
{
	return CameraRigEditorWidget->GetEditorMode();
}

void FCameraRigAssetEditorToolkitBase::OnAnyGraphChanged(const FEdGraphEditAction& InEditAction)
{
	// Called when something is modified in the node graph or transition graph.
	if (CameraRigAsset)
	{
		CameraRigAsset->BuildStatus = ECameraBuildStatus::Dirty;
	}
}

void FCameraRigAssetEditorToolkitBase::SetLiveEditManager(TSharedPtr<IGameplayCamerasLiveEditManager> InLiveEditManager)
{
	LiveEditManager = InLiveEditManager;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

