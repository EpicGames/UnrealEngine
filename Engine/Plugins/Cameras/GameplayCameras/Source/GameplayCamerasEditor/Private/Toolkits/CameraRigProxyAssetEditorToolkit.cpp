// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraRigProxyAssetEditorToolkit.h"

#include "AssetTools/CameraRigProxyAssetEditor.h"
#include "Core/CameraRigProxyAsset.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Helpers/AssetTypeMenuOverlayHelper.h"
#include "IGameplayCamerasFamily.h"
#include "PropertyEditorModule.h"
#include "Widgets/SCameraFamilyShortcutBar.h"

#define LOCTEXT_NAMESPACE "CameraRigProxyAssetEditorToolkit"

namespace UE::Cameras
{

const FName FCameraRigProxyAssetEditorToolkit::DetailsViewTabId(TEXT("CameraRigProxyAssetEditor_DetailsView"));

FCameraRigProxyAssetEditorToolkit::FCameraRigProxyAssetEditorToolkit(UCameraRigProxyAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	// Override base class default layout.
	StandaloneDefaultLayout = FTabManager::NewLayout("CameraRigProxyAssetEditor_Layout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(DetailsViewTabId, ETabState::OpenedTab)
				->SetForegroundTab(DetailsViewTabId)
			)
		);
}

FCameraRigProxyAssetEditorToolkit::~FCameraRigProxyAssetEditorToolkit()
{
}

void FCameraRigProxyAssetEditorToolkit::SetCameraRigProxyAsset(UCameraRigProxyAsset* InCameraRigProxyAsset)
{
	CameraRigProxyAsset = InCameraRigProxyAsset;
}

void FCameraRigProxyAssetEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraRigProxyAsset);
}

FString FCameraRigProxyAssetEditorToolkit::GetReferencerName() const
{
	return TEXT("FCameraRigProxyAssetEditorToolkit");
}

void FCameraRigProxyAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(DetailsViewTabId, FOnSpawnTab::CreateSP(this, &FCameraRigProxyAssetEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FCameraRigProxyAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(DetailsViewTabId);
}

void FCameraRigProxyAssetEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	// ...no up-call...

	// Do most of FBaseAssetToolkit's work except for the viewport.
	RegisterToolbar();
	LayoutExtender = MakeShared<FLayoutExtender>();

	DetailsView = CreateDetailsView();
}

void FCameraRigProxyAssetEditorToolkit::RegisterToolbar()
{
	TSharedRef<IGameplayCamerasFamily> Family = IGameplayCamerasFamily::CreateFamily(CameraRigProxyAsset).ToSharedRef();
	TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
	AddToolbarExtender(ToolbarExtender);
	ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this, Family](FToolBarBuilder& Builder)
				{
					AddToolbarWidget(SNew(SCameraFamilyShortcutBar, SharedThis(this), Family));
				})
			);
}

void FCameraRigProxyAssetEditorToolkit::PostInitAssetEditor()
{
	RegenerateMenusAndToolbars();
}

void FCameraRigProxyAssetEditorToolkit::PostRegenerateMenusAndToolbars()
{
	SetMenuOverlay(FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UCameraRigProxyAsset::StaticClass()));
}

FText FCameraRigProxyAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Rig Proxy Asset");
}

FName FCameraRigProxyAssetEditorToolkit::GetToolkitFName() const
{
	static FName SequencerName("CameraRigProxyAssetEditor");
	return SequencerName;
}

FString FCameraRigProxyAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Rig Proxy Asset ").ToString();
}

FLinearColor FCameraRigProxyAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

