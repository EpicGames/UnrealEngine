// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraRigTransitionEditorToolkit.h"

#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "ToolMenus.h"
#include "Toolkits/CameraRigTransitionEditorToolkitBase.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "CameraRigTransitionEditorToolkit"

namespace UE::Cameras
{

namespace Internal
{

class FCameraRigTransitionEditorToolkitBaseImpl : public FCameraRigTransitionEditorToolkitBase
{
public:
	
	FCameraRigTransitionEditorToolkitBaseImpl()
		: FCameraRigTransitionEditorToolkitBase(TEXT("CameraRigTransitionEditor_Layout_v2"))
	{}
};

}  // namespace Internal

FCameraRigTransitionEditorToolkit::FCameraRigTransitionEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	Impl = MakeShared<Internal::FCameraRigTransitionEditorToolkitBaseImpl>();

	// Override base class default layout.
	StandaloneDefaultLayout = Impl->GetStandardLayout()->GetLayout();
}

void FCameraRigTransitionEditorToolkit::SetTransitionOwner(UObject* InTransitionOwner)
{
	Impl->SetTransitionOwner(InTransitionOwner);
}

void FCameraRigTransitionEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	Impl->RegisterTabSpawners(InTabManager, AssetEditorTabsCategory);
}

void FCameraRigTransitionEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	Impl->UnregisterTabSpawners(InTabManager);
}

void FCameraRigTransitionEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab, and our base toolkit class
	// already has its own details view.
	// ...no up-call...

	RegisterToolbar();
	CreateEditorModeManager();
	LayoutExtender = MakeShared<FLayoutExtender>();
	
	// Now do our custom stuff.

	Impl->CreateWidgets();
}

void FCameraRigTransitionEditorToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(
				MenuName, ParentName, EMultiBoxType::ToolBar);
		Impl->BuildToolbarMenu(ToolbarMenu);
	}
}

FText FCameraRigTransitionEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Transitions");
}

FName FCameraRigTransitionEditorToolkit::GetToolkitFName() const
{
	static FName ToolkitName("CameraRigTransitionEditor");
	return ToolkitName;
}

FString FCameraRigTransitionEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Transitions ").ToString();
}

FLinearColor FCameraRigTransitionEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.1f, 0.8f, 0.2f, 0.5f);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

