// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraRigTransitionEditorToolkitBase.h"

#include "Build/CameraBuildStatus.h"
#include "Commands/CameraRigTransitionEditorCommands.h"
#include "Editors/CameraRigTransitionGraphSchemaBase.h"
#include "Editors/SCameraRigTransitionEditor.h"
#include "Editors/SObjectTreeGraphToolbox.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenu.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "CameraRigTransitionEditorToolkitBase"

namespace UE::Cameras
{

const FName FCameraRigTransitionEditorToolkitBase::ToolboxTabId(TEXT("CameraRigTransitionEditor_Toolbox"));
const FName FCameraRigTransitionEditorToolkitBase::TransitionEditorTabId(TEXT("CameraRigTransitionEditor_TransitionEditor"));
const FName FCameraRigTransitionEditorToolkitBase::DetailsViewTabId(TEXT("CameraRigTransitionEditor_DetailsView"));

FCameraRigTransitionEditorToolkitBase::FCameraRigTransitionEditorToolkitBase(FName InLayoutName)
	: StandardLayout(new FStandardToolkitLayout(InLayoutName))
{
	StandardLayout->AddLeftTab(ToolboxTabId);
	StandardLayout->AddCenterTab(TransitionEditorTabId);
	StandardLayout->AddRightTab(DetailsViewTabId);
}

FCameraRigTransitionEditorToolkitBase::~FCameraRigTransitionEditorToolkitBase()
{
	if (TransitionEditorWidget)
	{
		TransitionEditorWidget->RemoveOnGraphChanged(this);
	}
}

void FCameraRigTransitionEditorToolkitBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TransitionOwner);
}

FString FCameraRigTransitionEditorToolkitBase::GetReferencerName() const
{
	return TEXT("FCameraRigTransitionEditorToolkitBase");
}

void FCameraRigTransitionEditorToolkitBase::SetTransitionOwner(UObject* InTransitionOwner)
{
	TransitionOwner = InTransitionOwner;

	if (TransitionEditorWidget)
	{
		TransitionEditorWidget->SetTransitionOwner(InTransitionOwner);
	}
}

void FCameraRigTransitionEditorToolkitBase::RegisterTabSpawners(TSharedRef<FTabManager> InTabManager, TSharedPtr<FWorkspaceItem> InAssetEditorTabsCategory)
{
	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(ToolboxTabId, FOnSpawnTab::CreateSP(this, &FCameraRigTransitionEditorToolkitBase::SpawnTab_Toolbox))
		.SetDisplayName(LOCTEXT("Toolbox", "Toolbox"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraRigAssetEditor.Tabs.Toolbox"));

	InTabManager->RegisterTabSpawner(TransitionEditorTabId, FOnSpawnTab::CreateSP(this, &FCameraRigTransitionEditorToolkitBase::SpawnTab_TransitionEditor))
		.SetDisplayName(LOCTEXT("TransitionEditor", "Camera Transitions"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(DetailsViewTabId, FOnSpawnTab::CreateSP(this, &FCameraRigTransitionEditorToolkitBase::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

TSharedRef<SDockTab> FCameraRigTransitionEditorToolkitBase::SpawnTab_Toolbox(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> ToolboxTab = SNew(SDockTab)
		.Label(LOCTEXT("ToolboxTabTitle", "Toolbox"))
		[
			ToolboxWidget.ToSharedRef()
		];

	return ToolboxTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigTransitionEditorToolkitBase::SpawnTab_TransitionEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> TransitionEditorTab = SNew(SDockTab)
		.Label(LOCTEXT("TransitionEditorTabTitle", "Camera Transitions"))
		[
			TransitionEditorWidget.ToSharedRef()
		];

	return TransitionEditorTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraRigTransitionEditorToolkitBase::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label(LOCTEXT("BaseDetailsTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DetailsTab.ToSharedRef();
}

void FCameraRigTransitionEditorToolkitBase::UnregisterTabSpawners(TSharedRef<FTabManager> InTabManager)
{
	InTabManager->UnregisterTabSpawner(ToolboxTabId);
	InTabManager->UnregisterTabSpawner(TransitionEditorTabId);
	InTabManager->UnregisterTabSpawner(DetailsViewTabId);
}

void FCameraRigTransitionEditorToolkitBase::CreateWidgets()
{
	// Create the details view.
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TSubclassOf<UCameraRigTransitionGraphSchemaBase> SchemaClass = GetTransitionGraphSchemaClass();

	FGraphAppearanceInfo GraphAppearanceInfo;
	GetTransitionGraphAppearanceInfo(GraphAppearanceInfo);

	// Create the transition editor.
	TransitionEditorWidget = SNew(SCameraRigTransitionEditor)
		.DetailsView(DetailsView)
		.TransitionOwner(TransitionOwner)
		.TransitionGraphSchemaClass(SchemaClass)
		.TransitionGraphEditorAppearance(GraphAppearanceInfo);
	TransitionEditorWidget->AddOnGraphChanged(FOnGraphChanged::FDelegate::CreateSP(
				this, &FCameraRigTransitionEditorToolkitBase::OnTransitionGraphChanged));

	// Create the toolbox, default to the rig editor items.
	ToolboxWidget = SNew(SObjectTreeGraphToolbox)
		.GraphConfig(TransitionEditorWidget->GetTransitionGraphConfig());
}

TSubclassOf<UCameraRigTransitionGraphSchemaBase> FCameraRigTransitionEditorToolkitBase::GetTransitionGraphSchemaClass()
{
	return UCameraRigTransitionGraphSchemaBase::StaticClass();
}

void FCameraRigTransitionEditorToolkitBase::BuildToolbarMenu(UToolMenu* ToolbarMenu)
{
	FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
	const FCameraRigTransitionEditorCommands& Commands = FCameraRigTransitionEditorCommands::Get();
	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();
}

void FCameraRigTransitionEditorToolkitBase::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// Called when something is modified in the details view.
	if (IHasCameraBuildStatus* Buildable = Cast<IHasCameraBuildStatus>(TransitionOwner))
	{
		Buildable->DirtyBuildStatus();
	}
}

void FCameraRigTransitionEditorToolkitBase::OnTransitionGraphChanged(const FEdGraphEditAction& InEditAction)
{
	// Called when something is modified in the transition graph.
	if (IHasCameraBuildStatus* Buildable = Cast<IHasCameraBuildStatus>(TransitionOwner))
	{
		Buildable->DirtyBuildStatus();
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

