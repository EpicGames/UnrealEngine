// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraShakeAssetEditorToolkit.h"

#include "Build/CameraBuildLog.h"
#include "Build/CameraShakeAssetBuilder.h"
#include "Commands/CameraShakeAssetEditorCommands.h"
#include "Core/CameraShakeAsset.h"
#include "Editors/CameraShakeCameraNodeGraphSchema.h"
#include "Editors/ObjectTreeGraph.h"
#include "Editors/ObjectTreeGraphNode.h"
#include "Editors/SCameraNodeGraphEditor.h"
#include "Editors/SFindInObjectTreeGraph.h"
#include "Editors/SObjectTreeGraphEditor.h"
#include "Editors/SObjectTreeGraphToolbox.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Helpers/AssetTypeMenuOverlayHelper.h"
#include "IGameplayCamerasEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenus.h"
#include "Toolkits/BuildButtonToolkit.h"
#include "Toolkits/CameraBuildLogToolkit.h"
#include "Toolkits/CameraObjectInterfaceParametersToolkit.h"
#include "Toolkits/StandardToolkitLayout.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeAssetEditorToolkit)

#define LOCTEXT_NAMESPACE "CameraShakeAssetEditorToolkit"

namespace UE::Cameras
{

const FName FCameraShakeAssetEditorToolkit::CameraShakeEditorTabId(TEXT("CameraShakeAssetEditor_GraphEditor"));
const FName FCameraShakeAssetEditorToolkit::DetailsViewTabId(TEXT("CameraShakeAssetEditor_DetailsView"));
const FName FCameraShakeAssetEditorToolkit::SearchTabId(TEXT("CameraShakeAssetEditor_Search"));
const FName FCameraShakeAssetEditorToolkit::MessagesTabId(TEXT("CameraShakeAssetEditor_Messages"));
const FName FCameraShakeAssetEditorToolkit::ToolboxTabId(TEXT("CameraShakeAssetEditor_Toolbox"));
const FName FCameraShakeAssetEditorToolkit::InterfaceParametersTabId(TEXT("CameraShakeAssetEditor_InterfaceParameters"));

FCameraShakeAssetEditorToolkit::FCameraShakeAssetEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	BuildButtonToolkit = MakeShared<FBuildButtonToolkit>();
	BuildLogToolkit = MakeShared<FCameraBuildLogToolkit>();
	InterfaceParametersToolkit = MakeShared<FCameraObjectInterfaceParametersToolkit>();

	StandardLayout = MakeShared<FStandardToolkitLayout>(TEXT("CameraShakeAssetEditor_Layout_v1"));
	{
		StandardLayout->AddCenterTab(CameraShakeEditorTabId);

		StandardLayout->AddRightTab(DetailsViewTabId);

		StandardLayout->AddBottomTab(SearchTabId);
		StandardLayout->AddBottomTab(MessagesTabId);

		StandardLayout->AddLeftTab(ToolboxTabId);
		StandardLayout->AddLeftTab(InterfaceParametersTabId, ETabState::OpenedTab);
	}
	StandaloneDefaultLayout = StandardLayout->GetLayout();
}

FCameraShakeAssetEditorToolkit::~FCameraShakeAssetEditorToolkit()
{
	if (!GExitPurge)
	{
		DiscardNodeGraphEditor();
	}
}

void FCameraShakeAssetEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CameraShakeAsset);
}

FString FCameraShakeAssetEditorToolkit::GetReferencerName() const
{
	return TEXT("FCameraRigAssetEditorToolkitBase");
}

void FCameraShakeAssetEditorToolkit::SetCameraShakeAsset(UCameraShakeAsset* InCameraShake)
{
	CameraShakeAsset = InCameraShake;

	BuildButtonToolkit->SetTarget(InCameraShake);
	InterfaceParametersToolkit->SetCameraObject(InCameraShake);
}

FText FCameraShakeAssetEditorToolkit::GetCameraShakeAssetName() const
{
	if (CameraShakeAsset)
	{
		return NodeGraphConfig.GetDisplayNameText(CameraShakeAsset);
	}
	return LOCTEXT("NoCameraShake", "No Camera Shake");
}

bool FCameraShakeAssetEditorToolkit::IsGraphEditorEnabled() const
{
	return CameraShakeAsset != nullptr;
}

void FCameraShakeAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(ToolboxTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_Toolbox))
		.SetDisplayName(LOCTEXT("Toolbox", "Toolbox"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraShakeAssetEditor.Tabs.Toolbox"));

	InTabManager->RegisterTabSpawner(CameraShakeEditorTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_CameraShakeEditor))
		.SetDisplayName(LOCTEXT("CameraShakeEditor", "Camera Shake"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(DetailsViewTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(SearchTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_Search))
		.SetDisplayName(LOCTEXT("Search", "Search"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraShakeAssetEditor.Tabs.Search"));

	InTabManager->RegisterTabSpawner(MessagesTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_Messages))
		.SetDisplayName(LOCTEXT("Messages", "Messages"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraShakeAssetEditor.Tabs.Messages"));

	InTabManager->RegisterTabSpawner(InterfaceParametersTabId, FOnSpawnTab::CreateSP(this, &FCameraShakeAssetEditorToolkit::SpawnTab_InterfaceParameters))
		.SetDisplayName(LOCTEXT("InterfaceParameters", "InterfaceParameters"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(CamerasStyleSetName, "CameraShakeAssetEditor.Tabs.InterfaceParameters"));
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_CameraShakeEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> CameraShakeEditorTab = SNew(SDockTab)
		.Label(LOCTEXT("CameraShakeEditorTabTitle", "Camera Shake Editor"))
		[
			NodeGraphEditor.ToSharedRef()
		];

	return CameraShakeEditorTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label(LOCTEXT("DetailsTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DetailsTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_Search(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> SearchTab = SNew(SDockTab)
		.Label(LOCTEXT("SearchTabTitle", "Search"))
		[
			SearchWidget.ToSharedRef()
		];

	return SearchTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_Messages(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> MessagesTab = SNew(SDockTab)
		.Label(LOCTEXT("MessagesTabTitle", "Messages"))
		[
			BuildLogToolkit->GetMessagesWidget().ToSharedRef()
		];

	return MessagesTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_Toolbox(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> ToolboxTab = SNew(SDockTab)
		.Label(LOCTEXT("ToolboxTabTitle", "Toolbox"))
		[
			ToolboxWidget.ToSharedRef()
		];

	return ToolboxTab.ToSharedRef();
}

TSharedRef<SDockTab> FCameraShakeAssetEditorToolkit::SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> InterfaceParametersTab = SNew(SDockTab)
		.Label(LOCTEXT("InterfaceParametersTabTitle", "Parameters"))
		[
			InterfaceParametersToolkit->GetInterfaceParametersPanel().ToSharedRef()
		];

	return InterfaceParametersTab.ToSharedRef();
}

void FCameraShakeAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	
	InTabManager->UnregisterTabSpawner(CameraShakeEditorTabId);
	InTabManager->UnregisterTabSpawner(DetailsViewTabId);
	InTabManager->UnregisterTabSpawner(SearchTabId);
	InTabManager->UnregisterTabSpawner(MessagesTabId);
	InTabManager->UnregisterTabSpawner(ToolboxTabId);
	InTabManager->UnregisterTabSpawner(InterfaceParametersTabId);
}

void FCameraShakeAssetEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab, and our base class
	// has its own details view in order to get a notify hook.
	// ...no up-call...

	RegisterToolbar();
	CreateEditorModeManager();
	LayoutExtender = MakeShared<FLayoutExtender>();

	// Now do our custom stuff.

	// Create the details view.
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// Create the camera rig editor.
	CreateNodeGraphEditor();

	// Create the toolbox, default to the rig editor items.
	ToolboxWidget = SNew(SObjectTreeGraphToolbox)
		.GraphConfig(NodeGraphConfig);

	// Create the search panel.
	SearchWidget = SNew(SFindInObjectTreeGraph)
		.OnGetGraphsToSearch(this, &FCameraShakeAssetEditorToolkit::OnGetGraphsToSearch)
		.OnJumpToNodeRequested(this, &FCameraShakeAssetEditorToolkit::OnJumpToNode);

	// Create the message log.
	BuildLogToolkit->Initialize("CameraShakeAssetBuildMessages");

	// Hook-up the selection of interface parameters.
	InterfaceParametersToolkit->OnInterfaceParameterSelected().AddSP(this, &FCameraShakeAssetEditorToolkit::OnCameraObjectInterfaceParameterSelected);
}

void FCameraShakeAssetEditorToolkit::CreateNodeGraphEditor()
{
	UClass* SchemaClass = UCameraShakeCameraNodeGraphSchema::StaticClass();
	UCameraShakeCameraNodeGraphSchema* DefaultSchemaObject = Cast<UCameraShakeCameraNodeGraphSchema>(SchemaClass->GetDefaultObject());
	NodeGraphConfig = DefaultSchemaObject->BuildGraphConfig();

	NodeGraph = NewObject<UObjectTreeGraph>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Standalone);
	NodeGraph->Schema = SchemaClass;
	NodeGraph->Reset(CameraShakeAsset, NodeGraphConfig);

	FGraphAppearanceInfo Appearance;
	Appearance.CornerText = LOCTEXT("CameraShakeGraphText", "CAMERA SHAKE");

	NodeGraphEditor = SNew(SCameraNodeGraphEditor)
		.Appearance(Appearance)
		.DetailsView(DetailsView)
		.GraphTitle(this, &FCameraShakeAssetEditorToolkit::GetCameraShakeAssetName)
		.IsEnabled(this, &FCameraShakeAssetEditorToolkit::IsGraphEditorEnabled)
		.GraphToEdit(NodeGraph)
		.AssetEditorToolkit(SharedThis(this));
	NodeGraphEditor->RegisterEditor();
}

void FCameraShakeAssetEditorToolkit::DiscardNodeGraphEditor()
{
	NodeGraphEditor->UnregisterEditor();
}

void FCameraShakeAssetEditorToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);
		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);

		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(
				MenuName, ParentName, EMultiBoxType::ToolBar);

		ToolbarMenu->AddDynamicSection("Tools", FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					UCameraShakeAssetEditorMenuContext* Context = InMenu->FindContext<UCameraShakeAssetEditorMenuContext>();
					FCameraShakeAssetEditorToolkit* This = Context ? Context->Toolkit.Pin().Get() : nullptr;
					if (!ensure(This))
					{
						return;
					}

					const FCameraShakeAssetEditorCommands& Commands = FCameraShakeAssetEditorCommands::Get();

					FToolMenuSection& ToolsSection = InMenu->AddSection("Tools");
					ToolsSection.AddEntry(This->BuildButtonToolkit->MakeToolbarButton(Commands.Build));
					ToolsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.FindInCameraShake));
				}),
				InsertAfterAssetSection);

		const FCameraShakeAssetEditorCommands& Commands = FCameraShakeAssetEditorCommands::Get();

		FToolMenuSection& GraphsSection = ToolbarMenu->AddSection("Graphs", TAttribute<FText>(), InsertAfterAssetSection);

		GraphsSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.FocusHome));
	}
}

void FCameraShakeAssetEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(MenuContext);

	UCameraShakeAssetEditorMenuContext* Context = NewObject<UCameraShakeAssetEditorMenuContext>();
	Context->Toolkit = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FCameraShakeAssetEditorToolkit::PostInitAssetEditor()
{
	const FCameraShakeAssetEditorCommands& Commands = FCameraShakeAssetEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Build,
		FExecuteAction::CreateSP(this, &FCameraShakeAssetEditorToolkit::OnBuild));

	ToolkitCommands->MapAction(
		Commands.FocusHome,
		FExecuteAction::CreateSP(this, &FCameraShakeAssetEditorToolkit::OnFocusHome));

	ToolkitCommands->MapAction(
		Commands.FindInCameraShake,
		FExecuteAction::CreateSP(this, &FCameraShakeAssetEditorToolkit::OnFindInCameraShake));

	BuildLogToolkit->OnRequestJumpToObject().BindSPLambda(this, [this](UObject* Object)
		{
			OnJumpToObject(Object, NAME_None);
		});

	ToolkitCommands->MapAction(
			Commands.RenameInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersToolkit->RenameSelectedParameter();
				}));

	ToolkitCommands->MapAction(
			Commands.DeleteInterfaceParameter,
			FExecuteAction::CreateSPLambda(this, [this]()
				{
					InterfaceParametersToolkit->DeleteSelectedParameter();
				}));

	RegenerateMenusAndToolbars();
}

void FCameraShakeAssetEditorToolkit::PostRegenerateMenusAndToolbars()
{
	SetMenuOverlay(FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UCameraShakeAsset::StaticClass()));
}

void FCameraShakeAssetEditorToolkit::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// Called when something is modified in the details view.
	if (CameraShakeAsset)
	{
		CameraShakeAsset->BuildStatus = ECameraBuildStatus::Dirty;
	}
}

void FCameraShakeAssetEditorToolkit::OnCameraObjectInterfaceParameterSelected(UCameraObjectInterfaceParameterBase* Object)
{
	OnJumpToObject(Object, NAME_None);
}

void FCameraShakeAssetEditorToolkit::OnBuild()
{
	if (!CameraShakeAsset)
	{
		return;
	}

	FCameraBuildLog BuildLog;
	FCameraBuildContext BuildContext(BuildLog, ECameraBuildReason::UserAction);
	FCameraShakeAssetBuilder Builder(BuildContext);
	Builder.BuildCameraShake(CameraShakeAsset);

	BuildLogToolkit->PopulateMessageListing(BuildLog);

	if (CameraShakeAsset->BuildStatus != ECameraBuildStatus::Clean)
	{
		TabManager->TryInvokeTab(MessagesTabId);
	}

	DetailsView->RequestForceRefresh();
}

void FCameraShakeAssetEditorToolkit::OnFindInCameraShake()
{
	TabManager->TryInvokeTab(SearchTabId);
	SearchWidget->FocusSearchEditBox();
}

void FCameraShakeAssetEditorToolkit::OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources)
{
	OutSources.Add(FFindInObjectTreeGraphSource{ NodeGraph });
}

void FCameraShakeAssetEditorToolkit::OnJumpToNode(UEdGraphNode* Node, FName PinName)
{
	NodeGraphEditor->JumpToNode(Node, PinName);
}

void FCameraShakeAssetEditorToolkit::OnJumpToObject(UObject* Object, FName PropertyName)
{
	NodeGraphEditor->FindAndJumpToObjectNode(Object, PropertyName);
}

void FCameraShakeAssetEditorToolkit::OnFocusHome()
{
	NodeGraphEditor->FindAndJumpToObjectNode(CameraShakeAsset);
}

FText FCameraShakeAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Shake Asset");
}

FName FCameraShakeAssetEditorToolkit::GetToolkitFName() const
{
	static FName ToolkitName("CameraShakeAssetEditor");
	return ToolkitName;
}

FString FCameraShakeAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Shake Asset ").ToString();
}

FLinearColor FCameraShakeAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

