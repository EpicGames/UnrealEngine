// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraSharedTransitionsAssetEditorMode.h"

#include "Core/CameraAsset.h"
#include "Editors/CameraSharedTransitionGraphSchema.h"
#include "Editors/SCameraRigTransitionEditor.h"
#include "Editors/SFindInObjectTreeGraph.h"
#include "ToolMenus.h"
#include "Toolkits/CameraRigTransitionEditorToolkitBase.h"
#include "Toolkits/StandardToolkitLayout.h"

#define LOCTEXT_NAMESPACE "CameraSharedTransitionsAssetEditorMode"

namespace UE::Cameras
{

namespace Internal
{

class FCameraSharedTransitionsAssetEditorModeBaseImpl : public FCameraRigTransitionEditorToolkitBase
{
public:

	FCameraSharedTransitionsAssetEditorModeBaseImpl()
		: FCameraRigTransitionEditorToolkitBase(TEXT("CameraAssetEditor_Mode_SharedTransitions_v1"))
	{}

protected:

	virtual TSubclassOf<UCameraRigTransitionGraphSchemaBase> GetTransitionGraphSchemaClass() override
	{
		return UCameraSharedTransitionGraphSchema::StaticClass();
	}

	virtual void GetTransitionGraphAppearanceInfo(FGraphAppearanceInfo& OutGraphAppearanceInfo) override
	{
		OutGraphAppearanceInfo.CornerText = LOCTEXT("SharedTransitionGraphCornerText", "SHARED TRANSITIONS");
	}
};

}  // namespace Internal

FName FCameraSharedTransitionsAssetEditorMode::ModeName(TEXT("SharedTransitions"));

FCameraSharedTransitionsAssetEditorMode::FCameraSharedTransitionsAssetEditorMode(UCameraAsset* InCameraAsset)
	: FAssetEditorMode(ModeName)
	, CameraAsset(InCameraAsset)
{
	Impl = MakeShared<Internal::FCameraSharedTransitionsAssetEditorModeBaseImpl>();

	Impl->SetTransitionOwner(CameraAsset);

	DefaultLayout = Impl->GetStandardLayout()->GetLayout();

	UClass* SharedTransitionSchemaClass = UCameraSharedTransitionGraphSchema::StaticClass();
	UCameraSharedTransitionGraphSchema* DefaultTransitionGraphSchema = Cast<UCameraSharedTransitionGraphSchema>(SharedTransitionSchemaClass->GetDefaultObject());
	TransitionGraphConfig = DefaultTransitionGraphSchema->BuildGraphConfig();
}

void FCameraSharedTransitionsAssetEditorMode::OnActivateMode(const FAssetEditorModeActivateParams& InParams)
{
	if (!bInitializedToolkit)
	{
		Impl->CreateWidgets();
		bInitializedToolkit = true;
	}

	Impl->RegisterTabSpawners(InParams.TabManager.ToSharedRef(), InParams.AssetEditorTabsCategory);

	FToolMenuOwnerScoped OwnerScoped(this);
	UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu(InParams.ToolbarMenuName);
	Impl->BuildToolbarMenu(ToolbarMenu);
}

void FCameraSharedTransitionsAssetEditorMode::OnDeactivateMode(const FAssetEditorModeDeactivateParams& InParams)
{
	Impl->UnregisterTabSpawners(InParams.TabManager.ToSharedRef());

	UToolMenus::UnregisterOwner(this);
}

void FCameraSharedTransitionsAssetEditorMode::OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources)
{
	UEdGraph* TransitionGraph = Impl->GetCameraRigTransitionEditor()->GetTransitionGraph();
	OutSources.Add(FFindInObjectTreeGraphSource{ TransitionGraph });
}

void FCameraSharedTransitionsAssetEditorMode::JumpToNode(UEdGraphNode* InNode, FName InPinName)
{
	if (TSharedPtr<SCameraRigTransitionEditor> TransitionEditor = Impl->GetCameraRigTransitionEditor())
	{
		TransitionEditor->JumpToNode(InNode, InPinName);
	}
}

bool FCameraSharedTransitionsAssetEditorMode::JumpToObject(UObject* InObject, FName InPropertyName)
{
	if (TSharedPtr<SCameraRigTransitionEditor> TransitionEditor = Impl->GetCameraRigTransitionEditor())
	{
		return TransitionEditor->FindAndJumpToObjectNode(InObject);
	}
	return false;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

