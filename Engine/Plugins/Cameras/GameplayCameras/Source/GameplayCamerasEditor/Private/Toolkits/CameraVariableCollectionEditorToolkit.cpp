// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CameraVariableCollectionEditorToolkit.h"

#include "AssetTools/CameraVariableCollectionEditor.h"
#include "Commands/CameraVariableCollectionEditorCommands.h"
#include "ContentBrowserModule.h"
#include "Core/CameraVariableAssets.h"
#include "Core/CameraVariableCollection.h"
#include "Editors/SCameraVariableCollectionEditor.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Helpers/AssetTypeMenuOverlayHelper.h"
#include "IContentBrowserSingleton.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "ToolMenus.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SDeleteCameraObjectDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraVariableCollectionEditorToolkit)

#define LOCTEXT_NAMESPACE "CameraVariableCollectionEditorToolkit"

namespace UE::Cameras
{

const FName FCameraVariableCollectionEditorToolkit::VariableCollectionEditorTabId(TEXT("CameraVariableCollectionEditor_VariableCollectionEditor"));
const FName FCameraVariableCollectionEditorToolkit::DetailsViewTabId(TEXT("CameraVariableCollectionEditor_DetailsView"));

FCameraVariableCollectionEditorToolkit::FCameraVariableCollectionEditorToolkit(UCameraVariableCollectionEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
	, VariableCollection(InOwningAssetEditor->GetVariableCollection())
{
	// Override base class default layout.
	StandaloneDefaultLayout = FTabManager::NewLayout("CameraVariableCollectionEditor_Layout_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.8f)
				->SetHideTabWell(true)
				->AddTab(VariableCollectionEditorTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->SetHideTabWell(true)
				->AddTab(DetailsViewTabId, ETabState::OpenedTab)
			)
		);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FCameraVariableCollectionEditorToolkit::~FCameraVariableCollectionEditorToolkit()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void FCameraVariableCollectionEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

	InTabManager->RegisterTabSpawner(VariableCollectionEditorTabId, FOnSpawnTab::CreateSP(this, &FCameraVariableCollectionEditorToolkit::SpawnTab_VariableCollectionEditor))
		.SetDisplayName(LOCTEXT("VariableCollectionEditor", "Camera Variable Collection"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(DetailsViewTabId, FOnSpawnTab::CreateSP(this, &FCameraVariableCollectionEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

TSharedRef<SDockTab> FCameraVariableCollectionEditorToolkit::SpawnTab_VariableCollectionEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> VariableCollectionEditorTab = SNew(SDockTab)
		.Label(LOCTEXT("VariableCollectionEditorTabTitle", "Camera Variable Collection"))
		[
			VariableCollectionEditorWidget.ToSharedRef()
		];

	return VariableCollectionEditorTab.ToSharedRef();
}

void FCameraVariableCollectionEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(VariableCollectionEditorTabId);
	InTabManager->UnregisterTabSpawner(DetailsViewTabId);
}

void FCameraVariableCollectionEditorToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit here because we don't want a viewport tab.
	// ...no up-call...

	RegisterToolbar();
	LayoutExtender = MakeShared<FLayoutExtender>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	// Now do our custom stuff.

	// Create the variable collection editor.
	VariableCollectionEditorWidget = SNew(SCameraVariableCollectionEditor)
		.DetailsView(DetailsView)
		.VariableCollection(VariableCollection)
		.AdditionalCommands(ToolkitCommands);
}

void FCameraVariableCollectionEditorToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(
				MenuName, ParentName, EMultiBoxType::ToolBar);

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		const FCameraVariableCollectionEditorCommands& Commands = FCameraVariableCollectionEditorCommands::Get();
		const FName CamerasStyleSetName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();

		FToolMenuSection& VariablesSection = ToolbarMenu->AddSection("Variables", TAttribute<FText>(), InsertAfterAssetSection);

		FToolMenuEntry CreateVariableEntry = FToolMenuEntry::InitComboButton(
				"CreateVariable",
				FUIAction(),
				FNewToolMenuDelegate::CreateStatic(&FCameraVariableCollectionEditorToolkit::GenerateAddNewVariableMenu),
				LOCTEXT("CreateVariableCombo_Label", "Add"),
				LOCTEXT("CreateVariableCombo_ToolTip", "Add a new camera variable to the collection"),
				FSlateIcon(CamerasStyleSetName, "CameraVariableCollectionEditor.CreateVariable")
				);
		VariablesSection.AddEntry(CreateVariableEntry);

		FToolMenuEntry RenameVariableEntry = FToolMenuEntry::InitToolBarButton(Commands.RenameVariable);
		VariablesSection.AddEntry(RenameVariableEntry);

		FToolMenuEntry DeleteVariableEntry = FToolMenuEntry::InitToolBarButton(Commands.DeleteVariable);
		VariablesSection.AddEntry(DeleteVariableEntry);
	}
}

void FCameraVariableCollectionEditorToolkit::GenerateAddNewVariableMenu(UToolMenu* InMenu)
{
	UCameraVariableCollectionEditorMenuContext* Context = InMenu->FindContext<UCameraVariableCollectionEditorMenuContext>();
	if (!ensure(Context))
	{
		return;
	}

	FCameraVariableCollectionEditorToolkit* This = Context->EditorToolkit.Pin().Get();
	if (!ensure(This))
	{
		return;
	}

	const FCameraVariableCollectionEditorCommands& Commands = FCameraVariableCollectionEditorCommands::Get();
	FToolMenuSection& VariableTypesSection = InMenu->AddSection("VariableTypes");

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* VariableClass = *It;
		if (VariableClass->IsChildOf<UCameraVariableAsset>() &&
				!VariableClass->HasAnyClassFlags(CLASS_Abstract))
		{
			const FText VariableTypeDisplayName(VariableClass->GetDisplayNameText());
			VariableTypesSection.AddEntry(FToolMenuEntry::InitMenuEntry(
						FName(FString::Format(TEXT("AddCameraVariable_{0}"), { VariableClass->GetName() })),
						TAttribute<FText>(VariableTypeDisplayName),
						TAttribute<FText>(FText::Format(
								LOCTEXT("CreateVariableEntry_LabelFmt", "Add a {0} to the collection"), VariableTypeDisplayName)),
						TAttribute<FSlateIcon>(),
						FExecuteAction::CreateSP(
							This, &FCameraVariableCollectionEditorToolkit::OnCreateVariable, 
							TSubclassOf<UCameraVariableAsset>(VariableClass))
						));
		}
	}
}

void FCameraVariableCollectionEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(MenuContext);

	UCameraVariableCollectionEditorMenuContext* Context = NewObject<UCameraVariableCollectionEditorMenuContext>();
	Context->EditorToolkit = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FCameraVariableCollectionEditorToolkit::PostInitAssetEditor()
{
	const FCameraVariableCollectionEditorCommands& Commands = FCameraVariableCollectionEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.RenameVariable,
		FExecuteAction::CreateSP(this, &FCameraVariableCollectionEditorToolkit::OnRenameVariable),
		FCanExecuteAction::CreateSP(this, &FCameraVariableCollectionEditorToolkit::CanRenameVariable));

	ToolkitCommands->MapAction(
		Commands.DeleteVariable,
		FExecuteAction::CreateSP(this, &FCameraVariableCollectionEditorToolkit::OnDeleteVariable),
		FCanExecuteAction::CreateSP(this, &FCameraVariableCollectionEditorToolkit::CanDeleteVariable));

	RegenerateMenusAndToolbars();
}

void FCameraVariableCollectionEditorToolkit::PostRegenerateMenusAndToolbars()
{
	SetMenuOverlay(FAssetTypeMenuOverlayHelper::CreateMenuOverlay(UCameraVariableCollection::StaticClass()));
}

FText FCameraVariableCollectionEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Camera Variable Collection");
}

FName FCameraVariableCollectionEditorToolkit::GetToolkitFName() const
{
	static FName ToolkitName("CameraVariableCollectionEditor");
	return ToolkitName;
}

FString FCameraVariableCollectionEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Camera Variable Collection ").ToString();
}

FLinearColor FCameraVariableCollectionEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.1f, 0.8f, 0.2f, 0.5f);
}

void FCameraVariableCollectionEditorToolkit::OnCreateVariable(TSubclassOf<UCameraVariableAsset> InVariableClass)
{
	GEditor->BeginTransaction(LOCTEXT("CreateVariable", "Create camera variable"));
	
	VariableCollection->Modify();

	UCameraVariableAsset* NewVariable = NewObject<UCameraVariableAsset>(
			VariableCollection, 
			InVariableClass.Get(),
			NAME_None,
			RF_Transactional | RF_Public  // Must be referenceable by camera parameters.
			);
	VariableCollection->Variables.Add(NewVariable);

	VariableCollectionEditorWidget->RequestListRefresh();
	VariableCollectionEditorWidget->RequestRenameVariable(NewVariable, FSimpleDelegate::CreateLambda([]()
				{
					// End the transaction when the user exits the editing mode on the
					// editable text block for the new variable's name.
					GEditor->EndTransaction();
				}));
}

void FCameraVariableCollectionEditorToolkit::OnRenameVariable()
{
	VariableCollectionEditorWidget->RequestRenameSelectedVariable();
}

bool FCameraVariableCollectionEditorToolkit::CanRenameVariable()
{
	TArray<UCameraVariableAsset*> Selection;
	VariableCollectionEditorWidget->GetSelectedVariables(Selection);
	return !Selection.IsEmpty();
}

void FCameraVariableCollectionEditorToolkit::OnDeleteVariable()
{
	TArray<UCameraVariableAsset*> Selection;
	VariableCollectionEditorWidget->GetSelectedVariables(Selection);
	if (Selection.IsEmpty())
	{
		return;
	}

	TSharedRef<SWindow> DeleteVariableWindow = SNew(SWindow)
		.Title(LOCTEXT("DeleteVariablesWindowTitle", "Delete Variables"))
		.ClientSize(FVector2D(600, 700));

	TSharedRef<SDeleteCameraObjectDialog> DeleteVariableDialog = SNew(SDeleteCameraObjectDialog)
		.ParentWindow(DeleteVariableWindow)
		.ObjectsToDelete(TArray<UObject*>(Selection))
		.OnDeletedObject_Lambda([](UObject* Obj)
					{
						if (UCameraVariableAsset* TrashVariable = Cast<UCameraVariableAsset>(Obj))
						{
							SDeleteCameraObjectDialog::RenameObjectAsTrash(TrashVariable->DisplayName);
						}
					});
	DeleteVariableWindow->SetContent(DeleteVariableDialog);

	GEditor->EditorAddModalWindow(DeleteVariableWindow);

	const bool bPerformDelete = DeleteVariableDialog->ShouldPerformDelete();
	if (bPerformDelete)
	{
		FScopedTransaction DeleteTransaction(LOCTEXT("DeleteVariable", "Delete camera variable"));

		VariableCollection->Modify();

		for (UCameraVariableAsset* VariableToDelete : Selection)
		{
			VariableToDelete->Modify();
			VariableCollection->Variables.Remove(VariableToDelete);
		}

		DeleteVariableDialog->PerformReferenceReplacement();

		VariableCollectionEditorWidget->RequestListRefresh();
	}
}

bool FCameraVariableCollectionEditorToolkit::CanDeleteVariable()
{
	TArray<UCameraVariableAsset*> Selection;
	VariableCollectionEditorWidget->GetSelectedVariables(Selection);
	return !Selection.IsEmpty();
}

void FCameraVariableCollectionEditorToolkit::FocusWindow(UObject* ObjectToFocusOn)
{
	FBaseAssetToolkit::FocusWindow(ObjectToFocusOn);

	if (UCameraVariableAsset* VariableToFocusOn = Cast<UCameraVariableAsset>(ObjectToFocusOn))
	{
		VariableCollectionEditorWidget->SelectVariable(VariableToFocusOn);
	}
}

void FCameraVariableCollectionEditorToolkit::PostUndo(bool bSuccess)
{
	VariableCollectionEditorWidget->RequestListRefresh();
}

void FCameraVariableCollectionEditorToolkit::PostRedo(bool bSuccess)
{
	VariableCollectionEditorWidget->RequestListRefresh();
}

void FCameraVariableCollectionEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(VariableCollection);
}

FString FCameraVariableCollectionEditorToolkit::GetReferencerName() const
{
	return TEXT("FCameraVariableCollectionEditorToolkit");
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

