// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/CurveEditorToolkit.h"

#include "CurveEditor.h"
#include "CurveEditor/CurvePropertyEditorTreeItem.h"
#include "Curves/CameraRotatorCurve.h"
#include "Curves/CameraSingleCurve.h"
#include "Curves/CameraVectorCurve.h"
#include "Curves/RichCurve.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SCurveEditorPanel.h"
#include "Tree/SCurveEditorTree.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "CurveEditorToolkit"

namespace UE::Cameras
{

FCurveEditorToolkit::FCurveEditorToolkit()
{
}

void FCurveEditorToolkit::Initialize()
{
	if (!ensureMsgf(!CurveEditor.IsValid(), TEXT("This curve editor toolkit has already been initialized.")))
	{
		return;
	}

	// Create the editor.
	CurveEditor = MakeShared<FCurveEditor>();

	FCurveEditorInitParams InitParams;
	CurveEditor->InitCurveEditor(InitParams);

	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

	TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FStaticCurveEditorBounds>();
	CurveEditor->SetBounds(MoveTemp(EditorBounds));

	// Create the panel.
	TSharedRef<SCurveEditorPanel> CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.TreeContent()
		[
			SNew(SCurveEditorTree, CurveEditor)
		];

	// Create the toolbar.
	TSharedPtr<FUICommandList> Commands = CurveEditorPanel->GetCommands();
	TSharedPtr<FExtender> ToolbarExtender = CurveEditorPanel->GetToolbarExtender();

	FSlimHorizontalToolBarBuilder ToolBarBuilder(Commands, FMultiBoxCustomization::None, ToolbarExtender, true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.EndSection();
	TSharedRef<SWidget> ToolBarWidget = ToolBarBuilder.MakeWidget();

	TSharedRef<SBorder> CurveEditorPanelWrapper = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16.0f, 16.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				ToolBarWidget
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				CurveEditorPanel
			]
		];

	CurveEditorWidget = CurveEditorPanelWrapper;
}

void FCurveEditorToolkit::Initialize(TArrayView<UObject*> InCurveOwners)
{
	Initialize();

	for (UObject* CurveOwner : InCurveOwners)
	{
		AddCurves(CurveOwner);
	}
}

void FCurveEditorToolkit::Shutdown()
{
	if (!ensureMsgf(CurveEditor.IsValid(), TEXT("This curve editor toolkit was not initialized.")))
	{
		return;
	}

	CurveEditorWidget.Reset();
	CurveEditor.Reset();
}

void FCurveEditorToolkit::AddCurveOwner(UObject* InCurveOwner)
{
	if (IsInitialized())
	{
		AddCurves(InCurveOwner);
	}
}

void FCurveEditorToolkit::AddCurveOwners(TArrayView<UObject*> InCurveOwners)
{
	if (IsInitialized())
	{
		for (UObject* CurveOwner : InCurveOwners)
		{
			AddCurves(CurveOwner);
		}
	}
}

void FCurveEditorToolkit::RemoveCurveOwner(UObject* InCurveOwner)
{
	if (!IsInitialized())
	{
		return;
	}

	TArray<FCurveEditorTreeItemID> RootTreeItemIDs = CurveEditor->GetRootTreeItems();
	for (FCurveEditorTreeItemID RootTreeItemID : RootTreeItemIDs)
	{
		FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(RootTreeItemID);
		TSharedPtr<FCurvePropertyEditorTreeItem> SimpleTreeItem = StaticCastSharedPtr<FCurvePropertyEditorTreeItem>(TreeItem.GetItem());
		if (SimpleTreeItem->GetOwner() == InCurveOwner)
		{
			CurveEditor->RemoveTreeItem(RootTreeItemID);
		}
	}
}

void FCurveEditorToolkit::RemoveAllCurveOwners()
{
	CurveEditor->RemoveAllTreeItems();
}

void FCurveEditorToolkit::SelectCurves(UObject* InCurveOwner, FName InPropertyName)
{
	TArray<FCurveEditorTreeItemID> Selection;

	// Find all root items that relate to the given object.
	TArray<FCurveEditorTreeItemID> RootTreeItemIDs = CurveEditor->GetRootTreeItems();
	for (FCurveEditorTreeItemID RootTreeItemID : RootTreeItemIDs)
	{
		FCurveEditorTreeItem& RootTreeItem = CurveEditor->GetTreeItem(RootTreeItemID);
		TSharedPtr<FCurvePropertyEditorTreeItem> ObjectTreeItem = StaticCastSharedPtr<FCurvePropertyEditorTreeItem>(RootTreeItem.GetItem());
		if (ObjectTreeItem->GetOwner() == InCurveOwner)
		{
			// Find any curves, or curve groups, that match the given property name.
			for (FCurveEditorTreeItemID ChildID : RootTreeItem.GetChildren())
			{
				FCurveEditorTreeItem& ChildTreeItem = CurveEditor->GetTreeItem(ChildID);
				TSharedPtr<FCurvePropertyEditorTreeItem> PropertyTreeItem = StaticCastSharedPtr<FCurvePropertyEditorTreeItem>(ChildTreeItem.GetItem());
				if (PropertyTreeItem->Info.PropertyName == InPropertyName)
				{
					Selection.Add(ChildID);
				}
			}
		}
	}

	if (!Selection.IsEmpty())
	{
		CurveEditor->SetTreeSelection(MoveTemp(Selection));
	}
}

void FCurveEditorToolkit::AddCurves(UObject* InObject)
{
	if (!InObject)
	{
		return;
	}

	UClass* ObjectClass = InObject->GetClass();

	FCurveEditorTreeItem* ObjectItem = nullptr;
	auto GetObjectItemID = [this, InObject, &ObjectItem]() -> FCurveEditorTreeItemID
	{
		if (!ObjectItem)
		{
			FCurvePropertyInfo ObjectInfo;
			ObjectInfo.WeakOwner = InObject;
			ObjectInfo.DisplayName = FText::FromName(InObject->GetFName());
			ObjectItem = AddTreeItem(FCurveEditorTreeItemID::Invalid(), MoveTemp(ObjectInfo));
		}
		return ObjectItem->GetID();
	};

	const FLinearColor RGBColors[3] = { FLinearColor::Red, FLinearColor::Green, FLinearColor::Blue };

	for (TFieldIterator<FProperty> PropertyIt(ObjectClass); PropertyIt; ++PropertyIt)
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(*PropertyIt);
		if (!StructProperty)
		{
			continue;
		}

		const FName PropertyName = StructProperty->GetFName();
		const FText PropertyDisplayName = FText::FromName(StructProperty->GetFName());

		if (StructProperty->Struct == FRichCurve::StaticStruct())
		{
			FCurvePropertyInfo CurveInfo;
			CurveInfo.Curve = StructProperty->ContainerPtrToValuePtr<FRichCurve>(InObject);
			CurveInfo.DisplayName = PropertyDisplayName;
			CurveInfo.Color = FLinearColor::White;
			CurveInfo.WeakOwner = InObject;
			CurveInfo.PropertyName = PropertyName;
			AddTreeItem(GetObjectItemID(), MoveTemp(CurveInfo));
		}
		else if (StructProperty->Struct == FCameraSingleCurve::StaticStruct())
		{
			FCameraSingleCurve* SingleCurve = StructProperty->ContainerPtrToValuePtr<FCameraSingleCurve>(InObject);

			FCurvePropertyInfo CurveInfo;
			CurveInfo.Curve = &SingleCurve->Curve;
			CurveInfo.DisplayName = PropertyDisplayName;
			CurveInfo.Color = FLinearColor::White;
			CurveInfo.WeakOwner = InObject;
			CurveInfo.PropertyName = PropertyName;
			AddTreeItem(GetObjectItemID(), MoveTemp(CurveInfo));
		}
		else if (StructProperty->Struct == FCameraVectorCurve::StaticStruct())
		{
			const FText XYZNames[3] = { LOCTEXT("X", "X"), LOCTEXT("Y", "Y"), LOCTEXT("Z", "Z") };
			FCameraVectorCurve* VectorCurve = StructProperty->ContainerPtrToValuePtr<FCameraVectorCurve>(InObject);

			FCurvePropertyInfo ParentInfo;
			ParentInfo.DisplayName = PropertyDisplayName;
			ParentInfo.WeakOwner = InObject;
			ParentInfo.PropertyName = PropertyName;
			FCurveEditorTreeItem* ParentItem = AddTreeItem(GetObjectItemID(), MoveTemp(ParentInfo));

			for (int32 Index = 0; Index < 3; ++Index)
			{
				FCurvePropertyInfo CurveInfo;
				CurveInfo.Curve = &VectorCurve->Curves[Index];
				CurveInfo.DisplayName = XYZNames[Index];
				CurveInfo.Color = RGBColors[Index];
				CurveInfo.WeakOwner = InObject;
				CurveInfo.PropertyName = PropertyName;
				AddTreeItem(ParentItem->GetID(), MoveTemp(CurveInfo));
			}
		}
		else if (StructProperty->Struct == FCameraRotatorCurve::StaticStruct())
		{
			const FText YPRNames[3] = { LOCTEXT("Yaw", "Yaw"), LOCTEXT("Pitch", "Pitch"), LOCTEXT("Roll", "Roll") };
			FCameraRotatorCurve* RotatorCurve = StructProperty->ContainerPtrToValuePtr<FCameraRotatorCurve>(InObject);

			FCurvePropertyInfo ParentInfo;
			ParentInfo.DisplayName = PropertyDisplayName;
			ParentInfo.WeakOwner = InObject;
			ParentInfo.PropertyName = PropertyName;
			FCurveEditorTreeItem* ParentItem = AddTreeItem(GetObjectItemID(), MoveTemp(ParentInfo));

			for (int32 Index = 0; Index < 3; ++Index)
			{
				FCurvePropertyInfo CurveInfo;
				CurveInfo.Curve = &RotatorCurve->Curves[Index];
				CurveInfo.DisplayName = YPRNames[Index];
				CurveInfo.Color = RGBColors[Index];
				CurveInfo.WeakOwner = InObject;
				CurveInfo.PropertyName = PropertyName;
				AddTreeItem(ParentItem->GetID(), MoveTemp(CurveInfo));
			}
		}
	}
}

 FCurveEditorTreeItem* FCurveEditorToolkit::AddTreeItem(FCurveEditorTreeItemID ParentID, FCurvePropertyInfo&& CurveInfo)
{
	FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(ParentID);

	TSharedPtr<FCurvePropertyEditorTreeItem> TreeItemModel = MakeShared<FCurvePropertyEditorTreeItem>(MoveTemp(CurveInfo));
	TreeItem->SetStrongItem(TreeItemModel);

	return TreeItem;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

