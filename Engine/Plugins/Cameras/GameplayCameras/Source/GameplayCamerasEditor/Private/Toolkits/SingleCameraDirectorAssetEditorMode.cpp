// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SingleCameraDirectorAssetEditorMode.h"

#include "Directors/SingleCameraDirector.h"

namespace UE::Cameras
{

TSharedPtr<FCameraDirectorAssetEditorMode> FSingleCameraDirectorAssetEditorMode::CreateInstance(UCameraAsset* InCameraAsset)
{
	UCameraDirector* CameraDirector = InCameraAsset->GetCameraDirector();
	if (Cast<USingleCameraDirector>(CameraDirector))
	{
		return MakeShared<FSingleCameraDirectorAssetEditorMode>(InCameraAsset);
	}
	return nullptr;
}

FSingleCameraDirectorAssetEditorMode::FSingleCameraDirectorAssetEditorMode(UCameraAsset* InCameraAsset)
	: FCameraDirectorAssetEditorMode(InCameraAsset)
{
	if (InCameraAsset)
	{
		InCameraAsset->EventHandlers.Register(EventHandler, this);
	}
}

}  // namespace UE::Cameras

