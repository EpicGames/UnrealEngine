// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/StandardToolkitLayout.h"
#include "Framework/Docking/TabManager.h"

namespace UE::Cameras
{

const FName FStandardToolkitLayout::LeftStackExtensionId("LeftStackId");
const FName FStandardToolkitLayout::CenterStackExtensionId("CenterStackId");
const FName FStandardToolkitLayout::RightStackExtensionId("RightStackId");
const FName FStandardToolkitLayout::BottomStackExtensionId("BottomStackId");

FStandardToolkitLayout::FStandardToolkitLayout(FName InLayoutName)
	: LayoutName(InLayoutName)
{
	if (LayoutName.IsNone())
	{
		LayoutName = TEXT("StandardCameraEditorToolkit_Layout_v1");
	}

	BuildLayout();
}

TSharedPtr<FTabManager::FLayout> FStandardToolkitLayout::GetLayout() const
{
	return Layout;
}

void FStandardToolkitLayout::BuildLayout()
{
	if (Layout.IsValid())
	{
		return;
	}

	LeftTabStack = FTabManager::NewStack();
	CenterTabStack = FTabManager::NewStack();
	RightTabStack = FTabManager::NewStack();
	BottomTabStack = FTabManager::NewStack();

	Layout = FTabManager::NewLayout(LayoutName)
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					LeftTabStack
					->SetExtensionId(LeftStackExtensionId)
					->SetSizeCoefficient(0.2f)
				)
				->Split
				(
					CenterTabStack
					->SetExtensionId(CenterStackExtensionId)
					->SetSizeCoefficient(0.6f)
					->SetHideTabWell(true)
				)
				->Split
				(
					RightTabStack
					->SetExtensionId(RightStackExtensionId)
					->SetSizeCoefficient(0.2f)
				)
			)
			->Split
			(
				BottomTabStack
				->SetExtensionId(BottomStackExtensionId)
				->SetSizeCoefficient(0.2f)
			)
		);

	// Add content browsers to the bottom tab stack by default so that
	// we can dock them in our layout.
	BottomTabStack->AddTab("ContentBrowserTab1", ETabState::ClosedTab);
	BottomTabStack->AddTab("ContentBrowserTab2", ETabState::ClosedTab);
	BottomTabStack->AddTab("ContentBrowserTab3", ETabState::ClosedTab);
	BottomTabStack->AddTab("ContentBrowserTab4", ETabState::ClosedTab);
}

void FStandardToolkitLayout::AddLeftTab(FName InTabId, ETabState::Type InTabState)
{
	LeftTabStack->AddTab(InTabId, InTabState);
}

void FStandardToolkitLayout::AddRightTab(FName InTabId, ETabState::Type InTabState)
{
	RightTabStack->AddTab(InTabId, InTabState);
}

void FStandardToolkitLayout::AddBottomTab(FName InTabId, ETabState::Type InTabState)
{
	BottomTabStack->AddTab(InTabId, InTabState);
}

void FStandardToolkitLayout::AddCenterTab(FName InTabId, ETabState::Type InTabState)
{
	CenterTabStack->AddTab(InTabId, InTabState);
}

}  // namespace UE::Cameras

