// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/CameraSystemRewindDebuggerExtension.h"

#include "Debug/CameraDebugRenderer.h"
#include "Debug/CameraSystemTrace.h"
#include "Debug/DebugDrawService.h"
#include "Debug/RootCameraDebugBlock.h"
#include "Engine/Canvas.h"
#include "Engine/World.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#include "Trace/CameraSystemTraceProvider.h"
#include "TraceServices/Model/Frames.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

FCameraSystemRewindDebuggerExtension::FCameraSystemRewindDebuggerExtension()
{
}

FCameraSystemRewindDebuggerExtension::~FCameraSystemRewindDebuggerExtension()
{
	EnsureDebugDrawDelegate(false);
}

void FCameraSystemRewindDebuggerExtension::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	if (RewindDebugger->IsPIESimulating() || RewindDebugger->GetRecordingDuration() == 0.0)
	{
		EnsureDebugDrawDelegate(false);
		return;
	}

	EnsureDebugDrawDelegate(true);

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	
	const FCameraSystemTraceProvider* CameraSystemProvider = AnalysisSession->ReadProvider<FCameraSystemTraceProvider>(FCameraSystemTraceProvider::ProviderName);
	const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*RewindDebugger->GetAnalysisSession());

	double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

	TraceServices::FFrame Frame;
	if (FrameProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTraceTime, Frame))
	{
		const FCameraSystemTraceFrameData* FoundFrameData = nullptr;
		const bool bIsActiveCamera = false; // TODO

		const FCameraSystemTraceTimeline* CameraSystemTimeline = CameraSystemProvider->GetTimeline();
		CameraSystemTimeline->EnumerateEvents(
				Frame.StartTime, Frame.EndTime, 
				[&FoundFrameData, bIsActiveCamera](double InStartTime, double InEndTime, uint32 InDepth, const FCameraSystemTraceFrameData& FrameData)  
				{
					if (FRootCameraDebugBlock::ShouldDebugDraw(FrameData.CameraSystemID, bIsActiveCamera))
					{
						FoundFrameData = &FrameData;
					}
					return TraceServices::EEventEnumerate::Continue;
				});

		if (FoundFrameData && CurrentTraceTime != LastTraceTime)
		{
			LastTraceTime = CurrentTraceTime;
			WeakVisualizedWorld = RewindDebugger->GetWorldToVisualize();

			DebugBlockStorage.DestroyDebugBlocks();

			FCameraDebugBlock* ReadBlock = FCameraSystemTrace::ReadEvaluationTrace(
					FoundFrameData->SerializedBlocks, DebugBlockStorage);
			ensure(ReadBlock && ReadBlock->GetTypeID() == FRootCameraDebugBlock::StaticTypeID());
			RootDebugBlock = ReadBlock->CastThisChecked<FRootCameraDebugBlock>();
		}
	}
}

void FCameraSystemRewindDebuggerExtension::Clear(IRewindDebugger* RewindDebugger)
{
	EnsureDebugDrawDelegate(false);

	WeakVisualizedWorld = nullptr;

	DebugBlockStorage.DestroyDebugBlocks(true);
	RootDebugBlock = nullptr;
}

void FCameraSystemRewindDebuggerExtension::EnsureDebugDrawDelegate(bool bIsRegistered)
{
	if (bIsRegistered && !DebugDrawDelegateHandle.IsValid())
	{
		DebugDrawDelegateHandle = UDebugDrawService::Register(
				TEXT("GameplayDebug"), FDebugDrawDelegate::CreateRaw(this, &FCameraSystemRewindDebuggerExtension::DebugDraw));
	}
	else if (!bIsRegistered && DebugDrawDelegateHandle.IsValid())
	{
		UDebugDrawService::Unregister(DebugDrawDelegateHandle);
		DebugDrawDelegateHandle .Reset();
	}
}

void FCameraSystemRewindDebuggerExtension::DebugDraw(UCanvas* Canvas, APlayerController* PlayController)
{
	if (LevelEditorModule == nullptr)
	{
		LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	}

	UWorld* VisualizedWorld = WeakVisualizedWorld.Get();
	if (RootDebugBlock && VisualizedWorld)
	{
		bool bIsExternalRendering = false;
		if (LevelEditorModule)
		{
			TSharedPtr<SLevelViewport> LevelViewport = LevelEditorModule->GetFirstActiveLevelViewport();
			FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
			bIsExternalRendering = !LevelViewportClient.IsAnyActorLocked();
		}

		FRootCameraDebugDrawParams Params;
		FCameraDebugRenderer CameraDebugRenderer(VisualizedWorld, Canvas, bIsExternalRendering);
		RootDebugBlock->RootDebugDraw(Params, CameraDebugRenderer);
	}
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

