// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Debug/CameraDebugBlockStorage.h"
#include "GameplayCameras.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerExtension.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtrFwd.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

class APlayerController;
class FLevelEditorModule;
class UCanvas;

namespace UE::Cameras
{

class FRootCameraDebugBlock;

/**
 * Rewind debugger extension for the camera system evaluation trace.
 */
class FCameraSystemRewindDebuggerExtension : public IRewindDebuggerExtension
{
public:

	FCameraSystemRewindDebuggerExtension();
	virtual ~FCameraSystemRewindDebuggerExtension();

	// IRewindDebuggerExtension interface.
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual void Clear(IRewindDebugger* RewindDebugger) override;

private:

	void EnsureDebugDrawDelegate(bool bIsRegistered);
	void DebugDraw(UCanvas* Canvas, APlayerController* PlayController);

private:

	FDelegateHandle DebugDrawDelegateHandle;
	double LastTraceTime = 0.f;

	FLevelEditorModule* LevelEditorModule = nullptr;
	TWeakObjectPtr<UWorld> WeakVisualizedWorld;

	FCameraDebugBlockStorage DebugBlockStorage;
	FRootCameraDebugBlock* RootDebugBlock = nullptr;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

