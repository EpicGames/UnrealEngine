// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/CameraSystemRewindDebuggerTrack.h"

#include "CoreTypes.h"
#include "GameFramework/Pawn.h"
#include "IRewindDebugger.h"
#include "Math/Range.h"
#include "SSimpleTimeSlider.h"
#include "Styles/GameplayCamerasEditorStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define LOCTEXT_NAMESPACE "CameraSystemRewindDebuggerTrack"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

class SCameraSystemRewindDebuggerTrackTimeline : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraSystemRewindDebuggerTrackTimeline)
		: _ViewRange(TRange<double>(0,10))
		, _DesiredSize(FVector2D(100.f, 20.f))
	{}
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);
		SLATE_ATTRIBUTE(FVector2D, DesiredSize);
		SLATE_ATTRIBUTE(TSharedPtr<FCameraSystemRewindDebuggerTrackTimelineData>, TimelineData);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ViewRange = InArgs._ViewRange;
		DesiredSize = InArgs._DesiredSize;
		TimelineData = InArgs._TimelineData;

		BackgroundBrush = FAppStyle::GetBrush("Sequencer.SectionArea.Background");
	}

protected:

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return DesiredSize.Get();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const int32 NewLayer = PaintWindows(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		const int32 BaseLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
		return FMath::Max(NewLayer, BaseLayer);
	}

private:

	int32 PaintWindows(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		TSharedPtr<FCameraSystemRewindDebuggerTrackTimelineData> ActualTimelineData = TimelineData.Get();
		if (!ActualTimelineData)
		{
			return LayerId;
		}

		TRange<double> DebugTimeRange = ViewRange.Get();

		SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen( DebugTimeRange, AllottedGeometry.GetLocalSize() );
		FVector2D Size = AllottedGeometry.GetLocalSize();

		const TArray<FCameraSystemRewindDebuggerTrackTimelineData::DataWindow>& Windows(ActualTimelineData->Windows);
		const int NumWindows = Windows.Num();
		if (NumWindows > 0)
		{
			for (int32 Index = 0; Index < NumWindows; ++Index)
			{
				const FCameraSystemRewindDebuggerTrackTimelineData::DataWindow& Window(Windows[Index]);
			
				const float StartX = RangeToScreen.InputToLocalX(Window.TimeStart);
				const float EndX = RangeToScreen.InputToLocalX(Window.TimeEnd);
				const float SizeY = AllottedGeometry.Size.Y;

				FSlateDrawElement::MakeBox(
						OutDrawElements, 
						LayerId++,
						AllottedGeometry.ToPaintGeometry(FVector2D(EndX - StartX, SizeY - 2), FSlateLayoutTransform(FVector2D(StartX, 1))),
						BackgroundBrush, 
						ESlateDrawEffect::None, 
						Window.Color);
			}
		}
		
		LayerId++;
		return LayerId;
	}
	
private:

	TAttribute<TRange<double>> ViewRange;
	TAttribute<FVector2D> DesiredSize;
	TAttribute<TSharedPtr<FCameraSystemRewindDebuggerTrackTimelineData>> TimelineData;

	const FSlateBrush* BackgroundBrush;
};

FCameraSystemRewindDebuggerTrack::FCameraSystemRewindDebuggerTrack()
{
	const FName& GameplayCamerasEditorStyleName = FGameplayCamerasEditorStyle::Get()->GetStyleSetName();
	Icon = FSlateIcon(GameplayCamerasEditorStyleName,"DebugCategory.PoseStats.Icon");

	TimelineData = MakeShared<FCameraSystemRewindDebuggerTrackTimelineData>();
}

FSlateIcon FCameraSystemRewindDebuggerTrack::GetIconInternal()
{
	return Icon;
}

FText FCameraSystemRewindDebuggerTrack::GetDisplayNameInternal() const
{
	return LOCTEXT("DisplayName", "Gameplay Camera System");
}

TSharedPtr<SWidget> FCameraSystemRewindDebuggerTrack::GetTimelineViewInternal()
{
	return SNew(SCameraSystemRewindDebuggerTrackTimeline)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.TimelineData(TimelineData);
}

FName FCameraSystemRewindDebuggerTrackCreator::GetTargetTypeNameInternal() const
{
	// Only show this track once at the top level.
 	static const FName TargetTypeName = APawn::StaticClass()->GetFName();
	return TargetTypeName;
}

namespace
{
	static const FName TrackName("Gameplay Camera System");
}
	
FName FCameraSystemRewindDebuggerTrackCreator::GetNameInternal() const
{
	return TrackName;
}
	
FName FCameraSystemRewindDebuggerTrack::GetNameInternal() const
{
	return TrackName;
}

void FCameraSystemRewindDebuggerTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({GetNameInternal(), LOCTEXT("DisplayName", "Gameplay Camera System")});
}


#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FCameraSystemRewindDebuggerTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
#else
TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FCameraSystemRewindDebuggerTrackCreator::CreateTrackInternal(uint64 InObjectId) const
#endif
{
	return MakeShared<FCameraSystemRewindDebuggerTrack>();
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

#undef LOCTEXT_NAMESPACE

