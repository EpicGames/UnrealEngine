// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayCameras.h"
#include "IRewindDebuggerTrackCreator.h"
#include "Math/Color.h"
#include "Misc/EngineVersionComparison.h"
#include "RewindDebuggerTrack.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

/**
 * Data for drawing the rewind debugger track that corresponds to the 
 * camera system evaluation trace.
 */
struct FCameraSystemRewindDebuggerTrackTimelineData
{
public:

	struct DataWindow
	{
		double TimeStart;
		double TimeEnd;
		FLinearColor Color;
	};

	TArray<DataWindow> Windows;
};

/**
 * Rewing debugger track for the camera system evaluation trace.
 */
class FCameraSystemRewindDebuggerTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:

	FCameraSystemRewindDebuggerTrack();

protected:

	virtual FSlateIcon GetIconInternal() override;
	virtual FText GetDisplayNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;

private:

	FSlateIcon Icon;
	TSharedPtr<FCameraSystemRewindDebuggerTrackTimelineData> TimelineData;
};

#if WITH_ENGINE
/**
 * Factory class that creates the rewind debugger track for the camera system evaluation trace.
 */
class FCameraSystemRewindDebuggerTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
protected:

	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
#else
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(uint64 InObjectId) const override;
#endif
};
#endif // WITH_ENGINE

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

