// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/CameraSystemTraceAnalyzer.h"

#include "Debug/CameraSystemTrace.h"
#include "Trace/CameraSystemTraceProvider.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

FCameraSystemTraceAnalyzer::FCameraSystemTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FCameraSystemTraceProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FCameraSystemTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(0, "CameraSystem", "CameraSystemEvaluation");
}

bool FCameraSystemTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	TraceServices::FAnalysisSessionEditScope _(Session);

	FCameraSystemTraceFrameData FrameData;

	uint64 Cycle = Context.EventData.GetValue<uint64>("Cycle");
	double RecordingTime = Context.EventData.GetValue<double>("RecordingTime");
	FrameData.CameraSystemID = Context.EventData.GetValue<int32>("CameraSystemDebugID");
	FrameData.EvaluatedLocation.X = Context.EventData.GetValue<double>("EvaluatedLocationX");
	FrameData.EvaluatedLocation.Y = Context.EventData.GetValue<double>("EvaluatedLocationY");
	FrameData.EvaluatedLocation.Z = Context.EventData.GetValue<double>("EvaluatedLocationZ");
	FrameData.EvaluatedRotation.Yaw = Context.EventData.GetValue<double>("EvaluatedRotationYaw");
	FrameData.EvaluatedRotation.Pitch = Context.EventData.GetValue<double>("EvaluatedRotationPitch");
	FrameData.EvaluatedRotation.Roll = Context.EventData.GetValue<double>("EvaluatedRotationRoll");
	FrameData.EvaluatedFieldOfView = Context.EventData.GetValue<float>("EvaluatedFieldOfView");
	FrameData.SerializedBlocks = Context.EventData.GetArrayView<uint8>("SerializedBlocks");
	
	double EventTime = Context.EventTime.AsSeconds(Cycle);
	Provider.AppendFrameData(EventTime, MoveTemp(FrameData));

	return true;
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

