// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Trace/Analyzer.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace TraceServices { class IAnalysisSession; }

namespace UE::Cameras
{

class FCameraSystemTraceProvider;

/**
 * Trace analyzer for the camera system evaluation.
 */
class FCameraSystemTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:

	FCameraSystemTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FCameraSystemTraceProvider& InProvider);

	// IAnalyzer interface
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:

	TraceServices::IAnalysisSession& Session;
	FCameraSystemTraceProvider& Provider;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

