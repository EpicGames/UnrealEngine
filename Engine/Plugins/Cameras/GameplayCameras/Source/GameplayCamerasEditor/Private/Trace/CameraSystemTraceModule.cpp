// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/CameraSystemTraceModule.h"

#include "Debug/CameraSystemTrace.h"
#include "Trace/CameraSystemTraceModule.h"
#include "Trace/CameraSystemTraceProvider.h"
#include "Trace/CameraSystemTraceAnalyzer.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

FName FCameraSystemTraceModule::ModuleName("CameraSystem");

void FCameraSystemTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("Camera System");
}

void FCameraSystemTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FCameraSystemTraceProvider> CameraSystemProvider = MakeShared<FCameraSystemTraceProvider>(InSession);
	InSession.AddProvider(FCameraSystemTraceProvider::ProviderName, CameraSystemProvider);
	InSession.AddAnalyzer(new FCameraSystemTraceAnalyzer(InSession, *CameraSystemProvider));
}

void FCameraSystemTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(*FCameraSystemTrace::LoggerName);
}

void FCameraSystemTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

