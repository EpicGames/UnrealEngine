// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayCameras.h"
#include "TraceServices/ModuleService.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

/**
 * Trace module for camera system evaluation.
 */
class FCameraSystemTraceModule : public TraceServices::IModule
{
public:

	// TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("gameplaycameras"); }

private:

	static FName ModuleName;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

