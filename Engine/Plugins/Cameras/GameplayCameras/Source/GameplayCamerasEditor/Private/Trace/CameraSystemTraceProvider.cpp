// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/CameraSystemTraceProvider.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace UE::Cameras
{

FName FCameraSystemTraceProvider::ProviderName("CameraSystemTraceProvider");

FCameraSystemTraceProvider::FCameraSystemTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
	Timeline = MakeShared<FFrameDataPointTimeline>(Session.GetLinearAllocator());
}

void FCameraSystemTraceProvider::AppendFrameData(double InRecordingTime, FCameraSystemTraceFrameData&& InFrameData)
{
	Session.WriteAccessCheck();

	Timeline->AppendEvent(InRecordingTime, MoveTemp(InFrameData));
}

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

