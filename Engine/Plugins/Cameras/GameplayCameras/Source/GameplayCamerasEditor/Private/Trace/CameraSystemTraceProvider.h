// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameplayCameras.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Model/PointTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"

#if UE_GAMEPLAY_CAMERAS_TRACE

namespace TraceServices { class IAnalysisSession; }

namespace UE::Cameras
{

/** Data for one frame of tracing. */
struct FCameraSystemTraceFrameData
{
	int32 CameraSystemID;
	FVector3d EvaluatedLocation;
	FRotator3d EvaluatedRotation;
	float EvaluatedFieldOfView;
	TArray<uint8> SerializedBlocks;
};

using FCameraSystemTraceTimeline = TraceServices::ITimeline<FCameraSystemTraceFrameData>;

/**
 * Trace provider for the camera system evaluation.
 */
class FCameraSystemTraceProvider : public TraceServices::IProvider
{
public:

	static FName ProviderName;

	FCameraSystemTraceProvider(TraceServices::IAnalysisSession& InSession);

	void AppendFrameData(double InRecordingTime, FCameraSystemTraceFrameData&& InFrameData);

	const FCameraSystemTraceTimeline* GetTimeline() const { return Timeline.Get(); }	

private:

	TraceServices::IAnalysisSession& Session;

	using FFrameDataPointTimeline = TraceServices::TPointTimeline<FCameraSystemTraceFrameData>;
	TSharedPtr<FFrameDataPointTimeline> Timeline;
};

}  // namespace UE::Cameras

#endif  // UE_GAMEPLAY_CAMERAS_TRACE

