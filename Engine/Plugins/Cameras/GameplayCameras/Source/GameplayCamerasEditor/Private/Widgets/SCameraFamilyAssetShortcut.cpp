// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCameraFamilyAssetShortcut.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "GameplayCamerasDelegates.h"
#include "IAssetFamily.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IGameplayCamerasFamily.h"
#include "IGameplayCamerasEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Textures/SlateIcon.h"
#include "Tools/BaseAssetToolkit.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SCameraFamilyShortcutBar"

namespace UE::Cameras
{

void SCameraFamilyAssetShortcut::Construct(const FArguments& InArgs, const TSharedRef<FBaseAssetToolkit>& InToolkit, const TSharedRef<IGameplayCamerasFamily>& InFamily, UClass* InAssetType)
{
	WeakToolkit = InToolkit;

	Family = InFamily;
	FamilyAssetType = InAssetType;

	bRefreshAssetDatas = true;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SCameraFamilyAssetShortcut::HandleFilesLoaded);
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SCameraFamilyAssetShortcut::HandleAssetAdded);
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SCameraFamilyAssetShortcut::HandleAssetRemoved);
	AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &SCameraFamilyAssetShortcut::HandleAssetRenamed);

	FGameplayCamerasDelegates::OnCameraAssetBuilt().AddSP(this, &SCameraFamilyAssetShortcut::HandleCameraAssetBuilt);
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().AddSP(this, &SCameraFamilyAssetShortcut::HandleCameraRigAssetBuilt);

	const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar");

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Fat button, when there's only one asset of this type.
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(SoloCheckBox, SCheckBox)
			.Style(FAppStyle::Get(), "SegmentedCombo.ButtonOnly")
			.OnCheckStateChanged(this, &SCameraFamilyAssetShortcut::HandleButtonClick)
			.IsChecked(this, &SCameraFamilyAssetShortcut::GetCheckState)
			.IsEnabled(this, &SCameraFamilyAssetShortcut::IsSoloButtonEnabled)
			.Visibility(this, &SCameraFamilyAssetShortcut::GetSoloButtonVisibility)
			.ToolTipText(this, &SCameraFamilyAssetShortcut::GetButtonTooltip)
			.Padding(0.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(FMargin(28.f, 4.f))
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SCameraFamilyAssetShortcut::GetAssetTint)
					.Image(this, &SCameraFamilyAssetShortcut::GetAssetIcon)
				]
			]
		]

		// Button/dropdown pair, when there are more than one asset of this type.
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(ComboCheckBox, SCheckBox)
			.Style(FAppStyle::Get(), "SegmentedCombo.Left")
			.OnCheckStateChanged(this, &SCameraFamilyAssetShortcut::HandleButtonClick)
			.IsChecked(this, &SCameraFamilyAssetShortcut::GetCheckState)
			.Visibility(this, &SCameraFamilyAssetShortcut::GetComboButtonVisibility)
			.ToolTipText(this, &SCameraFamilyAssetShortcut::GetButtonTooltip)
			.Padding(0.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(FMargin(16.f, 4.f))
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SCameraFamilyAssetShortcut::GetAssetTint)
					.Image(this, &SCameraFamilyAssetShortcut::GetAssetIcon)
				]
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SSeparator)
			.Visibility(this, &SCameraFamilyAssetShortcut::GetComboDropdownVisibility)
			.Thickness(1.0f)
			.Orientation(EOrientation::Orient_Vertical)
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew(ComboDropdown, SComboButton)
			.Visibility(this, &SCameraFamilyAssetShortcut::GetComboDropdownVisibility)
			.ContentPadding(FMargin(7.f, 0.f))
			.ForegroundColor(FSlateColor::UseForeground())
			.ComboButtonStyle(&FAppStyle::Get(), "SegmentedCombo.Right")
			.OnGetMenuContent(this, &SCameraFamilyAssetShortcut::HandleGetDropdownMenuContent)
		]
	];

	EnableToolTipForceField(true);
}

SCameraFamilyAssetShortcut::~SCameraFamilyAssetShortcut()
{
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}

	FGameplayCamerasDelegates::OnCameraAssetBuilt().RemoveAll(this);
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().RemoveAll(this);
}

void SCameraFamilyAssetShortcut::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bRefreshAssetDatas)
	{
		bRefreshAssetDatas = false;

		AssetDatas.Reset();
		Family->FindAssetsOfType(FamilyAssetType, AssetDatas);
	}
}

const FSlateBrush* SCameraFamilyAssetShortcut::GetAssetIcon() const 
{
	return Family->GetAssetIcon(FamilyAssetType);
}

FSlateColor SCameraFamilyAssetShortcut::GetAssetTint() const
{
	if (GetCheckState() == ECheckBoxState::Checked)
	{
		return FSlateColor::UseForeground();
	}
	return Family->GetAssetTint(FamilyAssetType);
}

ECheckBoxState SCameraFamilyAssetShortcut::GetCheckState() const
{
	if (TSharedPtr<FBaseAssetToolkit> Toolkit = WeakToolkit.Pin())
	{
		const TArray<UObject*>* Objects = Toolkit->GetObjectsCurrentlyBeingEdited();
		if (Objects != nullptr && AssetDatas.Num() == 1)
		{
			const FAssetData& AssetData = AssetDatas[0];
			for (UObject* Object : *Objects)
			{
				if (Object->GetPathName().Compare(AssetData.GetObjectPathString(), ESearchCase::IgnoreCase) == 0)
				{
					return ECheckBoxState::Checked;
				}
			}
		}
	}
	return ECheckBoxState::Unchecked;
}

FText SCameraFamilyAssetShortcut::GetButtonTooltip() const
{
	if (AssetDatas.Num() > 1)
	{
		return FText::Format(
				LOCTEXT("ShortcutComboTooltipFmt", "{0}\nAsset Types: {1}"),
				Family->GetAssetTypeTooltip(FamilyAssetType),
				FamilyAssetType->GetDisplayNameText());
	}
	else if (AssetDatas.Num() == 1)
	{
		return FText::Format(
				LOCTEXT("ShortcutSoloTooltipFmt", "Open {0}\nAsset Type: {1}"),
				FText::FromString(AssetDatas[0].GetFullName()),
				FamilyAssetType->GetDisplayNameText());
	}
	else
	{
		return FText::Format(
				LOCTEXT("ShortcutNoMatchTooltipFmt", "No related assets found\nAsset Type: {0}"),
				FamilyAssetType->GetDisplayNameText());
	}
}

void SCameraFamilyAssetShortcut::HandleButtonClick(ECheckBoxState InState)
{
	if (AssetDatas.Num() == 1)
	{
		const FAssetData& AssetData(AssetDatas[0]);
		if (UObject* AssetObject = AssetData.GetAsset())
		{
			TArray<UObject*> Assets;
			Assets.Add(AssetObject);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
		}
		else
		{
			UE_LOGF(LogCameraSystemEditor, Error, "Asset cannot be opened: %ls", *AssetData.GetObjectPathString());
		}
	}
	else if (AssetDatas.Num() > 1)
	{
		ComboDropdown->SetIsOpen(true);
	}
}

EVisibility SCameraFamilyAssetShortcut::GetSoloButtonVisibility() const
{
	return AssetDatas.Num() <= 1 ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SCameraFamilyAssetShortcut::IsSoloButtonEnabled() const
{
	return AssetDatas.Num() > 0;
}

EVisibility SCameraFamilyAssetShortcut::GetComboButtonVisibility() const
{
	return AssetDatas.Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCameraFamilyAssetShortcut::GetComboDropdownVisibility() const
{
	return AssetDatas.Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SCameraFamilyAssetShortcut::HandleGetDropdownMenuContent()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	if (AssetDatas.Num() > 1)
	{
		MenuBuilder.BeginSection("AssetSelection", LOCTEXT("AssetSelectionSection", "Select Asset"));
		{
			FAssetPickerConfig AssetPickerConfig;
			AssetPickerConfig.bCanShowClasses = false;
			for (const FAssetData& AssetData : AssetDatas)
			{
				AssetPickerConfig.Filter.SoftObjectPaths.Add(AssetData.GetSoftObjectPath());
			}

			AssetPickerConfig.SelectionMode = ESelectionMode::SingleToggle;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCameraFamilyAssetShortcut::HandleOpenSecondaryAsset);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.ThumbnailLabel = EThumbnailLabel::ClassName;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

			IContentBrowserSingleton& ContentBrowserSingleton = IContentBrowserSingleton::Get();

			MenuBuilder.AddWidget(
				SNew(SBox)
				.WidthOverride(300.f)
				.HeightOverride(600.f)
				[
					ContentBrowserSingleton.CreateAssetPicker(AssetPickerConfig)
				],
				FText(), true);
		}
		MenuBuilder.EndSection();
	}
	
	return MenuBuilder.MakeWidget();
}

void SCameraFamilyAssetShortcut::HandleOpenSecondaryAsset(const FAssetData& InAssetData)
{
	if (InAssetData.IsValid())
	{
		FSlateApplication::Get().DismissAllMenus();

		TArray<UObject*> Assets;
		Assets.Add(InAssetData.GetAsset());
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
	}
}

bool SCameraFamilyAssetShortcut::DoesFamilySupport(const FAssetData& InAssetData)
{
	TArray<UClass*> AssetTypes;
	Family->GetAssetTypes(AssetTypes);
	return AssetTypes.ContainsByPredicate([&InAssetData](UClass* AssetType)
				{
					return InAssetData.AssetClassPath == AssetType->GetClassPathName();
				});
}

void SCameraFamilyAssetShortcut::HandleFilesLoaded()
{
	bRefreshAssetDatas = true;
}

void SCameraFamilyAssetShortcut::HandleAssetRemoved(const FAssetData& InAssetData)
{
	if (DoesFamilySupport(InAssetData))
	{
		bRefreshAssetDatas = true;
	}
}

void SCameraFamilyAssetShortcut::HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	if (DoesFamilySupport(InAssetData))
	{
		bRefreshAssetDatas = true;
	}
}

void SCameraFamilyAssetShortcut::HandleAssetAdded(const FAssetData& InAssetData)
{
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (!AssetRegistry.IsLoadingAssets())
	{
		if (DoesFamilySupport(InAssetData))
		{
			bRefreshAssetDatas = true;
		}
	}
}

void SCameraFamilyAssetShortcut::HandleCameraAssetBuilt(const UCameraAsset* InCameraAsset)
{
	bRefreshAssetDatas = true;
}

void SCameraFamilyAssetShortcut::HandleCameraRigAssetBuilt(const UCameraRigAsset* InCameraAsset)
{
	bRefreshAssetDatas = true;
}

} // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

