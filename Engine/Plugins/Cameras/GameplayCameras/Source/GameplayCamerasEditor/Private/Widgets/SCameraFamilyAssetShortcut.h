// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"

class FAssetThumbnail;
class FBaseAssetToolkit;
class SCheckBox;
class SComboButton;
class UCameraAsset;
class UCameraRigAsset;

namespace UE::Cameras
{

class IGameplayCamerasFamily;

class SCameraFamilyAssetShortcut : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraFamilyAssetShortcut)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FBaseAssetToolkit>& InToolkit, const TSharedRef<IGameplayCamerasFamily>& InFamily, UClass* InAssetType);
	~SCameraFamilyAssetShortcut();

protected:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

public:

	const FSlateBrush* GetAssetIcon() const;
	FSlateColor GetAssetTint() const;
	ECheckBoxState GetCheckState() const;
	FText GetButtonTooltip() const;
	void HandleButtonClick(ECheckBoxState InState);

	EVisibility GetSoloButtonVisibility() const;
	bool IsSoloButtonEnabled() const;

	EVisibility GetComboButtonVisibility() const;
	EVisibility GetComboDropdownVisibility() const;
	TSharedRef<SWidget> HandleGetDropdownMenuContent();
	void HandleOpenSecondaryAsset(const FAssetData& InAssetData);

	bool DoesFamilySupport(const FAssetData& InAssetData);

	void HandleFilesLoaded();
	void HandleAssetRemoved(const FAssetData& InAssetData);
	void HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath);
	void HandleAssetAdded(const FAssetData& InAssetData);

	void HandleCameraAssetBuilt(const UCameraAsset* InCameraAsset);
	void HandleCameraRigAssetBuilt(const UCameraRigAsset* InCameraAsset);

private:

	TWeakPtr<FBaseAssetToolkit> WeakToolkit;
	
	TSharedPtr<IGameplayCamerasFamily> Family;
	UClass* FamilyAssetType;

	TSharedPtr<SCheckBox> SoloCheckBox;
	TSharedPtr<SCheckBox> ComboCheckBox;
	TSharedPtr<SComboButton> ComboDropdown;

	TArray<FAssetData> AssetDatas;
	bool bRefreshAssetDatas = false;
};

}  // namespace UE::Cameras

