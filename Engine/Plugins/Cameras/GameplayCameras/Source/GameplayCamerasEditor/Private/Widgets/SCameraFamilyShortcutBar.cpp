// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCameraFamilyShortcutBar.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IAssetTools.h"
#include "IGameplayCamerasFamily.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCameraFamilyAssetShortcut.h"

#define LOCTEXT_NAMESPACE "SCameraFamilyShortcutBar"

namespace UE::Cameras
{

void SCameraFamilyShortcutBar::Construct(const FArguments& InArgs, const TSharedRef<FBaseAssetToolkit>& InToolkit, const TSharedRef<IGameplayCamerasFamily>& InFamily)
{
	WeakToolkit = InToolkit;

	Family = InFamily;

	HorizontalBox = SNew(SHorizontalBox);
	{
		BuildShortcuts();
	}

	ChildSlot
	[
		HorizontalBox.ToSharedRef()
	];
}

void SCameraFamilyShortcutBar::BuildShortcuts()
{
	TArray<UClass*> AssetTypes;
	Family->GetAssetTypes(AssetTypes);

	TSharedRef<FBaseAssetToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	for (UClass* Class : AssetTypes)
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 4.0f, 16.0f, 4.0f)
		[
			SNew(SCameraFamilyAssetShortcut, Toolkit, Family.ToSharedRef(), Class)
		];
	}
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

