// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDeleteCameraObjectDialog.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Helpers/ObjectReferenceFinder.h"
#include "IContentBrowserSingleton.h"
#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/ObjectRedirector.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDeleteCameraObjectDialog"

namespace UE::Cameras
{

void SDeleteCameraObjectDialog::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;

	ObjectsToDelete = InArgs._ObjectsToDelete;
	ObjectsToDelete.Remove(nullptr);

	State = (ObjectsToDelete.Num() > 0) ? EState::StartScanning : EState::Waiting;

	bRenameObjectsAsTrash = InArgs._RenameObjectsAsTrash.Get();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Green)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility(this, &SDeleteCameraObjectDialog::GetNoReferencesVisibility)
			.Padding(5.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT(
							"ObjectsOkToDelete", 
							"No assets reference the objects being deleted."))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Red)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility(this, &SDeleteCameraObjectDialog::GetReferencesVisiblity)
			.Padding(5.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT(
							"ObjectsPendingDeleteAreInUse", 
							"Some of the objects being deleted are referenced by other assets."))
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(5.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(0, 0, 0, 3))
			.Visibility(this, &SDeleteCameraObjectDialog::GetReferencesVisiblity)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
					.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
					.Padding(3.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT(
									"AssetsReferencingObjectsPendingDelete", 
									"Assets Referencing the Objects to Delete"))
						.Font(FAppStyle::GetFontStyle("BoldFont"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SDeleteCameraObjectDialog::BuildReferencerAssetPicker()
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 4.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(6, 0)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("Delete", "Delete"))
					.ToolTipText(LOCTEXT("DeleteTooltipText", "Perform the delete"))
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
					.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
					.IsEnabled(this, &SDeleteCameraObjectDialog::IsDeleteEnabled)
					.OnClicked(this, &SDeleteCameraObjectDialog::OnDeleteClicked)
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(6, 0)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("CancelDeleteTooltipText", "Cancel the delete"))
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
					.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
					.OnClicked(this, &SDeleteCameraObjectDialog::OnCancelClicked)
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SProgressBar)
			.Visibility(this, &SDeleteCameraObjectDialog::GetProgressBarVisibility)
			.Percent(this, &SDeleteCameraObjectDialog::GetProgressBarPercent)
		]
	];
}

bool SDeleteCameraObjectDialog::ShouldPerformDelete() const
{
	return bPerformDelete;
}

void SDeleteCameraObjectDialog::PerformReferenceReplacement()
{
	// Replace references to the objects we want to delete.
	//
	// We need to specify ObjectsToReplaceWithin otherwise the replacement also occurs inside the undo buffer!
	// Note that we also need to pass the assets, not the packages, in that list, otherwise ForceReplaceReferences
	// fails to recurse into the package (that's because it's using object references, and packages don't really
	// reference their asset directly).
	TArray<UPackage*> DirtiedPackages;
	TArray<UPackage*> PackagesToDeleteFrom;
	TSet<UObject*> ObjectsToReplaceWithin;
	for (const FName PackageName : ReferencingPackages)
	{
		UPackage* Package = FindPackage(nullptr, *PackageName.ToString());
		if (ensure(Package))
		{
			DirtiedPackages.Add(Package);

			UObject* PackageAsset = Package->FindAssetInPackage();
			if (ensure(PackageAsset))
			{
				ObjectsToReplaceWithin.Add(PackageAsset);
			}
		}
	}
	for (UObject* Object : ObjectsToDelete)
	{
		UPackage* Package = Object->GetOutermost();
		if (ensure(Package))
		{
			DirtiedPackages.Add(Package);
			PackagesToDeleteFrom.Add(Package);

			UObject* PackageAsset = Package->FindAssetInPackage();
			if (ensure(PackageAsset))
			{
				ObjectsToReplaceWithin.Add(PackageAsset);
			}
		}
	}
	ensure(ObjectsToReplaceWithin.Num() > 0);
	ObjectTools::ForceReplaceReferences(nullptr, ObjectsToDelete, ObjectsToReplaceWithin);

	// Prompt for checking out and saving changed packages.
	if (DirtiedPackages.Num() > 0)
	{
		FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
		SaveParams.bCheckDirty = false;
		SaveParams.bPromptToSave = true;
		SaveParams.bCanBeDeclined = false;
		SaveParams.bIsExplicitSave = true;

		FEditorFileUtils::PromptForCheckoutAndSave(ObjectPtrDecay(DirtiedPackages), SaveParams);
	}

	// Remove some flags that prevent objects from being collected.
	for (UObject* Object : ObjectsToDelete)
	{
		Object->ClearFlags(RF_Public | RF_Standalone);
	}

	// Remove any object redirectors. This should be safe since we deleted any references.
	TSet<UObject*> ObjectsToDeleteSet(ObjectsToDelete);
	for (UPackage* Package : PackagesToDeleteFrom)
	{
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(Package, ObjectsInPackage);
		for (UObject* Object : ObjectsInPackage)
		{
			UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object);
			if (Redirector && ObjectsToDeleteSet.Contains(Redirector->DestinationObject))
			{
				Redirector->ClearFlags(RF_Public | RF_Standalone);
				Redirector->DestinationObject = nullptr;
			}
		}
	}

	// Optionally rename objects with a TRASH prefix, helpful for debugging sometimes.
	if (bRenameObjectsAsTrash)
	{
		TStringBuilder<256> StringBuilder;
		for (UObject* Object : ObjectsToDelete)
		{
			StringBuilder.Reset();
			StringBuilder.Append("TRASH_");
			StringBuilder.Append(Object->GetName());
			Object->Rename(StringBuilder.ToString());
		}
	}

	// Call custom callback.
	if (OnDeletedObject.IsBound())
	{
		for (UObject* Object : ObjectsToDelete)
		{
			OnDeletedObject.Execute(Object);
		}
	}
}

void SDeleteCameraObjectDialog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	static const double MaxTickTime = 0.1;  // 100ms

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	const double StartTickTime = FPlatformTime::Seconds();
	while (State != EState::Waiting && FPlatformTime::Seconds() - StartTickTime < MaxTickTime)
	{
		switch (State)
		{
			case EState::Waiting:
				// We should never get here but some compiler emit warnings for missing case statements.
				ensure(false);
				break;
			case EState::StartScanning:
				StartScanning();
				break;
			case EState::ScanNextObject:
				ScanNextObject(AssetRegistry);
				break;
			case EState::ScanNextReferencingPackage:
				ScanNextReferencingPackage();
				break;
			case EState::FinishScanning:
				FinishScanning();
				break;
		}
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

FReply SDeleteCameraObjectDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		OnCancelClicked();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SDeleteCameraObjectDialog::StartScanning()
{
	ReferencingPackages.Reset();
	bIsAnyReferencedInMemoryByNonUndo = false;
	bIsAnyReferencedInMemoryByUndo = false;

	PossiblyReferencingPackages.Reset();
	NextObjectToScan = 0;
	NextPackageToScan = 0;

	State = ObjectsToDelete.Num() > 0 ? EState::ScanNextObject : EState::FinishScanning;
}

void SDeleteCameraObjectDialog::ScanNextObject(IAssetRegistry& AssetRegistry)
{
	if (!ensure(ObjectsToDelete.IsValidIndex(NextObjectToScan)))
	{
		State = EState::ScanNextReferencingPackage;
		return;
	}

	UObject* ObjectToScan = ObjectsToDelete[NextObjectToScan];
	NextObjectToScan++;
	if (NextObjectToScan >= ObjectsToDelete.Num())
	{
		State = EState::ScanNextReferencingPackage;
	}

	if (!ensure(ObjectToScan))
	{
		return;
	}

	UPackage* Package = ObjectToScan->GetOutermost();

	// Check on-disk references to the object's outer package.
	// We will have to later scan them properly to check if they are actually referencing the exact
	// object we are trying to delete.
	if (Package)
	{
		TArray<FName> OnDiskReferencers;
		AssetRegistry.GetReferencers(Package->GetFName(), OnDiskReferencers);
		PossiblyReferencingPackages.Append(OnDiskReferencers);
	}

	// Check in-memory references to the object.
	{
		bool bIsReferencedInMemoryByNonUndo = false;
		bool bIsReferencedInMemoryByUndo = false;
		FReferencerInformationList MemoryReferences;
		ObjectTools::GatherObjectReferencersForDeletion(
				ObjectToScan, bIsReferencedInMemoryByNonUndo, bIsReferencedInMemoryByUndo, &MemoryReferences);

		UPackage* TransientPackage = GetTransientPackage();
		for (const FReferencerInformation& ExternalReference : MemoryReferences.ExternalReferences)
		{
			UPackage* ExternalReferencePackage = ExternalReference.Referencer->GetOutermost();
			if (ExternalReferencePackage != Package && ExternalReferencePackage != TransientPackage)
			{
				ReferencingPackages.Add(ExternalReferencePackage->GetFName());
			}
		}

		bIsAnyReferencedInMemoryByNonUndo |= bIsReferencedInMemoryByNonUndo;
		bIsAnyReferencedInMemoryByUndo |= bIsReferencedInMemoryByUndo;
	}
}

void SDeleteCameraObjectDialog::ScanNextReferencingPackage()
{
	// If this is the first call, initialize the list of packages to scan.
	// But if this is the first call and there's nothing to do, we can immediately
	// skip to the next step.
	if (ReferencingPackagesToScan.IsEmpty())
	{
		if (!PossiblyReferencingPackages.IsEmpty())
		{
			ReferencingPackagesToScan = PossiblyReferencingPackages.Array();
		}
		else
		{
			State = EState::FinishScanning;
			return;
		}
	}

	if (!ensure(ReferencingPackagesToScan.IsValidIndex(NextPackageToScan)))
	{
		State = EState::FinishScanning;
		return;
	}

	const FName ReferencingPackageName = ReferencingPackagesToScan[NextPackageToScan];
	NextPackageToScan++;
	if (NextPackageToScan >= ReferencingPackagesToScan.Num())
	{
		State = EState::FinishScanning;
	}

	UPackage* Package = FindPackage(nullptr, *ReferencingPackageName.ToString());
	if (!ensure(Package))
	{
		return;
	}

	UObject* PackageAsset = Package->FindAssetInPackage();
	if (!ensure(PackageAsset))
	{
		return;
	}

	FObjectReferenceFinder ReferenceFinder(PackageAsset, ObjectsToDelete);
	ReferenceFinder.CollectReferences();
	if (ReferenceFinder.HasAnyObjectReference())
	{
		// That package does reference one of the objects we want to delete. Keep it.
		ReferencingPackages.Add(ReferencingPackageName);
	}
}

void SDeleteCameraObjectDialog::FinishScanning()
{
	State = EState::Waiting;
}

TOptional<float> SDeleteCameraObjectDialog::GetProgressBarPercent() const
{
	switch (State)
	{
		case EState::Waiting:
		default:
			return TOptional<float>();
		case EState::StartScanning:
			return 0.f;
		case EState::ScanNextObject:
			return ((float)NextObjectToScan / (float)FMath::Max(1, ObjectsToDelete.Num())) * 0.5f;
		case EState::ScanNextReferencingPackage:
			return ((float)NextPackageToScan / (float)FMath::Max(1, ReferencingPackagesToScan.Num())) * 0.5f + 0.5f;
		case EState::FinishScanning:
			return 1.f;
	}
}

EVisibility SDeleteCameraObjectDialog::GetNoReferencesVisibility() const
{
	return ReferencingPackages.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDeleteCameraObjectDialog::GetReferencesVisiblity() const
{
	return ReferencingPackages.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SDeleteCameraObjectDialog::GetProgressBarVisibility() const
{
	return State == EState::Waiting ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget> SDeleteCameraObjectDialog::BuildReferencerAssetPicker()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.bAutohideSearchBar = true;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SDeleteCameraObjectDialog::OnShouldFilterReferencerAsset);
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SDeleteCameraObjectDialog::OnAssetsActivated);

	return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
}

bool SDeleteCameraObjectDialog::OnShouldFilterReferencerAsset(const FAssetData& InAssetData) const
{
	if (ReferencingPackages.Contains(InAssetData.PackageName))
	{
		return false;
	}
	return true;
}

void SDeleteCameraObjectDialog::OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod)
{
	if (ActivationMethod == EAssetTypeActivationMethod::DoubleClicked || ActivationMethod == EAssetTypeActivationMethod::Opened)
	{
		CloseWindow();

		for (const FAssetData& ActivatedAsset : ActivatedAssets)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ActivatedAsset.GetAsset());
		}
	}
}

bool SDeleteCameraObjectDialog::IsDeleteEnabled() const
{
	return State == EState::Waiting;
}

FReply SDeleteCameraObjectDialog::OnDeleteClicked()
{
	bPerformDelete = true;
	CloseWindow();
	return FReply::Handled();
}

FReply SDeleteCameraObjectDialog::OnCancelClicked()
{
	bPerformDelete = false;
	CloseWindow();
	return FReply::Handled();
}

void SDeleteCameraObjectDialog::CloseWindow()
{
	if (TSharedPtr<SWindow> ParentWindow = WeakParentWindow.Pin())
	{
		ParentWindow->RequestDestroyWindow();
	}
}

void SDeleteCameraObjectDialog::RenameObjectAsTrash(FString& InOutName)
{
	if (!InOutName.IsEmpty())
	{
		TStringBuilder<256> StringBuilder;
		StringBuilder.Append(TEXT("TRASH_"));
		StringBuilder.Append(InOutName);
		InOutName = StringBuilder.ToString();
	}
}

FString SDeleteCameraObjectDialog::MakeTrashName(const FString& InName)
{
	FString Result(InName);
	RenameObjectAsTrash(Result);
	return Result;
}

}  // namespace UE::Cameras

#undef LOCTEXT_NAMESPACE

