// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraAssetEditor.generated.h"

class FBaseAssetToolkit;
class UCameraAsset;

/**
 * Editor for a camera asset.
 */
UCLASS(Transient)
class UCameraAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UCameraAsset> InCameraAsset);

	UCameraAsset* GetCameraAsset() const { return CameraAsset; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	UPROPERTY()
	TObjectPtr<UCameraAsset> CameraAsset;
};

