// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraRigAssetEditor.generated.h"

class FBaseAssetToolkit;
class UCameraRigAsset;

/**
 * Editor for a camera rig asset.
 */
UCLASS(Transient)
class UCameraRigAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UCameraRigAsset> InCameraRigAsset);

	UCameraRigAsset* GetCameraRigAsset() const { return CameraRigAsset; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	UPROPERTY()
	TObjectPtr<UCameraRigAsset> CameraRigAsset;
};

