// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraRigProxyAssetEditor.generated.h"

class FBaseAssetToolkit;
class UCameraRigProxyAsset;

/**
 * Editor for a camera rig proxy asset.
 */
UCLASS(Transient)
class UCameraRigProxyAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UCameraRigProxyAsset> InCameraRigProxyAsset);

	UCameraRigProxyAsset* GetCameraRigProxyAsset() const { return CameraRigProxyAsset; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	UPROPERTY()
	TObjectPtr<UCameraRigProxyAsset> CameraRigProxyAsset;
};

