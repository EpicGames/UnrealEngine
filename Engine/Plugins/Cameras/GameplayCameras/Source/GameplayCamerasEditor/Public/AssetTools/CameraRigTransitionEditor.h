// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraRigTransitionEditor.generated.h"

class FBaseAssetToolkit;

/**
 * Editor for camera transitions.
 */
UCLASS(Transient)
class UCameraRigTransitionEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UObject> InTransitionOwner);

	UObject* GetTransitionOwner() const { return TransitionOwner; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	UPROPERTY()
	TObjectPtr<UObject> TransitionOwner;
};

