// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraShakeAssetEditor.generated.h"

class FBaseAssetToolkit;
class UCameraShakeAsset;

/**
 * Editor for a camera rig asset.
 */
UCLASS(Transient)
class UCameraShakeAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UCameraShakeAsset> InCameraShakeAsset);

	UCameraShakeAsset* GetCameraShakeAsset() const { return CameraShakeAsset; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	UPROPERTY()
	TObjectPtr<UCameraShakeAsset> CameraShakeAsset;
};

