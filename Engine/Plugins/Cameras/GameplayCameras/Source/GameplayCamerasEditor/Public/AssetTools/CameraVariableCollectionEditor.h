// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "CameraVariableCollectionEditor.generated.h"

class FBaseAssetToolkit;
class UCameraVariableCollection;

/**
 * Editor for a camera variable collection.
 */
UCLASS(Transient)
class UCameraVariableCollectionEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	void Initialize(TObjectPtr<UCameraVariableCollection> InVariableCollection);

	UCameraVariableCollection* GetVariableCollection() const { return VariableCollection; }

public:

	// UAssetEditor interface
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:

	TObjectPtr<UCameraVariableCollection> VariableCollection;
};

