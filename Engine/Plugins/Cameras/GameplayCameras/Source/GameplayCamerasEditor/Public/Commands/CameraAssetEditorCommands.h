// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"

class FUICommandInfo;
class UClass;
struct FInputChord;

namespace UE::Cameras
{

class FCameraAssetEditorCommands : public TCommands<FCameraAssetEditorCommands>
{
public:

	FCameraAssetEditorCommands();

	virtual void RegisterCommands() override;
	
public:

	TSharedPtr<FUICommandInfo> Build;

	TSharedPtr<FUICommandInfo> ShowCameraDirector;
	TSharedPtr<FUICommandInfo> ShowCameraRigs;
	TSharedPtr<FUICommandInfo> ShowSharedTransitions;

	TSharedPtr<FUICommandInfo> ChangeCameraDirector;

	TSharedPtr<FUICommandInfo> ShowMessages;
	TSharedPtr<FUICommandInfo> FindInCamera;

	TSharedPtr<FUICommandInfo> RenameInterfaceParameter;
	TSharedPtr<FUICommandInfo> DeleteInterfaceParameter;
};

}  // namespace UE::Cameras

