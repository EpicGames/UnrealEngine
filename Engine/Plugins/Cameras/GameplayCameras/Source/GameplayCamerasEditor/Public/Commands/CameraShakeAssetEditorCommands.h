// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"

class FUICommandInfo;
class UClass;
struct FInputChord;

namespace UE::Cameras
{

class FCameraShakeAssetEditorCommands : public TCommands<FCameraShakeAssetEditorCommands>
{
public:

	FCameraShakeAssetEditorCommands();

	virtual void RegisterCommands() override;
	
public:

	TSharedPtr<FUICommandInfo> Build;
	
	TSharedPtr<FUICommandInfo> ShowMessages;
	TSharedPtr<FUICommandInfo> FindInCameraShake;
	TSharedPtr<FUICommandInfo> FocusHome;

	TSharedPtr<FUICommandInfo> RenameInterfaceParameter;
	TSharedPtr<FUICommandInfo> DeleteInterfaceParameter;
};

}  // namespace UE::Cameras

