// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"

class FUICommandInfo;
class UClass;
struct FInputChord;

namespace UE::Cameras
{

class FGameplayCamerasDebuggerCommands : public TCommands<FGameplayCamerasDebuggerCommands>
{
public:

	FGameplayCamerasDebuggerCommands();

	virtual void RegisterCommands() override;
	
public:

	TSharedPtr<FUICommandInfo> EnableDebugInfo;
};

}  // namespace UE::Cameras

