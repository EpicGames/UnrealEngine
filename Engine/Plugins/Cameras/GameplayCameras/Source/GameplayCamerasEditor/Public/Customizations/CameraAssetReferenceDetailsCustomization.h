// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;
class UCameraAsset;

namespace UE::Cameras
{

class FCameraAssetReferenceDetailsCustomization : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:

	~FCameraAssetReferenceDetailsCustomization();

	// IPropertyTypeCustomization interface.
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

private:

	void OnCameraAssetBuilt(const UCameraAsset* InCameraAsset);
	void RebuildParametersIfNeeded();

private:

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> CameraAssetPropertyHandle;
	TSharedPtr<IPropertyHandle> ParametersPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};

}  // namespace UE::Cameras

