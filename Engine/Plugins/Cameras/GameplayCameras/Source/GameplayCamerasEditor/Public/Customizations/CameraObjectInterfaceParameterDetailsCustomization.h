// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyUtilities;
class UBaseCameraObject;

namespace UE::Cameras
{

/**
 * Details customization for a camera object parameter.
 */
class FCameraObjectInterfaceParameterDetailsCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

public:

	// IDetailCustomization interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

}  // namespace UE::Cameras

