// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BaseCameraObject.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"

namespace UE::Cameras
{

/**
 * Helper class for detail customizations that want to show the camera object interface's parameters
 * in the UI as a property bag of overridable properties.
 */
template<typename AssetReferenceStruct>
class TCameraObjectInterfaceParameterOverrideDataDetails : public FPropertyBagInstanceDataDetails
{
public:

	TCameraObjectInterfaceParameterOverrideDataDetails(
			TSharedPtr<IPropertyHandle> InStructPropertyHandle,
			TSharedPtr<IPropertyHandle> InParametersPropertyHandle,
			TSharedPtr<IPropertyUtilities>& InPropertyUtilities)
		: FPropertyBagInstanceDataDetails(InParametersPropertyHandle, InPropertyUtilities, true)
		, StructPropertyHandle(InStructPropertyHandle)
	{
	}

protected:

	struct FOverrideProvider : public IPropertyBagOverrideProvider
	{
		FOverrideProvider(AssetReferenceStruct& InAssetReference)
			: AssetReference(InAssetReference)
		{
		}
		
		virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
		{
			return AssetReference.IsParameterOverridden(PropertyID);
		}
		
		virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
		{
			AssetReference.SetParameterOverridden(PropertyID, bIsOverridden);
		}

	private:
		AssetReferenceStruct& AssetReference;
	};

	virtual bool HasPropertyOverrides() const override
	{
		return true;
	}

	virtual void PreChangeOverrides() override
	{
		StructPropertyHandle->NotifyPreChange();
	}

	virtual void PostChangeOverrides() override
	{
		StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructPropertyHandle->NotifyFinishedChangingProperties();
	}

	virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override
	{
		StructPropertyHandle->EnumerateRawData([Func](void* RawData, const int32 DataIndex, const int32 NumDatas)
		{
			if (AssetReferenceStruct* AssetReference = static_cast<AssetReferenceStruct*>(RawData))
			{
				if (const UBaseCameraObject* CameraObject = AssetReference->GetCameraObject())
				{
					const FInstancedPropertyBag& DefaultParameters = CameraObject->GetDefaultParameters();
					FInstancedPropertyBag& Parameters = AssetReference->GetParameters();
					FOverrideProvider OverrideProvider(*AssetReference);
					if (!Func(DefaultParameters, Parameters, OverrideProvider))
					{
						return false;
					}
				}
			}
			return true;
		});
	}

private:
	
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};

}  // namespace UE::Cameras


