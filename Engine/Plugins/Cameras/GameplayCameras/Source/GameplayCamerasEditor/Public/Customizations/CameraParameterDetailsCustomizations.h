// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraParameters.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "TickableEditorObject.h"
#include "Types/SlateStructs.h"

class FPropertyEditorModule;
class IDetailLayoutBuilder;
class IPropertyUtilities;
class SComboButton;
class SHorizontalBox;
class SWidget;

namespace UE::Cameras
{

/**
 * Base details customization for camera parameters.
 */
class FCameraParameterDetailsCustomization 
	: public IPropertyTypeCustomization
	, public FTickableEditorObject
{
public:

	/** Registers details customizations for all camera parameter types. */
	static void Register(FPropertyEditorModule& PropertyEditorModule);
	/** Unregisters details customizations for all camera parameter types. */
	static void Unregister(FPropertyEditorModule& PropertyEditorModule);

public:

	// IPropertyTypeCustomization interface.
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	// FTickableEditorObject interface.
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FCameraParameterDetailsCustomization, STATGROUP_Tickables); }

protected:

	virtual bool HasOverride(void* InRawData) = 0;
	virtual bool HasNonUserOverride(void* InRawData) = 0;
	virtual void SetParameterVariable(void* InRawData, UCameraVariableAsset* InVariable) = 0;

private:

	enum class ECameraVariableValue
	{
		NotSet,
		Set,
		MultipleSet,
		Invalid
	};

	struct FCameraVariableInfo
	{
		UCameraVariableAsset* CommonVariable = nullptr;
		ECameraVariableValue VariableValue = ECameraVariableValue::NotSet;
		bool bHasNonUserOverride = false;
		bool bHasOverride = false;

		FText InfoText;
		FText ErrorText;
	};

	void UpdateVariableInfo();

	TSharedRef<SWidget> BuildCameraVariableBrowser();

	bool IsValueEditorEnabled() const;
	bool IsCameraVariableBrowserEnabled() const;
	FText GetCameraVariableBrowserToolTip() const;

	FText GetVariableInfoText() const;
	EVisibility GetVariableInfoTextVisibility() const;
	FOptionalSize GetVariableInfoTextMaxWidth() const;

	FText GetVariableErrorText() const;
	EVisibility GetVariableErrorTextVisibility() const;
	FOptionalSize GetVariableErrorTextMaxWidth() const;

	bool CanGoToVariable() const;
	void OnGoToVariable();

	bool CanClearVariable() const;
	void OnClearVariable();

	void OnSetVariable(UCameraVariableAsset* InVariable);

	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const;
	void OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

protected:

	UClass* VariableClass = nullptr;

	FCameraVariableInfo VariableInfo;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> ValueProperty;
	TSharedPtr<IPropertyHandle> VariableProperty;

	TSharedPtr<SHorizontalBox> LayoutBox;
	TSharedPtr<SComboButton> VariableBrowserButton;
};

// Create all the individual classes.
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
class F##ValueName##CameraParameterDetailsCustomization : public FCameraParameterDetailsCustomization\
{\
protected:\
	virtual bool HasOverride(void* InRawData) override;\
	virtual bool HasNonUserOverride(void* InRawData) override;\
	virtual void SetParameterVariable(void* InRawData, UCameraVariableAsset* InVariable) override;\
};
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

}  // namespace UE::Cameras
 
