// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;
class UCameraShakeAsset;

namespace UE::Cameras
{

class FCameraShakeAssetReferenceDetailsCustomization : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:

	~FCameraShakeAssetReferenceDetailsCustomization();

	// IPropertyTypeCustomization interface.
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

private:

	void OnCameraShakeAssetBuilt(const UCameraShakeAsset* CameraShake);
	void RebuildParametersIfNeeded();

private:

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> CameraShakeAssetPropertyHandle;
	TSharedPtr<IPropertyHandle> ParametersPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};

}  // namespace UE::Cameras

