// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableReferences.h"
#include "IPropertyTypeCustomization.h"

class FPropertyEditorModule;
class SComboButton;
class SWidget;
class UCameraVariableAsset;

namespace UE::Cameras
{

/**
 * Base details customization for camera variable references.
 */
class FCameraVariableReferenceDetailsCustomization : public IPropertyTypeCustomization
{
public:

	/** Registers details customizations for all camera variable reference types. */
	static void Register(FPropertyEditorModule& PropertyEditorModule);
	/** Unregisters details customizations for all camera variable reference types. */
	static void Unregister(FPropertyEditorModule& PropertyEditorModule);

public:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

protected:

	virtual bool HasNonUserOverride(void* InRawData) const = 0;
	virtual void SetReferenceVariable(void* InRawData, UCameraVariableAsset* InVariable) = 0;

private:

	TSharedRef<SWidget> BuildCameraVariableBrowser();

	bool IsCameraVariableBrowserEnabled() const;

	FText GetVariableName() const;

	bool CanClearVariable() const;
	void OnClearVariable();

	void OnSetVariable(UCameraVariableAsset* InVariable);
	void OnResetToDefault();

protected:

	UClass* VariableClass = nullptr;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> VariableProperty;

	TSharedPtr<SComboButton> VariableBrowserButton;
};

// Create all the individual classes.
#define UE_CAMERA_VARIABLE_FOR_TYPE(ValueType, ValueName)\
class F##ValueName##CameraVariableReferenceDetailsCustomization : public FCameraVariableReferenceDetailsCustomization\
{\
protected:\
	virtual bool HasNonUserOverride(void* InRawData) const override;\
	virtual void SetReferenceVariable(void* InRawData, UCameraVariableAsset* InVariable) override;\
};
UE_CAMERA_VARIABLE_FOR_ALL_TYPES()
#undef UE_CAMERA_VARIABLE_FOR_TYPE

}  // namespace UE::Cameras
 

