// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/Input/SComboBox.h"

class IPropertyHandle;
class SComboButton;
class SWidget;

namespace UE::Cameras
{

/**
 * Details customization for the filmback camera node.
 */
class FFilmbackCameraNodeDetailsCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

public:

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	void BuildPresetComboList();

	bool IsPresetEnabled() const;
	TSharedRef<SWidget> MakePresetComboWidget(TSharedPtr<FText> InItem);
	void OnPresetChanged(TSharedPtr<FText> NewSelection, ESelectInfo::Type SelectInfo);

	FText GetPresetComboBoxContent() const;

private:

	TSharedPtr<IPropertyHandle> SensorWidthProperty;
	TSharedPtr<IPropertyHandle> SensorHeightProperty;

	TSharedPtr<SComboBox<TSharedPtr<FText>>> PresetComboBox;
	TArray<TSharedPtr<FText>> PresetComboList;
};

}  // namespace UE::Cameras

