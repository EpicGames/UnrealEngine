// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointerFwd.h"

class FCurveEditor;
class FCurveModel;
class FPropertyEditorModule;
class SHorizontalBox;
class SWidget;

namespace UE::Cameras
{

class SRichCurveViewport;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInvokeCurveEditor, UObject*, FName);

class FRichCurveDetailsCustomization : public IPropertyTypeCustomization
{
public:

	static void Register(FPropertyEditorModule& PropertyEditorModule);
	static void Unregister(FPropertyEditorModule& PropertyEditorModule);

	static FOnInvokeCurveEditor& OnInvokeCurveEditor() { return OnInvokeCurveEditorDelegate; }

private:

	static FOnInvokeCurveEditor OnInvokeCurveEditorDelegate;

public:

	~FRichCurveDetailsCustomization();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

protected:

	virtual void AddCurves(
			TSharedRef<IPropertyHandle> PropertyHandle, 
			TSharedRef<SRichCurveViewport> InRichCurveViewport,
			const FText& PropertyDisplayName,
			UObject* OuterObject,
			void* RawData) = 0;

private:

	void OnPropertyValueChanged();
	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	FReply OnFocusInCurvesTab();

	TSharedRef<FCurveEditor> CreateCurveEditor();
	TSharedRef<SWidget> CreateCurveEditorPanel(TSharedRef<FCurveEditor> CurveEditor);

private:

	TSharedPtr<IPropertyHandle> PrivatePropertyHandle;
	TSharedPtr<SHorizontalBox> HeaderLayout;
	TSharedPtr<SRichCurveViewport> RichCurveViewport;
};

class FRichSingleCurveDetailsCustomization : public FRichCurveDetailsCustomization
{
protected:
	virtual void AddCurves(
			TSharedRef<IPropertyHandle> PropertyHandle, 
			TSharedRef<SRichCurveViewport> InRichCurveViewport,
			const FText& PropertyDisplayName,
			UObject* OuterObject,
			void* RawData) override;
};

class FRichRotatorCurveDetailsCustomization : public FRichCurveDetailsCustomization
{
protected:
	virtual void AddCurves(
			TSharedRef<IPropertyHandle> PropertyHandle, 
			TSharedRef<SRichCurveViewport> InRichCurveViewport,
			const FText& PropertyDisplayName,
			UObject* OuterObject,
			void* RawData) override;
};

class FRichVectorCurveDetailsCustomization : public FRichCurveDetailsCustomization
{
protected:
	virtual void AddCurves(
			TSharedRef<IPropertyHandle> PropertyHandle, 
			TSharedRef<SRichCurveViewport> InRichCurveViewport,
			const FText& PropertyDisplayName,
			UObject* OuterObject,
			void* RawData) override;
};

}  // namespace UE::Cameras
 
