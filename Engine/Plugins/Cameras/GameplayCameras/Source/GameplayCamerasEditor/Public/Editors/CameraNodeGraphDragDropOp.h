// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class SGraphEditor;
class UCameraObjectInterfaceParameterBase;

class FCameraNodeGraphInterfaceParameterDragDropOp : public FDecoratedDragDropOp
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FCameraNodeGraphInterfaceParameterDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FCameraNodeGraphInterfaceParameterDragDropOp> New(UCameraObjectInterfaceParameterBase* InInterfaceParameter);

	FReply ExecuteDragOver(TSharedPtr<SGraphEditor> GraphEditor);
	FReply ExecuteDrop(TSharedPtr<SGraphEditor> GraphEditor, const FSlateCompatVector2f& NewLocation);

private:

	UCameraObjectInterfaceParameterBase* InterfaceParameter;
};

