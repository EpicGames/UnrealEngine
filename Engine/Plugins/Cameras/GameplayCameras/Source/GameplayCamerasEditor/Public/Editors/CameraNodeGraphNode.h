// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/ObjectTreeGraphNode.h"

#include "CameraNodeGraphNode.generated.h"

class UCameraNode;

/**
 * Custom graph node for camera nodes. They mostly differ by showing input pins for any 
 * camera parameter property.
 */
UCLASS()
class UCameraNodeGraphNode : public UObjectTreeGraphNode
{
	GENERATED_BODY()

public:

	/** Creates a new graph node. */
	UCameraNodeGraphNode(const FObjectInitializer& ObjInit);

	virtual void BeginDestroy() override;

public:

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	// UObjectTreeGraphNode interface.
	virtual void OnInitialize() override;

private:

	void OnCustomCameraNodeParametersChanged(const UCameraNode* CameraNode);
};

