// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "Core/CameraObjectInterface.h"
#include "Core/CameraObjectInterfaceParameterDefinition.h"
#include "EdGraph/EdGraphPin.h"
#include "Editors/CameraNodeGraphPinColors.h"
#include "Editors/ObjectTreeGraphSchema.h"

#include "CameraObjectGraphSchemaBase.generated.h"

class UCameraObjectInterfaceParameterGraphNode;
struct FObjectTreeGraphConfig;

/**
 * Base schema class for any kind of camera node graph.
 */
UCLASS()
class UCameraObjectGraphSchemaBase : public UObjectTreeGraphSchema
{
	GENERATED_BODY()

public:

	static const FName PC_CameraParameter;			// A camera parameter pin.
	static const FName PC_CameraVariableReference;	// A variable reference pin.
	static const FName PC_CameraContextData;		// A context data pin.

	UCameraObjectGraphSchemaBase(const FObjectInitializer& ObjInit);

	/** Builds the config for a graph managed by this schema. */
	FObjectTreeGraphConfig BuildGraphConfig() const;

protected:

	// UEdGraphSchema interface.
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool OnBreakCustomPinLinks(UEdGraphPin& TargetPin) const;
	virtual bool OnBreakSingleCustomPinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;

	// UObjectTreeGraphSchema interface.
	virtual void OnCreateAllNodes(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const override;
	virtual bool OnTryCreateCustomConnection(UEdGraphPin* A, UEdGraphPin* B) const;

protected:

	// UCameraObjectGraphSchemaBase interface.
	virtual void OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const {}

private:

	void CreateValueFlowConnections(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const;
	UEdGraphPin* FindPinByType(UEdGraphNode* InNode, const FName& InPinCategory) const;

	UE::Cameras::FCameraNodeGraphPinColors PinColors;
};

/**
 * Graph editor action for adding a new camera rig parameter, plus an associated getter node in the graph.
 */
USTRUCT()
struct FCameraObjectGraphSchemaAction_NewInterfaceParameterNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	/** The new parameter's definition. */
	UPROPERTY()
	FCameraObjectInterfaceParameterDefinition ParameterDefinition;

public:

	FCameraObjectGraphSchemaAction_NewInterfaceParameterNode();
	FCameraObjectGraphSchemaAction_NewInterfaceParameterNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping = 0, FText InKeywords = FText());

public:

	// FEdGraphSchemaAction interface.
	static FName StaticGetTypeId() { static FName Type("FCameraObjectGraphSchemaAction_NewInterfaceParameterNode"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode = true) override;
};

/**
 * Graph editor action for adding a new getter node for an existing camera rig parameter.
 */
USTRUCT()
struct FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()
	
public:

	/** The existing camera rig parameter to create a getter node for. */
	UPROPERTY()
	TObjectPtr<UCameraObjectInterfaceParameterBase> InterfaceParameter;

public:
	
	FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode();
	FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping = 0, FText InKeywords = FText());

public:

	// FEdGraphSchemaAction interface.
	static FName StaticGetTypeId() { static FName Type("FCameraObjectGraphSchemaAction_AddInterfaceParameterGetterNode"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode = true) override;
};

