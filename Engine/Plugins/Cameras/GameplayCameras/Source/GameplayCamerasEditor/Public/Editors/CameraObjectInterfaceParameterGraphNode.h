// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/ObjectTreeGraphNode.h"

#include "CameraObjectInterfaceParameterGraphNode.generated.h"

class UCameraObjectInterfaceParameterBase;

/**
 * Custom graph editor node for a camera rig parameter getter.
 */
UCLASS()
class UCameraObjectInterfaceParameterGraphNode : public UObjectTreeGraphNode
{
	GENERATED_BODY()

public:

	/** Creates a new graph node. */
	UCameraObjectInterfaceParameterGraphNode(const FObjectInitializer& ObjInit);

	/** Finds the camera interface parameter. */
	UCameraObjectInterfaceParameterBase* GetInterfaceParameter() const;

public:

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
};

