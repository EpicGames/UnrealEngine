// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/CameraObjectGraphSchemaBase.h"

#include "CameraRigCameraNodeGraphSchema.generated.h"

/**
 * Schema class for camera node graph.
 */
UCLASS()
class UCameraRigCameraNodeGraphSchema : public UCameraObjectGraphSchemaBase
{
	GENERATED_BODY()

public:

	UCameraRigCameraNodeGraphSchema(const FObjectInitializer& ObjInit);

protected:

	// UObjectTreeGraphSchema interface.
	virtual void CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const override;

	// UCameraObjectGraphSchemaBase interface.
	virtual void OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const override;
};

