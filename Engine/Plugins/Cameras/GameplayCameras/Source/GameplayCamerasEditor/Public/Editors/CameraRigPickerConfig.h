// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Text.h"

class IPropertyHandle;
class UCameraAsset;
class UCameraRigAsset;

namespace UE::Cameras
{

DECLARE_DELEGATE_OneParam(FOnCameraRigSelected, UCameraRigAsset*);

/**
 * Configuration structure for a camera rig picker widget.
 * This is a widget that shows the list of camera rigs inside a given
 * camera asset.
 *
 * See IGameplayCamerasEditorModule for creating that widget.
 */
struct FCameraRigPickerConfig
{
	/** The initially selected camera asset, if any. */
	FAssetData InitialCameraAssetSelection;

	/** The initially selected camera rig, specified as a pointer, if any. */
	UCameraRigAsset* InitialCameraRigSelection = nullptr;

	/** The initially selected camera rig, specified by its Guid, if any. */
	FGuid InitialCameraRigSelectionGuid;

	/** 
	 * Whether a camera asset can be picked, or whether InitialCameraAssetSelection
	 * determines the only asset to be used. If the latter (when the value is false),
	 * then no camera asset picker is shown. Only the list of rigs is shown.
	 */
	bool bCanSelectCameraAsset = true;

	/** Asset picker view type for the camera asset picker. */
	EAssetViewType::Type CameraAssetViewType = EAssetViewType::List;

	/** Asset picker settings name for the camera asset picker. */
	FString CameraAssetSaveSettingsName;

	/** Whether the camera rig search box should be focused initially. */
	bool bFocusCameraRigSearchBoxWhenOpened = true;

	/** An optional warning message to display. */
	FText WarningMessage;

	/** An optional error message to display. */
	FText ErrorMessage;

	/** Callback for when a camera rig has been selected. */
	FOnCameraRigSelected OnCameraRigSelected;

	/** 
	 * An optional property handle to set the selected camera rig on.
	 * Properties of type FGuid and UCameraRigAsset* are handled.
	 */
	TSharedPtr<IPropertyHandle> PropertyToSet;
};

}  // namespace UE::Cameras

