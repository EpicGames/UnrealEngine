// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/CameraRigTransitionGraphSchemaBase.h"

#include "CameraRigTransitionGraphSchema.generated.h"

/**
 * Schema class for a camera rig's transition graph.
 */
UCLASS()
class UCameraRigTransitionGraphSchema : public UCameraRigTransitionGraphSchemaBase
{
	GENERATED_BODY()

protected:

	// UCameraRigTransitionGraphSchemaBase interface.
	virtual void OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const override;
};

