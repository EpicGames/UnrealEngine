// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/CameraObjectGraphSchemaBase.h"

#include "CameraRigTransitionGraphSchemaBase.generated.h"

struct FObjectTreeGraphConfig;

/**
 * Base schema class for camera transition graph.
 */
UCLASS()
class UCameraRigTransitionGraphSchemaBase : public UCameraObjectGraphSchemaBase
{
	GENERATED_BODY()

protected:

	// UObjectTreeGraphSchema interface.
	virtual void CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const override;

	// UCameraObjectGraphSchemaBase interface.
	virtual void OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const override;
};

