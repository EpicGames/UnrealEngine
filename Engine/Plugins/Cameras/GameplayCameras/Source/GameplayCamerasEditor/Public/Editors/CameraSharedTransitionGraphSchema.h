// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/CameraRigTransitionGraphSchemaBase.h"

#include "CameraSharedTransitionGraphSchema.generated.h"

/**
 * Schema class for a camera asset's shared transition graph.
 */
UCLASS()
class UCameraSharedTransitionGraphSchema : public UCameraRigTransitionGraphSchemaBase
{
	GENERATED_BODY()

protected:

	// UCameraRigTransitionGraphSchemaBase interface.
	virtual void OnBuildGraphConfig(FObjectTreeGraphConfig& InOutGraphConfig) const override;
};

