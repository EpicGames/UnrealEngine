// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IContentBrowserSingleton.h"

class IPropertyHandle;
class UCameraVariableAsset;
class UCameraVariableCollection;

namespace UE::Cameras
{

DECLARE_DELEGATE_OneParam(FOnCameraVariableSelected, UCameraVariableAsset*);

/**
 * Configuration structure for a camera variable picker widget.
 * This is a widget that shows the list of camera variables inside a given
 * camera variable collection. It can be filtered by variable type.
 *
 * See IGameplayCamerasEditorModule for creating that widget.
 */
struct FCameraVariablePickerConfig
{
	/** The initially selected camera variable collection, if any. */
	FAssetData InitialCameraVariableCollectionSelection;

	/** 
	 * The initially selected camera variable, if any.
	 * When set, InitialCameraVariableCollectionSelection is ignored, and the outer
	 * collection of the selected variable is used instead.
	 */
	UCameraVariableAsset* InitialCameraVariableSelection = nullptr;

	/** The type of variable to select. */
	UClass* CameraVariableClass = nullptr;

	/** Asset picker view type for the camera variable collection picker. */
	EAssetViewType::Type CameraAssetViewType = EAssetViewType::List;

	/** Asset picker settings name for the camera variable collection picker. */
	FString CameraVariableCollectionSaveSettingsName;

	/** Callback for when a camera variable has been selected. */
	FOnCameraVariableSelected OnCameraVariableSelected;
};

}  // namespace UE::Cameras

