// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class SGraphPin;
class UK2Node_CallFunction;

namespace UE::Cameras
{

/**
 * Graph editor pin factory for camera-specific pin widgets.
 */
struct FGameplayCamerasGraphPanelPinFactory : public FGraphPanelPinFactory
{
public:

	// FGraphPanelPinFactory interface.
	virtual TSharedPtr<SGraphPin> CreatePin(UEdGraphPin* Pin) const override;
};

}  // namespace UE::Cameras

