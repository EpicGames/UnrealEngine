// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class UObjectTreeGraph;
class SGraphEditor;

/**
 * Drag-drop operation for creating a new object (and corresponding graph node) in an object tree graph
 * by dragging one of the entries from the toolbox widget.
 */
class FObjectTreeClassDragDropOp : public FDecoratedDragDropOp
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FObjectTreeClassDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FObjectTreeClassDragDropOp> New(UClass* InObjectClass);
	static TSharedRef<FObjectTreeClassDragDropOp> New(TArrayView<UClass*> InObjectClasses);

	TArrayView<UClass* const> GetObjectClasses() const { return ObjectClasses; }

	FReply ExecuteDragOver(TSharedPtr<SGraphEditor> GraphEditor);
	FReply ExecuteDrop(TSharedPtr<SGraphEditor> GraphEditor, const FSlateCompatVector2f& NewLocation);

private:

	TArray<UClass*> FilterPlaceableObjectClasses(UObjectTreeGraph* InGraph);

private:

	TArray<UClass*> ObjectClasses;
};

