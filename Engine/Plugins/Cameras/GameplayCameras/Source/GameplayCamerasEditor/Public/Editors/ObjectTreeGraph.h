// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraph.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrFwd.h"

#include "ObjectTreeGraph.generated.h"

class UObjectTreeGraphNode;
class UObjectTreeGraphSchema;

/**
 * A node graph that represents a hierarchy of objects and their relationships.
 */
UCLASS()
class UObjectTreeGraph : public UEdGraph
{
	GENERATED_BODY()

public:

	/** Creates a new graph. */
	UObjectTreeGraph(const FObjectInitializer& ObjInit);

	/** Initializes the graph given a root object and a graph configuration. */
	void Reset(TObjectPtr<UObject> InRootObject, const FObjectTreeGraphConfig& InConfig);

	/** Gets the root object. */
	UObject* GetRootObject() const { return WeakRootObject.Get(); }
	/** Gets the root object's graph node. */
	UObjectTreeGraphNode* GetRootObjectNode() const { return RootObjectNode; }

	/** Finds a node for the given object. */
	UObjectTreeGraphNode* FindObjectNode(UObject* InObject) const;

	/** Gets the graph configuration. */
	const FObjectTreeGraphConfig& GetConfig() const;

public:

	void RebuildGraph();

private:

	TWeakObjectPtr<> WeakRootObject;

	FObjectTreeGraphConfig Config;

	UPROPERTY()
	TObjectPtr<UObjectTreeGraphNode> RootObjectNode;

	friend class UObjectTreeGraphSchema;
};

