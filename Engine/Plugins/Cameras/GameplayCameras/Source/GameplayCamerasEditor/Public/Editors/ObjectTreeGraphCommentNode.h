// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "EdGraphNode_Comment.h"
#include "UObject/WeakObjectPtrFwd.h"

#include "ObjectTreeGraphCommentNode.generated.h"

class UObjectTreeGraphComment;

UCLASS(MinimalAPI, Optional)
class UObjectTreeGraphCommentNode : public UEdGraphNode_Comment
{
	GENERATED_BODY()

public:

	/** Initializes this graph node for the given comment object. */
	void Initialize(UObjectTreeGraphComment* InObject);

	/** Gets the underlying comment object represented by this graph node. */
	UObjectTreeGraphComment* GetObject() const { return WeakObject.Get(); }

public:

	// UEdGraphNode interface.
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual void PostPlacedNewNode() override;
	virtual void ResizeNode(const FSlateCompatVector2f& NewSize) override;
	virtual void OnRenameNode(const FString& NewName) override;

public:

	void OnGraphNodeMoved(bool bMarkDirty);

private:

	UPROPERTY()
	TWeakObjectPtr<UObjectTreeGraphComment> WeakObject;
};

