// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTypeTraits.h"

#include <initializer_list>

class FText;
class UClass;
class UObject;
class UObjectTreeGraph;
class UObjectTreeGraphNode;
struct FObjectTreeGraphConfig;

DECLARE_DELEGATE_OneParam(FOnBuildObjectTreeGraphConfig, FObjectTreeGraphConfig& InOutConfig);

DECLARE_DELEGATE_OneParam(FOnSetupNewObject, UObject*);
DECLARE_DELEGATE_RetVal_OneParam(FText, FOnGetObjectClassDisplayName, const UClass*);

DECLARE_DELEGATE_TwoParams(FOnGetGraphDisplayInfo, const UObjectTreeGraph*, FGraphDisplayInfo&);
DECLARE_DELEGATE_TwoParams(FOnFormatObjectDisplayName, const UObject*, FText&);

#define OTGCC_FIELD(FieldType, FieldName)\
	public:\
		typename TCallTraits<FieldType>::ConstReference FieldName() const\
			{ return _##FieldName; }\
		FObjectTreeGraphClassConfig& FieldName(typename TCallTraits<FieldType>::ParamType InValue)\
			{ _##FieldName = InValue; _bOverride##FieldName = true; return *this; }\
		bool Has##FieldName##Override() const\
			{ return _bOverride##FieldName; }\
	private:\
		FieldType _##FieldName;

#define OTGCC_FIELD_FLAG(FieldName)\
		bool _bOverride##FieldName;

/**
 * A structure providing optional configuration options for a given object class.
 */
struct FObjectTreeGraphClassConfig
{
public:

	/** The subclass of graph nodes to create. */
	OTGCC_FIELD(TSubclassOf<UObjectTreeGraphNode>, GraphNodeClass)

	/** The name of the self pin. */
	OTGCC_FIELD(FName, SelfPinName)
	/** The display name of the self pin. */
	OTGCC_FIELD(FText, SelfPinFriendlyName)
	/** The direction of the self pin. */
	OTGCC_FIELD(TOptional<EEdGraphPinDirection>, SelfPinDirectionOverride)
	/** Whether graph nodes for this class have a self pin. */
	OTGCC_FIELD(bool, HasSelfPin)

	/** Default direction of property pins. */
	OTGCC_FIELD(TOptional<EEdGraphPinDirection>, DefaultPropertyPinDirectionOverride)

	/** Color of the graph node's title background. */
	OTGCC_FIELD(TOptional<FLinearColor>, NodeTitleColor)
	/** Color of the graph node's title text. */
	OTGCC_FIELD(TOptional<FLinearColor>, NodeTitleTextColor)
	/** Color of the graph node's body. */
	OTGCC_FIELD(TOptional<FLinearColor>, NodeBodyTintColor)

	/** A custom callback to setup a newly created object added in the graph editor. */
	OTGCC_FIELD(FOnSetupNewObject, OnSetupNewObject)

	/** Whether the graph node title uses the underlying object's name instead of its class name. */
	OTGCC_FIELD(bool, NodeTitleUsesObjectName)
	/** A custom callback to get the object's display name used in the graph node title. */
	OTGCC_FIELD(FOnGetObjectClassDisplayName, OnGetObjectClassDisplayName)

	/** Whether users can create new objects of this class in the graph. */
	OTGCC_FIELD(bool, CanCreateNew)
	/** Whether users can duplicate objects of this class in the graph. */
	OTGCC_FIELD(bool, CanDelete)

	/** The metadata specifier to look for in order to categorize the 'create node' action for this class. */
	OTGCC_FIELD(FName, CreateCategoryMetaData)

private:

	// Packed flags.
	OTGCC_FIELD_FLAG(GraphNodeClass)
	OTGCC_FIELD_FLAG(SelfPinName)
	OTGCC_FIELD_FLAG(SelfPinFriendlyName)
	OTGCC_FIELD_FLAG(SelfPinDirectionOverride)
	OTGCC_FIELD_FLAG(HasSelfPin)
	OTGCC_FIELD_FLAG(DefaultPropertyPinDirectionOverride)
	OTGCC_FIELD_FLAG(NodeTitleColor)
	OTGCC_FIELD_FLAG(NodeTitleTextColor)
	OTGCC_FIELD_FLAG(NodeBodyTintColor)
	OTGCC_FIELD_FLAG(OnSetupNewObject)
	OTGCC_FIELD_FLAG(NodeTitleUsesObjectName)
	OTGCC_FIELD_FLAG(OnGetObjectClassDisplayName)
	OTGCC_FIELD_FLAG(CanCreateNew)
	OTGCC_FIELD_FLAG(CanDelete)
	OTGCC_FIELD_FLAG(CreateCategoryMetaData)

public:

	FObjectTreeGraphClassConfig();

	/** A shortcut for disabling CanCreateNew and CanDelete. */
	FObjectTreeGraphClassConfig& OnlyAsRoot();

public:

	/** Gets the name suffixes to strip. */
	TArrayView<const FString> StripDisplayNameSuffixes() const { return _StripDisplayNameSuffixes; }

	/** Adds a new suffix to strip from the display name. */
	FObjectTreeGraphClassConfig& StripDisplayNameSuffix(const FString& InSuffix)
	{
		_StripDisplayNameSuffixes.Add(InSuffix);
		return *this;
	}

	/** Adds multiple suffixes to strip from the display name. */
	FObjectTreeGraphClassConfig& StripDisplayNameSuffixes(std::initializer_list<FString> InSuffixes)
	{
		_StripDisplayNameSuffixes.Append(InSuffixes);
		return *this;
	}

	/** Gets the custom property pin directions for given named properties. */
	const TMap<FName, EEdGraphPinDirection>& PropertyPinDirectionOverrides() const { return _PropertyPinDirectionOverrides; }

	/** Adds a new custom property pin direction for a given named property. */
	FObjectTreeGraphClassConfig& SetPropertyPinDirectionOverride(const FName& InPropertyName, EEdGraphPinDirection InDirection)
	{
		_PropertyPinDirectionOverrides.Add(InPropertyName, InDirection);
		return *this;
	}

	/** Gets the custom property pin direction for a given named property. */
	TOptional<EEdGraphPinDirection> GetPropertyPinDirectionOverride(const FName& InPropertyName) const
	{
		if (const EEdGraphPinDirection* PinDirection = _PropertyPinDirectionOverrides.Find(InPropertyName))
		{
			return *PinDirection;
		}
		return TOptional<EEdGraphPinDirection>();
	}

private:

	TArray<FString> _StripDisplayNameSuffixes;
	TMap<FName, EEdGraphPinDirection> _PropertyPinDirectionOverrides;
};

#undef OTGCC_FIELD
#undef OTGCC_FIELD_FLAG

#define OTGCCS_FIELD(FieldType, FieldName)\
	public:\
		typename TCallTraits<FieldType>::ConstReference FieldName(typename TCallTraits<FieldType>::ConstReference DefaultValue) const\
		{\
			for (const FObjectTreeGraphClassConfig* InnerConfig : InnerConfigs)\
			{\
				if (InnerConfig->Has##FieldName##Override())\
				{\
					return InnerConfig->FieldName();\
				}\
			}\
			return DefaultValue;\
		}\
		typename TCallTraits<FieldType>::ConstReference FieldName() const\
		{\
			return FieldName(DefaultConfig.FieldName());\
		}

/**
 * A composite of multiple class configurations, for handling configuration options set on
 * different classes in a class hierarchy.
 */
struct FObjectTreeGraphClassConfigs
{
public:

	/** The subclass of graph nodes to create. */
	OTGCCS_FIELD(TSubclassOf<UObjectTreeGraphNode>, GraphNodeClass)

	/** The name of the self pin. */
	OTGCCS_FIELD(FName, SelfPinName)
	/** The display name of the self pin. */
	OTGCCS_FIELD(FText, SelfPinFriendlyName)
	/** The direction of the self pin. */
	OTGCCS_FIELD(TOptional<EEdGraphPinDirection>, SelfPinDirectionOverride)
	/** Whether graph nodes for this class have a self pin. */
	OTGCCS_FIELD(bool, HasSelfPin)

	/** Default direction of property pins. */
	OTGCCS_FIELD(TOptional<EEdGraphPinDirection>, DefaultPropertyPinDirectionOverride)

	/** Color of the graph node's title background. */
	OTGCCS_FIELD(TOptional<FLinearColor>, NodeTitleColor)
	/** Color of the graph node's title text. */
	OTGCCS_FIELD(TOptional<FLinearColor>, NodeTitleTextColor)
	/** Color of the graph node's body. */
	OTGCCS_FIELD(TOptional<FLinearColor>, NodeBodyTintColor)

	/** A custom callback to setup a newly created object added in the graph editor. */
	OTGCCS_FIELD(FOnSetupNewObject, OnSetupNewObject)

	/** Whether the graph node title uses the underlying object's name instead of its class name. */
	OTGCCS_FIELD(bool, NodeTitleUsesObjectName)
	/** A custom callback to get the object's display name used in the graph node title. */
	OTGCCS_FIELD(FOnGetObjectClassDisplayName, OnGetObjectClassDisplayName)

	/** Whether users can create new objects of this class in the graph. */
	OTGCCS_FIELD(bool, CanCreateNew)
	/** Whether users can duplicate objects of this class in the graph. */
	OTGCCS_FIELD(bool, CanDelete)

	/** The metadata specifier to look for in order to categorize the 'create node' action for this class. */
	OTGCCS_FIELD(FName, CreateCategoryMetaData)

public:

	FObjectTreeGraphClassConfigs();
	FObjectTreeGraphClassConfigs(TArrayView<const FObjectTreeGraphClassConfig*> InClassConfigs);

public:

	/** Gets the name suffixes to strip. */
	void GetStripDisplayNameSuffixes(TArray<FString>& OutSuffixes) const;

	/** Gets the custom property pin direction for a given named property. */
	TOptional<EEdGraphPinDirection> GetPropertyPinDirectionOverride(const FName& InPropertyName) const;

private:

	static FObjectTreeGraphClassConfig DefaultConfig;

	TArray<const FObjectTreeGraphClassConfig*, TInlineAllocator<2>> InnerConfigs;
};

#undef OTGCCS_FIELD

/**
 * A structure that provides all the information needed to build, edit, and maintain an 
 * object tree graph.
 */
struct FObjectTreeGraphConfig
{
public:

	/**
	 * The name of the graph, passed to some APIs like IObjectTreeGraphRootObject.
	 */
	FName GraphName;

	/**
	 * The list of connectable object classes.
	 *
	 * Objects whose class is connectable (which includes sub-classes) will be eligible 
	 * to be nodes in the graph. Properties on those objects that point to other connectable
	 * objects (either with a direct object property or an array property) will show up
	 * as pins on the object's node.
	 */
	TArray<UClass*> ConnectableObjectClasses;
	/**
	 * The list of unconnectable object classes.
	 *
	 * This serves as an exception list to the ConnectableObjectClasses list.
	 */
	TArray<UClass*> NonConnectableObjectClasses;
	
	/**
	 * The default graph node class to use in the graph. Defaults to UObjectTreeGraphNode.
	 */
	TSubclassOf<UObjectTreeGraphNode> DefaultGraphNodeClass;

	/** The default name for a node's self pin. */
	FName DefaultSelfPinName;
	/** The default friendly name for a node's self pin. */
	FText DefaultSelfPinFriendlyName;

	/** The default title background color for an object's graph node. */
	FLinearColor DefaultGraphNodeTitleColor;
	/** The default title text color for an object's graph node. */
	FLinearColor DefaultGraphNodeTitleTextColor;
	/** The default body color for an object's graph node. */
	FLinearColor DefaultGraphNodeBodyTintColor;

	/** A custom callback to format an object's display name. */
	FOnFormatObjectDisplayName OnFormatObjectDisplayName;

	/** The graph display information. */
	FGraphDisplayInfo GraphDisplayInfo;

	/** A custom callback to get the graph display information, to override GraphDisplayInfo. */
	FOnGetGraphDisplayInfo OnGetGraphDisplayInfo;

	/** Advanced, optional bits of configuration for specific classes and sub-classes of objects. */
	TMap<UClass*, FObjectTreeGraphClassConfig> ObjectClassConfigs;

public:

	/** Creates a new graph config. */
	FObjectTreeGraphConfig();

	/**
	 * Returns whether the given class is connectable.
	 *
	 * It is connectable if it, or one of its parent classes, is inside ConnectableObjectClasses,
	 * and nor it or any of its parent classes is in NonConnectableObjectClasses.
	 */
	bool IsConnectable(UClass* InObjectClass) const;

	/**
	 * Returns whether the given object reference property is connectable.
	 *
	 * It is connectable if the property's reference type is for a connectable class, and if the
	 * property doesn't have the ObjectTreeGraphHidden metadata.
	 */
	bool IsConnectable(FObjectProperty* InObjectProperty) const;

	/**
	 * Returns whether the given object array property is connectable.
	 *
	 * It is connectable if the array's item type is for a connectable class, and if the array
	 * property doesn't have the ObjectTreeGraphHidden metadata.
	 */
	bool IsConnectable(FArrayProperty* InArrayProperty) const;

	/**
	 * Gets all possible known connectable classes.
	 *
	 * @param bPlaceableOnly  If set, only return those that can be created.
	 */
	void GetConnectableClasses(TArray<UClass*>& OutClasses, bool bPlaceableOnly = false);
	
	/** Gets the advanced class-specific configuration for the given class. */
	FObjectTreeGraphClassConfigs GetObjectClassConfigs(const UClass* InObjectClass) const;

	/** Computes the display name of the given object. */
	FText GetDisplayNameText(const UObject* InObject) const;
	/** Computes the display name of the given object class. */
	FText GetDisplayNameText(const UClass* InClass) const;

	/** Gets the "self" pin direction for a given class. */
	EEdGraphPinDirection GetSelfPinDirection(const UClass* InObjectClass) const;

	/** Gets the custom property pin direction for a given named property. */
	EEdGraphPinDirection GetPropertyPinDirection(const UClass* InObjectClass, const FName& InPropertyName) const;

private:

	FText GetDisplayNameText(const UClass* InClass, const FObjectTreeGraphClassConfigs& InClassConfig) const;
	void FormatDisplayNameText(const UObject* InObject, const FObjectTreeGraphClassConfigs& InClassConfig, FText& InOutDisplayNameText) const;
};

