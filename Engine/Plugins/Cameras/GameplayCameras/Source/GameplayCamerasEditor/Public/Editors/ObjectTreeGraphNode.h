// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EdGraph/EdGraphNode.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Misc/EngineVersionComparison.h"
#include "UObject/UObjectGlobals.h"

#include "ObjectTreeGraphNode.generated.h"

class FObjectProperty;
class UEdGraphPin;
class UObjectTreeGraph;

/**
 * A graph node that represents an object inside an object tree graph.
 */
UCLASS()
class UObjectTreeGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

public:

	/** Creates a new graph node. */
	UObjectTreeGraphNode(const FObjectInitializer& ObjInit);

	/** Initializes this graph node for the given object. */
	void Initialize(UObject* InObject);

	/** Gets the underlying object represented by this graph node. */
	UObject* GetObject() const { return WeakObject.Get(); }
	/** Gets whether we have a valid underlying object, and that it's a type of ObjectClass. */
	template<typename ObjectClass> bool IsObjectA() const;
	/** Gets the underlying object as a point to the given sub-class. */
	template<typename ObjectClass> ObjectClass* CastObject() const;

	/** Gets all connectable properties on the underlying object. */
	void GetAllConnectableProperties(TArray<FProperty*>& OutProperties) const;

	/** Finds the self pin that represents the underlying object itself. */
	UEdGraphPin* GetSelfPin() const;
	/** Changes the direction of the self pin. */
	void OverrideSelfPinDirection(EEdGraphPinDirection Direction);

	/** Finds the pin for the given object property. */
	UEdGraphPin* GetPinForProperty(FObjectProperty* InProperty) const;
	/** Finds the invisible parent pin for the given array property. */
	UEdGraphPin* GetPinForProperty(FArrayProperty* InProperty) const;
	/** Finds the pin for the given item in an array property. */
	UEdGraphPin* GetPinForProperty(FArrayProperty* InProperty, int32 Index) const;
	/** Gets the underlying property represented by the given pin. */
	FProperty* GetPropertyForPin(const UEdGraphPin* InPin) const;
	/** Gets the type of object that can connect to the given pin. */
	UClass* GetConnectedObjectClassForPin(const UEdGraphPin* InPin) const;
	/** Gets the index of the given pin's underlying value inside an array property. */
	int32 GetIndexOfArrayPin(const UEdGraphPin* InPin) const;

public:

	// UEdGraphNode interface.
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual FLinearColor GetNodeTitleColor() const override;
#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
	virtual FLinearColor GetNodeTitleTextColor() const override;
#endif
	virtual FLinearColor GetNodeBodyTintColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void AllocateDefaultPins() override;
	virtual void PostPlacedNewNode() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void NodeConnectionListChanged() override;
	virtual void OnPinRemoved(UEdGraphPin* InRemovedPin) override;
	virtual void ReconstructNode() override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool GetCanRenameNode() const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual bool CanDuplicateNode() const override;
	virtual bool CanUserDeleteNode() const override;
	virtual bool SupportsCommentBubble() const override;
	virtual void OnUpdateCommentText(const FString& NewComment) override;

	// UObjectTreeGraphNode interface.
	virtual void OnInitialize() {}
	virtual void OnGraphNodeMoved(bool bMarkDirty);
	virtual void OnDoubleClicked() {}

public:

	// Internal API.
	void GetArrayProperties(TArray<FArrayProperty*>& OutArrayProperties, EEdGraphPinDirection Direction) const;
	void CreateNewItemPins(FArrayProperty& InArrayProperty, int32 NumExtraPins);
	void CreateNewItemPins(UEdGraphPin* InParentArrayPin, int32 NumExtraPins);
	void InsertNewItemPin(UEdGraphPin* InParentArrayPin, int32 Index);
	void RemoveItemPin(UEdGraphPin* InItemPin);
	void RefreshArrayPropertyPinNames();

protected:

	struct FNodeContext
	{
		UClass* ObjectClass;
		UObjectTreeGraph* Graph;
		const FObjectTreeGraphConfig& GraphConfig;
		const FObjectTreeGraphClassConfigs ObjectClassConfigs;
	};

	FNodeContext GetNodeContext() const;
	const FObjectTreeGraphClassConfigs GetObjectClassConfigs() const;

private:

	UPROPERTY()
	TWeakObjectPtr<UObject> WeakObject;

	UPROPERTY()
	TEnumAsByte<EEdGraphPinDirection> SelfPinDirectionOverride;

	UPROPERTY()
	bool bOverrideSelfPinDirection;
};

template<typename ObjectClass>
bool UObjectTreeGraphNode::IsObjectA() const
{
	if (UObject* Object = WeakObject.Get())
	{
		return Object->IsA<ObjectClass>();
	}
	return false;
}

template<typename ObjectClass> 
ObjectClass* UObjectTreeGraphNode::CastObject() const
{
	if (UObject* Object = WeakObject.Get())
	{
		return Cast<ObjectClass>(Object);
	}
	return nullptr;
}

