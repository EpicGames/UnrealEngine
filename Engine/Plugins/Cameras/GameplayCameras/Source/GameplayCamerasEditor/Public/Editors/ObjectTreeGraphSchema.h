// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "Misc/EngineVersionComparison.h"

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,7,0)
#include "Misc/StringOutputDevice.h"
#endif

#include "ObjectTreeGraphSchema.generated.h"

class IObjectTreeGraphRootObject;
class UEdGraph;
class UObjectTreeGraph;
class UObjectTreeGraphComment;
class UObjectTreeGraphNode;
struct FObjectTreeGraphClassConfigs;

/**
 * Schema class for an object tree graph.
 */
UCLASS()
class UObjectTreeGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:

	// Pin categories.
	static const FName PC_Self;			// A "self" pin.
	static const FName PC_Property;		// A property pin.

	// Pin sub-categories.
	static const FName PSC_ObjectProperty;		// A normal object property pin.
	static const FName PSC_ArrayProperty;		// An array property pin (generally hidden).
	static const FName PSC_ArrayPropertyItem;	// A pin for an item inside an array property.

public:

	/** Creates a new schema. */
	UObjectTreeGraphSchema(const FObjectInitializer& ObjInit);

	/** Rebuilds the graph from scratch. */
	void RebuildGraph(UObjectTreeGraph* InGraph) const;

	/** Creates an object graph node for the given object. */
	UEdGraphNode* CreateObjectNode(UObjectTreeGraph* InGraph, UObject* InObject) const;

	/** Adds an object to the underlying data after it has been added to the graph. */
	void AddConnectableObject(UObjectTreeGraph* InGraph, UObject* InNewObject) const;

	/** Removes an object from the underlying data after it has been removed from the graph. */
	void RemoveConnectableObject(UObjectTreeGraph* InGraph, UObject* InRemovedObject) const;

	/** Export the given selection into a text suitable for copy/pasting. */
	FString ExportNodesToText(const FGraphPanelSelectionSet& Nodes, bool bOnlyCanDuplicateNodes, bool bOnlyCanDeleteNodes) const;

	/** Imports the given text into the given graph. */
	void ImportNodesFromText(UObjectTreeGraph* InGraph, const FString& TextToImport, TArray<UEdGraphNode*>& OutPastedNodes) const;

	/** Checks if the given text is suitable for importing. */
	bool CanImportNodesFromText(UObjectTreeGraph* InGraph, const FString& TextToImport) const;

	/** Inserts a new array property item pin. */
	void InsertArrayItemPin(UEdGraphPin* ArrayPin, int32 Index = INDEX_NONE) const;

	/** Inserts a new array property item pin before the given pin. */
	void InsertArrayItemPinBefore(UEdGraphPin* ArrayItemPin) const;

	/** Inserts a new array property item pin after the given pin. */
	void InsertArrayItemPinAfter(UEdGraphPin* ArrayItemPin) const;

	/** Removes an array property item pin. */
	void RemoveArrayItemPin(UEdGraphPin* ArrayItemPin) const;

public:

	// UEdGraphSchema interface.
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	virtual FName GetParentContextMenuName() const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, UEdGraph* InGraph) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override;
	virtual FPinConnectionResponse CanCreateNewNodes(UEdGraphPin* InSourcePin) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& Graph, FGraphDisplayInfo& OutDisplayInfo) const override;

protected:

	struct FCreatedNodes
	{
		TMap<UObject*, UEdGraphNode*> CreatedNodes;
	};

	// UObjectTreeGraphSchema interface.
	virtual void CollectAllObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects) const;
	virtual void OnCreateAllNodes(UObjectTreeGraph* InGraph, const FCreatedNodes& InCreatedNodes) const;
	virtual UEdGraphNode* OnCreateObjectNode(UObjectTreeGraph* InGraph, UObject* InObject) const;
	virtual void OnAddConnectableObject(UObjectTreeGraph* InGraph, UObject* InNewObject) const;
	virtual void OnRemoveConnectableObject(UObjectTreeGraph* InGraph, UObject* InRemovedObject) const;
	virtual void CopyNonObjectNodes(TArrayView<UObject*> InObjects, FStringOutputDevice& OutDevice) const;
	virtual bool OnTryCreateCustomConnection(UEdGraphPin* A, UEdGraphPin* B) const;
	virtual bool OnBreakCustomPinLinks(UEdGraphPin& TargetPin) const;
	virtual bool OnBreakSingleCustomPinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const;
	virtual bool OnApplyConnection(UEdGraphPin* A, UEdGraphPin* B) const;
	virtual bool OnApplyDisconnection(UEdGraphPin* TargetPin) const;
	virtual bool OnApplyDisconnection(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const;
	virtual void OnDeleteNodeFromGraph(UObjectTreeGraph* Graph, UEdGraphNode* Node) const;
	virtual void FilterGraphContextPlaceableClasses(TArray<UClass*>& InOutClasses) const;

protected:

	static void CollectAllReferencedObjects(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects);
	static bool CollectAllConnectableObjectsFromRootInterface(UObjectTreeGraph* InGraph, TSet<UObject*>& OutAllObjects, bool bAllowNoRootInterface);

	const FObjectTreeGraphClassConfigs GetObjectClassConfigs(const UObjectTreeGraphNode* InNode) const;
	const FObjectTreeGraphClassConfigs GetObjectClassConfigs(const UObjectTreeGraph* InGraph, UClass* InObjectClass) const;

	void ApplyConnection(UEdGraphPin* A, UEdGraphPin* B) const;
	void ApplyDisconnection(UEdGraphPin* TargetPin) const;
	void ApplyDisconnection(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const;

private:

	void RemoveAllNodes(UObjectTreeGraph* InGraph) const;
	void CreateAllNodes(UObjectTreeGraph* InGraph) const;
	void CreateConnections(UObjectTreeGraphNode* InGraphNode, const FCreatedNodes& InCreatedNodes) const;

	UEdGraphNode* CreateCommentNode(UObjectTreeGraph* InGraph, UObjectTreeGraphComment* InComment) const;
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder) const;
};

/**
 * Graph action to create a new object (and corresponding graph node) of a given class.
 */
USTRUCT()
struct FObjectTreeGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	/** The outer for the new object. Defaults to the root object's package. */
	UPROPERTY()
	TObjectPtr<UObject> ObjectOuter;

	/** The class of the new object. */
	UPROPERTY()
	TObjectPtr<UClass> ObjectClass;

public:

	FObjectTreeGraphSchemaAction_NewNode();
	FObjectTreeGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping = 0, FText InKeywords = FText());

public:

	// FEdGraphSchemaAction interface.
	static FName StaticGetTypeId() { static FName Type("FObjectTreeGraphSchemaAction_NewNode"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode = true) override;

protected:

	virtual UObject* CreateObject();
	virtual void AutoSetupNewNode(UEdGraphNode* NewNode, UEdGraphPin* FromPin);
};

USTRUCT()
struct FObjectTreeGraphSchemaAction_NewComment : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	FObjectTreeGraphSchemaAction_NewComment();
	FObjectTreeGraphSchemaAction_NewComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping = 0, FText InKeywords = FText());

public:

	// FEdGraphSchemaAction interface.
	static FName StaticGetTypeId() { static FName Type("FObjectTreeGraphSchemaAction_NewComment"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FPerformGraphActionLocation Location, bool bSelectNewNode = true) override;
};

