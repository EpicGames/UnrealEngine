// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class UEdGraph;
class UEdGraphNode;
class UObject;
struct FObjectTreeGraphConfig;

/** Search result for an object tree graph search. */
struct FObjectTreeGraphSearchResult
{
	/** The graph in which the result was found. */
	UEdGraph* Graph = nullptr;

	/** The node found to match the search. */
	UEdGraphNode* Node = nullptr;

	/** The name of the pin that matched the search. None if the node itself matches. */
	FName PinName;

public:

	FObjectTreeGraphSearchResult()
	{}
	FObjectTreeGraphSearchResult(UEdGraph* InGraph, UEdGraphNode* InNode, FName InPinName = NAME_None)
		: Graph(InGraph), Node(InNode), PinName(InPinName)
	{}
};

/**
 * A utility class that can search a series of string tokens across an
 * object tree graph.
 */
class FObjectTreeGraphSearch
{
public:

	FObjectTreeGraphSearch();

public:

	/** Adds a graph to search inside of. */
	void AddGraph(UEdGraph* InGraph);
	/** Sets the graphs to search inside of. */
	void SetGraphs(TConstArrayView<UEdGraph*> InGraphs);

	/** Searchs for the given string tokens. */
	void Search(TArrayView<FString> InTokens, TArray<FObjectTreeGraphSearchResult>& OutResults) const;

private:

	using FSearchResult = FObjectTreeGraphSearchResult;

	struct FSearchState
	{
		UEdGraph* Graph = nullptr;
		TArrayView<FString> Tokens;
		TArray<FSearchResult> Results;
	};

	void SearchGraph(UEdGraph* Graph, FSearchState& State) const;
	void SearchNode(UEdGraphNode* Node, FSearchState& State) const;

	bool MatchString(const FString& InString, const FSearchState& InState) const;

private:

	TArray<UEdGraph*> SearchGraphs;
};

