// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraAsset.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/SCompoundWidget.h"

class UCameraAssetInterfaceParameter;
class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

namespace UE::Cameras
{

class FCameraAssetEditorToolkit;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraAssetInterfaceParameterEvent, UCameraAssetInterfaceParameter*);

class SCameraAssetInterfaceParametersPanel 
	: public SCompoundWidget
	, public ICameraAssetEventHandler
{
public:

	static const FName ParameterTypeColumn;
	static const FName ParameterNameColumn;
	static const FName SourceCameraRigColumn;
	static const FName SourceParameterNameColumn;

public:

	SLATE_BEGIN_ARGS(SCameraAssetInterfaceParametersPanel)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, FCameraAssetEditorToolkit* OwnerToolkit);

	/** Request that the list of parameters be refreshed. */
	void RequestListRefresh();

	/** Rename the selected parameter in the focused panel. */
	void RenameSelectedParameter();

	/** Delete the selected parameter in the focused panel. */
	void DeleteSelectedParameter();

	/** Delegate invoked when a parmeter is selected in the panel. */
	FOnCameraAssetInterfaceParameterEvent& OnInterfaceParameterSelected() { return OnInterfaceParameterSelectedDelegate; }

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// ICameraAssetEventHandler interface.
	virtual void OnCameraAssetInterfaceChanged() override;

private:

	TSharedRef<ITableRow> OnGenerateInterfaceParameterRow(TObjectPtr<UCameraAssetInterfaceParameter> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnInterfaceParameterSelectionChanged(TObjectPtr<UCameraAssetInterfaceParameter> Item, ESelectInfo::Type Type);
	TSharedPtr<SWidget> OnInterfaceParameterContextMenuOpening();

	FReply OnAddInterfaceParameter();
	FReply OnDeleteSelectedInterfaceParameter();

	void OnRenameInterfaceParameter(TObjectPtr<UCameraAssetInterfaceParameter> Item);
	void OnDeleteInterfaceParameter(TObjectPtr<UCameraAssetInterfaceParameter> Item);

	FString GetNewParameterName();

private:

	UCameraAsset* CameraAsset = nullptr;
	FCameraAssetEditorToolkit* Toolkit = nullptr;

	TCameraEventHandler<ICameraAssetEventHandler> EventHandler;

	TSharedPtr<SListView<TObjectPtr<UCameraAssetInterfaceParameter>>> ParametersListView;

	FOnCameraAssetInterfaceParameterEvent OnInterfaceParameterSelectedDelegate;

	bool bListRefreshRequested = false;
};

}  // namespace UE::Cameras

