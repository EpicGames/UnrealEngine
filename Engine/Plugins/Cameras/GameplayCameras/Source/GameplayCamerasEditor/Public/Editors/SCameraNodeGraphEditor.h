// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/SObjectTreeGraphEditor.h"

class SCameraNodeGraphEditor 
	: public SObjectTreeGraphEditor 
{
protected:

	// SWidget interface.
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
};

