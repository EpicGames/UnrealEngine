// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/SObjectTreeGraphNode.h"

class SCameraNodeGraphNode : public SObjectTreeGraphNode
{
public:

	SLATE_BEGIN_ARGS(SCameraNodeGraphNode)
		: _GraphNode(nullptr)
	{}
		SLATE_ARGUMENT(UObjectTreeGraphNode*, GraphNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:

	// SGraphNode interface.
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin) const override;
};

