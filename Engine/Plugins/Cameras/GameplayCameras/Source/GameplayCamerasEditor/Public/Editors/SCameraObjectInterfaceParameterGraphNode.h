// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editors/SObjectTreeGraphNode.h"

class UCameraObjectInterfaceParameterGraphNode;

/**
 * Custom graph editor node widget for a camera rig parameter getter node.
 */
class SCameraObjectInterfaceParameterGraphNode : public SObjectTreeGraphNode
{
public:

	SLATE_BEGIN_ARGS(SCameraObjectInterfaceParameterGraphNode)
		: _GraphNode(nullptr)
	{}
		SLATE_ARGUMENT(UCameraObjectInterfaceParameterGraphNode*, GraphNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	// SGraphNode interface.
	virtual void UpdateGraphNode() override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin) const override;

private:

	FText GetInterfaceParameterName() const;
};

