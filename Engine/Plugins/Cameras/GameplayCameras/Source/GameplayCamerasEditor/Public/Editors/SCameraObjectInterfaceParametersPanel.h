// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/BaseCameraObject.h"
#include "Core/CameraEventHandler.h"
#include "EdGraphSchema_K2.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/SCompoundWidget.h"

#include "SCameraObjectInterfaceParametersPanel.generated.h"

class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

namespace UE::Cameras
{

class FCameraObjectInterfaceParametersToolkit;

/**
 * The overall interface parameters panel, showing two sub-panels, one for blendable parameters, and
 * one for data parameters.
 */
class SCameraObjectInterfaceParametersPanel 
	: public SCompoundWidget
	, public ICameraObjectEventHandler
{
public:

	static const FName ParameterTypeColumn;
	static const FName ParameterNameColumn;

public:

	SLATE_BEGIN_ARGS(SCameraObjectInterfaceParametersPanel)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, FCameraObjectInterfaceParametersToolkit* OwnerToolkit);

	/** Refresh the lists of parameters. */
	void RequestListRefresh();

	/** Rename the selected parameter in the focused panel. */
	void RenameSelectedParameter();

	/** Delete the selected parameter in the focused panel. */
	void DeleteSelectedParameter();

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// ICameraObjectEventHandler interface.
	virtual void OnCameraObjectInterfaceChanged() override;

private:

	TSharedRef<ITableRow> OnGenerateBlendableParameterRow(TObjectPtr<UCameraObjectInterfaceBlendableParameter> Item, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGenerateDataParameterRow(TObjectPtr<UCameraObjectInterfaceDataParameter> Item, const TSharedRef<STableViewBase>& OwnerTable);

	void OnBlendableSelectionChanged(TObjectPtr<UCameraObjectInterfaceBlendableParameter> Item, ESelectInfo::Type Type);
	void OnDataParameterSelectionChanged(TObjectPtr<UCameraObjectInterfaceDataParameter> Item, ESelectInfo::Type Type);

	TSharedPtr<SWidget> OnBlendableParameterContextMenuOpening();
	TSharedPtr<SWidget> OnDataParameterContextMenuOpening();

	void OnSearchInterfaceParameter(TObjectPtr<UCameraObjectInterfaceParameterBase> Item);

	template<typename ItemType>
	TSharedPtr<SWidget> OnInterfaceParameterContextMenuOpening(
			TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item);
	template<typename ItemType>
	void OnRenameInterfaceParameter(
			TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item);
	template<typename ItemType>
	void OnDeleteInterfaceParameter(
			TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView, TObjectPtr<ItemType> Item);

	template<typename ItemType>
	void OnRenameSelectedInterfaceParameter(TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView);
	template<typename ItemType>
	void OnDeleteSelectedInterfaceParameter(TSharedPtr<SListView<TObjectPtr<ItemType>>> ListView);

	FReply OnAddBlendableParameter();
	FReply OnAddDataParameter();

	void OnRenameSelectedBlendableParameter();
	void OnRenameSelectedDataParameter();

	FReply OnDeleteSelectedBlendableParameter();
	FReply OnDeleteSelectedDataParameter();

	FString GetNewParameterName() const;

private:

	UBaseCameraObject* CameraObject = nullptr;
	FCameraObjectInterfaceParametersToolkit* Toolkit = nullptr;

	TCameraEventHandler<ICameraObjectEventHandler> EventHandler;

	TSharedPtr<SListView<TObjectPtr<UCameraObjectInterfaceBlendableParameter>>> BlendableParametersListView;
	TSharedPtr<SListView<TObjectPtr<UCameraObjectInterfaceDataParameter>>> DataParametersListView;

	bool bListRefreshRequested = false;
};

}  // namespace UE::Cameras

// Dummy Blueprint schema used only for getting proper icons and colors for parameter types.
UCLASS(MinimalAPI, Hidden)
class UEdGraphSchema_CameraNodeK2 : public UEdGraphSchema_K2
{
	GENERATED_BODY()

public:

	virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
	{
		if (ContainerType == EPinContainerType::None || ContainerType == EPinContainerType::Array)
		{
			return Super::SupportsPinTypeContainer(SchemaAction, PinType, ContainerType);
		}
		return false;
	}
};

