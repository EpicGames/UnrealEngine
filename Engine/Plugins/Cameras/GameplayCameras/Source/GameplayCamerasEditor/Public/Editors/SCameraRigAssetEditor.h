// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "CoreTypes.h"
#include "GraphEditor.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FAssetEditorToolkit;
class IDetailsView;
class SBox;
class SObjectTreeGraphEditor;
class UCameraRigAsset;
class UEdGraphNode;
class UObjectTreeGraph;
struct FObjectTreeGraphConfig;

namespace UE::Cameras
{

/** The current mode of the camera rig asset editor. */
enum class ECameraRigAssetEditorMode
{
	/** Show the node hierarchy editor. */
	NodeGraph,
	/** Show the transition editor. */
	TransitionGraph
};

/**
 * A camera rig asset editor.
 *
 * This implements only the dual-graph editor, for the node hierarchy and transitions.
 * The rest of the UI such as the details view or the toolbox aren't included here.
 */
class SCameraRigAssetEditor 
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraRigAssetEditor)
	{}
		/** The camera rig asset to edit. */
		SLATE_ARGUMENT(TObjectPtr<UCameraRigAsset>, CameraRigAsset)
		/** The details view to synchronize with the graph selection. */
		SLATE_ARGUMENT(TSharedPtr<IDetailsView>, DetailsView)
		/** The toolkit inside which this editor lives, if any. */
		SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SCameraRigAssetEditor();

	void SetCameraRigAsset(UCameraRigAsset* InCameraRig);

public:

	/** Gets the current editor mode. */
	ECameraRigAssetEditorMode GetEditorMode() const;
	/** Checks if the editor is in the current mode. */
	bool IsEditorMode(ECameraRigAssetEditorMode InMode) const;
	/** Changes the editor's current mode. */
	void SetEditorMode(ECameraRigAssetEditorMode InMode);

	/** Gets both the node hierarchy and transition graphs. */
	void GetGraphs(TArray<UEdGraph*>& OutGraphs) const;

	/** Gets the graph for the current mode. */
	UEdGraph* GetFocusedGraph() const;
	/** Gets the graph configuration for the current mode. */
	const FObjectTreeGraphConfig& GetFocusedGraphConfig() const;

	/** Focuses the current graph to the root object node. */
	void FocusHome();
	/** Jumps to a node. */
	bool JumpToNode(UEdGraphNode* InNode, FName InPinName = NAME_None);
	/** Finds a node for the given object and, if so, jumps to it. */
	bool FindAndJumpToObjectNode(UObject* InObject);

	/** Adds a callback that will be invoked when a graph editor is changed. */
	FDelegateHandle AddOnAnyGraphChanged(FOnGraphChanged::FDelegate InAddDelegate);
	/** Removes a previous added callback. */
	void RemoveOnAnyGraphChanged(FDelegateHandle InDelegateHandle);
	/** Removes a previous added callback. */
	void RemoveOnAnyGraphChanged(FDelegateUserObjectConst InUserObject);

protected:

	void CreateGraphEditors();
	void CreateNodeGraphEditor();
	void CreateTransitionGraphEditor();
	void DiscardGraphEditors();

	void SetEditorModeImpl(ECameraRigAssetEditorMode InMode, bool bForceSet);

	void OnGraphChanged(const FEdGraphEditAction& InEditAction);

	FText GetCameraRigAssetName(UObjectTreeGraph* ForGraph) const;
	bool IsGraphEditorEnabled() const;

	bool FindAndJumpToObjectNode(UObject* InObject, ECameraRigAssetEditorMode InEditorMode);

private:

	/** The asset being edited */
	TObjectPtr<UCameraRigAsset> CameraRigAsset;

	/** Reference to the details view */
	TSharedPtr<IDetailsView> DetailsView;

	/** Reference to the owning asset editor */
	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit;

	/** The node hierarchy graph */
	TObjectPtr<UObjectTreeGraph> NodeGraph;
	/** The node hierarchy graph editor */
	TSharedPtr<SObjectTreeGraphEditor> NodeGraphEditor;

	/** The transition graph */
	TObjectPtr<UObjectTreeGraph> TransitionGraph;
	/** The transition graph editor */
	TSharedPtr<SObjectTreeGraphEditor> TransitionGraphEditor;

	/** Box panel holding either the node hierarchy or transition graph editor */
	TSharedPtr<SBox> BoxPanel;

	/** The mode for the currently shown graph editor */
	ECameraRigAssetEditorMode CurrentMode;

	/** Handles for listening to changes in the graphs editors */
	FDelegateHandle NodeGraphChangedHandle;
	FDelegateHandle TransitionGraphChangedHandle;

	/* Forwarding delegate for changes in any of the graphs */
	FOnGraphChanged OnAnyGraphChanged;
};

}  // namespace UE::Cameras

