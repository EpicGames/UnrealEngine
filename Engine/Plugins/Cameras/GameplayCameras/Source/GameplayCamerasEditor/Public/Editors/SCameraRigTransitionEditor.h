// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "CoreTypes.h"
#include "Editors/CameraRigTransitionGraphSchema.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "GraphEditor.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FAssetEditorToolkit;
class IDetailsView;
class SBox;
class SObjectTreeGraphEditor;
class UCameraRigTransition;
class UEdGraphNode;
class UObjectTreeGraph;
struct FObjectTreeGraphConfig;

namespace UE::Cameras
{

/**
 * A graph editor for any object that has enter and exit transitions.
 */
class SCameraRigTransitionEditor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraRigTransitionEditor)
	{}
		/** The object owning the transitions. */
		SLATE_ARGUMENT(UObject*, TransitionOwner)
		/** The graph schema class to use. */
		SLATE_ARGUMENT(TSubclassOf<UCameraRigTransitionGraphSchemaBase>, TransitionGraphSchemaClass)
		/** The details view to synchronize with the graph selection. */
		SLATE_ARGUMENT(TSharedPtr<IDetailsView>, DetailsView)
		/** The toolkit inside which this editor lives, if any. */
		SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
		/** Appearance info for the transition graph editor. */
		SLATE_ATTRIBUTE(FGraphAppearanceInfo, TransitionGraphEditorAppearance)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SCameraRigTransitionEditor();

	/** Sets the asset being edited for transitions. */
	void SetTransitionOwner(UObject* InTransitionOwner);

public:

	/** Gets the transition graph. */
	UEdGraph* GetTransitionGraph() const;
	/** Gets the transition graph configuration. */
	const FObjectTreeGraphConfig& GetTransitionGraphConfig() const;

	/** Focuses the current graph to the root object node. */
	void FocusHome();
	/** Jumps the current graph to the given node. */
	void JumpToNode(UEdGraphNode* InGraphNode, FName InPinName = NAME_None);
	/** Finds a node for the given object and, if so, jumps to it. */
	bool FindAndJumpToObjectNode(UObject* InObject);

	/** Adds a callback that will be invoked when the editor is changed. */
	FDelegateHandle AddOnGraphChanged(FOnGraphChanged::FDelegate InAddDelegate);
	/** Removes a previous added callback. */
	void RemoveOnGraphChanged(FDelegateHandle InDelegateHandle);
	/** Removes a previous added callback. */
	void RemoveOnGraphChanged(FDelegateUserObjectConst InUserObject);

protected:

	void CreateTransitionGraphEditor();
	void DiscardTransitionGraphEditor();

	void OnGraphChanged(const FEdGraphEditAction& InEditAction);

	FText GetTransitionOwnerName() const;

private:

	/** The asset being edited. */
	UObject* TransitionOwner;

		/** The graph schema class to use. */
	TSubclassOf<UCameraRigTransitionGraphSchemaBase> TransitionGraphSchemaClass;

	/** The details view for this editor. */
	TSharedPtr<IDetailsView> DetailsView;

	/** The owning toolkit. */
	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit;

	/** The transition graph. */
	TObjectPtr<UObjectTreeGraph> TransitionGraph;
	/** The transition graph editor. */
	TSharedPtr<SObjectTreeGraphEditor> TransitionGraphEditor;

	/** Appearance info for the transition graph editor. */
	TAttribute<FGraphAppearanceInfo> TransitionGraphEditorAppearance;

	/** The panel holding the graph editor. */
	TSharedPtr<SBox> BoxPanel;

	/** Handle for listening to changes in the graph editor. */
	FDelegateHandle TransitionGraphChangedHandle;

	/* Forwarding delegate for changes in the graph editor. */
	FOnGraphChanged OnTransitionGraphChanged;
};

}  // namespace UE::Cameras

