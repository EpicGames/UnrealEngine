// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TextFilter.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FAssetEditorToolkit;
class FUICommandList;
class IDetailsView;
class ITableRow;
class SSearchBox;
class STableViewBase;
class UCameraVariableAsset;
class UCameraVariableCollection;
struct FToolMenuContext;
template<typename> class SListView;

namespace UE::Cameras
{

/**
 * An editor widget for a camera variable collection.
 */
class SCameraVariableCollectionEditor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraVariableCollectionEditor)
	{}
		/** The camera variable collection to edit. */
		SLATE_ARGUMENT(TObjectPtr<UCameraVariableCollection>, VariableCollection)
		/** The details view to synchronize with the variable list selection. */
		SLATE_ARGUMENT(TWeakPtr<IDetailsView>, DetailsView)
		/** The toolkit inside which this editor lives, if any. */
		SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
		/** Command bindings for manipulating camera variables. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, AdditionalCommands)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SCameraVariableCollectionEditor();

public:

	/** Gets the selected variables in the list view. */
	void GetSelectedVariables(TArray<UCameraVariableAsset*>& OutSelection) const;

	/** Selects the given variable if it is in the list view. */
	void SelectVariable(UCameraVariableAsset* InItem);

	/** Enter editing mode for the given variable's name. */
	void RequestRenameVariable(UCameraVariableAsset* InItem, FSimpleDelegate InOnRenamedItem);

	/** Enter editing mode for the first currently selected variable's name. */
	void RequestRenameSelectedVariable();

	/** Requests that the list view be refreshed by the next tick. */
	void RequestListRefresh();

protected:

	//~ SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	void UpdateFilteredItemSource();
	void SetDetailsViewObject(UObject* InObject) const;

	TSharedRef<ITableRow> OnListGenerateRow(UCameraVariableAsset* Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnListSectionChanged(UCameraVariableAsset* Item, ESelectInfo::Type SelectInfo) const;
	void OnListItemScrolledIntoView(UCameraVariableAsset* Item, const TSharedPtr<ITableRow>& ItemWidget);
	TSharedPtr<SWidget> OnListContextMenuOpening();

	void GetEntryStrings(const UCameraVariableAsset* InItem, TArray<FString>& OutStrings);
	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	FText GetHighlightText() const;

private:

	TObjectPtr<UCameraVariableCollection> VariableCollection;

	TWeakPtr<IDetailsView> WeakDetailsView;

	TSharedPtr<FUICommandList> CommandList;

	TSharedPtr<SListView<UCameraVariableAsset*>> ListView;

	TArray<UCameraVariableAsset*> FilteredItemSource;

	using FEntryTextFilter = TTextFilter<const UCameraVariableAsset*>;
	TSharedPtr<FEntryTextFilter> SearchTextFilter;
	TSharedPtr<SSearchBox> SearchBox;

	bool bUpdateFilteredItemSource = false;
	bool bDeferredRequestRenameItem = false;
	FSimpleDelegate OnDeferredRenamedItem;
};

}  // namespace UE::Cameras

