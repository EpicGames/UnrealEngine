// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserModule.h"
#include "SGraphPin.h"

class SComboButton;
class UCameraVariableAsset;

namespace UE::Cameras
{

class SCameraVariablePicker;

/**
 * A custom widget for a graph editor pin that shows a camera rig picker dialog.
 */
class SCameraVariableNameGraphPin : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SCameraVariableNameGraphPin)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	// SGraphPin interface.
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual bool DoesWidgetHandleSettingEditingEnabled() const override;

private:

	static constexpr float ActiveComboAlpha = 1.f;
	static constexpr float InactiveComboAlpha = 0.6f;
	static constexpr float ActivePinForegroundAlpha = 1.f;
	static constexpr float InactivePinForegroundAlpha = 0.15f;
	static constexpr float ActivePinBackgroundAlpha = 0.8f;
	static constexpr float InactivePinBackgroundAlpha = 0.4f;

	FSlateColor OnGetComboForeground() const;
	FSlateColor OnGetWidgetForeground() const;
	FSlateColor OnGetWidgetBackground() const;

	FText OnGetSelectedCameraVariableName() const;
	FText OnGetCameraVariablePickerToolTipText() const;
	TSharedRef<SWidget> OnBuildCameraVariablePicker() const;
	void OnPickerAssetSelected(UCameraVariableAsset* SelectedItem) const;

	FReply OnResetButtonClicked();

	void SetCameraVariable(UCameraVariableAsset* SelectedCameraVariable) const;

private:

	TSharedPtr<SComboButton> CameraVariablePickerButton;
};

}  // namespace UE::Cameras

