// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class SFindInObjectTreeGraph;
class SSearchBox;
class UEdGraph;
class UEdGraphNode;
class UEdGraphSchema;
class UObjectTreeGraph;
struct FObjectTreeGraphConfig;

/**
 * Struct describing one source of possible results for a search.
 */
struct FFindInObjectTreeGraphSource
{
	/** The graph where results are to be found. */
	UEdGraph* Graph = nullptr;
};

/** 
 * Structure for a search result inside an object tree graph.
 */
struct FFindInObjectTreeGraphResult
{
public:

	/** Parent result. */
	TWeakPtr<FFindInObjectTreeGraphResult> Parent;
	/** Children results. */
	TArray<TSharedPtr<FFindInObjectTreeGraphResult>> Children;

	/** Custom text for this result. */
	FText CustomText;
	/** The node that this result refers to. */
	TWeakObjectPtr<UEdGraphNode> WeakNode;
	/** The pin name that this result refers to. */
	FName PinName;

public:

	/** Creates a new result with a custom text. */
	FFindInObjectTreeGraphResult(const FText& InCustomText);
	/** Creates a new result referring to an object, under a parent result. */
	FFindInObjectTreeGraphResult(
			TSharedPtr<FFindInObjectTreeGraphResult>& InParent, 
			UEdGraphNode* InNode,
			FName PinName = NAME_None);

	/** Gets the icon for this result. */
	TSharedRef<SWidget>	GetIcon() const;
	/** Gets the category for this result. */
	FText GetCategory() const;
	/** Gets the display text for this result. */
	FText GetText() const;
	/** Gets the comment text for this result. */
	FText GetCommentText() const;

	/** Go to the graph node, pin, etc. */
	FReply OnClick(TSharedRef<SFindInObjectTreeGraph> FindInObjectTreeGraph);
};

/**
 * A search panel to find things in one or more object tree graphs.
 */
class SFindInObjectTreeGraph : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnGetGraphsToSearch, TArray<FFindInObjectTreeGraphSource>&);
	DECLARE_DELEGATE_TwoParams(FOnJumpToNodeRequested, UEdGraphNode*, FName);

	SLATE_BEGIN_ARGS(SFindInObjectTreeGraph)
	{}
		/** The callback to get the graphs to search. */
		SLATE_EVENT(FOnGetGraphsToSearch, OnGetGraphsToSearch)
		/** The callback to invoke when a search result wants to focus a node or one of its pins. */
		SLATE_EVENT(FOnJumpToNodeRequested, OnJumpToNodeRequested)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void FocusSearchEditBox();

	void Search(const FString& InSearchQuery);

protected:

	typedef TSharedPtr<FFindInObjectTreeGraphResult> FResultPtr;
	typedef STreeView<FResultPtr> SResultTreeView;

	void OnSearchTextChanged(const FText& Text);
	void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	TSharedRef<ITableRow> OnResultTreeViewGenerateRow(FResultPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnResultTreeViewGetChildren(FResultPtr InItem, TArray<FResultPtr>& OutChildren);
	void OnResultTreeViewSelectionChanged(FResultPtr Item, ESelectInfo::Type SelectInfo);
	void OnResultTreeViewMouseButtonDoubleClick(FResultPtr Item);

protected:

	void StartSearch();

protected:

	FOnGetGraphsToSearch OnGetGraphsToSearch;
	FOnJumpToNodeRequested OnJumpToNodeRequested;

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SResultTreeView> ResultTreeView;

	FString SearchQuery;
	TArray<FResultPtr> Results;

	FText HighlightText;

	friend struct FFindInObjectTreeGraphResult;
};

