// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "SGraphNodeComment.h"

class UObjectTreeGraphCommentNode;

/**
 * The widget for object tree graph comment nodes.
 */
class SObjectTreeGraphCommentNode : public SGraphNodeComment
{
public:

	SLATE_BEGIN_ARGS(SObjectTreeGraphCommentNode)
		: _GraphNode(nullptr)
	{}
		SLATE_ARGUMENT(UObjectTreeGraphCommentNode*, GraphNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	UObjectTreeGraphCommentNode* GetObjectGraphNode() const { return ObjectGraphNode; }

public:

	// SNodePanel::SNode interface.
	virtual void MoveTo(const FSlateCompatVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty=true) override;

protected:

	UObjectTreeGraphCommentNode* ObjectGraphNode;
};

