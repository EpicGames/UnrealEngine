// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"
#include "Widgets/SCompoundWidget.h"

class FAssetEditorToolkit;
class IDetailsView;
class UObjectTreeGraph;

/**
 * A graph editor for an object tree graph.
 */
class SObjectTreeGraphEditor 
	: public SCompoundWidget
	, public FEditorUndoClient
{
public:

	static TSharedPtr<SObjectTreeGraphEditor> FindGraphEditor(UObjectTreeGraph* InGraph);
	static void OnBeginEditingGraph(UObjectTreeGraph* InGraph, TSharedRef<SObjectTreeGraphEditor> InGraphEditor);
	static void OnEndEditingGraph(UObjectTreeGraph* InGraph, TSharedRef<SObjectTreeGraphEditor> InGraphEditor);

private:

	static TMap<UObjectTreeGraph*, TSharedPtr<SObjectTreeGraphEditor>> ActiveGraphEditors;

public:

	SLATE_BEGIN_ARGS(SObjectTreeGraphEditor)
	{}
		/** Any additional command mappings to use in the graph editor. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, AdditionalCommands)
		/** A custom graph title bar. A default is provided if this isn't specified. */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, GraphTitleBar)
		/** The details view to use for showing the current graph selection. */
		SLATE_ARGUMENT(TSharedPtr<IDetailsView>, DetailsView)
		/** The graph to show in the editor. */
		SLATE_ARGUMENT(UObjectTreeGraph*, GraphToEdit)
		/** The toolkit inside which this editor lives, if any. */
		SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
		/** The graph editor appearance. */
		SLATE_ATTRIBUTE(FGraphAppearanceInfo, Appearance)
		/** The graph editor title. */
		SLATE_ATTRIBUTE(FText, GraphTitle)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SObjectTreeGraphEditor();

	void RegisterEditor();
	void UnregisterEditor();

	void JumpToNode(UEdGraphNode* InNode, FName InPinName = NAME_None);
	bool FindAndJumpToObjectNode(UObject* InObject, FName InPropertyName = NAME_None);

	void ResyncDetailsView();

	TSharedPtr<SGraphEditor> GetGraphEditor() { return GraphEditor; }

protected:

	// SWidget interface.
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	// FEditorUndoClient interface.
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

protected:

	void InitializeBuiltInCommands();

	void OnGraphSelectionChanged(const FGraphPanelSelectionSet& SelectionSet);
	void OnNodeTextCommitted(const FText& InText, ETextCommit::Type InCommitType, UEdGraphNode* InEditedNode);
	void OnNodeDoubleClicked(UEdGraphNode* InClickedNode);
	void OnDoubleClicked();

	FString ExportNodesToText(const FGraphPanelSelectionSet& Nodes, bool bOnlyCanDuplicateNodes, bool bOnlyCanDeleteNodes);
	void ImportNodesFromText(const FSlateCompatVector2f& Location, const FString& TextToImport);
	bool CanImportNodesFromText(const FString& TextToImport);
	void DeleteNodes(TArrayView<UEdGraphNode*> NodesToDelete);

	void SelectAllNodes();
	bool CanSelectAllNodes();

	void DeleteSelectedNodes();
	bool CanDeleteSelectedNodes();

	void CopySelectedNodes();
	bool CanCopySelectedNodes();

	void CutSelectedNodes();
	bool CanCutSelectedNodes();

	void PasteNodes();
	bool CanPasteNodes();

	void DuplicateNodes();
	bool CanDuplicateNodes();

	void OnRenameNode();
	bool CanRenameNode();

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();

	void OnStraightenConnections();
	void OnDistributeNodesHorizontally();
	void OnDistributeNodesVertically();

	void OnInsertArrayItemPinBefore();
	void OnInsertArrayItemPinAfter();
	void OnRemoveArrayItemPin();

protected:

	TSharedPtr<SGraphEditor> GraphEditor;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<FUICommandList> BuiltInCommands;
};

