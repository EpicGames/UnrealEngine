// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compat/EditorCompat.h"
#include "SGraphNode.h"
#include "SGraphPin.h"

class FArrayProperty;
class UObjectTreeGraphNode;

/**
 * The widget used by default for object tree graph nodes.
 */
class SObjectTreeGraphNode : public SGraphNode
{
public:

	SLATE_BEGIN_ARGS(SObjectTreeGraphNode)
		: _GraphNode(nullptr)
	{}
		SLATE_ARGUMENT(UObjectTreeGraphNode*, GraphNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	UObjectTreeGraphNode* GetObjectGraphNode() const { return ObjectGraphNode; }

public:

	// SGraphNode interface.
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin) const override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual const FSlateBrush* GetNodeBodyBrush() const override;

	// SNodePanel::SNode interface.
	virtual void MoveTo(const FSlateCompatVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty=true) override;

protected:

	void MakeAllAddArrayPropertyPinButtons(TSharedPtr<SVerticalBox> Box, EEdGraphPinDirection Direction);
	TSharedRef<SWidget> MakeAddArrayPropertyPinButton(FArrayProperty* ArrayProperty);

	FReply OnAddArrayPropertyPin(FArrayProperty* ArrayProperty);

protected:

	UObjectTreeGraphNode* ObjectGraphNode;

	bool bHasAddPinButtons = false;
};

