// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SScrollBox;
class STableViewBase;
class UEdGraph;

template<typename> class SBreadcrumbTrail;
template<typename> class SListView;

DECLARE_DELEGATE_OneParam(FObjectTreeGraphEvent, UEdGraph*);

struct FObjectTreeGraphInfo
{
	FString GraphName;
};

/**
 * The default object graph title bar used by the object tree graph editor.
 */
class SObjectTreeGraphTitleBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SObjectTreeGraphTitleBar)
		: _Graph(nullptr)
	{}
		SLATE_ATTRIBUTE(FText, TitleText)
		SLATE_ARGUMENT(UEdGraph*, Graph)
		SLATE_ARGUMENT(const TArray<TSharedPtr<FObjectTreeGraphInfo>>*, GraphList)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, HistoryNavigationWidget)
		SLATE_EVENT(FObjectTreeGraphEvent, OnBreadcrumbClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	TSharedRef<ITableRow> GenerateGraphInfoRow(TSharedPtr<FObjectTreeGraphInfo> Item, const TSharedRef<STableViewBase>& OwnerTable);

	void OnBreadcrumbClickedImpl(UEdGraph* const& Item);
	void RebuildBreadcrumbTrail();
	static FText GetTitleForOneCrumb(const UEdGraph* BaseGraph, const UEdGraph* CurGraph);

protected:

	UEdGraph* Graph;

	TSharedPtr<SListView<TSharedPtr<FObjectTreeGraphInfo>>> GraphListView;

	TSharedPtr<SBreadcrumbTrail<UEdGraph*>> BreadcrumbTrail;
	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;
	FObjectTreeGraphEvent OnBreadcrumbClicked;
};

