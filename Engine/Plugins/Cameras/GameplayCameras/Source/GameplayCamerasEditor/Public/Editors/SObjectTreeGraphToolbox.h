// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Misc/TextFilter.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
class UClass;

template<typename> class SListView;

/**
 * A widget for an object tree graph toolbox entry, showing a specific
 * instantiable object class that can be added to a graph.
 */
class SObjectTreeGraphToolboxEntry : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SObjectTreeGraphToolboxEntry)
		: _ObjectClass(nullptr)
		, _GraphConfig(nullptr)
	{}
		/** The object class represented by this entry. */
		SLATE_ARGUMENT(UClass*, ObjectClass)
		/** The configuration of the graph this toolbox works for. */
		SLATE_ARGUMENT(const FObjectTreeGraphConfig*, GraphConfig)
		/** Text to highlight if a search is ongoing. */
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	UClass* GetObjectClass() const { return ObjectClass; }

protected:

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	const FSlateBrush* GetBorder() const;

private:

	UClass* ObjectClass = nullptr;
	FText DisplayNameText;

	bool bIsPressed = false;

	const FSlateBrush* NormalImage = nullptr;
	const FSlateBrush* HoverImage = nullptr;
	const FSlateBrush* PressedImage = nullptr;
};

/**
 * A toolbox widget that shows all the possible instantiable classes of objects for a
 * given object tree graph.
 */
class SObjectTreeGraphToolbox : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SObjectTreeGraphToolbox)
	{}
		SLATE_ARGUMENT(FObjectTreeGraphConfig, GraphConfig)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/**
	 * Updates the toolbox to reflect the list of instantiable objects for
	 * the given graph configuration.
	 */
	void SetGraphConfig(const FObjectTreeGraphConfig& InGraphConfig);

protected:

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	void GetEntryStrings(const UClass* InItem, TArray<FString>& OutStrings);

	void UpdateItemSource();
	void UpdateFilteredItemSource();

	TSharedRef<ITableRow> OnGenerateItemRow(UClass* Item, const TSharedRef<STableViewBase>& OwnerTable);

	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	FText GetHighlightText() const;

private:

	FObjectTreeGraphConfig GraphConfig;

	TArray<UClass*> ItemSource;
	TSharedPtr<SListView<UClass*>> ListView;

	using FEntryTextFilter = TTextFilter<const UClass*>;
	TSharedPtr<FEntryTextFilter> SearchTextFilter;
	TSharedPtr<SSearchBox> SearchBox;

	TArray<UClass*> FilteredItemSource;

	bool bUpdateItemSource = false;
	bool bUpdateFilteredItemSource = false;
};

