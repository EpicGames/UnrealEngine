// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "FrameNumberDisplayFormat.h"

#include "GameplayCamerasEditorSettings.generated.h"

UCLASS(Config=EditorPerProjectUserSettings, MinimalAPI, meta=(DisplayName="Gameplay Cameras Editor"))
class UGameplayCamerasEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UGameplayCamerasEditorSettings(const FObjectInitializer& ObjectInitializer);	

public:

	/** Title color for common camera nodes. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraNodeTitleColor;

	/** Title color for a camera asset's root node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraAssetTitleColor;

	/** Title color for a camera rig asset's root node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraRigAssetTitleColor;

	/** Title color for a camera shake asset's root node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraShakeAssetTitleColor;

	/** Title color for a camera transition node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraRigTransitionTitleColor;

	/** Title color for a camera transition condition node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraRigTransitionConditionTitleColor;

	/** Title color for a camera blend node. */
	UPROPERTY(EditAnywhere, Config, Category=NodeTitleColors)
	FLinearColor CameraBlendNodeTitleColor;

public:

	/** Camera asset mode to restore on open (director, shared transitions, etc.) */
	UPROPERTY()
	FName LastCameraAssetToolkitModeName;

public:

	/** Global enable/disable flag for running camera rigs in editor. */
	UPROPERTY(EditAnywhere, Config, Category="Editor Preview")
	bool bEnableRunInEditor = true;
};

