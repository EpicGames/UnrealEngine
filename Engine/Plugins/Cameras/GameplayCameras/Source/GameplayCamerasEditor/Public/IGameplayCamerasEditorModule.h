// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Build/CameraBuildLog.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

class SWidget;
class UCameraAsset;
class UCameraAssetEditor;
class UCameraRigAsset;
class UCameraRigAssetEditor;
class UCameraRigProxyAsset;
class UCameraRigProxyAssetEditor;
class UCameraShakeAsset;
class UCameraShakeAssetEditor;
class UCameraVariableCollection;
class UCameraVariableCollectionEditor;

namespace UE::Cameras
{

class FCameraBuildLog;
class FCameraDirectorAssetEditorMode;
struct FCameraVariablePickerConfig;

struct FCameraDebugCategoryInfo
{
	FString Name;
	FText DisplayText;
	FText ToolTipText;
	FSlateIcon IconImage;
};

}  // namespace UE::Cameras

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<UE::Cameras::FCameraDirectorAssetEditorMode>, FOnCreateCameraDirectorAssetEditorMode, UCameraAsset*);
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnCreateDebugCategoryPanel, const FString&);

/**
 * The gameplay cameras editor module.
 */
class IGameplayCamerasEditorModule : public IModuleInterface
{
public:

	static const FName GameplayCamerasEditorAppIdentifier;
	static const FName CameraRigAssetEditorToolBarName;

	GAMEPLAYCAMERASEDITOR_API static IGameplayCamerasEditorModule& Get();

	virtual ~IGameplayCamerasEditorModule() = default;

public:

	using FCameraVariablePickerConfig = UE::Cameras::FCameraVariablePickerConfig;

	/** Creates an editor for the given camera asset */
	virtual UCameraAssetEditor* CreateCameraAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraAsset* CameraAsset) = 0;

	/** Creates an editor for the given camera rig asset */
	virtual UCameraRigAssetEditor* CreateCameraRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraRigAsset* CameraRig) = 0;

	/** Creates an editor for the given camera rig proxy asset */
	virtual UCameraRigProxyAssetEditor* CreateCameraRigProxyEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraRigProxyAsset* CameraRigProxy) = 0;

	/** Creates an editor for the given camera shake asset */
	virtual UCameraShakeAssetEditor* CreateCameraShakeEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraShakeAsset* CameraShake) = 0;

	/** Creates an editor for the given variable collection */
	virtual UCameraVariableCollectionEditor* CreateCameraVariableCollectionEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCameraVariableCollection* VariableCollection) = 0;

	/** Creates a new camera varable asset picker widget */
	virtual TSharedRef<SWidget> CreateCameraVariablePicker(const FCameraVariablePickerConfig& InPickerConfig) = 0;

public:

	/** Registers a new camera director editor creator. */
	virtual FDelegateHandle RegisterCameraDirectorEditor(FOnCreateCameraDirectorAssetEditorMode InOnCreateEditor) = 0;
	/** Gets the registered camera director editor creators. */
	virtual TArrayView<const FOnCreateCameraDirectorAssetEditorMode> GetCameraDirectorEditorCreators() const = 0;
	/** Unregisters a camera director editor creator. */
	virtual void UnregisterCameraDirectorEditor(FDelegateHandle InHandle) = 0;

public:

	using FCameraDebugCategoryInfo = UE::Cameras::FCameraDebugCategoryInfo;

	/** Registers a new debug category, to be displayed in the camera debugger tool. */
	virtual void RegisterDebugCategory(const UE::Cameras::FCameraDebugCategoryInfo& InCategoryInfo) = 0;
	/** Gets all registered debug categories. */
	virtual void GetRegisteredDebugCategories(TArray<UE::Cameras::FCameraDebugCategoryInfo>& OutCategoryInfos) = 0;
	/** Unregisters a debug category. */
	virtual void UnregisterDebugCategory(const FString& InCategoryName) = 0;

	/** Registers a custom UI panel for a given debug category. */
	virtual void RegisterDebugCategoryPanel(const FString& InCategoryName, FOnCreateDebugCategoryPanel OnCreatePanel) = 0;
	/** Creates the custom UI panel (if any) for a given debug category. */
	virtual TSharedPtr<SWidget> CreateDebugCategoryPanel(const FString& InCategoryName) = 0;
	/** Unregisters a debug category's custom UI panel. */
	virtual void UnregisterDebugCategoryPanel(const FString& InCategoryName) = 0;
};

DECLARE_LOG_CATEGORY_EXTERN(LogCameraSystemEditor, Log, All);

