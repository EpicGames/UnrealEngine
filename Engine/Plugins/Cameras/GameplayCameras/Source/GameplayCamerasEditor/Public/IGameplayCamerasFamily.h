// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

struct FSlateBrush;

namespace UE::Cameras
{

class IGameplayCamerasFamily : public TSharedFromThis<IGameplayCamerasFamily>
{
public:

	virtual ~IGameplayCamerasFamily() = default;

	virtual UObject* GetRootAsset() const = 0;

	virtual void GetAssetTypes(TArray<UClass*>& OutAssetTypes) const = 0;

	virtual void FindAssetsOfType(UClass* InAssetType, TArray<FAssetData>& OutAssets) const = 0;

	virtual FText GetAssetTypeTooltip(UClass* InAssetType) const = 0;

	virtual const FSlateBrush* GetAssetIcon(UClass* InAssetType) const = 0;

	virtual FSlateColor GetAssetTint(UClass* InAssetType) const = 0;

public:

	static TSharedPtr<IGameplayCamerasFamily> CreateFamily(UObject* InAsset);
};

}  // namespace UE::Cameras

