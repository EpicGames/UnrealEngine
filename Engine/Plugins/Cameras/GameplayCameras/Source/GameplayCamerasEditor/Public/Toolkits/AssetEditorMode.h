// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Docking/TabManager.h"

class FAssetEditorToolkit;
class FExtender;
class FLayoutExtender;
class FUICommandList;
struct FToolMenuContext;

namespace UE::Cameras
{

/**
 * Parameter structure for activating an asset editor mode.
 */
struct FAssetEditorModeActivateParams
{
	TSharedPtr<FAssetEditorToolkit> Toolkit;
	TSharedPtr<FTabManager> TabManager;
	TSharedPtr<FWorkspaceItem> AssetEditorTabsCategory;
	TSharedPtr<FUICommandList> CommandList;
	FName ToolbarMenuName;
};

/**
 * Parameter structure for deactivating an asset editor mode.
 */
struct FAssetEditorModeDeactivateParams
{
	TSharedPtr<FAssetEditorToolkit> Toolkit;
	TSharedPtr<FTabManager> TabManager;
};

/**
 * An editor mode inside an FAssetEditorModeManagerToolkit.
 *
 * This changes the toolkit's editor to match a desired "editing mode" or "workflow"
 * in that editor. The layout changes, the tabs change, the toolbars change, etc.
 */
class FAssetEditorMode 
	: public TSharedFromThis<FAssetEditorMode>
{
public:

	FAssetEditorMode();
	FAssetEditorMode(FName InModeName);
	virtual ~FAssetEditorMode();

	void ActivateMode(const FAssetEditorModeActivateParams& InParams);
	void DeactivateMode(const FAssetEditorModeDeactivateParams& InParams);

	FName GetModeName() const { return ModeName; }
	TSharedPtr<FTabManager::FLayout> GetDefaultLayout() const { return DefaultLayout; }
	TSharedPtr<FExtender> GetToolbarExtender() const { return ToolbarExtender; }
	TSharedPtr<FLayoutExtender> GetLayoutExtender() const { return LayoutExtender; }

	void InitToolMenuContext(FToolMenuContext& MenuContext);

protected:

	virtual void OnActivateMode(const FAssetEditorModeActivateParams& InParams) {}
	virtual void OnInitToolMenuContext(FToolMenuContext& MenuContext) {}
	virtual void OnDeactivateMode(const FAssetEditorModeDeactivateParams& InParams) {}

protected:

	FName ModeName;

	TSharedPtr<FTabManager::FLayout> DefaultLayout;
	TSharedPtr<FExtender> ToolbarExtender;
	TSharedPtr<FLayoutExtender> LayoutExtender;
};

}  // namespace UE::Cameras

