// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Tools/BaseAssetToolkit.h"

struct FToolMenuContext;

namespace UE::Cameras
{

class FAssetEditorMode;

/**
 * An editor toolkit that can manage different "editor modes".
 *
 * Similar to FWorkflowCentricApplication but with a few nuances such as making it
 * possible to retain some common tabs between modes.
 */
class FAssetEditorModeManagerToolkit 
	: public FBaseAssetToolkit
{
public:

	FAssetEditorModeManagerToolkit(UAssetEditor* InOwningAssetEditor);
	~FAssetEditorModeManagerToolkit();

protected:

	// FBaseAssetToolkit interface
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	// FCameraAssetEditorToolkitMode interface
	virtual void OnEditorToolkitModeActivated() {}

protected:

	void AddEditorMode(TSharedRef<FAssetEditorMode> InMode);
	void RemoveEditorMode(TSharedRef<FAssetEditorMode> InMode);
	void RemoveEditorMode(FName InModeName);

	TSharedPtr<FAssetEditorMode> GetEditorMode(FName InModeName) const;
	void GetEditorModes(TArray<TSharedPtr<FAssetEditorMode>>& OutModes) const;

	void SetEditorMode(FName InModeName);
	bool CanSetEditorMode(FName InModeName) const;
	bool IsEditorMode(FName InModeName) const;
	FName GetCurrentEditorModeName() const;

	template<typename EditorModeClass>
	TSharedPtr<EditorModeClass> GetTypedEditorMode(FName InModeName) const
	{
		return StaticCastSharedPtr<EditorModeClass>(GetEditorMode(InModeName));
	}

private:

	TMap<FName, TSharedPtr<FAssetEditorMode>> EditorModes;

	FName CurrentEditorModeName;
	TSharedPtr<FAssetEditorMode> CurrentEditorMode;
};

}  // namespace UE::Cameras

