// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraAsset.h"
#include "Toolkits/CameraDirectorAssetEditorMode.h"

namespace UE::Cameras
{

class FBlueprintCameraDirectorAssetEditorMode
	: public FCameraDirectorAssetEditorMode
	, public ICameraAssetEventHandler
{
public:

	static TSharedPtr<FCameraDirectorAssetEditorMode> CreateInstance(UCameraAsset* InCameraAsset);

	FBlueprintCameraDirectorAssetEditorMode(UCameraAsset* InCameraAsset);

private:

	TCameraEventHandler<ICameraAssetEventHandler> EventHandler;
};

}  // namespace UE::Cameras

