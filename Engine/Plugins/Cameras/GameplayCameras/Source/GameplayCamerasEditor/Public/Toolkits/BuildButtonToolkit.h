// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ScriptInterface.h"

class FUICommandInfo;
class IHasCameraBuildStatus;
struct FSlateIcon;
struct FToolMenuEntry;

namespace UE::Cameras
{

/**
 * A utility toolkit for have a "Build" toolbar button that features a different
 * icon based on the target asset's build status.
 */
class FBuildButtonToolkit : public TSharedFromThis<FBuildButtonToolkit>
{
public:

	FBuildButtonToolkit();
	FBuildButtonToolkit(TScriptInterface<IHasCameraBuildStatus> InTarget);

	void SetTarget(TScriptInterface<IHasCameraBuildStatus> InTarget);

	FToolMenuEntry MakeToolbarButton(TSharedPtr<FUICommandInfo> InCommand);

private:

	FSlateIcon GetBuildButtonIcon() const;
	FText GetBuildButtonTooltip() const;

private:

	TScriptInterface<IHasCameraBuildStatus> Target;
};

}  // namespace UE::Cameras

