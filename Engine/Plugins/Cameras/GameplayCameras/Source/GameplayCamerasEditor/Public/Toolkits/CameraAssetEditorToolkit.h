// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraAsset.h"
#include "Core/CameraEventHandler.h"
#include "EditorUndoClient.h"
#include "Toolkits/AssetEditorModeManagerToolkit.h"
#include "Tools/BaseAssetToolkit.h"
#include "UObject/GCObject.h"

#include "CameraAssetEditorToolkit.generated.h"

class SFindInObjectTreeGraph;
class UCameraAssetEditor;
class UEdGraph;
class UEdGraphNode;
class UGameplayCamerasEditorSettings;
struct FFindInObjectTreeGraphSource;

namespace UE::Cameras
{

class FBuildButtonToolkit;
class FCameraBuildLogToolkit;
class FStandardToolkitLayout;
class IGameplayCamerasLiveEditManager;
class SCameraAssetInterfaceParametersPanel;

/**
 * Editor toolkit for a camera asset.
 */
class FCameraAssetEditorToolkit 
	: public FAssetEditorModeManagerToolkit
	, public FEditorUndoClient
	, public FGCObject
	, public UE::Cameras::ICameraAssetEventHandler
{
public:

	FCameraAssetEditorToolkit(UCameraAssetEditor* InOwningAssetEditor);
	~FCameraAssetEditorToolkit();

	UCameraAsset* GetCameraAsset() const { return CameraAsset; }

protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual void PostInitAssetEditor() override;
	virtual void PostRegenerateMenusAndToolbars() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// FAssetEditorModeManagerToolkit interface
	virtual void OnEditorToolkitModeActivated() override;

	// ICameraAssetEventHandler interface
	virtual void OnCameraDirectorChanged(UCameraAsset* InCameraAsset, const TCameraPropertyChangedEvent<UCameraDirector*>& Event) override;

private:

	TSharedRef<SDockTab> SpawnTab_Search(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Messages(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args);

	void FillCameraMenu(FMenuBuilder& MenuBuilder);

	TSharedPtr<FAssetEditorMode> CreateCameraDirectorAssetEditorMode();

	void OnChangeCameraDirector();
	void OnBuild();
	void OnFindInCamera();

	void OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources);
	void OnJumpToNode(UEdGraphNode* Node, FName PinName = NAME_None);

	void OnJumpToObject(UObject* Object);
	void OnJumpToObject(UObject* Object, FName PropertyName);

	void OnInterfaceParameterSelected(UCameraAssetInterfaceParameter* InterfaceParameter);

	void UpgradeLegacyCameraAssets();

private:

	static const FName SearchTabId;
	static const FName MessagesTabId;
	static const FName InterfaceParametersTabId;

	/** The asset being edited */
	TObjectPtr<UCameraAsset> CameraAsset;

	/** Event listener for the camera asset */
	UE::Cameras::TCameraEventHandler<UE::Cameras::ICameraAssetEventHandler> CameraAssetEventHandler;

	/** The layout for this toolkit */
	TSharedPtr<FStandardToolkitLayout> StandardLayout;

	/** The "Build" button */
	TSharedPtr<FBuildButtonToolkit> BuildButtonToolkit;

	/** The output/log window */
	TSharedPtr<FCameraBuildLogToolkit> BuildLogToolkit;

	/** The search results window */
	TSharedPtr<SFindInObjectTreeGraph> SearchWidget;

	/** The panel for exposing parameters. */
	TSharedPtr<SCameraAssetInterfaceParametersPanel> InterfaceParametersPanel;

	/** A cached pointer to the editor settings. */
	TObjectPtr<UGameplayCamerasEditorSettings> Settings;

	/** Live edit manager for updating the assets in the runtime */
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager;
};

}  // namespace UE::Cameras

UCLASS()
class UCameraAssetEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Cameras::FCameraAssetEditorToolkit> Toolkit;
};

