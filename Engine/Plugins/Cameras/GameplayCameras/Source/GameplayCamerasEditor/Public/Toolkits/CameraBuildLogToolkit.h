// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class IMessageLogListing;
class IMessageToken;
class SWidget;
class UObject;

namespace UE::Cameras
{

class FCameraBuildLog;

DECLARE_DELEGATE_OneParam(FRequestJumpToObject, UObject*);

/**
 * A utility toolkit class that creates and manages a "messages" or "output" tab that can
 * show the build log of a camera asset or camera rig asset.
 */
class FCameraBuildLogToolkit : public TSharedFromThis<FCameraBuildLogToolkit>
{
public:

	FCameraBuildLogToolkit();

	void Initialize(FName InLogName);

	TSharedPtr<IMessageLogListing> GetMessageListing() const { return MessageListing; }
	TSharedPtr<SWidget> GetMessagesWidget() const { return MessagesWidget; }

	FRequestJumpToObject& OnRequestJumpToObject() { return RequestJumpToObject; }

	void PopulateMessageListing(FCameraBuildLog& InBuildLog);

private:

	void OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken);

private:

	TSharedPtr<IMessageLogListing> MessageListing;

	TSharedPtr<SWidget> MessagesWidget;

	FRequestJumpToObject RequestJumpToObject;
};

}  // namespace UE::Cameras

