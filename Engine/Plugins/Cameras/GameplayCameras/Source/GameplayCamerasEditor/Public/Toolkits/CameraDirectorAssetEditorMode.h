// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/NotifyHook.h"
#include "Toolkits/AssetEditorMode.h"

class IDetailsView;
class UCameraAsset;
class UEdGraphNode;

namespace UE::Cameras
{

class FStandardToolkitLayout;

class FCameraDirectorAssetEditorMode
	: public FAssetEditorMode
	, public FNotifyHook
{
public:

	static FName ModeName;

	FCameraDirectorAssetEditorMode(UCameraAsset* InCameraAsset);

	bool JumpToObject(UObject* InObject, FName InPropertyName);

protected:

	// FAssetEditorMode interface.
	virtual void OnActivateMode(const FAssetEditorModeActivateParams& InParams) override;
	virtual void OnDeactivateMode(const FAssetEditorModeDeactivateParams& InParams) override;

	// FNotifyHook interface.
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

private:

	TSharedRef<SDockTab> SpawnTab_DirectorEditor(const FSpawnTabArgs& Args);

private:

	static FName DirectorEditorTabId;

	UCameraAsset* CameraAsset;

	TSharedPtr<FStandardToolkitLayout> StandardLayout;

	TSharedPtr<IDetailsView> DetailsView;

	bool bInitializedToolkit = false;
};

}  // namespace UE::Cameras

