// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2.h"
#include "EditorUndoClient.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

class FUICommandList;
class SBox;
class SWidget;
class UBaseCameraObject;
class UCameraObjectInterfaceParameterBase;

namespace UE::Cameras
{

class SCameraObjectInterfaceParametersPanel;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCameraObjectInterfaceParameterEvent, UCameraObjectInterfaceParameterBase*);

/**
 * Utility toolkit for the "interface parameters" panel of any camera object editor.
 */
class FCameraObjectInterfaceParametersToolkit 
	: public TSharedFromThis<FCameraObjectInterfaceParametersToolkit>
	, public FEditorUndoClient
{
public:

	FCameraObjectInterfaceParametersToolkit();
	~FCameraObjectInterfaceParametersToolkit();

	/** Gets the camera object asset to edit. */
	UBaseCameraObject* GetCameraObject() const { return CameraObject; }
	/** Sets the camera object to edit. This re-creates the panel widget. */
	void SetCameraObject(UBaseCameraObject* InCameraObject);

	/** Gets the panel widget. */
	TSharedPtr<SWidget> GetInterfaceParametersPanel() const;

public:

	/** Rename the selected parameter in the focused panel. */
	void RenameSelectedParameter();

	/** Delete the selected parameter in the focused panel. */
	void DeleteSelectedParameter();

public:

	/** Delegate invoked when a parmeter is selected in the panel. */
	FOnCameraObjectInterfaceParameterEvent& OnInterfaceParameterSelected() { return OnInterfaceParameterSelectedDelegate; }

	/** Delegate invoked when the user requests finding getter nodes in the graph. */
	FOnCameraObjectInterfaceParameterEvent& OnSearchInterfaceParameterNodes() { return OnSearchInterfaceParameterNodesDelegate; }

protected:

	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:

	TObjectPtr<UBaseCameraObject> CameraObject;

	FOnCameraObjectInterfaceParameterEvent OnInterfaceParameterSelectedDelegate;
	FOnCameraObjectInterfaceParameterEvent OnSearchInterfaceParameterNodesDelegate;

	TSharedPtr<SBox> PanelContainer;
	TSharedPtr<SCameraObjectInterfaceParametersPanel> Panel;
};

}  // namespace UE::Cameras

