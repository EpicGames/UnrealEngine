// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraRigAsset.h"
#include "CoreTypes.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Tools/BaseAssetToolkit.h"

#include "CameraRigAssetEditorToolkit.generated.h"

class SFindInObjectTreeGraph;
class UCameraRigAsset;
struct FEdGraphEditAction;
struct FFindInObjectTreeGraphSource;

namespace UE::Cameras
{

class FBuildButtonToolkit;
class FCameraBuildLogToolkit;
class FCameraObjectInterfaceParametersToolkit;
class FCameraRigAssetEditorToolkitBase;
class FCurveEditorToolkit;
class IGameplayCamerasLiveEditManager;

class FCameraRigAssetEditorToolkit
	: public FBaseAssetToolkit
	, public ICameraRigAssetEventHandler
{
public:

	FCameraRigAssetEditorToolkit(UAssetEditor* InOwningAssetEditor);
	~FCameraRigAssetEditorToolkit();

	void SetCameraRigAsset(UCameraRigAsset* InCameraRig);

protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual void PostInitAssetEditor() override;
	virtual void PostRegenerateMenusAndToolbars() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// ICameraRigAssetEventHandler interface
	virtual void OnObjectAddedToGraph(const FName GraphName, UObject* Object) override;
	virtual void OnObjectRemovedFromGraph(const FName GraphName, UObject* Object) override;

private:

	TSharedRef<SDockTab> SpawnTab_Search(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Messages(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Curves(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args);

	void OnCurvesTabClosed(TSharedRef<SDockTab> InTab);

	void OnCameraObjectInterfaceParameterSelected(UCameraObjectInterfaceParameterBase* Object);
	void OnSearchCameraObjectInterfaceParameterNodes(UCameraObjectInterfaceParameterBase* Object);

	void OnBuild();
	void OnFindInCameraRig();

	void OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources);
	void OnJumpToNode(UEdGraphNode* Node, FName PinName);

	void OnInvokeCurveEditor(UObject* Object, FName PropertyName);

private:

	static const FName SearchTabId;
	static const FName MessagesTabId;
	static const FName CurvesTabId;
	static const FName InterfaceParametersTabId;

	/** Base implementation */
	TSharedPtr<FCameraRigAssetEditorToolkitBase> Impl;

	/** Cached config for the node graph */
	FObjectTreeGraphConfig NodeGraphConfig;
	/** Cached config for the transition graph */
	FObjectTreeGraphConfig TransitionGraphConfig;

	/** The build button */
	TSharedPtr<FBuildButtonToolkit> BuildButtonToolkit;
	/** The output log */
	TSharedPtr<FCameraBuildLogToolkit> BuildLogToolkit;
	/** The curve editor */
	TSharedPtr<FCurveEditorToolkit> CurveEditorToolkit;
	/** The interface parameters panel */
	TSharedPtr<FCameraObjectInterfaceParametersToolkit> InterfaceParametersToolkit;

	/** Search widget */
	TSharedPtr<SFindInObjectTreeGraph> SearchWidget;

	/** Live edit manager for updating the assets in the runtime */
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager;

	/** Event handler */
	TCameraEventHandler<ICameraRigAssetEventHandler> EventHandler;

	/** Whether the toolkit has finished initializing */
	bool bIsInitialized = false;
};

}  // namespace UE::Cameras

UCLASS()
class UCameraRigAssetEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Cameras::FCameraRigAssetEditorToolkit> Toolkit;
};

