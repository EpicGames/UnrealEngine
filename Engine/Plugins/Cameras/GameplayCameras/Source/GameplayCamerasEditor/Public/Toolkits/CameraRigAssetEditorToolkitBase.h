// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "UObject/GCObject.h"

class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class FWorkspaceItem;
class IDetailsView;
class SDockTab;
class SObjectTreeGraphToolbox;
class UCameraRigAsset;
class UToolMenu;
struct FEdGraphEditAction;

namespace UE::Cameras
{

class IGameplayCamerasLiveEditManager;
class FStandardToolkitLayout;
class SCameraRigAssetEditor;
enum class ECameraRigAssetEditorMode;

/**
 * Editor toolkit for a camera rig asset.
 */
class FCameraRigAssetEditorToolkitBase
	: public FGCObject
	, public FNotifyHook
	, public TSharedFromThis<FCameraRigAssetEditorToolkitBase>
{
public:

	FCameraRigAssetEditorToolkitBase(FName InLayoutName);
	~FCameraRigAssetEditorToolkitBase();

	UCameraRigAsset* GetCameraRigAsset() const { return CameraRigAsset; }
	void SetCameraRigAsset(UCameraRigAsset* InCameraRig);

	TSharedPtr<FStandardToolkitLayout> GetStandardLayout() const { return StandardLayout; }
	TSharedPtr<IDetailsView> GetDetailsView() const { return DetailsView; }
	TSharedPtr<SCameraRigAssetEditor> GetCameraRigAssetEditor() const { return CameraRigEditorWidget; }

	void RegisterTabSpawners(TSharedRef<FTabManager> InTabManager, TSharedPtr<FWorkspaceItem> InAssetEditorTabsCategory);
	void UnregisterTabSpawners(TSharedRef<FTabManager> InTabManager);
	void CreateWidgets();
	void BuildToolbarMenu(UToolMenu* ToolbarMenu);
	void BindCommands(TSharedRef<FUICommandList> CommandList);

	void SetLiveEditManager(TSharedPtr<IGameplayCamerasLiveEditManager> InLiveEditManager);

	ECameraRigAssetEditorMode GetCameraRigEditorMode() const;
	bool IsCameraRigEditorMode(ECameraRigAssetEditorMode InEditorMode) const;

protected:

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

protected:

	void SetCameraRigEditorMode(ECameraRigAssetEditorMode InEditorMode);

private:

	TSharedRef<SDockTab> SpawnTab_Toolbox(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CameraRigEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);

	void OnAnyGraphChanged(const FEdGraphEditAction& InEditAction);

private:

	static const FName ToolboxTabId;
	static const FName CameraRigEditorTabId;
	static const FName DetailsViewTabId;

	/** The asset being edited */
	TObjectPtr<UCameraRigAsset> CameraRigAsset;

	/** The layout for this editor toolkit */
	TSharedPtr<FStandardToolkitLayout> StandardLayout;

	/** The details view */
	TSharedPtr<IDetailsView> DetailsView;

	/** Camera rig editor widget */
	TSharedPtr<SCameraRigAssetEditor> CameraRigEditorWidget;

	/** Toolbox widget */
	TSharedPtr<SObjectTreeGraphToolbox> ToolboxWidget;

	/** Live editing manager */
	TSharedPtr<IGameplayCamerasLiveEditManager> LiveEditManager;
};

}  // namespace UE::Cameras

