// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/BaseAssetToolkit.h"
#include "UObject/GCObject.h"

class IMessageLogListing;
class UCameraRigProxyAsset;
class UCameraRigProxyAssetEditor;
class SWidget;

namespace UE::Cameras
{

/**
 * Editor toolkit for a camera rig proxy asset.
 */
class FCameraRigProxyAssetEditorToolkit 
	: public FBaseAssetToolkit
	, public FGCObject
{
public:

	FCameraRigProxyAssetEditorToolkit(UCameraRigProxyAssetEditor* InOwningAssetEditor);
	~FCameraRigProxyAssetEditorToolkit();

	void SetCameraRigProxyAsset(UCameraRigProxyAsset* InCameraRigProxyAsset);

protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;
	virtual void PostInitAssetEditor() override;
	virtual void PostRegenerateMenusAndToolbars() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:

	static const FName DetailsViewTabId;

	/** The asset being edited */
	TObjectPtr<UCameraRigProxyAsset> CameraRigProxyAsset;
};

}  // namespace UE::Cameras

