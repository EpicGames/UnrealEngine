// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Tools/BaseAssetToolkit.h"

class FTabManager;
class ICameraRigTransitionOwner;
class UAssetEditor;

namespace UE::Cameras
{

class FCameraRigTransitionEditorToolkitBase;

/**
 * Editor toolkit for camera transitions.
 */
class FCameraRigTransitionEditorToolkit
	: public FBaseAssetToolkit
{
public:

	FCameraRigTransitionEditorToolkit(UAssetEditor* InOwningAssetEditor);

	void SetTransitionOwner(UObject* InTransitionOwner);
		
protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

private:

	TSharedPtr<FCameraRigTransitionEditorToolkitBase> Impl;
};

}  // namespace UE::Cameras

