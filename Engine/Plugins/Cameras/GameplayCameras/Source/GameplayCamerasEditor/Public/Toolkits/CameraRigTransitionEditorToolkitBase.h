// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"

class FSpawnTabArgs;
class FTabManager;
class FWorkspaceItem;
class IDetailsView;
class SDockTab;
class SObjectTreeGraphToolbox;
class UCameraRigTransitionGraphSchemaBase;
class UToolMenu;
struct FEdGraphEditAction;
struct FGraphAppearanceInfo;

namespace UE::Cameras
{

class FStandardToolkitLayout;
class SCameraRigTransitionEditor;

class FCameraRigTransitionEditorToolkitBase
	: public FGCObject
	, public FNotifyHook
	, public TSharedFromThis<FCameraRigTransitionEditorToolkitBase>
{
public:

	FCameraRigTransitionEditorToolkitBase(FName InLayoutName);
	~FCameraRigTransitionEditorToolkitBase();

	UObject* GetTransitionOwner() const { return TransitionOwner; }
	void SetTransitionOwner(UObject* InTransitionOwner);

	TSharedPtr<FStandardToolkitLayout> GetStandardLayout() const { return StandardLayout; }
	TSharedPtr<SCameraRigTransitionEditor> GetCameraRigTransitionEditor() const { return TransitionEditorWidget; }

	void RegisterTabSpawners(TSharedRef<FTabManager> InTabManager, TSharedPtr<FWorkspaceItem> InAssetEditorTabsCategory);
	void UnregisterTabSpawners(TSharedRef<FTabManager> InTabManager);
	void CreateWidgets();
	void BuildToolbarMenu(UToolMenu* ToolbarMenu);

protected:

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	// FCameraRigTransitionEditorToolkitBase interface
	virtual TSubclassOf<UCameraRigTransitionGraphSchemaBase> GetTransitionGraphSchemaClass();
	virtual void GetTransitionGraphAppearanceInfo(FGraphAppearanceInfo& OutGraphAppearanceInfo) {}

private:

	TSharedRef<SDockTab> SpawnTab_Toolbox(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_TransitionEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);

	void OnTransitionGraphChanged(const FEdGraphEditAction& InEditAction);

private:

	static const FName ToolboxTabId;
	static const FName TransitionEditorTabId;
	static const FName DetailsViewTabId;

	/** The object being edited */
	TObjectPtr<UObject> TransitionOwner;

	/** The layout for this editor */
	TSharedPtr<FStandardToolkitLayout> StandardLayout;

	/** The details view for this editor */
	TSharedPtr<IDetailsView> DetailsView;

	/** Camera transition editor widget */
	TSharedPtr<SCameraRigTransitionEditor> TransitionEditorWidget;

	/** Toolbox widget */
	TSharedPtr<SObjectTreeGraphToolbox> ToolboxWidget;
};

}  // namespace UE::Cameras

