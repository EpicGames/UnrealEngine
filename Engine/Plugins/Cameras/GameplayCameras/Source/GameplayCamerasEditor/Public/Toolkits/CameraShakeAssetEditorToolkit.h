// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraShakeAsset.h"
#include "CoreTypes.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Misc/NotifyHook.h"
#include "Tools/BaseAssetToolkit.h"
#include "UObject/GCObject.h"

#include "CameraShakeAssetEditorToolkit.generated.h"

class SFindInObjectTreeGraph;
class SObjectTreeGraphEditor;
class SObjectTreeGraphToolbox;
class UCameraShakeAsset;
struct FEdGraphEditAction;
struct FFindInObjectTreeGraphSource;

namespace UE::Cameras
{

class FBuildButtonToolkit;
class FCameraBuildLogToolkit;
class FCameraObjectInterfaceParametersToolkit;
class FStandardToolkitLayout;
class IGameplayCamerasLiveEditManager;

class FCameraShakeAssetEditorToolkit
	: public FBaseAssetToolkit
	, public FGCObject
	, public FNotifyHook
{
public:

	FCameraShakeAssetEditorToolkit(UAssetEditor* InOwningAssetEditor);
	~FCameraShakeAssetEditorToolkit();

	void SetCameraShakeAsset(UCameraShakeAsset* InCameraShake);

protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual void PostInitAssetEditor() override;
	virtual void PostRegenerateMenusAndToolbars() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

private:

	TSharedRef<SDockTab> SpawnTab_CameraShakeEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Search(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Messages(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Toolbox(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_InterfaceParameters(const FSpawnTabArgs& Args);

	void CreateNodeGraphEditor();
	void DiscardNodeGraphEditor();

	FText GetCameraShakeAssetName() const;
	bool IsGraphEditorEnabled() const;

	void OnCameraObjectInterfaceParameterSelected(UCameraObjectInterfaceParameterBase* Object);

	void OnBuild();
	void OnFindInCameraShake();

	void OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources);
	void OnJumpToNode(UEdGraphNode* Node, FName PinName);

	void OnJumpToObject(UObject* Object, FName PropertyName);
	void OnFocusHome();

private:

	static const FName CameraShakeEditorTabId;

	static const FName DetailsViewTabId;

	static const FName SearchTabId;
	static const FName MessagesTabId;

	static const FName ToolboxTabId;
	static const FName InterfaceParametersTabId;

	/** The asset being edited */
	TObjectPtr<UCameraShakeAsset> CameraShakeAsset;

	/** The layout for this editor toolkit */
	TSharedPtr<FStandardToolkitLayout> StandardLayout;

	/** Cached config for the node graph */
	FObjectTreeGraphConfig NodeGraphConfig;

	/** The node hierarchy graph */
	TObjectPtr<UObjectTreeGraph> NodeGraph;
	/** The node hierarchy graph editor */
	TSharedPtr<SObjectTreeGraphEditor> NodeGraphEditor;

	/** The build button */
	TSharedPtr<FBuildButtonToolkit> BuildButtonToolkit;
	/** The output log */
	TSharedPtr<FCameraBuildLogToolkit> BuildLogToolkit;
	/** Toolbox widget */
	TSharedPtr<SObjectTreeGraphToolbox> ToolboxWidget;
	/** The interface parameters panel */
	TSharedPtr<FCameraObjectInterfaceParametersToolkit> InterfaceParametersToolkit;

	/** Search widget */
	TSharedPtr<SFindInObjectTreeGraph> SearchWidget;
};

}  // namespace UE::Cameras

UCLASS()
class UCameraShakeAssetEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Cameras::FCameraShakeAssetEditorToolkit> Toolkit;
};

