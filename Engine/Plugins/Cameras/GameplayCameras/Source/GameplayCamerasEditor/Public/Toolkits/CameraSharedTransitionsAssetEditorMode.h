// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Editors/ObjectTreeGraphConfig.h"
#include "Toolkits/AssetEditorMode.h"

class UCameraAsset;
class UEdGraphNode;
struct FFindInObjectTreeGraphSource;

namespace UE::Cameras
{

class FCameraRigTransitionEditorToolkitBase;

class FCameraSharedTransitionsAssetEditorMode
	: public FAssetEditorMode
{
public:

	static FName ModeName;

	FCameraSharedTransitionsAssetEditorMode(UCameraAsset* InCameraAsset);

	void OnGetGraphsToSearch(TArray<FFindInObjectTreeGraphSource>& OutSources);
	void JumpToNode(UEdGraphNode* InNode, FName InPinName);
	bool JumpToObject(UObject* InObject, FName InPropertyName);

protected:

	virtual void OnActivateMode(const FAssetEditorModeActivateParams& InParams) override;
	virtual void OnDeactivateMode(const FAssetEditorModeDeactivateParams& InParams) override;

private:

	UCameraAsset* CameraAsset;

	TSharedPtr<FCameraRigTransitionEditorToolkitBase> Impl;

	FObjectTreeGraphConfig TransitionGraphConfig;

	bool bInitializedToolkit = false;
};

}  // namespace UE::Cameras

