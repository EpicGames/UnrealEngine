// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EditorUndoClient.h"
#include "Tools/BaseAssetToolkit.h"
#include "UObject/GCObject.h"

#include "CameraVariableCollectionEditorToolkit.generated.h"

class UCameraVariableAsset;
class UCameraVariableCollection;
class UCameraVariableCollectionEditor;

namespace UE::Cameras
{

class SCameraVariableCollectionEditor;

/**
 * Editor toolkit for a camera variable collection.
 */
class FCameraVariableCollectionEditorToolkit
	: public FBaseAssetToolkit
	, public FEditorUndoClient
	, public FGCObject
{
public:

	FCameraVariableCollectionEditorToolkit(UCameraVariableCollectionEditor* InOwningAssetEditor);
	~FCameraVariableCollectionEditorToolkit();

	// IAssetEditorInstance interface
	virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) override;

protected:

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void CreateWidgets() override;
	virtual void RegisterToolbar() override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual void PostInitAssetEditor() override;
	virtual void PostRegenerateMenusAndToolbars() override;

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:

	TSharedRef<SDockTab> SpawnTab_VariableCollectionEditor(const FSpawnTabArgs& Args);

	static void GenerateAddNewVariableMenu(UToolMenu* InMenu);

	void OnCreateVariable(TSubclassOf<UCameraVariableAsset> InVariableClass);

	void OnRenameVariable();
	bool CanRenameVariable();

	void OnDeleteVariable();
	bool CanDeleteVariable();

private:

	static const FName VariableCollectionEditorTabId;
	static const FName DetailsViewTabId;

	/** The asset being edited */
	TObjectPtr<UCameraVariableCollection> VariableCollection;

	/** Camera variable collection editor widget */
	TSharedPtr<SCameraVariableCollectionEditor> VariableCollectionEditorWidget;
};

}  // namespace UE::Cameras

UCLASS()
class UCameraVariableCollectionEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Cameras::FCameraVariableCollectionEditorToolkit> EditorToolkit;
};

