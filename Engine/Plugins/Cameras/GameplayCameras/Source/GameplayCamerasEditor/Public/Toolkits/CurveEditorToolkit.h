// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FCurveEditor;
class FUICommandInfo;
class SWidget;
struct FCurveEditorTreeItem;
struct FCurveEditorTreeItemID;
struct FRichCurve;
struct FSlateIcon;
struct FToolMenuEntry;

namespace UE::Cameras
{

struct FCurvePropertyInfo;

/**
 * A utility toolkit that hosts a curve editor showing curves from any object that
 * has curve properties.
 */
class FCurveEditorToolkit : public TSharedFromThis<FCurveEditorToolkit>
{
public:

	FCurveEditorToolkit();

	/** Initializes the toolkit and creates the curve editor. */
	void Initialize();
	/** Initializes the toolkit and creates the curve editor, plus adds curves from the given objects. */
	void Initialize(TArrayView<UObject*> InCurveOwners);

	/** Whether this toolkit is initialized and has a valid curve editor. */
	bool IsInitialized() const { return CurveEditor.IsValid(); }

	/** Removes all curves and destroys the curve editor. */
	void Shutdown();

	/**
	 * Adds curves for any curve property on the given object. If the object doesn't have any recognizable
	 * curve properties, nothing happens.
	 */
	void AddCurveOwner(UObject* InCurveOwner);
	/**
	 * Adds curves for any curve property on the given objects. If the object doesn't have any recognizable
	 * curve properties, nothing happens.
	 */
	void AddCurveOwners(TArrayView<UObject*> InCurveOwners);
	/**
	 * Removes any existing curves belonging to the given object.
	 */
	void RemoveCurveOwner(UObject* InCurveOwner);

	/** Removes all curves from all curve owners. */
	void RemoveAllCurveOwners();

	/** Selects the curves associated with the given object and property name. */
	void SelectCurves(UObject* InCurveOwner, FName InPropertyName);

	/** Gets the curve editor widget. */
	TSharedPtr<SWidget> GetCurveEditorWidget() const { return CurveEditorWidget; }

private:

	void AddCurves(UObject* InObject);
	FCurveEditorTreeItem* AddTreeItem(FCurveEditorTreeItemID ParentID, FCurvePropertyInfo&& CurveInfo);

private:

	TSharedPtr<FCurveEditor> CurveEditor;
	TSharedPtr<SWidget> CurveEditorWidget;
};

}  // namespace UE::Cameras

