// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Docking/TabManager.h"

namespace UE::Cameras
{

/**
 * A utility class for building a standard layout with tab stacks on the
 * left, right, and bottom of a central document area.
 */
class FStandardToolkitLayout
{
public:

	static const FName LeftStackExtensionId;
	static const FName CenterStackExtensionId;
	static const FName RightStackExtensionId;
	static const FName BottomStackExtensionId;

	FStandardToolkitLayout(FName LayoutName);

	TSharedPtr<FTabManager::FLayout> GetLayout() const;

	void AddLeftTab(FName InTabId, ETabState::Type InTabState = ETabState::OpenedTab);
	void AddRightTab(FName InTabId, ETabState::Type InTabState = ETabState::OpenedTab);
	void AddBottomTab(FName InTabId, ETabState::Type InTabState = ETabState::OpenedTab);
	void AddCenterTab(FName InTabId, ETabState::Type InTabState = ETabState::OpenedTab);

private:

	void BuildLayout();

private:

	FName LayoutName;

	TSharedPtr<FTabManager::FLayout> Layout;

	TSharedPtr<FTabManager::FStack> LeftTabStack;
	TSharedPtr<FTabManager::FStack> CenterTabStack;
	TSharedPtr<FTabManager::FStack> RightTabStack;
	TSharedPtr<FTabManager::FStack> BottomTabStack;
};

}  // namespace UE::Cameras

