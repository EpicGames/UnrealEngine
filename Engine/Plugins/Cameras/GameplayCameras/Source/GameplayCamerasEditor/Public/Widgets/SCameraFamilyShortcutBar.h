// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FBaseAssetToolkit;
class SHorizontalBox;

namespace UE::Cameras
{

class IGameplayCamerasFamily;

class SCameraFamilyShortcutBar : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCameraFamilyShortcutBar)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FBaseAssetToolkit>& InToolkit, const TSharedRef<IGameplayCamerasFamily>& InFamily);

private:

	void BuildShortcuts();

private:
	
	TWeakPtr<FBaseAssetToolkit> WeakToolkit;

	TSharedPtr<IGameplayCamerasFamily> Family;

	TSharedPtr<SHorizontalBox> HorizontalBox;
};

}  // namespace UE::Cameras

