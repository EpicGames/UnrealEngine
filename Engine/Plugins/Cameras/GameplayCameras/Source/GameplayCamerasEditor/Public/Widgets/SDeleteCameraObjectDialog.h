// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAssetTypeActions.h"
#include "Widgets/SCompoundWidget.h"

class IAssetRegistry;

namespace UE::Cameras
{

DECLARE_DELEGATE_OneParam(FOnDeletedObject, UObject*);

/**
 * Re-usable dialog for deleting camera objects that are sub-objects of an asset.
 */
class SDeleteCameraObjectDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeleteCameraObjectDialog)
		: _RenameObjectsAsTrash(false)
	{}
		/** The modal window containing this dialog. */
		SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
		/** The list of objects to delete. */
		SLATE_ARGUMENT(TArrayView<UObject*>, ObjectsToDelete)
		/** Whether deleted objects should also get `TRASH_` prepended to their name. */
		SLATE_ATTRIBUTE(bool, RenameObjectsAsTrash)
		/** Custom callback invoked when objects are actually deleted. */
		SLATE_EVENT(FOnDeletedObject, OnDeletedObject)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	bool ShouldPerformDelete() const;
	void PerformReferenceReplacement();

public:

	static void RenameObjectAsTrash(FString& InOutName);
	static FString MakeTrashName(const FString& InName);

protected:

	// SCompoundWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	void StartScanning();
	void ScanNextObject(IAssetRegistry& AssetRegistry);
	void ScanNextReferencingPackage();
	void FinishScanning();

	TOptional<float> GetProgressBarPercent() const;

	TSharedRef<SWidget> BuildReferencerAssetPicker();
	bool OnShouldFilterReferencerAsset(const FAssetData& InAssetData) const;

	EVisibility GetNoReferencesVisibility() const;
	EVisibility GetReferencesVisiblity() const;
	EVisibility GetProgressBarVisibility() const;

	void OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod);
	bool IsDeleteEnabled() const;
	FReply OnDeleteClicked();
	FReply OnCancelClicked();

	void CloseWindow();

private:

	enum class EState { Waiting, StartScanning, ScanNextObject, ScanNextReferencingPackage, FinishScanning };

	TWeakPtr<SWindow> WeakParentWindow;

	TArray<UObject*> ObjectsToDelete;

	EState State = EState::Waiting;

	TSet<FName> ReferencingPackages;
	TSet<FName> PossiblyReferencingPackages;
	bool bIsAnyReferencedInMemoryByNonUndo = false;
	bool bIsAnyReferencedInMemoryByUndo = false;

	TArray<FName> ReferencingPackagesToScan;
	int32 NextObjectToScan = INDEX_NONE;
	int32 NextPackageToScan = INDEX_NONE;

	bool bPerformDelete = false;
	bool bRenameObjectsAsTrash = false;
	FOnDeletedObject OnDeletedObject;
};

}  // namespace UE::Cameras

