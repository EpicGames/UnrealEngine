// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_CameraRigBase.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "GameFramework/BlueprintCameraEvaluationDataRef.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "GameplayCamerasDelegates.h"
#include "Helpers/CameraContextDataPinTypeHelper.h"
#include "Helpers/CameraVariablePinTypeHelper.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EnumLiteral.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CameraRigBase)

#define LOCTEXT_NAMESPACE "K2Node_CameraRigBase"

const FName UK2Node_CameraRigBase::CameraNodeEvaluationResultPinName(TEXT("CameraData"));

UK2Node_CameraRigBase::UK2Node_CameraRigBase(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
	using namespace UE::Cameras;
	
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().AddUObject(this, &UK2Node_CameraRigBase::OnCameraRigAssetBuilt);
}

void UK2Node_CameraRigBase::BeginDestroy()
{
	using namespace UE::Cameras;
	
	FGameplayCamerasDelegates::OnCameraRigAssetBuilt().RemoveAll(this);

	Super::BeginDestroy();
}

void UK2Node_CameraRigBase::AllocateDefaultPins()
{
	using namespace UE::Cameras;

	if (!IsNodePure())
	{
		// Add execution pins.
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	}

	// Add evalation result pin.
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FBlueprintCameraEvaluationDataRef::StaticStruct(), CameraNodeEvaluationResultPinName);

	Super::AllocateDefaultPins();
}

void UK2Node_CameraRigBase::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!CameraRig)
	{
		const FText MessageText = LOCTEXT("MissingCameraRig", "Invalid camera rig reference inside node @@");
		MessageLog.Error(*MessageText.ToString(), this);
	}
}

bool UK2Node_CameraRigBase::CanJumpToDefinition() const
{
	return CameraRig != nullptr;
}

void UK2Node_CameraRigBase::JumpToDefinition() const
{
	if (CameraRig)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CameraRig);
	}
}

FText UK2Node_CameraRigBase::GetMenuCategory() const
{
	const FText BaseCategoryString = FEditorCategoryUtils::BuildCategoryString(
			FCommonEditorCategory::Gameplay, 
			LOCTEXT("CameraRigAssetsEditorCategory", "Camera Rigs"));
	return BaseCategoryString;
}

void UK2Node_CameraRigBase::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(UCameraRigAsset::StaticClass()->GetClassPathName());

		TArray<FAssetData> CameraRigAssetDatas;
		IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
		AssetRegistry.GetAssets(Filter, CameraRigAssetDatas);

		for (const FAssetData& CameraRigAssetData : CameraRigAssetDatas)
		{
			GetMenuActions(ActionRegistrar, CameraRigAssetData);
		}
	}
	else if (const UCameraRigAsset* CameraRigKeyFilter = Cast<const UCameraRigAsset>(ActionRegistrar.GetActionKeyFilter()))
	{
		const FAssetData CameraRigAssetData(CameraRigKeyFilter);
		GetMenuActions(ActionRegistrar, CameraRigAssetData);
	}
}

UEdGraphPin* UK2Node_CameraRigBase::GetCameraNodeEvaluationResultPin() const
{
	return FindPinChecked(CameraNodeEvaluationResultPinName);
}

bool UK2Node_CameraRigBase::ValidateCameraRigBeforeExpandNode(FKismetCompilerContext& CompilerContext) const
{
	if (!CameraRig)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingCameraRig", "SetCameraRigParameter node @@ doesn't have a valid camera rig set.").ToString(), this);
		return false;
	}
	return true;
}

void UK2Node_CameraRigBase::OnCameraRigAssetBuilt(const UCameraRigAsset* InBuiltCameraRig)
{
	if (CameraRig && CameraRig == InBuiltCameraRig)
	{
		ReconstructNode();
	}
}

FEdGraphPinType UK2Node_CameraRigBase::MakeBlendableParameterPinType(const UCameraObjectInterfaceBlendableParameter* BlendableParameter)
{
	return MakeBlendableParameterPinType(BlendableParameter->VariableType, BlendableParameter->BlendableStructType);
}

FEdGraphPinType UK2Node_CameraRigBase::MakeBlendableParameterPinType(ECameraVariableType CameraVariableType, const UScriptStruct* BlendableStructType)
{
	return UE::Cameras::FCameraVariablePinTypeHelper::GetPinType(CameraVariableType, BlendableStructType);
}

FEdGraphPinType UK2Node_CameraRigBase::MakeDataParameterPinType(const UCameraObjectInterfaceDataParameter* DataParameter)
{
	return MakeDataParameterPinType(DataParameter->DataType, DataParameter->DataContainerType, DataParameter->DataTypeObject);
}

FEdGraphPinType UK2Node_CameraRigBase::MakeDataParameterPinType(ECameraContextDataType CameraContextDataType, ECameraContextDataContainerType CameraContextDataContainerType, const UObject* CameraContextDataTypeObject)
{
	return UE::Cameras::FCameraContextDataPinTypeHelper::GetPinType(CameraContextDataType, CameraContextDataContainerType, CameraContextDataTypeObject);
}

UK2Node_CallFunction* UK2Node_CameraRigBase::CreateMakeLiteralNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node* SourceNode, UClass* FunctionLibraryClass, const TCHAR* FunctionName, UEdGraphPin* SourceValuePin)
{
	UK2Node_CallFunction* MakeLiteralNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(SourceNode, SourceGraph);
	MakeLiteralNode->FunctionReference.SetExternalMember(FunctionName, FunctionLibraryClass);
	MakeLiteralNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeLiteralNode, SourceNode);

	UEdGraphPin* LiteralValuePin = MakeLiteralNode->FindPinChecked(TEXT("Value"));
	LiteralValuePin->DefaultValue = SourceValuePin->DefaultValue;
	LiteralValuePin->DefaultTextValue = SourceValuePin->DefaultTextValue;
	LiteralValuePin->AutogeneratedDefaultValue = SourceValuePin->AutogeneratedDefaultValue;
	LiteralValuePin->DefaultObject = SourceValuePin->DefaultObject;

	return MakeLiteralNode;
}

UK2Node* UK2Node_CameraRigBase::MakeLiteralValueForPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node* SourceNode, UEdGraphPin* SourceValuePin)
{
#define UE_NEW_MAKE_LITERAL_NODE(FunctionLibraryClassName, MakeLiteralFunctionName)\
	CreateMakeLiteralNode(\
			CompilerContext, SourceGraph, SourceNode,\
			FunctionLibraryClassName::StaticClass(), TEXT(#MakeLiteralFunctionName),\
			SourceValuePin);

#define UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PinCategoryName, MakeLiteralFunctionName)\
	if (SourceValuePin->PinType.PinCategory == UEdGraphSchema_K2::PinCategoryName)\
	{\
		return UE_NEW_MAKE_LITERAL_NODE(UKismetSystemLibrary, MakeLiteralFunctionName);\
	}

#define UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(SubCategoryObject, MakeLiteralFunctionName)\
	if (SourceValuePin->PinType.PinSubCategoryObject == SubCategoryObject)\
	{\
		return UE_NEW_MAKE_LITERAL_NODE(UCameraRigParameterInteropLibrary, MakeLiteralFunctionName);\
	}

	// Support for all types with and inline widget for editing the default value.
	// Start with those supported by the built-in FNodeFactory class.
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Boolean, MakeLiteralBool)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Text, MakeLiteralText)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Int, MakeLiteralInt)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Int64, MakeLiteralInt64)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Name, MakeLiteralName)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_String, MakeLiteralString)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Text, MakeLiteralText)

	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Float, MakeLiteralFloat)
	UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY(PC_Double, MakeLiteralDouble)
	if (SourceValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real && 
			SourceValuePin->PinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
	{
		return UE_NEW_MAKE_LITERAL_NODE(UKismetSystemLibrary, MakeLiteralFloat);
	}
	if (SourceValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real && 
			SourceValuePin->PinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
	{
		return UE_NEW_MAKE_LITERAL_NODE(UKismetSystemLibrary, MakeLiteralDouble);
	}

	if (SourceValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		if (UEnum* EnumType = Cast<UEnum>(SourceValuePin->PinType.PinSubCategoryObject))
		{
			UK2Node_EnumLiteral* EnumLiteralNode = CompilerContext.SpawnIntermediateNode<UK2Node_EnumLiteral>(SourceNode, SourceGraph);
			EnumLiteralNode->Enum = EnumType;
			EnumLiteralNode->AllocateDefaultPins();
			CompilerContext.MessageLog.NotifyIntermediateObjectCreation(EnumLiteralNode, SourceNode);

			UEdGraphPin* EnumLiteralValuePin = EnumLiteralNode->FindPinChecked(UK2Node_EnumLiteral::GetEnumInputPinName());
			EnumLiteralValuePin->DefaultValue = SourceValuePin->DefaultValue;
			EnumLiteralValuePin->DefaultTextValue = SourceValuePin->DefaultTextValue;
			EnumLiteralValuePin->AutogeneratedDefaultValue = SourceValuePin->AutogeneratedDefaultValue;
			EnumLiteralValuePin->DefaultObject = SourceValuePin->DefaultObject;

			return EnumLiteralNode;
		}
		else
		{
			return UE_NEW_MAKE_LITERAL_NODE(UKismetSystemLibrary, MakeLiteralByte);
		}
	}

	if (SourceValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(TBaseStructure<FLinearColor>::Get(), MakeLiteralLinearColor)
		UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(TBaseStructure<FVector>::Get(), MakeLiteralVector)
		UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(TVariantStructure<FVector3f>::Get(), MakeLiteralVector3f)
		UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(TBaseStructure<FVector2D>::Get(), MakeLiteralVector2D)
		UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT(TBaseStructure<FRotator>::Get(), MakeLiteralRotator)
	}

	return nullptr;

#undef UE_NEW_MAKE_LITERAL_NODE
#undef UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_CATEGORY
#undef UE_CREATE_MAKE_LITERAL_NODE_IF_PIN_SUB_CATEGORY_OBJECT
}

#undef LOCTEXT_NAMESPACE

