// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTableFwd.h"
#include "Core/CameraVariableTableFwd.h"
#include "EdGraph/EdGraphNode.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CameraRigBase.generated.h"

class UCameraRigAsset;
class UCameraObjectInterfaceBlendableParameter;
class UCameraObjectInterfaceDataParameter;
class UK2Node_CallFunction;

/**
 * Utility base class for Blueprint nodes that can set camera rig parameters.
 */
UCLASS(MinimalAPI, Abstract)
class UK2Node_CameraRigBase : public UK2Node
{
	GENERATED_BODY()

public:

	UK2Node_CameraRigBase(const FObjectInitializer& ObjectInit);

public:

	// UObject interface.
	virtual void BeginDestroy() override;

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	virtual bool CanJumpToDefinition() const override;
	virtual void JumpToDefinition() const override;

	// UK2Node interface.
	virtual FText GetMenuCategory() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

public:

	static FEdGraphPinType MakeBlendableParameterPinType(const UCameraObjectInterfaceBlendableParameter* BlendableParameter);
	static FEdGraphPinType MakeBlendableParameterPinType(ECameraVariableType CameraVariableType, const UScriptStruct* BlendableStructType);
	static FEdGraphPinType MakeDataParameterPinType(const UCameraObjectInterfaceDataParameter* DataParameter);
	static FEdGraphPinType MakeDataParameterPinType(ECameraContextDataType CameraContextDataType, ECameraContextDataContainerType CameraContextDataContainerType, const UObject* CameraContextDataTypeObject);

protected:

	// UK2Node_CameraRigBase interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const {}

protected:

	static const FName CameraNodeEvaluationResultPinName;

	UEdGraphPin* GetCameraNodeEvaluationResultPin() const;
	bool ValidateCameraRigBeforeExpandNode(FKismetCompilerContext& CompilerContext) const;

	void OnCameraRigAssetBuilt(const UCameraRigAsset* InBuiltCameraRig);

protected:

	static UK2Node_CallFunction* CreateMakeLiteralNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node* SourceNode, UClass* FunctionLibraryClass, const TCHAR* FunctionName, UEdGraphPin* SourceValuePin);

	static UK2Node* MakeLiteralValueForPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node* SourceNode, UEdGraphPin* InValuePin);

protected:

	UPROPERTY()
	TObjectPtr<UCameraRigAsset> CameraRig;
};

