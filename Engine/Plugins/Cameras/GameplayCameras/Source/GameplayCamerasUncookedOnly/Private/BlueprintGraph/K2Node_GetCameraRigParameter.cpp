// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_GetCameraRigParameter.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetCameraRigParameter)

#define LOCTEXT_NAMESPACE "K2Node_GetCameraRigParameter"

void UK2Node_GetCameraRigParameter::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Add parameter value pin.
	FEdGraphPinType PinType = GetParameterPinType();
	CreatePin(EGPD_Output, PinType, FName(CameraParameterName));
}

FText UK2Node_GetCameraRigParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("BaseNodeTitle", "GET on {0}"), FText::FromString(GetNameSafe(CameraRig)));
}

FText UK2Node_GetCameraRigParameter::GetTooltipText() const
{
	return FText::Format(
			LOCTEXT("NodeTooltip", "Gets the value of camera rig {0}'s parameter {1} on the given evaluation data."),
			FText::FromString(GetNameSafe(CameraRig)),
			FText::FromString(CameraParameterName));
}

void UK2Node_GetCameraRigParameter::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const
{
	const FText BaseCategoryString = GetMenuCategory();

	for (TPair<FName, FAssetTagValueRef> It : CameraRigAssetData.TagsAndValues)
	{
		const FName ParameterName = It.Key;

		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		NodeSpawner->DefaultMenuSignature.Category = FText::Join(
				FText::FromString(TEXT("|")), BaseCategoryString, FText::FromName(CameraRigAssetData.AssetName));
		NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(
				LOCTEXT("GetCameraRigParameterActionMenuName", "Get {0}"),
				FText::FromName(ParameterName));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(
				[ParameterName, CameraRigAssetData](UEdGraphNode* NewNode, bool bIsTemplateNode)
				{
					UK2Node_GetCameraRigParameter* NewSetter = CastChecked<UK2Node_GetCameraRigParameter>(NewNode);
					NewSetter->Initialize(CameraRigAssetData, ParameterName.ToString(), bIsTemplateNode);
				});

		ActionRegistrar.AddBlueprintAction(CameraRigAssetData, NodeSpawner);
	}
}

void UK2Node_GetCameraRigParameter::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ValidateCameraRigBeforeExpandNode(CompilerContext))
	{
		BreakAllNodeLinks();
		return;
	}

	if (CameraParameterName.IsEmpty())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingCameraParameterName", "GetCameraRigParameter node @@ doesn't have a valid camera parameter name set.").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	UEdGraphPin* const CameraNodeEvaluationResultPin = GetCameraNodeEvaluationResultPin();
	UEdGraphPin* const CameraParameterValuePin = FindPinChecked(CameraParameterName);

	// Make the GetXxxParameter function call node.
	UK2Node_CallFunction* CallGetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallGetParameter->FunctionReference.SetExternalMember(TEXT("GetCameraParameter"), UCameraRigParameterInterop::StaticClass());
	CallGetParameter->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallGetParameter, this);

	// Connect the camera evaluation result argument.
	UEdGraphPin* CallGetParameterResultPin = CallGetParameter->FindPinChecked(TEXT("CameraData"));
	CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallGetParameterResultPin);

	// Set the camera rig argument.
	UEdGraphPin* CallGetParameterCameraRigPin = CallGetParameter->FindPinChecked(TEXT("CameraRig"));
	CallGetParameterCameraRigPin->DefaultObject = CameraRig;

	// Set the parameter name argument.
	UEdGraphPin* CallGetParameterNamePin = CallGetParameter->FindPinChecked(TEXT("ParameterName"));
	CallGetParameterNamePin->DefaultValue = CameraParameterName;

	// Connect the output parameter value.
	UEdGraphPin* CallGetParameterValuePin = CallGetParameter->FindPinChecked(TEXT("ReturnValue"));
	CallGetParameterValuePin->PinType = CameraParameterValuePin->PinType;
	if (CameraParameterValuePin->LinkedTo.Num() > 0)
	{
		CompilerContext.MovePinLinksToIntermediate(*CameraParameterValuePin, *CallGetParameterValuePin);
	}

	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE

