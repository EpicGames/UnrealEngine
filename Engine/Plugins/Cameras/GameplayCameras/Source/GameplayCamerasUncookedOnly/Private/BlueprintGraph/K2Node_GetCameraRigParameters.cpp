// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_GetCameraRigParameters.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetCameraRigParameters)

#define LOCTEXT_NAMESPACE "K2Node_GetCameraRigParameters"

void UK2Node_GetCameraRigParameters::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreateParameterPins(EGPD_Output);
}

FText UK2Node_GetCameraRigParameters::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(
			LOCTEXT("BaseNodeTitle", "GET on {0}"),
			FText::FromString(GetNameSafe(CameraRig)));
}

FText UK2Node_GetCameraRigParameters::GetTooltipText() const
{
	return FText::Format(
			LOCTEXT("NodeTooltip", "Gets the values of all camera rig parameters on {0} on the given evaluation data."),
			FText::FromString(GetNameSafe(CameraRig)));
}

void UK2Node_GetCameraRigParameters::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const
{
	const FText BaseCategoryString = GetMenuCategory();

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
	NodeSpawner->DefaultMenuSignature.Category = BaseCategoryString;
	NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(
			LOCTEXT("GetCameraRigParameterActionMenuName", "Get all parameters on {0}"),
			FText::FromName(CameraRigAssetData.AssetName));
	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(
			[CameraRigAssetData](UEdGraphNode* NewNode, bool bIsTemplateNode)
			{
				UK2Node_GetCameraRigParameters* NewGetter = CastChecked<UK2Node_GetCameraRigParameters>(NewNode);
				NewGetter->Initialize(CameraRigAssetData, bIsTemplateNode);
			});

	ActionRegistrar.AddBlueprintAction(CameraRigAssetData, NodeSpawner);
}

void UK2Node_GetCameraRigParameters::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ValidateCameraRigBeforeExpandNode(CompilerContext))
	{
		BreakAllNodeLinks();
		return;
	}

	UEdGraphPin* const CameraNodeEvaluationResultPin = GetCameraNodeEvaluationResultPin();

	// For each blendable and data parameter, we figure out the type of GetXxxParameter function to call on the UCameraRigParameterInterop
	// function library. We then make a K2Node_CallFunction node for it, and connect all its inputs, including connecting the parameter
	// value to whatever our node's corresponding parameter value pin was connected to. We basically transform our GetCameraRigParameters
	// node into an array of single parameter getter nodes.

	TArray<UEdGraphPin*> BlendableParameterPins;
	FindBlendableParameterPins(BlendableParameterPins);
	for (UEdGraphPin* RigParameterPin : BlendableParameterPins)
	{
		const UCameraObjectInterfaceBlendableParameter* BlendableParameter = CameraRig->Interface.FindBlendableParameterByName(RigParameterPin->GetName());
		if (!BlendableParameter)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameter", "GetCameraRigParameters node @@ is trying to set parameter @@ but camera rig @@ has no such parameter.").ToString(), this, *RigParameterPin->GetName(), CameraRig);
			continue;
		}

		if (!BlendableParameter->PrivateVariableID)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameterVariable", "GetCameraRigParameters node @@ needs camera rig @@ to be built.").ToString(), this, CameraRig);
			continue;
		}

		// Make the GetXxxParameter function call node.
		UK2Node_CallFunction* CallGetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallGetParameter->FunctionReference.SetExternalMember(TEXT("GetCameraParameter"), UCameraRigParameterInterop::StaticClass());
		CallGetParameter->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallGetParameter, this);

		// Connect the camera evaluation result argument.
		UEdGraphPin* CallGetParameterResultPin = CallGetParameter->FindPinChecked(TEXT("CameraData"));
		CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallGetParameterResultPin);

		// Set the camera rig argument.
		UEdGraphPin* CallGetParameterCameraRigPin = CallGetParameter->FindPinChecked(TEXT("CameraRig"));
		CallGetParameterCameraRigPin->DefaultObject = CameraRig;

		// Set the parameter name argument.
		UEdGraphPin* CallGetParameterNamePin = CallGetParameter->FindPinChecked(TEXT("ParameterName"));
		CallGetParameterNamePin->DefaultValue = BlendableParameter->InterfaceParameterName;

		// Connect the output parameter value.
		UEdGraphPin* CallGetParameterValuePin = CallGetParameter->FindPinChecked(TEXT("ReturnValue"));
		CallGetParameterValuePin->PinType = RigParameterPin->PinType;
		if (RigParameterPin->LinkedTo.Num() > 0)
		{
			CompilerContext.MovePinLinksToIntermediate(*RigParameterPin, *CallGetParameterValuePin);
		}
	}

	TArray<UEdGraphPin*> DataParameterPins;
	FindDataParameterPins(DataParameterPins);
	for (UEdGraphPin* RigParameterPin : DataParameterPins)
	{
		const UCameraObjectInterfaceDataParameter* DataParameter = CameraRig->Interface.FindDataParameterByName(RigParameterPin->GetName());
		if (!DataParameter)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameter", "GetCameraRigParameters node @@ is trying to set parameter @@ but camera rig @@ has no such parameter.").ToString(), this, *RigParameterPin->GetName(), CameraRig);
			continue;
		}

		if (!DataParameter->PrivateDataID.IsValid())
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameterVariable", "GetCameraRigParameters node @@ needs camera rig @@ to be built.").ToString(), this, CameraRig);
			continue;
		}

		// Make the SetXxxData function call node.
		UK2Node_CallFunction* CallGetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallGetParameter->FunctionReference.SetExternalMember(TEXT("GetCameraParameter"), UCameraRigParameterInterop::StaticClass());
		CallGetParameter->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallGetParameter, this);

		// Connect the camera evaluation result argument.
		UEdGraphPin* CallGetParameterResultPin = CallGetParameter->FindPinChecked(TEXT("CameraData"));
		CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallGetParameterResultPin);

		// Set the camera rig argument.
		UEdGraphPin* CallGetParameterCameraRigPin = CallGetParameter->FindPinChecked(TEXT("CameraRig"));
		CallGetParameterCameraRigPin->DefaultObject = CameraRig;

		// Set the parameter name argument.
		UEdGraphPin* CallGetParameterNamePin = CallGetParameter->FindPinChecked(TEXT("ParameterName"));
		CallGetParameterNamePin->DefaultValue = DataParameter->InterfaceParameterName;

		// Set or connect the parameter value argument.
		UEdGraphPin* CallGetParameterValuePin = CallGetParameter->FindPinChecked(TEXT("ReturnValue"));
		CallGetParameterValuePin->PinType = RigParameterPin->PinType;
		if (RigParameterPin->LinkedTo.Num() > 0)
		{
			CompilerContext.MovePinLinksToIntermediate(*RigParameterPin, *CallGetParameterValuePin);
		}
	}

	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE

