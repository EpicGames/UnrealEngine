// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_MultiCameraRigParametersBase.h"

#include "K2Node_GetCameraRigParameters.generated.h"

class UCameraRigAsset;

/**
 * Blueprint node that, given a camera rig, lets the user get the values of all exposed parameters
 * on that camera rig.
 */
UCLASS(MinimalAPI)
class UK2Node_GetCameraRigParameters : public UK2Node_MultiCameraRigParametersBase
{
	GENERATED_BODY()

public:

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	//virtual bool ShouldDrawCompact() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

protected:

	// UK2Node_CameraRigBase interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const override;
};

