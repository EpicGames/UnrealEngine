// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_MultiCameraRigParametersBase.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MultiCameraRigParametersBase)

#define LOCTEXT_NAMESPACE "K2Node_MultiCameraRigParametersBase"

UK2Node_MultiCameraRigParametersBase::UK2Node_MultiCameraRigParametersBase(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UK2Node_MultiCameraRigParametersBase::Initialize(const FAssetData& UnloadedCameraRig, bool bIsTemplateNode)
{
	if (!bIsTemplateNode)
	{
		CameraRig = Cast<UCameraRigAsset>(UnloadedCameraRig.GetAsset());
	}
}

void UK2Node_MultiCameraRigParametersBase::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	// The camera rig might not be loaded yet when we are rebuilt on startup, so make sure
	// this is all good otherwise we won't be able to get all our pins back.
	EnsureCameraRigAssetLoaded();

	// Now do as usual: create all default pins (including the parameter pins) and 
	// restore split pins.
	AllocateDefaultPins();
	RestoreSplitPins(OldPins);
}

void UK2Node_MultiCameraRigParametersBase::EnsureCameraRigAssetLoaded()
{
	if (CameraRig)
	{
		PreloadObject(CameraRig);
		for (UCameraObjectInterfaceBlendableParameter* BlendableParameter : CameraRig->Interface.BlendableParameters)
		{
			PreloadObject(BlendableParameter);
			if (BlendableParameter)
			{
				PreloadObject(const_cast<UScriptStruct*>(BlendableParameter->BlendableStructType.Get()));
			}
		}
		for (UCameraObjectInterfaceDataParameter* DataParameter : CameraRig->Interface.DataParameters)
		{
			PreloadObject(DataParameter);
			if (DataParameter)
			{
				PreloadObject(const_cast<UObject*>(DataParameter->DataTypeObject.Get()));
			}
		}
	}
}

void UK2Node_MultiCameraRigParametersBase::CreateParameterPins(EEdGraphPinDirection PinDirection)
{
	BlendableParameterPinNames.Reset();
	DataParameterPinNames.Reset();

	if (!CameraRig)
	{
		return;
	}

	for (const UCameraObjectInterfaceBlendableParameter* BlendableParameter : CameraRig->Interface.BlendableParameters)
	{
		if (!ensure(BlendableParameter))
		{
			continue;
		}

		if (!BlendableParameter->PrivateVariableID)
		{
			// Camera rig isn't fully built.
			continue;
		}
		
		FEdGraphPinType PinType = MakeBlendableParameterPinType(BlendableParameter);
		if (PinType.PinCategory.IsNone())
		{
			// Unsupported type for Blueprints.
			continue;
		}

		UEdGraphPin* NewPin = CreatePin(PinDirection, PinType, FName(BlendableParameter->InterfaceParameterName));
		BlendableParameterPinNames.Add(NewPin->PinName);
	}

	for (const UCameraObjectInterfaceDataParameter* DataParameter : CameraRig->Interface.DataParameters)
	{
		if (!ensure(DataParameter))
		{
			continue;
		}
		
		if (!DataParameter->PrivateDataID.IsValid())
		{
			// Camera rig isn't fully built.
			continue;
		}

		FEdGraphPinType PinType = MakeDataParameterPinType(DataParameter);
		if (PinType.PinCategory.IsNone())
		{
			// Unsupported type for Blueprints.
			continue;
		}

		UEdGraphPin* NewPin = CreatePin(PinDirection, PinType, FName(DataParameter->InterfaceParameterName));
		DataParameterPinNames.Add(NewPin->PinName);
	}
}

void UK2Node_MultiCameraRigParametersBase::FindBlendableParameterPins(TArray<UEdGraphPin*>& OutPins) const
{
	for (const FName& PinName : BlendableParameterPinNames)
	{
		UEdGraphPin* Pin = FindPin(PinName);
		if (ensure(Pin))
		{
			OutPins.Add(Pin);
		}
	}
}

void UK2Node_MultiCameraRigParametersBase::FindDataParameterPins(TArray<UEdGraphPin*>& OutPins) const
{
	for (const FName& PinName : DataParameterPinNames)
	{
		UEdGraphPin* Pin = FindPin(PinName);
		if (ensure(Pin))
		{
			OutPins.Add(Pin);
		}
	}
}

#undef LOCTEXT_NAMESPACE

