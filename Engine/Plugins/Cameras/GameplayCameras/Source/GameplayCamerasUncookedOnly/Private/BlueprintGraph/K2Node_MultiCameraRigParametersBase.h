// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CameraRigBase.h"

#include "K2Node_MultiCameraRigParametersBase.generated.h"

class UCameraRigAsset;

/**
 * A base class for a Blueprint node that gets or sets multiple camera rig parameters.
 */
UCLASS(MinimalAPI)
class UK2Node_MultiCameraRigParametersBase : public UK2Node_CameraRigBase
{
	GENERATED_BODY()

public:

	UK2Node_MultiCameraRigParametersBase(const FObjectInitializer& ObjectInit);

	void Initialize(const FAssetData& UnloadedCameraRig, bool bIsTemplateNode);

public:

	// UK2Node interface.
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;

protected:

	void EnsureCameraRigAssetLoaded();
	void CreateParameterPins(EEdGraphPinDirection PinDirection);
	void FindBlendableParameterPins(TArray<UEdGraphPin*>& OutPins) const;
	void FindDataParameterPins(TArray<UEdGraphPin*>& OutPins) const;

protected:

	UPROPERTY()
	TArray<FName> BlendableParameterPinNames;

	UPROPERTY()
	TArray<FName> DataParameterPinNames;
};

