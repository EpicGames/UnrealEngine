// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_SetCameraRigParameter.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SetCameraRigParameter)

#define LOCTEXT_NAMESPACE "K2Node_SetCameraRigParameter"

void UK2Node_SetCameraRigParameter::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Add parameter value pin and return pin.
	FEdGraphPinType PinType = GetParameterPinType();
	CreatePin(EGPD_Input, PinType, FName(CameraParameterName));
	UEdGraphPin* ReturnPin = CreatePin(EGPD_Output, PinType, UEdGraphSchema_K2::PN_ReturnValue);
	ReturnPin->PinFriendlyName = FText::GetEmpty();
}

FText UK2Node_SetCameraRigParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("BaseNodeTitle", "SET on {0}"), FText::FromString(GetNameSafe(CameraRig)));
}

FText UK2Node_SetCameraRigParameter::GetTooltipText() const
{
	return FText::Format(
			LOCTEXT("NodeTooltip", "Sets the value of camera rig {0}'s parameter {1} on the given evaluation data."),
			FText::FromString(GetNameSafe(CameraRig)),
			FText::FromString(CameraParameterName));
}

void UK2Node_SetCameraRigParameter::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const
{
	const FText BaseCategoryString = GetMenuCategory();

	for (TPair<FName, FAssetTagValueRef> It : CameraRigAssetData.TagsAndValues)
	{
		const FName ParameterName = It.Key;

		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		NodeSpawner->DefaultMenuSignature.Category = FText::Join(
				FText::FromString(TEXT("|")), BaseCategoryString, FText::FromName(CameraRigAssetData.AssetName));
		NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(
				LOCTEXT("SetCameraRigParameterActionMenuName", "Set {0}"),
				FText::FromName(ParameterName));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(
				[ParameterName, CameraRigAssetData](UEdGraphNode* NewNode, bool bIsTemplateNode)
				{
					UK2Node_SetCameraRigParameter* NewSetter = CastChecked<UK2Node_SetCameraRigParameter>(NewNode);
					NewSetter->Initialize(CameraRigAssetData, ParameterName.ToString(), bIsTemplateNode);
				});

		ActionRegistrar.AddBlueprintAction(CameraRigAssetData, NodeSpawner);
	}
}

void UK2Node_SetCameraRigParameter::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ValidateCameraRigBeforeExpandNode(CompilerContext))
	{
		BreakAllNodeLinks();
		return;
	}

	if (CameraParameterName.IsEmpty())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingCameraParameterName", "SetCameraRigParameter node @@ doesn't have a valid camera parameter name set.").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	UEdGraphPin* const CameraNodeEvaluationResultPin = GetCameraNodeEvaluationResultPin();
	UEdGraphPin* const CameraParameterValuePin = FindPinChecked(CameraParameterName);
	UEdGraphPin* const ReturnValuePin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);

	// Make the SetXxxParameter function call node.
	UK2Node_CallFunction* CallSetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallSetParameter->FunctionReference.SetExternalMember(TEXT("SetCameraParameter"), UCameraRigParameterInterop::StaticClass());
	CallSetParameter->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallSetParameter, this);
	UEdGraphPin* FirstExecPin = CallSetParameter->GetExecPin();

	// Connect the camera evaluation result argument.
	UEdGraphPin* CallSetParameterResultPin = CallSetParameter->FindPinChecked(TEXT("CameraData"));
	CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallSetParameterResultPin);

	// Set the camera rig argument.
	UEdGraphPin* CallSetParameterCameraRigPin = CallSetParameter->FindPinChecked(TEXT("CameraRig"));
	CallSetParameterCameraRigPin->DefaultObject = CameraRig;

	// Set the parameter name argument.
	UEdGraphPin* CallSetParameterNamePin = CallSetParameter->FindPinChecked(TEXT("ParameterName"));
	CallSetParameterNamePin->DefaultValue = CameraParameterName;

	// Set or connect the parameter value argument. Wildcard pins don't accept default values so if the value doesn't
	// come from a connection, we need to use a "MakeLiteralXxx" node.
	UEdGraphPin* CallSetParameterValuePin = CallSetParameter->FindPinChecked(TEXT("NewValue"));
	CallSetParameterValuePin->PinType = CameraParameterValuePin->PinType;
	if (CameraParameterValuePin->LinkedTo.Num() > 0)
	{
		CompilerContext.MovePinLinksToIntermediate(*CameraParameterValuePin, *CallSetParameterValuePin);
	}
	else if (UK2Node* MakeLiteral = MakeLiteralValueForPin(CompilerContext, SourceGraph, this, CameraParameterValuePin))
	{
		UEdGraphPin* LiteralValuePin = MakeLiteral->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		LiteralValuePin->MakeLinkTo(CallSetParameterValuePin);
	}
	else
	{
		FText MissingParameterConnectionMsg = FText::Format(
				LOCTEXT(
					"ErrorRequiresLiteral", 
					"SetCameraRigParameter node @@ parameter value '{0}' must have an input connected into it "
					"(try connecting a MakeStruct, EnumLiteral, or appropriate node)."),
				FText::FromString(CameraParameterName));
		CompilerContext.MessageLog.Error(*MissingParameterConnectionMsg.ToString(), this);
	}

	// Connect the pin that provides the value to the return pin of our source node.
	for (UEdGraphPin* InValuePin : CallSetParameterValuePin->LinkedTo)
	{
		for (UEdGraphPin* OutValuePin : ReturnValuePin->LinkedTo)
		{
			InValuePin->MakeLinkTo(OutValuePin);
		}
	}
	
	// Setup the execution flow.
	UEdGraphPin* ThisExecPin = GetExecPin();
	CompilerContext.MovePinLinksToIntermediate(*ThisExecPin, *FirstExecPin);

	UEdGraphPin* ThisThenPin = GetThenPin();
	UEdGraphPin* CallSetParameterThenPin = CallSetParameter->GetThenPin();
	CompilerContext.MovePinLinksToIntermediate(*ThisThenPin, *CallSetParameterThenPin);

	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE

