// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_SingleCameraRigParameterBase.h"

#include "K2Node_SetCameraRigParameter.generated.h"

class FKismetCompilerContext;
class UCameraRigAsset;

/**
 * Blueprint node that, given a camera rig, lets the user set the value of one single
 * exposed parameter.
 */
UCLASS(MinimalAPI)
class UK2Node_SetCameraRigParameter : public UK2Node_SingleCameraRigParameterBase
{
	GENERATED_BODY()

public:

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

protected:

	// UK2Node_CameraRigBase interface.
	void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const;
};

