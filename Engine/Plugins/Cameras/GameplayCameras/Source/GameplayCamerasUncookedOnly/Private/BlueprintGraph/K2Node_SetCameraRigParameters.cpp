// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_SetCameraRigParameters.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SetCameraRigParameters)

#define LOCTEXT_NAMESPACE "K2Node_SetCameraRigParameters"

void UK2Node_SetCameraRigParameters::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreateParameterPins(EGPD_Input);
}

void UK2Node_SetCameraRigParameters::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	// Remove old camera rig pin name. Do this before calling AllocateDefaultPins() because
	// we need our CameraRig property set to (re)create our parameter pins.
	const FName OldCameraRigPinName = TEXT("CameraRig");
	UEdGraphPin** OldCameraRigPin = OldPins.FindByPredicate([&OldCameraRigPinName](UEdGraphPin* OldPin)
			{
				return OldPin->PinName == OldCameraRigPinName;
			});
	if (OldCameraRigPin)
	{
		CameraRig = Cast<UCameraRigAsset>((*OldCameraRigPin)->DefaultObject);
		(*OldCameraRigPin)->DefaultObject = nullptr;
		(*OldCameraRigPin)->DefaultValue.Reset();
	}

	// Do like the base class does, but fixup old-style pins before restoring split pins.
	EnsureCameraRigAssetLoaded();

	AllocateDefaultPins();

	// Upgrade old result pin name to new pin name.
	const FName OldResultPinName = TEXT("CameraEvaluationResult");
	UEdGraphPin** OldResultPin = OldPins.FindByPredicate([&OldResultPinName](UEdGraphPin* OldPin)
			{
				return OldPin->PinName == OldResultPinName;
			});
	if (OldResultPin)
	{
		UEdGraphPin* NewResultPin = FindPin(CameraNodeEvaluationResultPinName);
		check(NewResultPin);
		NewResultPin->MovePersistentDataFromOldPin(**OldResultPin);
	}

	RestoreSplitPins(OldPins);
}

FText UK2Node_SetCameraRigParameters::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(
			LOCTEXT("BaseNodeTitle", "SET on {0}"),
			FText::FromString(GetNameSafe(CameraRig)));
}

FText UK2Node_SetCameraRigParameters::GetTooltipText() const
{
	return FText::Format(
			LOCTEXT("NodeTooltip", "Sets the values of all camera rig parameters on {0} on the given evaluation data."),
			FText::FromString(GetNameSafe(CameraRig)));
}

void UK2Node_SetCameraRigParameters::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const
{
	const FText BaseCategoryString = GetMenuCategory();

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
	NodeSpawner->DefaultMenuSignature.Category = BaseCategoryString;
	NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(
			LOCTEXT("SetCameraRigParameterActionMenuName", "Set all parameters on {0}"),
			FText::FromName(CameraRigAssetData.AssetName));
	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(
			[CameraRigAssetData](UEdGraphNode* NewNode, bool bIsTemplateNode)
			{
				UK2Node_SetCameraRigParameters* NewSetter = CastChecked<UK2Node_SetCameraRigParameters>(NewNode);
				NewSetter->Initialize(CameraRigAssetData, bIsTemplateNode);
			});

	ActionRegistrar.AddBlueprintAction(CameraRigAssetData, NodeSpawner);
}

void UK2Node_SetCameraRigParameters::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ValidateCameraRigBeforeExpandNode(CompilerContext))
	{
		BreakAllNodeLinks();
		return;
	}

	UEdGraphPin* const CameraNodeEvaluationResultPin = GetCameraNodeEvaluationResultPin();

	UEdGraphPin* OriginalThenPin = GetThenPin();
	UEdGraphPin* PreviousThenPin = nullptr;
	
	// For each blendable and data parameter, we figure out the type of SetXxxParameter function to call on the UCameraRigParameterInterop
	// function library. We then make a K2Node_CallFunction node for it, and connect all its inputs, including connecting the parameter
	// value to whatever our node's corresponding parameter value pin was connected to. As we go, we chain the exec/then pins, basically
	// transforming our SetCameraRigParameters node into a chain of individual setter function calls.

	TArray<UEdGraphPin*> BlendableParameterPins;
	FindBlendableParameterPins(BlendableParameterPins);
	for (UEdGraphPin* RigParameterPin : BlendableParameterPins)
	{
		const UCameraObjectInterfaceBlendableParameter* BlendableParameter = CameraRig->Interface.FindBlendableParameterByName(RigParameterPin->GetName());
		if (!BlendableParameter)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameter", "SetCameraRigParameters node @@ is trying to set parameter @@ but camera rig @@ has no such parameter.").ToString(), this, *RigParameterPin->GetName(), CameraRig);
			continue;
		}

		if (!BlendableParameter->PrivateVariableID)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameterVariable", "SetCameraRigParameters node @@ needs camera rig @@ to be built.").ToString(), this, CameraRig);
			continue;
		}

		// Make the SetXxxParameter function call node.
		UK2Node_CallFunction* CallSetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallSetParameter->FunctionReference.SetExternalMember(TEXT("SetCameraParameter"), UCameraRigParameterInterop::StaticClass());
		CallSetParameter->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallSetParameter, this);

		// Connect the camera evaluation result argument.
		UEdGraphPin* CallSetParameterResultPin = CallSetParameter->FindPinChecked(TEXT("CameraData"));
		CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallSetParameterResultPin);

		// Set the camera rig argument.
		UEdGraphPin* CallSetParameterCameraRigPin = CallSetParameter->FindPinChecked(TEXT("CameraRig"));
		CallSetParameterCameraRigPin->DefaultObject = CameraRig;

		// Set the parameter name argument.
		UEdGraphPin* CallSetParameterNamePin = CallSetParameter->FindPinChecked(TEXT("ParameterName"));
		CallSetParameterNamePin->DefaultValue = BlendableParameter->InterfaceParameterName;

		// Set or connect the parameter value argument. Wildcard pins don't accept default values so if the value doesn't
		// come from a connection, we need to use a "MakeLiteralXxx" node.
		UEdGraphPin* CallSetParameterValuePin = CallSetParameter->FindPinChecked(TEXT("NewValue"));
		CallSetParameterValuePin->PinType = RigParameterPin->PinType;
		if (RigParameterPin->LinkedTo.Num() > 0)
		{
			CompilerContext.MovePinLinksToIntermediate(*RigParameterPin, *CallSetParameterValuePin);
		}
		else if (UK2Node* MakeLiteral = MakeLiteralValueForPin(CompilerContext, SourceGraph, this, RigParameterPin))
		{
			UEdGraphPin* ReturnValuePin = MakeLiteral->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
			ReturnValuePin->MakeLinkTo(CallSetParameterValuePin);
		}
		else
		{
			FText MissingParameterConnectionMsg = FText::Format(
					LOCTEXT(
						"ErrorRequiresLiteral", 
						"SetCameraRigParameter node @@ parameter value '{0}' must have an input connected into it "
						"(try connecting a MakeStruct, EnumLiteral, or appropriate node)."),
					FText::FromString(BlendableParameter->InterfaceParameterName));
			CompilerContext.MessageLog.Error(*MissingParameterConnectionMsg.ToString(), this);
		}

		// Chain the execution.
		UEdGraphPin* CallSetParameterExecPin = CallSetParameter->GetExecPin();
		if (PreviousThenPin)
		{
			PreviousThenPin->MakeLinkTo(CallSetParameterExecPin);
		}
		else
		{
			UEdGraphPin* ThisExecPin = GetExecPin();
			CompilerContext.MovePinLinksToIntermediate(*ThisExecPin, *CallSetParameterExecPin);
		}

		PreviousThenPin = CallSetParameter->GetThenPin();
	}

	TArray<UEdGraphPin*> DataParameterPins;
	FindDataParameterPins(DataParameterPins);
	for (UEdGraphPin* RigParameterPin : DataParameterPins)
	{
		const UCameraObjectInterfaceDataParameter* DataParameter = CameraRig->Interface.FindDataParameterByName(RigParameterPin->GetName());
		if (!DataParameter)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameter", "SetCameraRigParameters node @@ is trying to set parameter @@ but camera rig @@ has no such parameter.").ToString(), this, *RigParameterPin->GetName(), CameraRig);
			continue;
		}

		if (!DataParameter->PrivateDataID.IsValid())
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ErrorMissingParameterVariable", "SetCameraRigParameters node @@ needs camera rig @@ to be built.").ToString(), this, CameraRig);
			continue;
		}

		// Make the SetXxxData function call node.
		UK2Node_CallFunction* CallSetParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallSetParameter->FunctionReference.SetExternalMember(TEXT("SetCameraParameter"), UCameraRigParameterInterop::StaticClass());
		CallSetParameter->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallSetParameter, this);

		UEdGraphPin* CurExecPin = CallSetParameter->GetExecPin();

		// Connect the camera evaluation result argument.
		UEdGraphPin* CallSetParameterResultPin = CallSetParameter->FindPinChecked(TEXT("CameraData"));
		CompilerContext.CopyPinLinksToIntermediate(*CameraNodeEvaluationResultPin, *CallSetParameterResultPin);

		// Set the camera rig argument.
		UEdGraphPin* CallSetParameterCameraRigPin = CallSetParameter->FindPinChecked(TEXT("CameraRig"));
		CallSetParameterCameraRigPin->DefaultObject = CameraRig;

		// Set the parameter name argument.
		UEdGraphPin* CallSetParameterNamePin = CallSetParameter->FindPinChecked(TEXT("ParameterName"));
		CallSetParameterNamePin->DefaultValue = DataParameter->InterfaceParameterName;

		// Set or connect the parameter value argument. Wildcard pins don't accept default values so if the value doesn't
		// come from a connection, we need to use a "MakeLiteralXxx" node.
		UEdGraphPin* CallSetParameterValuePin = CallSetParameter->FindPinChecked(TEXT("NewValue"));
		CallSetParameterValuePin->PinType = RigParameterPin->PinType;
		if (RigParameterPin->LinkedTo.Num() > 0)
		{
			CompilerContext.MovePinLinksToIntermediate(*RigParameterPin, *CallSetParameterValuePin);
		}
		else if (UK2Node* MakeLiteral = MakeLiteralValueForPin(CompilerContext, SourceGraph, this, RigParameterPin))
		{
			UEdGraphPin* ReturnValuePin = MakeLiteral->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
			ReturnValuePin->MakeLinkTo(CallSetParameterValuePin);
		}
		else
		{
			FText MissingParameterConnectionMsg = FText::Format(
					LOCTEXT(
						"ErrorRequiresLiteral", 
						"SetCameraRigParameter node @@ parameter value '{0}' must have an input connected into it "
						"(try connecting a MakeStruct, EnumLiteral, or appropriate node)."),
					FText::FromString(DataParameter->InterfaceParameterName));
			CompilerContext.MessageLog.Error(*MissingParameterConnectionMsg.ToString(), this);
		}

		// Chain the execution.
		if (PreviousThenPin)
		{
			PreviousThenPin->MakeLinkTo(CurExecPin);
		}
		else
		{
			UEdGraphPin* ThisExecPin = GetExecPin();
			CompilerContext.MovePinLinksToIntermediate(*ThisExecPin, *CurExecPin);
		}

		PreviousThenPin = CallSetParameter->GetThenPin();
	}

	// Connect the last node if necessary.
	if (OriginalThenPin && PreviousThenPin && OriginalThenPin->LinkedTo.Num() > 0)
	{
		CompilerContext.MovePinLinksToIntermediate(*OriginalThenPin, *PreviousThenPin);
	}

	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE

