// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_MultiCameraRigParametersBase.h"

#include "K2Node_SetCameraRigParameters.generated.h"

class UCameraRigAsset;

/**
 * Blueprint node that, given a camera rig, lets the user set the values of all exposed parameters
 * on that camera rig. Any camera rig running with the specific evaluation data will use those
 * values.
 */
UCLASS(MinimalAPI)
class UK2Node_SetCameraRigParameters : public UK2Node_MultiCameraRigParametersBase
{
	GENERATED_BODY()

public:

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

	// UK2Node interface.
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;

protected:

	// UK2Node_CameraRigBase interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FAssetData& CameraRigAssetData) const override;
};

