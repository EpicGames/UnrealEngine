// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraph/K2Node_SingleCameraRigParameterBase.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintGraph/K2Node_CameraRigBase.h"
#include "Core/CameraRigAsset.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "GameFramework/CameraRigParameterInterop.h"
#include "K2Node_CallFunction.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "KismetCompiler.h"
#include "Misc/EngineVersionComparison.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SingleCameraRigParameterBase)

#define LOCTEXT_NAMESPACE "K2Node_SingleCameraRigParameterBase"

UK2Node_SingleCameraRigParameterBase::UK2Node_SingleCameraRigParameterBase(const FObjectInitializer& ObjectInit)
	: Super(ObjectInit)
{
}

void UK2Node_SingleCameraRigParameterBase::Initialize(const FAssetData& UnloadedCameraRig, const FString& InCameraParameterName, bool bIsTemplateNode)
{
	if (!bIsTemplateNode)
	{
		UCameraRigAsset* LoadedCameraRig = Cast<UCameraRigAsset>(UnloadedCameraRig.GetAsset());
		if (ensure(LoadedCameraRig))
		{
			if (const UCameraObjectInterfaceBlendableParameter* BlendableParameter = LoadedCameraRig->Interface.FindBlendableParameterByName(InCameraParameterName))
			{
				Initialize(LoadedCameraRig, InCameraParameterName, BlendableParameter->VariableType, BlendableParameter->BlendableStructType);
			}
			else if (const UCameraObjectInterfaceDataParameter* DataParameter = LoadedCameraRig->Interface.FindDataParameterByName(InCameraParameterName))
			{
				Initialize(LoadedCameraRig, InCameraParameterName, DataParameter->DataType, DataParameter->DataContainerType, DataParameter->DataTypeObject);
			}
			// else, no parameter of that name found...
		}
	}
}

void UK2Node_SingleCameraRigParameterBase::Initialize(UCameraRigAsset* InCameraRig, const FString& InCameraParameterName, ECameraVariableType InCameraVariableType, const UScriptStruct* InBlendableStructType)
{
	CameraRig = InCameraRig;
	CameraParameterName = InCameraParameterName;
	CameraParameterType = EK2Node_CameraParameterType::Blendable;
	BlendableCameraParameterType = InCameraVariableType;
	BlendableStructType = InBlendableStructType;
}

void UK2Node_SingleCameraRigParameterBase::Initialize(UCameraRigAsset* InCameraRig, const FString& InCameraParameterName, ECameraContextDataType InCameraContextDataType, ECameraContextDataContainerType InCameraContextDataContainerType, const UObject* InCameraContextDataTypeObject)
{
	CameraRig = InCameraRig;
	CameraParameterName = InCameraParameterName;
	CameraParameterType = EK2Node_CameraParameterType::Data;
	DataCameraParameterType = InCameraContextDataType;
	DataCameraParameterContainerType = InCameraContextDataContainerType;
	DataCameraParameterTypeObject = const_cast<UObject*>(InCameraContextDataTypeObject);
}

FSlateIcon UK2Node_SingleCameraRigParameterBase::GetIconAndTint(FLinearColor& OutColor) const
{
	UEdGraphPin* ParameterValuePin = FindPin(CameraParameterName);
	if (ParameterValuePin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		OutColor = K2Schema->GetPinTypeColor(ParameterValuePin->PinType);

#if UE_VERSION_NEWER_THAN_OR_EQUAL(5,6,0)
		if (UStruct* Struct = Cast<UStruct>(ParameterValuePin->PinType.PinSubCategoryObject.Get()))
		{
			return FSlateIconFinder::FindIconForClass(Struct);
		}
#endif
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.VariableIcon");
}

FEdGraphPinType UK2Node_SingleCameraRigParameterBase::GetParameterPinType() const
{
	FEdGraphPinType PinType;
	switch (CameraParameterType)
	{
		case EK2Node_CameraParameterType::Blendable:
			PinType = UK2Node_CameraRigBase::MakeBlendableParameterPinType(BlendableCameraParameterType, BlendableStructType);
			break;
		case EK2Node_CameraParameterType::Data:
			PinType = UK2Node_CameraRigBase::MakeDataParameterPinType(DataCameraParameterType, DataCameraParameterContainerType, DataCameraParameterTypeObject);
			break;
		default:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			break;
	}
	return PinType;
}

#undef LOCTEXT_NAMESPACE

