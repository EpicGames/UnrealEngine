// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CameraRigBase.h"

#include "K2Node_SingleCameraRigParameterBase.generated.h"

class FKismetCompilerContext;
class UCameraRigAsset;

UENUM()
enum class EK2Node_CameraParameterType : uint8
{
	Unknown,
	Blendable,
	Data
};

/**
 * A base class for a Blueprint node that gets or sets a single camera rig parameter.
 */
UCLASS(MinimalAPI)
class UK2Node_SingleCameraRigParameterBase : public UK2Node_CameraRigBase
{
	GENERATED_BODY()

public:

	UK2Node_SingleCameraRigParameterBase(const FObjectInitializer& ObjectInit);

	void Initialize(const FAssetData& UnloadedCameraRig, const FString& InCameraParameterName, bool bIsTemplateNode);
	void Initialize(UCameraRigAsset* InCameraRig, const FString& InCameraParameterName, ECameraVariableType InCameraVariableType, const UScriptStruct* InBlendableStructType);
	void Initialize(UCameraRigAsset* InCameraRig, const FString& InCameraParameterName, ECameraContextDataType InCameraContextDataType, ECameraContextDataContainerType InCameraContextDataContainerType, const UObject* InCameraContextDataTypeObject);

public:

	// UEdGraphNode interface.
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

protected:

	FEdGraphPinType GetParameterPinType() const;

protected:

	UPROPERTY()
	FString CameraParameterName;

	UPROPERTY()
	EK2Node_CameraParameterType CameraParameterType = EK2Node_CameraParameterType::Unknown;

	UPROPERTY()
	ECameraVariableType BlendableCameraParameterType;

	UPROPERTY()
	TObjectPtr<const UScriptStruct> BlendableStructType;

	UPROPERTY()
	ECameraContextDataType DataCameraParameterType;

	UPROPERTY()
	ECameraContextDataContainerType DataCameraParameterContainerType;

	UPROPERTY()
	TObjectPtr<UObject> DataCameraParameterTypeObject;
};

