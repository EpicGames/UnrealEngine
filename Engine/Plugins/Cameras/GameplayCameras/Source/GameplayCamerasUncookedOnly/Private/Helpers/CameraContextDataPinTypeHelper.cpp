// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CameraContextDataPinTypeHelper.h"

#include "EdGraphSchema_K2.h"

namespace UE::Cameras
{

FEdGraphPinType FCameraContextDataPinTypeHelper::GetPinType(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject)
{
	FName PinCategory = UEdGraphSchema_K2::PC_Name;
	UObject* PinSubCategoryObject = const_cast<UObject*>(DataTypeObject);
	switch (DataType)
	{
		case ECameraContextDataType::Name:
			PinCategory = UEdGraphSchema_K2::PC_Name;
			break;
		case ECameraContextDataType::String:
			PinCategory = UEdGraphSchema_K2::PC_String;
			break;
		case ECameraContextDataType::Enum:
			PinCategory = UEdGraphSchema_K2::PC_Byte;
			break;
		case ECameraContextDataType::Struct:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			break;
		case ECameraContextDataType::Object:
			PinCategory = UEdGraphSchema_K2::PC_Object;
			break;
		case ECameraContextDataType::Class:
			PinCategory = UEdGraphSchema_K2::PC_Class;
			break;
	}

	EPinContainerType PinContainerType = EPinContainerType::None;
	switch (DataContainerType)
	{
		case ECameraContextDataContainerType::Array:
			PinContainerType = EPinContainerType::Array;
		default:
			break;
	}

	FEdGraphPinType PinType;
	PinType.PinCategory = PinCategory;
	PinType.PinSubCategoryObject = PinSubCategoryObject;
	PinType.ContainerType = PinContainerType;
	return PinType;
}

}  // namespace UE::Cameras

