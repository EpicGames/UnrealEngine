// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CameraVariablePinTypeHelper.h"

#include "EdGraphSchema_K2.h"

namespace UE::Cameras
{

FEdGraphPinType FCameraVariablePinTypeHelper::GetPinType(ECameraVariableType VariableType, const UScriptStruct* BlendableStructType)
{
	FName PinCategory;
	FName PinSubCategory;
	UObject* PinSubCategoryObject = nullptr;
	switch (VariableType)
	{
		case ECameraVariableType::Boolean:
			PinCategory = UEdGraphSchema_K2::PC_Boolean;
			break;
		case ECameraVariableType::Integer32:
			PinCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case ECameraVariableType::Float:
			// We'll cast down to float.
			PinCategory = UEdGraphSchema_K2::PC_Real;
			PinSubCategory = UEdGraphSchema_K2::PC_Float;
			break;
		case ECameraVariableType::Double:
			PinCategory = UEdGraphSchema_K2::PC_Real;
			PinSubCategory = UEdGraphSchema_K2::PC_Double;
			break;
		case ECameraVariableType::Vector2f:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TVariantStructure<FVector2f>::Get();
			break;
		case ECameraVariableType::Vector2d:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TBaseStructure<FVector2D>::Get();
			break;
		case ECameraVariableType::Vector3f:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TVariantStructure<FVector3f>::Get();
			break;
		case ECameraVariableType::Vector3d:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TBaseStructure<FVector>::Get();
			break;
		case ECameraVariableType::Vector4f:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TVariantStructure<FVector4f>::Get();
			break;
		case ECameraVariableType::Vector4d:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TBaseStructure<FVector4>::Get();
			break;
		case ECameraVariableType::Rotator3f:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TVariantStructure<FRotator3f>::Get();
			break;
		case ECameraVariableType::Rotator3d:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TBaseStructure<FRotator>::Get();
			break;
		case ECameraVariableType::Transform3f:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TVariantStructure<FTransform3f>::Get();
			break;
		case ECameraVariableType::Transform3d:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = TBaseStructure<FTransform>::Get();
			break;
		case ECameraVariableType::BlendableStruct:
			PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinSubCategoryObject = const_cast<UScriptStruct*>(BlendableStructType);
			break;
	}

	FEdGraphPinType PinType;
	PinType.PinCategory = PinCategory;
	PinType.PinSubCategory = PinSubCategory;
	PinType.PinSubCategoryObject = PinSubCategoryObject;
	return PinType;
}

}  // namespace UE::Cameras

