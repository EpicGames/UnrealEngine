// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraContextDataTable.h"
#include "EdGraph/EdGraphPin.h"

#define UE_API GAMEPLAYCAMERASUNCOOKEDONLY_API

namespace UE::Cameras
{

class FCameraContextDataPinTypeHelper
{
public:

	UE_API static FEdGraphPinType GetPinType(ECameraContextDataType DataType, ECameraContextDataContainerType DataContainerType, const UObject* DataTypeObject);
};

}  // namespace UE::Cameras

#undef UE_API

