// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraVariableTable.h"
#include "EdGraph/EdGraphPin.h"

#define UE_API GAMEPLAYCAMERASUNCOOKEDONLY_API

namespace UE::Cameras
{

class FCameraVariablePinTypeHelper
{
public:

	UE_API static FEdGraphPinType GetPinType(ECameraVariableType VariableType, const UScriptStruct* BlendableStructType);
};

}  // namespace UE::Cameras

#undef UE_API

